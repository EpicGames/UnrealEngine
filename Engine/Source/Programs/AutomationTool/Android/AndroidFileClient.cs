// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Diagnostics;
using System.IO;
using System.IO.Compression;
using System.Net;
using System.Net.Sockets;
using System.Text;
using System.Threading;
using EpicGames.Core;
using Microsoft.Extensions.Logging;


namespace AutomationTool
{
	public class AndroidFileClient
	{
		public delegate bool ShellOutputCallbackType(object Data, string Message);

		protected static ILogger Logger => Log.Logger;

		private const bool TimeBatching = false;

		private const int DefaultPort = 57099;
		private const int ServerVersion_Min = 100;
		private const int ServerVersion_Max = 101;
		private const int ServerVersion_Handshake = 101;
		private const int FILE_BUFFERSIZE = 65536 * 16;
		private const int SOCKET_READBUFFER = 625 * 1024;
		private const int SOCKET_SENDBUFFER = 625 * 1024;
		private const int BATCH_BUFFERSIZE = !TimeBatching ? 65536 * 16 : 200000000;
		private const int BATCH_SENDSIZE = 65536 * 16;

		private string Device;
		private int ServerPort;
		private int HostPort;

		private string ActivePackageName = "";
		private string ActiveToken = "";
		private string ActiveIPAddress = "127.0.0.1";
		private int ActiveVersion = 0;

		private static OptimalADB adb = new OptimalADB();
		static string LastListenDevice = "";
		static bool bHave_ss = true;
		static bool bHave_netstat = true;

		private Socket ClientSocket = null;

		private FileStream RecordFile = null;
		private BufferedStream RecordStream = null;
		private bool bBlockSend = false;

		private byte[] BatchBuffer = null;
		private int BatchBufferIndex = 0;

		private string LastBaseDir = "";

		private static int Command_Terminate = 0;
		private static int Command_Close = 1;
		private static int Command_Info = 2;
		private static int Command_Query = 3;
		private static int Command_GetProp = 4;
		private static int Command_SetBaseDir = 5;

		private static int Command_DirExists = 10;
		private static int Command_DirList = 11;
		private static int Command_DirListFlat = 12;
		private static int Command_DirCreate = 13;
		private static int Command_DirDelete = 14;
		private static int Command_DirDeleteRecurse = 15;

		private static int Command_FileExists = 20;
		private static int Command_FileDelete = 21;
		private static int Command_FileCopy = 22;
		private static int Command_FileMove = 23;
		private static int Command_FileRead = 24;
		private static int Command_FileWrite = 25;
		private static int Command_FileWriteCompressed = 26;
		private static int Command_FileWriteHandshake = 27;

		// ------------------ Helpers to deal with ADB ------------------ 

		// optional way to talk directly with the adbd instead of shelling out to adb.exe
		public class OptimalADB
		{
			private const int DefaultPort = 5037;

			private string Executable;
			private int Port = 6037;
			private Socket ClientSocket = null;

			private string LastShellDevice = "";
			private string LastShellFeatures = "";

			// header is 1 byte id, 4 byte uint32 length
			private static int kHeaderSize = 1 + 4;

			private const int kIdStdin = 0;
			private const int kIdStdOut = 1;
			private const int kIdStdErr = 2;
			private const int kIdExit = 3;
			private const int kIdCloseStdin = 4;
			private const int kIdWindowSizeChange = 5;
			private const int kIdInvalid = 255;

			public OptimalADB(string ADBExecutable = "", int inPort = DefaultPort)
			{
				Executable = (ADBExecutable != "") ? ADBExecutable : Environment.ExpandEnvironmentVariables("%ANDROID_HOME%/platform-tools/adb" + (OperatingSystem.IsWindows() ? ".exe" : ""));
				Port = inPort;
			}

			~OptimalADB()
			{
				CloseConnection();
			}

			private bool OpenConnection()
			{
				if (ClientSocket != null)
				{
					return true;
				}

				// create TCP/IP socket
				try
				{
					IPAddress ipAddr = IPAddress.Parse("127.0.0.1");
					IPEndPoint localEndPoint = new IPEndPoint(ipAddr, Port);

					// create TCP/IP socket
					ClientSocket = new Socket(ipAddr.AddressFamily, SocketType.Stream, ProtocolType.Tcp);
					ClientSocket.Connect(localEndPoint);
				}
				catch (Exception)
				{
					// adb server not running so start it
					IProcessResult Result = CommandUtils.Run(Executable, "start-server", null, CommandUtils.ERunOptions.NoLoggingOfRunCommand);
					if (Result.ExitCode != 0)
					{
						Logger.LogError("Unable to execute adb to start adb server");
						return false;
					}

					// try again
					return OpenConnection();
				}

				return true;
			}

			private void CloseConnection()
			{
				if (ClientSocket != null)
				{
					try
					{
						ClientSocket.Shutdown(SocketShutdown.Both);
						ClientSocket.Close();
					}
					catch (SocketException se)
					{
						Logger.LogWarning("SocketException: {Arg0}", se.ToString());
					}
					catch (Exception e)
					{
						Logger.LogError("Unexpected Exception: {Arg0}", e.ToString());
					}
					finally
					{
						ClientSocket.Dispose();
					}
					ClientSocket = null;
				}
			}

			private bool sendData(string Data)
			{
				if (!OpenConnection())
				{
					return false;
				}

				byte[] buffer = Encoding.UTF8.GetBytes(Data.Length.ToString("X4") + Data);
				int sentBytes = ClientSocket.Send(buffer);

				return true;
			}

			private bool receiveData(out string Output, bool bOnce = false)
			{
				Output = "";

				if (ClientSocket == null)
				{
					return false;
				}

				byte[] buffer = new byte[65536];

				int bytesRecv = ClientSocket.Receive(buffer);
				string WorkBuffer = Encoding.UTF8.GetString(buffer, 0, bytesRecv);

				if (WorkBuffer.Length >= 4)
				{
					if (WorkBuffer.Substring(0, 4) == "OKAY")
					{
						WorkBuffer = WorkBuffer.Substring(4);
						if (bOnce)
						{
							while (ClientSocket.Available > 0)
							{
								bytesRecv = ClientSocket.Receive(buffer);
								WorkBuffer += Encoding.UTF8.GetString(buffer, 0, bytesRecv);
							}
							Output += WorkBuffer;
							return true;
						}
						int remaining = 0;
						while (true)
						{
							while (ClientSocket.Available > 0 || WorkBuffer.Length < 4)
							{
								bytesRecv = ClientSocket.Receive(buffer);
								WorkBuffer += Encoding.UTF8.GetString(buffer, 0, bytesRecv);
							}
							if (WorkBuffer.Length >= 4)
							{
								if (WorkBuffer.Substring(0, 4) == "OKAY")
								{
									WorkBuffer = WorkBuffer.Substring(4);
									continue;
								}
								try
								{
									remaining = int.Parse(WorkBuffer.Substring(0, 4), System.Globalization.NumberStyles.HexNumber);
									WorkBuffer = WorkBuffer.Substring(4);
									remaining -= WorkBuffer.Length;
								}
								catch (Exception)
								{
									while (ClientSocket.Available > 0)
									{
										bytesRecv = ClientSocket.Receive(buffer);
										WorkBuffer += Encoding.UTF8.GetString(buffer, 0, bytesRecv);
									}
									Output = WorkBuffer;
									return true;
								}
								break;
							}
						}
						while (remaining > 0)
						{
							bytesRecv = ClientSocket.Receive(buffer);
							WorkBuffer += Encoding.UTF8.GetString(buffer, 0, bytesRecv);
							remaining -= bytesRecv;
						}
						Output = WorkBuffer;
						return true;
					}
					else if (WorkBuffer.Substring(0, 4) == "FAIL")
					{
						WorkBuffer = WorkBuffer.Substring(4);
						int remaining = 0;
						while (true)
						{
							while (ClientSocket.Available > 0 || WorkBuffer.Length < 4)
							{
								bytesRecv = ClientSocket.Receive(buffer);
								WorkBuffer += Encoding.UTF8.GetString(buffer, 0, bytesRecv);
							}
							if (WorkBuffer.Length >= 4)
							{
								try
								{
									remaining = int.Parse(WorkBuffer.Substring(0, 4), System.Globalization.NumberStyles.HexNumber);
									WorkBuffer = WorkBuffer.Substring(4);
									remaining -= WorkBuffer.Length;
								}
								catch (Exception)
								{
									while (ClientSocket.Available > 0)
									{
										bytesRecv = ClientSocket.Receive(buffer);
										WorkBuffer += Encoding.UTF8.GetString(buffer, 0, bytesRecv);
									}
									Output = WorkBuffer;
									return false;
								}
								break;
							}
						}
						while (remaining > 0)
						{
							bytesRecv = ClientSocket.Receive(buffer);
							WorkBuffer += Encoding.UTF8.GetString(buffer, 0, bytesRecv);
							remaining -= bytesRecv;
						}
						Output = WorkBuffer;
						return false;
					}
				}

				// fail
				Output = WorkBuffer;
				return false;
			}

			private bool execute(string Command, out string Output, bool bOnce = false)
			{
				bool Result = true;
				Output = "";

				if (sendData(Command))
				{
					Result = receiveData(out Output, bOnce);
				}
				return Result;
			}

			private bool executeDevice(string Device, string Command, out string Output, bool bOnce = false)
			{
				if (!execute("host:transport" + (Device == "" ? "-any" : ":" + Device), out Output, true))
				{
					return false;
				}
				return execute(Command, out Output, bOnce);
			}

			public string GetVersion()
			{
				string Result;
				execute("host:version", out Result);
				CloseConnection();
				return Result;
			}

			public string GetDevices()
			{
				string Result;
				execute("host:devices", out Result);
				CloseConnection();
				return Result;
			}

			public string GetFeatures(string Device)
			{
				string Result;

				executeDevice(Device, "host:features", out Result);
				CloseConnection();
				return Result;
			}

			private string shellEscape(string source)
			{
				// wraps whole thing in ' ' after escaping it with \'
				return "'" + source.Replace("'", "'\\''") + "'";
			}

			public string Shell(string Device, string Command, int Timeout = 0, ShellOutputCallbackType OutputCallback = null, object OutputCallbackData = null)
			{
				string Result = "FAIL";

				if (Device != LastShellDevice)
				{
					LastShellFeatures = GetFeatures(Device);
					LastShellDevice = Device;
				}

				if (!execute("host:transport" + (Device == "" ? "-any" : ":" + Device), out Result, true))
				{
					CloseConnection();
					return Result;
				}

				bool bUseShellv2 = LastShellFeatures.Contains("shell_v2");
				string ShellCommand = string.Format("shell{0}:{1}", bUseShellv2 ? ",v2" : "", Command);

				Result = "FAIL";
				try
				{
					if (!sendData(ShellCommand))
					{
						CloseConnection();
						return Result;
					}

					byte[] buffer = new byte[65536];
					Result = "";

					while (Result.Length < 4)
					{
						int bytesRecv = ClientSocket.Receive(buffer, 4, SocketFlags.None);
						Result += Encoding.UTF8.GetString(buffer, 0, bytesRecv);
					}
					if (Result.Substring(0, 4) != "OKAY")
					{
						CloseConnection();
						return Result;
					}
					Result = "";
					string EncodedData = "";

					if (bUseShellv2)
					{
						bool bExit = false;
						while (!bExit)
						{
							if (Timeout > 0 && !ClientSocket.Poll(Timeout * 1000, SelectMode.SelectRead))
							{
								if (OutputCallback != null)
								{
									Result = "TIMEOUT";
								}
								break;
							}
							int bytesRecv = ClientSocket.Receive(buffer, kHeaderSize, SocketFlags.None);
							if (bytesRecv <= 0)
							{
								break;
							}

							int id = buffer[0];
							int length = buffer[1] + (buffer[2] << 8) + (buffer[3] << 16) + (buffer[4] << 24);

							while (length > 0)
							{
								int request = length < buffer.Length ? length : buffer.Length;
								bytesRecv = ClientSocket.Receive(buffer, request, SocketFlags.None);
								length -= bytesRecv;

								switch (id)
								{
									case kIdStdOut:
										EncodedData = Encoding.UTF8.GetString(buffer, 0, bytesRecv);
										Result += EncodedData;
										if (OutputCallback != null && !OutputCallback(OutputCallbackData, EncodedData))
										{
											bExit = true;
											break;
										}
										break;
									case kIdStdErr:
										EncodedData = Encoding.UTF8.GetString(buffer, 0, bytesRecv);
										Result += EncodedData;
										if (OutputCallback != null && !OutputCallback(OutputCallbackData, EncodedData))
										{
											bExit = true;
											break;
										}
										break;
									case kIdExit:
										// exit code is buffer[0]
										bExit = true;
										break;
								}
							}
						}
						CloseConnection();
						return Result;
					}

					while (true)
					{
						if (Timeout > 0 && !ClientSocket.Poll(Timeout * 1000, SelectMode.SelectRead))
						{
							if (OutputCallback != null)
							{
								Result = "TIMEOUT";
							}
							break;
						}
						int bytesRecv = ClientSocket.Receive(buffer);
						if (bytesRecv <= 0)
						{
							break;
						}
						EncodedData = Encoding.UTF8.GetString(buffer, 0, bytesRecv);
						Result += EncodedData;
						if (OutputCallback != null && !OutputCallback(OutputCallbackData, EncodedData))
						{
							break;
						}
					}
				}
				catch (SocketException)
				{
				}

				CloseConnection();
				return Result;
			}

			public string ListForward(string Device)
			{
				string Result;

				executeDevice(Device, "host:list-forward", out Result);
				CloseConnection();
				return Result;
			}

			public string AddForward(string Device, string Local, string Remote)
			{
				string Result;

				executeDevice(Device, "host:forward:" + Local + ";" + Remote, out Result);
				CloseConnection();
				return Result;
			}

			public string RemoveForward(string Device, string Local)
			{
				string Result;

				executeDevice(Device, "host:killforward:" + Local, out Result, true);
				CloseConnection();
				return Result;
			}
		}

		private static string GetAdbCommandLine(string SerialNumber, string Args)
		{
			if (string.IsNullOrEmpty(SerialNumber) == false)
			{
				SerialNumber = "-s " + SerialNumber;
			}

			return string.Format("{0} {1}", SerialNumber, Args);
		}

		static string LastSpewFilename = "";

		public static string ADBSpewFilter(string Message)
		{
			if (Message.StartsWith("[") && Message.Contains("%]"))
			{
				int LastIndex = Message.IndexOf(":");
				LastIndex = LastIndex == -1 ? Message.Length : LastIndex;

				if (Message.Length > 7)
				{
					string Filename = Message.Substring(7, LastIndex - 7);
					if (Filename == LastSpewFilename)
					{
						return null;
					}
					LastSpewFilename = Filename;
				}
				return Message;
			}
			return Message;
		}

		public static IProcessResult RunAdbCommand(string SerialNumber, string Args, string Input = null, CommandUtils.ERunOptions Options = CommandUtils.ERunOptions.NoLoggingOfRunCommand, bool bShouldLogCommand = false)
		{
			string AdbCommand = Environment.ExpandEnvironmentVariables("%ANDROID_HOME%/platform-tools/adb" + (OperatingSystem.IsWindows() ? ".exe" : ""));
			if (Options.HasFlag(CommandUtils.ERunOptions.AllowSpew) || Options.HasFlag(CommandUtils.ERunOptions.SpewIsVerbose))
			{
				LastSpewFilename = "";
				return CommandUtils.Run(AdbCommand, GetAdbCommandLine(SerialNumber, Args), Input, Options, SpewFilterCallback: new ProcessResult.SpewFilterCallbackType(ADBSpewFilter));
			}
			return CommandUtils.Run(AdbCommand, GetAdbCommandLine(SerialNumber, Args), Input, Options);
		}

		public static void GetConnectedDevices(out List<string> Devices)
		{
			Devices = new List<string>();


			string Result = adb.GetDevices();
			if (Result.Length > 0)
			{
				string[] LogLines = Result.Split(new char[] { '\n', '\r' });
				for (int i = 0; i < LogLines.Length; ++i)
				{
					string[] DeviceLine = LogLines[i].Split(new char[] { '\t' });

					if (DeviceLine.Length == 2)
					{
						// the second param should be "device"
						// if it's not setup correctly it might be "unattached" or "powered off" or something like that
						// warning in that case
						if (DeviceLine[1] == "device")
						{
							Devices.Add("@" + DeviceLine[0]);
						}
						else
						{
							Logger.LogWarning("Device attached but in bad state {Arg0}:{Arg1}", DeviceLine[0], DeviceLine[1]);
						}
					}
				}
			}
		}

		// Checks for existing forward of device port and returns local host port if found, otherwise sets one
		public static int ForwardPort_FindOrAdd(string Device, int DevicePort)
		{
			int HostPort = -1;

			// Check if port already forwarded to this device
			string Result = adb.ListForward(Device);
			if (Result.Length > 0)
			{
				string[] LogLines = Result.Split(new char[] { '\n', '\r' });
				for (int i = 0; i < LogLines.Length; ++i)
				{
					string[] DeviceLine = LogLines[i].Split(new char[] { '\t', ' ' });
					if (DeviceLine.Length == 3)
					{
						if (DeviceLine[1].StartsWith("tcp:") && DeviceLine[2].StartsWith("tcp:"))
						{
							int ListDevicePort = -1;
							if (int.TryParse(DeviceLine[2].Substring(4), out ListDevicePort))
							{
								if (ListDevicePort == DevicePort)
								{
									if (int.TryParse(DeviceLine[1].Substring(4), out HostPort))
									{
										return HostPort;
									}
								}
							}
						}
					}
				}
			}

			// request forwarding from an available host port
			Result = adb.AddForward(Device, "tcp:0", "tcp:" + DevicePort.ToString());
			if (Result.Length > 0)
			{
				string[] LogLines = Result.Split(new char[] { '\n', '\r' });
				for (int i = 0; i < LogLines.Length; ++i)
				{
					if (int.TryParse(LogLines[i], out HostPort))
					{
						return HostPort;
					}
				}
			}

			// did not work, return -1 as error
			return -1;
		}

		// Checks for existing forwards of device port and removes them then requests a new one to return
		// This deals better with stale binds and leftover forward requests
		public static int ForwardPort_Add(string Device, int DevicePort)
		{
			int HostPort = -1;

			// Check if port already forwarded to this device
			string Result = adb.ListForward(Device);
			if (Result.Length > 0)
			{
				string[] LogLines = Result.Split(new char[] { '\n', '\r' });
				for (int i = 0; i < LogLines.Length; ++i)
				{
					string[] DeviceLine = LogLines[i].Split(new char[] { '\t', ' ' });
					if (DeviceLine.Length == 3)
					{
						if (DeviceLine[1].StartsWith("tcp:") && DeviceLine[2].StartsWith("tcp:"))
						{
							int ListDevicePort = -1;
							if (int.TryParse(DeviceLine[2].Substring(4), out ListDevicePort))
							{
								if (ListDevicePort == DevicePort)
								{
									adb.RemoveForward(Device, DeviceLine[1]);
								}
							}
						}
					}
				}
			}

			// request forwarding from an available host port
			Result = adb.AddForward(Device, "tcp:0", "tcp:" + DevicePort.ToString());
			if (Result.Length > 0)
			{
				string[] LogLines = Result.Split(new char[] { '\n', '\r' });
				for (int i = 0; i < LogLines.Length; ++i)
				{
					if (int.TryParse(LogLines[i], out HostPort))
					{
						return HostPort;
					}
				}
			}

			// did not work, return -1 as error
			return -1;
		}

		public static void ForwardPort_Remove(string Device, int HostPort)
		{
			// request removal of forwarding from host port
			adb.RemoveForward(Device, "tcp:" + HostPort.ToString());
		}

		public static void ReversePort_Add(string Device, int DevicePort, int HostPort)
		{
			// request reversing from device port to local host port
			IProcessResult Result = RunAdbCommand(Device, "reverse tcp:" + DevicePort.ToString() + " tcp:" + HostPort.ToString());
			Result.DisposeProcess();
		}

		public static void ReversePort_Remove(string Device, int DevicePort)
		{
			// request removal of forwarding from host port
			IProcessResult Result = RunAdbCommand(Device, "reverse --remove tcp:" + DevicePort.ToString());
			Result.DisposeProcess();
		}

		// ------------------ Helpers to deal with packets ------------------ 

		private long Stats_FilesRead = 0;
		private long Stats_FilesWrite = 0;

		private long Stats_TotalBytesReceived = 0;
		private long Stats_TotalBytesSent = 0;

		private long Stats_PayloadBytesReceived = 0;
		private long Stats_PayloadBytesSent = 0;

		private long Stats_CompressedBytesReceived = 0;
		private long Stats_CompressedBytesSent = 0;

		private Stopwatch Stats_Stopwatch = null;

		public void Stats_Clear(bool bStartTimer = true)
		{
			Stats_FilesRead = 0;
			Stats_FilesWrite = 0;

			Stats_TotalBytesReceived = 0;
			Stats_TotalBytesSent = 0;

			Stats_PayloadBytesReceived = 0;
			Stats_PayloadBytesSent = 0;

			Stats_CompressedBytesReceived = 0;
			Stats_CompressedBytesSent = 0;

			if (bStartTimer)
			{
				Stats_StartStopwatch();
			}
		}

		public void Stats_StartStopwatch()
		{
			Stats_Stopwatch = System.Diagnostics.Stopwatch.StartNew();
		}

		public void Stats_Combine(AndroidFileClient client)
		{
			Stats_FilesRead += client.Stats_FilesRead;
			Stats_FilesWrite += client.Stats_FilesWrite;

			Stats_TotalBytesReceived += client.Stats_TotalBytesReceived;
			Stats_TotalBytesSent += client.Stats_TotalBytesSent;

			Stats_PayloadBytesReceived += client.Stats_PayloadBytesReceived;
			Stats_PayloadBytesSent += client.Stats_PayloadBytesSent;

			Stats_CompressedBytesReceived += client.Stats_CompressedBytesReceived;
			Stats_CompressedBytesSent += client.Stats_CompressedBytesSent;

			client.Stats_Clear(false);
		}

		public void Stats_Report()
		{
			Logger.LogInformation("Total bytes received:    {Stats_TotalBytesReceived}", Stats_TotalBytesReceived);
			Logger.LogInformation("Payload bytes received:  {Stats_PayloadBytesReceived}", Stats_PayloadBytesReceived);
			Logger.LogInformation("Total files read:        {Stats_FilesRead}", Stats_FilesRead);
			Logger.LogInformation(".");
			Logger.LogInformation("Total bytes sent:        {Stats_TotalBytesSent}", Stats_TotalBytesSent);
			Logger.LogInformation("Payload bytes sent:      {Stats_PayloadBytesSent}", Stats_PayloadBytesSent);
			Logger.LogInformation("Total files written:     {Stats_FilesWrite}", Stats_FilesWrite);
			Logger.LogInformation(".");
			Logger.LogInformation("Uncompressed bytes recv: {Stats_CompressedBytesReceived}", Stats_CompressedBytesReceived);
			Logger.LogInformation("Uncompressed bytes sent: {Stats_CompressedBytesSent}", Stats_CompressedBytesSent);
			Logger.LogInformation(".");

			if (Stats_Stopwatch != null)
			{
				Stats_Stopwatch.Stop();

				var ElapsedMs = Stats_Stopwatch.ElapsedMilliseconds;
				double MBPerSec = (double)Stats_TotalBytesSent / ((double)ElapsedMs / 1000.0) / 1024.0 / 1024.0;
				Logger.LogInformation("Actual sent:  {0:N1} MB/s ({Stats_TotalBytesSent} in {2:N3}s)", MBPerSec, Stats_TotalBytesSent, (float)(ElapsedMs / 1000.0));
				MBPerSec = (double)Stats_CompressedBytesSent / ((double)ElapsedMs / 1000.0) / 1024.0 / 1024.0;
				Logger.LogInformation("Uncompressed: {0:N1} MB/s ({Stats_CompressedBytesSent} in {2:N3}s)", MBPerSec, Stats_CompressedBytesSent, (float)(ElapsedMs / 1000.0));
			}
		}

		public void Record_Stop()
		{
			bBlockSend = false;
			if (RecordStream != null)
			{
				RecordStream.Close();
				RecordStream.Dispose();
				RecordStream = null;
			}
			if (RecordFile != null)
			{
				RecordFile.Close();
				RecordFile.Dispose();
				RecordFile = null;
			}
		}

		public bool Record_Begin(string Filename, bool BlockSend = false)
		{
			bBlockSend = BlockSend;
			Record_Stop();

			try
			{
				RecordFile = File.Open(Filename, FileMode.Create, FileAccess.Write);
				try
				{
					RecordStream = new BufferedStream(RecordFile);
				}
				catch (Exception)
				{
					RecordFile.Close();
					RecordFile.Dispose();
					RecordFile = null;
					return false;
				}
			}
			catch (Exception)
			{
				RecordFile = null;
				return false;
			}
			return true;
		}

		public bool Record_Play(string Filepath)
		{
			Boolean Result = false;

			if (ClientSocket == null)
			{
				return Result;
			}
			if (!File.Exists(Filepath))
			{
				return Result;
			}

			using (FileStream fileStream = File.Open(Filepath, FileMode.Open, FileAccess.Read, FileShare.Read))
			{
				try
				{
					int bytesRead;
					byte[] buffer = new byte[FILE_BUFFERSIZE];

					while ((bytesRead = fileStream.Read(buffer, 0, FILE_BUFFERSIZE)) != 0)
					{
						SocketSend(true, buffer, bytesRead);
					}
					fileStream.Close();
					Result = true;
				}
				catch (Exception e)
				{
					Logger.LogError("Unexpected Exception: {Arg0}", e.ToString());
					CloseConnection();
				}
			}
			return Result;
		}

		private int SocketReceive(bool bPayload, byte[] packet)
		{
			int received = 0;
			try
			{
				received = ClientSocket.Receive(packet);

				Stats_TotalBytesReceived += received;
				if (bPayload)
				{
					Stats_PayloadBytesReceived += received;
				}

				return received;
			}
			catch (Exception)
			{
			}
			return received;
		}

		private int SocketReceive(bool bPayload, byte[] packet, int readSize)
		{
			int received = ClientSocket.Receive(packet, readSize, SocketFlags.None);

			Stats_TotalBytesReceived += received;
			if (bPayload)
			{
				Stats_PayloadBytesReceived += received;
			}

			return received;
		}

		public bool Batch_Start()
		{
			if (BatchBuffer != null)
			{
				return false;
			}

			BatchBuffer = new byte[BATCH_BUFFERSIZE];
			BatchBufferIndex = 0;

			return true;
		}

		public bool Batch_Flush()
		{
			if (BatchBuffer == null)
			{
				return false;
			}

			if (BatchBufferIndex > 0)
			{
				int index = 0;
				int remaining = BatchBufferIndex;
				while (remaining > 0)
				{
					int chunk = remaining < BATCH_SENDSIZE ? remaining : BATCH_SENDSIZE;
					int sent = ClientSocket.Send(BatchBuffer, index, chunk, SocketFlags.None);
					remaining -= sent;
					index += sent;
				}
				BatchBufferIndex = 0;
			}

			return true;
		}

		public void Batch_Stop()
		{
			Batch_Flush();

			BatchBuffer = null;
			BatchBufferIndex = 0;
		}

		private void Batch_Append(byte[] packet, int writeSize)
		{
			int readOffset = 0;
			while (writeSize > 0)
			{
				int copySize = (BatchBufferIndex + writeSize > BATCH_BUFFERSIZE) ? BATCH_BUFFERSIZE - BatchBufferIndex : writeSize;
				Buffer.BlockCopy(packet, readOffset, BatchBuffer, BatchBufferIndex, copySize);

				BatchBufferIndex += copySize;
				readOffset += copySize;
				writeSize -= copySize;

				if (BatchBufferIndex == BATCH_BUFFERSIZE)
				{
					Batch_Flush();
				}
			}
		}

		private int SocketSend(bool bPayload, byte[] packet)
		{
			int writeSize = packet.Length;
			int sent = writeSize;

			if (BatchBuffer != null)
			{
				Batch_Append(packet, writeSize);
			}
			else if (!bBlockSend)
			{
				sent = ClientSocket.Send(packet);
			}

			if (RecordStream != null)
			{
				RecordStream.Write(packet);
			}

			Stats_TotalBytesSent += sent;
			if (bPayload)
			{
				Stats_PayloadBytesSent += sent;
			}

			if (sent < writeSize)
			{
				// did not send all
			}

			return sent;
		}

		private int SocketSend(bool bPayload, byte[] packet, int writeSize)
		{
			int sent = writeSize;

			if (BatchBuffer != null)
			{
				Batch_Append(packet, writeSize);
			}
			else if (!bBlockSend)
			{
				sent = ClientSocket.Send(packet, writeSize, SocketFlags.None);
			}

			if (RecordStream != null)
			{
				RecordStream.Write(packet, 0, writeSize);
			}

			Stats_TotalBytesSent += sent;
			if (bPayload)
			{
				Stats_PayloadBytesSent += sent;
			}

			if (sent < writeSize)
			{
				// did not send all
			}

			return sent;
		}

		private byte[] CommandPacket(int Command, long Size)
		{
			byte[] Result = { (byte)(Command & 255),
								(byte)((Command >> 8) & 255),
								(byte)(Size & 255),
								(byte)((Size >> 8) & 255),
								(byte)((Size >> 16) & 255),
								(byte)((Size >> 24) & 255),
								(byte)((Size >> 32) & 255),
								(byte)((Size >> 40) & 255) };
			return Result;
		}

		private long ResultSize()
		{
			long Result = -1;
			try
			{
				byte[] sizePacket = new byte[8];
				int byteRecv = SocketReceive(false, sizePacket);
				if (byteRecv == 8)
				{
					Result = (long)sizePacket[0] + ((long)sizePacket[1] << 8) + ((long)sizePacket[2] << 16) + ((long)sizePacket[3] << 24) +
							((long)sizePacket[4] << 32) + ((long)sizePacket[5] << 40) + ((long)sizePacket[6] << 48) + ((long)sizePacket[7] << 56);
				}
			}
			catch (Exception e)
			{
				Logger.LogError("Unexpected Exception: {Arg0}", e.ToString());
				CloseConnection();
			}
			return Result;
		}

		private string GetStringResult()
		{
			string Result = null;
			try
			{
				int resultSize = (int)ResultSize();
				if (resultSize > 0)
				{
					byte[] buffer = new byte[resultSize < FILE_BUFFERSIZE ? resultSize : FILE_BUFFERSIZE];

					Result = "";
					long remaining = resultSize;
					while (remaining > 0)
					{
						int readSize = remaining > FILE_BUFFERSIZE ? FILE_BUFFERSIZE : (int)remaining;
						int bytesRecv = SocketReceive(true, buffer, readSize);
						remaining -= bytesRecv;
						Result += Encoding.UTF8.GetString(buffer, 0, bytesRecv);
					}
				}
			}
			catch (Exception e)
			{
				Logger.LogError("Unexpected Exception: {Arg0}", e.ToString());
				CloseConnection();
			}
			return Result;
		}

		private string GetStringResult(int Command, string Params)
		{
			string Result = null;

			if (ClientSocket == null)
			{
				return Result;
			}

			byte[] paramMessage = Encoding.UTF8.GetBytes(Params + "\0");
			long paramSize = paramMessage.Length;
			byte[] commandMessage = CommandPacket(Command, paramSize);

			try
			{
				int bytesSent = SocketSend(false, commandMessage);
				bytesSent = SocketSend(false, paramMessage);
				Result = GetStringResult();
			}
			catch (Exception e)
			{
				Logger.LogError("Unexpected Exception: {Arg0}", e.ToString());
				CloseConnection();
			}
			return Result;
		}

		private bool GetBoolResult(int Command, string Params)
		{
			string Result = GetStringResult(Command, Params);
			if (Result == null)
			{
				return false;
			}

			return (Result == "true");
		}

		// ------------------ Optimizes destination paths -----------------

		private string OptimizePath(string inPath)
		{
			string inDir = Path.GetDirectoryName(inPath).Replace("\\", "/") + "/";
			string inFilename = Path.GetFileName(inPath);

			if (inDir == "/")
			{
				return inPath;
			}

			if (LastBaseDir == "")
			{
				LastBaseDir = inDir;
				return inPath;
			}

			// trivial case of same as last base
			if (LastBaseDir == inDir)
			{
				return "^^/" + inFilename;
			}

			// basedir matches start of new path
			if (inDir.StartsWith(LastBaseDir))
			{
				int BaseDirLength = LastBaseDir.Length;
				LastBaseDir = inDir;
				return "^^/" + inDir.Substring(BaseDirLength) + inFilename;
			}

			int startIndex;
			int nextIndex;
			int dropCount;

			// if indir is below basedir, figure out how many directories to drop
			if (LastBaseDir.StartsWith(inDir))
			{
				startIndex = inDir.Length;
				dropCount = 0;
				while ((nextIndex = LastBaseDir.Substring(startIndex).IndexOf('/')) >= 0)
				{
					dropCount++;
					startIndex += nextIndex + 1;
				}
				LastBaseDir = LastBaseDir.Substring(0, inDir.Length);
				return "^-" + dropCount + "/" + inFilename;
			}

			// lastly, see what portion of the two paths match (if any)
			int searchIndex = 0;
			int lastSlash = -1;
			int maxIndex = inDir.Length < LastBaseDir.Length ? inDir.Length : LastBaseDir.Length;
			while (searchIndex < maxIndex && inDir[searchIndex] == LastBaseDir[searchIndex])
			{
				if (inDir[searchIndex] == '/')
				{
					lastSlash = searchIndex;
				}
				searchIndex++;
			}

			if (lastSlash < 1)
			{
				LastBaseDir = inDir;
				return inPath;
			}

			startIndex = lastSlash + 1;
			dropCount = 0;
			while ((nextIndex = LastBaseDir.Substring(startIndex).IndexOf('/')) >= 0)
			{
				dropCount++;
				startIndex += nextIndex + 1;
			}
			LastBaseDir = inDir;
			return "^-" + dropCount + inDir.Substring(lastSlash) + inFilename;
		}

		// ------------------ Interace ------------------ 

		public AndroidFileClient(string inDevice, int inPort = DefaultPort)
		{
			Device = inDevice;
			ServerPort = inPort != 0 ? inPort : DefaultPort;
			HostPort = -1;
		}

		~AndroidFileClient()
		{
			CloseConnection();
		}

		public static int GetDefaultPort()
		{
			return DefaultPort;
		}

		public string GetDevice()
		{
			return Device;
		}

		public int GetServerPort()
		{
			return ServerPort;
		}

		public void SetActive(String PackageName, String Token, String Address)
		{
			ActivePackageName = PackageName;
			ActiveToken = Token;
			ActiveIPAddress = Address;
		}

		public bool OpenConnection(String Address = "127.0.0.1")
		{
			int UsePort = ServerPort;
			if (Address == null || Address == "")
			{
				Address = "127.0.0.1";
			}
			if (Address == "127.0.0.1")
			{
				HostPort = ForwardPort_Add(Device, ServerPort);
				if (HostPort == -1)
				{
					return false;
				}
				UsePort = HostPort;
			}

			// create TCP/IP socket
			try
			{
				IPAddress ipAddr = IPAddress.Parse(Address);
				IPEndPoint localEndPoint = new IPEndPoint(ipAddr, UsePort);

				// create TCP/IP socket
				ClientSocket = new Socket(ipAddr.AddressFamily, SocketType.Stream, ProtocolType.Tcp);

				ClientSocket.Connect(localEndPoint);
				ClientSocket.NoDelay = false;
				ClientSocket.SendBufferSize = SOCKET_SENDBUFFER;
				ClientSocket.ReceiveBufferSize = SOCKET_READBUFFER;

				Stats_Clear();
				LastBaseDir = "";

				int bytesSent = SocketSend(false, CommandPacket(Command_Info, 0));

				int defaultTimeout = ClientSocket.ReceiveTimeout;
				ClientSocket.ReceiveTimeout = 1000;

				byte[] sizePacket = new byte[8];
				int bytesRecv = SocketReceive(false, sizePacket);
				if (bytesRecv == 8)
				{
					ClientSocket.ReceiveTimeout = defaultTimeout;

					int resultSize = sizePacket[0] + (sizePacket[1] << 8) + (sizePacket[2] << 8) + (sizePacket[3] << 16) +
									(sizePacket[4] << 24) + (sizePacket[5] << 32) + (sizePacket[6] << 40) + (sizePacket[7] << 48);
					byte[] message = new byte[resultSize];
					bytesRecv = SocketReceive(false, message);
					if (bytesRecv == resultSize)
					{
						int version = message[0] + (message[1] << 8);
						if (version >= ServerVersion_Min && version <= ServerVersion_Max)
						{
							ActiveVersion = version;
							Logger.LogInformation("Connected to RemoteFileManager");
							return true;
						}
					}
				}
				else
				{
					//Log.TraceInformation("Did not get response from RemoteFileManager");
				}

				CloseConnection();
				return false;
			}
			catch (ArgumentNullException ane)
			{
				Logger.LogWarning("ArgumentNullException: {Arg0}", ane.ToString());
			}
			catch (SocketException se)
			{
				string message = se.ToString();
				if (message.Contains("No connection could be made because the target machine actively refused it"))
				{
					// ok not to display this one
				}
				else
				{
					Logger.LogWarning("SocketException: {message}", message);
				}
			}
			catch (Exception e)
			{
				Logger.LogWarning("OpenConnection failed: {Arg0}", e.ToString());
			}

			if (ClientSocket != null)
			{
				ClientSocket.Dispose();
				ClientSocket = null;
			}
			if (HostPort != -1)
			{
				ForwardPort_Remove(Device, HostPort);
				HostPort = -1;
			}
			return false;
		}

		public static string Logcat(string Device, string Options, int Timeout = 0, ShellOutputCallbackType InCallback = null, object InCallbackData = null)
		{
			return adb.Shell(Device, "export ANDROID_LOG_TAGS=; exec logcat " + Options, Timeout, InCallback, InCallbackData);
		}

		public static string GetLastLogcatTime(string Device)
		{
			string LogcatResult = Logcat(Device, "-t 1 -v monotonic");
			string Result = "0.000";
			foreach (string Line in LogcatResult.Split('\n'))
			{
				string Trimmed = Line.TrimStart();
				if (string.IsNullOrWhiteSpace(Trimmed) || !char.IsDigit(Trimmed[0]))
				{
					continue;
				}
				int SpaceIndex = Trimmed.IndexOf(' ');
				int TabIndex = Trimmed.IndexOf('\t');
				if (SpaceIndex > 0 && TabIndex > 0)
				{
					Result = Trimmed.Substring(0, Math.Min(SpaceIndex, TabIndex));
				}
				else if (SpaceIndex > 0)
				{
					Result = Trimmed.Substring(0, SpaceIndex);
				}
				else if (TabIndex > 0)
				{
					Result = Trimmed.Substring(0, TabIndex);
				}
			}
			return Result;
		}

		private class WaitCallbackData
		{
			public double AfterTime;
			public string WaitForLine;
			public string MessageBuffer;

			public WaitCallbackData(double InAfterTime, string InWaitForLine)
			{
				AfterTime = InAfterTime;
				WaitForLine = InWaitForLine;
				MessageBuffer = "";
			}

			public bool CheckForLine(string InNewData)
			{
				// incoming data may not be a complete line so always append the new data
				MessageBuffer += InNewData;

				// check if we now have the search line
				while (true)
				{
					int WaitLineIndex = MessageBuffer.IndexOf(WaitForLine);
					if (WaitLineIndex < 0)
					{
						// stop if not found
						break;
					}
	
					// done if don't care about time
					if (AfterTime < 0.0)
					{
						MessageBuffer = "";
						return true;
					}

					// get from beginning of line with the match
					string Trimmed = MessageBuffer.Substring(0, WaitLineIndex);
					int EOLIndex = Trimmed.LastIndexOf("\n");
					if (EOLIndex >= 0)
					{
						Trimmed = Trimmed.Substring(EOLIndex + 1);
					}

					// keep only past this found line for next iteration
					MessageBuffer = MessageBuffer.Substring(WaitLineIndex + 1);
					int FirstLineIndex = MessageBuffer.IndexOf('\n');
					if (FirstLineIndex >= 0)
					{
						MessageBuffer = MessageBuffer.Substring(FirstLineIndex + 1);
					}

					// parse the time
					string Result = "0.000";
					Trimmed = Trimmed.TrimStart();
					if (string.IsNullOrWhiteSpace(Trimmed) || !char.IsDigit(Trimmed[0]))
					{
						continue;
					}

					int SpaceIndex = Trimmed.IndexOf(' ');
					int TabIndex = Trimmed.IndexOf('\t');
					if (SpaceIndex > 0 && TabIndex > 0)
					{
						Result = Trimmed.Substring(0, Math.Min(SpaceIndex, TabIndex));
					}
					else if (SpaceIndex > 0)
					{
						Result = Trimmed.Substring(0, SpaceIndex);
					}
					else if (TabIndex > 0)
					{
						Result = Trimmed.Substring(0, TabIndex);
					}

					double LineTime = 0.0;
					if (Double.TryParse(Result, out LineTime))
					{
						if (LineTime > AfterTime)
						{
//								Console.WriteLine("GOT IT at: " + LineTime);
							MessageBuffer = "";
							return true;
						}
					}
				}

				// drop old lines to keep the search faster and free memory
				int LastLineIndex = MessageBuffer.LastIndexOf('\n');
				if (LastLineIndex >= 0)
				{
					MessageBuffer = MessageBuffer.Substring(LastLineIndex + 1);
				}

				return false;
			}
		}

		private static bool LogcatWaitCallback(object InData, string Message)
		{
			WaitCallbackData WorkData = (WaitCallbackData)InData;
			return WorkData.CheckForLine(Message) ? false : true;
		}

		public static bool WaitForLogcat(string Device, string Options, string Line, int Timeout, double AfterTime = -1.0)
		{
			WaitCallbackData WorkData = new WaitCallbackData(AfterTime, Line);

			string Result = Logcat(Device, Options, Timeout, LogcatWaitCallback, (object)WorkData);
			return Result != "TIMEOUT";
		}

		public static List<string> GetInstalledReceivers(string Device)
		{
			List<string> Result = new List<string>();

			String InstalledResult = adb.Shell(Device, "cmd package query-receivers --components -a com.epicgames.unreal.RemoteFileManager.intent.COMMAND");
			if (InstalledResult.Contains("not found") || InstalledResult.Contains("FAIL"))
			{
				InstalledResult = adb.Shell(Device, "dumpsys package");
				bool bFoundReceiver = false;
				foreach (string Line in InstalledResult.Split('\n'))
				{
					if (!bFoundReceiver)
					{
						if (Line.Contains("com.epicgames.unreal.RemoteFileManager.intent.COMMAND:"))
						{
							bFoundReceiver = true;
						}
					}
					else
					{
						int SlashIndex = Line.IndexOf("/com.epicgames.unreal.RemoteFileManagerReceiver");
						if (SlashIndex > 0)
						{
							int StartIndex = Line.LastIndexOf(' ');
							if (StartIndex >= 0)
							{
								Result.Add(Line.Substring(StartIndex + 1, SlashIndex - StartIndex - 1));
							}
						}
						else if (Line.IndexOf(":") > 0)
						{
							break;
						}
					}
				}
				return Result;
			}
			foreach (string Line in InstalledResult.Split('\n'))
			{
				int SlashIndex = Line.IndexOf('/');
				if (SlashIndex > 0 && (Line.Substring(SlashIndex + 1) == "com.epicgames.unreal.RemoteFileManagerReceiver"))
				{
					Result.Add(Line.Substring(0, SlashIndex));
				}
			}
			return Result;
		}

		public static List<string> GetInstalledActivities(string Device)
		{
			List<string> Result = new List<string>();

			String InstalledResult = adb.Shell(Device, "cmd package query-activities --components -a com.epicgames.unreal.RemoteFileManager.intent.COMMAND2");
			if (InstalledResult.Contains("not found") || InstalledResult.Contains("FAIL"))
			{
				InstalledResult = adb.Shell(Device, "dumpsys package");
				bool bFoundActivity = false;
				foreach (string Line in InstalledResult.Split('\n'))
				{
					if (!bFoundActivity)
					{
						if (Line.Contains("com.epicgames.unreal.RemoteFileManager.intent.COMMAND2:"))
						{
							bFoundActivity = true;
						}
					}
					else
					{
						int SlashIndex = Line.IndexOf("/com.epicgames.unreal.RemoteFileManagerActivity");
						if (SlashIndex > 0)
						{
							int StartIndex = Line.LastIndexOf(' ');
							if (StartIndex >= 0)
							{
								Result.Add(Line.Substring(StartIndex + 1, SlashIndex - StartIndex - 1));
							}
						}
						else if (Line.IndexOf(":") > 0)
						{
							break;
						}
					}
				}
				return Result;
			}
			foreach (string Line in InstalledResult.Split('\n'))
			{
				int SlashIndex = Line.IndexOf('/');
				if (SlashIndex > 0 && (Line.Substring(SlashIndex + 1) == "com.epicgames.unreal.RemoteFileManagerActivity"))
				{
					Result.Add(Line.Substring(0, SlashIndex));
				}
			}
			return Result;
		}

		public static bool GetListenStatus_netstat(string Device, int Port, out bool bHaveApp, out bool bUSB, out bool bWifi, out string WifiAddress)
		{
			bUSB = false;
			bWifi = false;
			WifiAddress = "";

			string Search = ":" + Port;
			string netstatResult = adb.Shell(Device, "netstat -atn");
			if (netstatResult.Contains("not found") || netstatResult.Contains("FAIL"))
			{
				bHaveApp = false;
				return false;
			}
			bHaveApp = true;
			foreach (string Line in netstatResult.Split('\n'))
			{
				if (Line.Contains("LISTEN") && Line.Contains(Search))
				{
					String[] parts = Line.Split(new char[0], StringSplitOptions.RemoveEmptyEntries);
					if (parts.Length > 3 && parts[3].EndsWith(Search))
					{
						int SearchIndex = parts[3].LastIndexOf(Search);
						string LineIP = parts[3].Substring(0, SearchIndex);
						if (LineIP.Contains("127.0.0.1"))
						{
							bUSB = true;
						}
						else
						{
							bWifi = true;
							// ip address is partial so don't try to parse it
						}
					}
				}
			}
			return true;
		}

		public static bool GetListenStatus_ss(string Device, int Port, out bool bHaveApp, out bool bUSB, out bool bWifi, out string WifiAddress)
		{
			bUSB = false;
			bWifi = false;
			WifiAddress = "";

			string Search = ":" + Port;
			string ssResult = adb.Shell(Device, "ss -ta");
			if (ssResult.Contains("not found") || ssResult.Contains("FAIL"))
			{
				bHaveApp = false;
				return false;
			}
			bHaveApp = true;
			bool bConnectionsPresent = false;
			bool bFirstLine = true;
			foreach (string Line in ssResult.Split('\n', StringSplitOptions.RemoveEmptyEntries))
			{
				if (bFirstLine)
				{
					bFirstLine = false;
					continue;
				}
				if (Line.Contains("Permission denied"))
				{
					continue;
				}
				bConnectionsPresent = true;
				if (Line.Contains("LISTEN") && Line.Contains(Search))
				{
					String[] parts = Line.Split(new char[0], StringSplitOptions.RemoveEmptyEntries);
					if (parts.Length > 3 && parts[3].EndsWith(Search))
					{
						// either ip:port or [*]:port
						int SearchIndex = parts[3].LastIndexOf(Search);
						string LineIP = parts[3].Substring(0, SearchIndex);
						if (LineIP.StartsWith("["))
						{
							// handle [*:ip]:port or [ip]:port
							SearchIndex = LineIP.LastIndexOf(":");
							if (SearchIndex > 0)
							{
								LineIP = LineIP.Substring(SearchIndex + 1, LineIP.Length - SearchIndex - 2);
							}
							else
							{
								LineIP = LineIP.Substring(1, LineIP.Length - 2);
							}
						}
						if (LineIP == "127.0.0.1")
						{
							bUSB = true;
						}
						else
						{
							WifiAddress = LineIP;
							bWifi = true;
						}
					}
				}
			}

			return bConnectionsPresent;
		}

		public static bool GetListenStatus(string Device, int Port, out bool bUSB, out bool bWifi, out string WifiAddress)
		{
			bUSB = false;
			bWifi = false;
			WifiAddress = "";

			if (LastListenDevice != Device)
			{
				// reset availability of commands if different device
				LastListenDevice = Device;
				bHave_ss = true;
				bHave_netstat = true;
			}

			// try ss first and remember if available for next time
			if (bHave_ss)
			{
				if (GetListenStatus_ss(Device, Port == 0 ? DefaultPort : Port, out bHave_ss, out bUSB, out bWifi, out WifiAddress))
				{
					return true;
				}
			}

			// next fall back to netstat and remember if available for next time
			if (bHave_netstat)
			{
				if (GetListenStatus_netstat(Device, Port == 0 ? DefaultPort : Port, out bHave_netstat, out bUSB, out bWifi, out WifiAddress))
				{
					return true;
				}
			}

			// neither available so cannot determine listen status
			return false;
		}

		private static string CurrentAndroidSDKLevelDevice = "";
		private static int CurrentAndroidSDKLevel = -1;

		private static int GetDeviceAndroidSDKLevel(string Device)
		{
			if (CurrentAndroidSDKLevelDevice != Device || CurrentAndroidSDKLevel == -1)
			{
				string Result = adb.Shell(Device, "getprop ro.build.version.sdk");
				if (int.TryParse(Result.Trim(), out CurrentAndroidSDKLevel))
				{
					CurrentAndroidSDKLevelDevice = Device;
				}
			}
			return CurrentAndroidSDKLevel;
		}

		private static bool ActivityManagerSendAndWait(string Device, string PackageName, string Arguments)
		{
			// am start -W seems to work past Android 10 reliably (but use a timeout just in case)
			if (GetDeviceAndroidSDKLevel(Device) > 29)
			{
				adb.Shell(Device, "am start -W -a com.epicgames.unreal.RemoteFileManager.intent.COMMAND2 -n " + PackageName + "/com.epicgames.unreal.RemoteFileManagerActivity " + Arguments, 1500);
				return true;
			}

			// need to monitor logcat for response from receiver (with timeout)
			string LastTime = GetLastLogcatTime(Device);
			double LastTimeValue = 0.0;
			Double.TryParse(LastTime, out LastTimeValue);
//			Console.WriteLine("Sending " + Arguments + " to " + PackageName + " at " + LastTime);

			adb.Shell(Device, "am start -a com.epicgames.unreal.RemoteFileManager.intent.COMMAND2 -n " + PackageName + "/com.epicgames.unreal.RemoteFileManagerActivity " + Arguments);

//			Stopwatch stopwatch = System.Diagnostics.Stopwatch.StartNew();
			bool retval = WaitForLogcat(Device, "-v monotonic -s UEFS", "package = " + PackageName, 2500, LastTimeValue);
//			stopwatch.Stop();
//			Console.WriteLine("Time: " + (double)stopwatch.ElapsedMilliseconds / 1000.0);
			return retval;
		}

		public static bool StopAnyServers(string Device, int Port, bool bWaitForStop = true)
		{
			// get a list of installed receivers and verify package is available first
			List<string> InstalledReceivers = GetInstalledReceivers(Device);

			// deal with stopping all client servers if requested
			bool bDidSendStops = false;
			foreach (string Receiver in InstalledReceivers)
			{
				bDidSendStops = true;
				adb.Shell(Device, "am broadcast -a com.epicgames.unreal.RemoteFileManager.intent.COMMAND -n " + Receiver + "/com.epicgames.unreal.RemoteFileManagerReceiver -e cmd stop");
			}

			// get a list of installed activities and verify package is available first
			List<string> InstalledActivities = GetInstalledActivities(Device);
			foreach (string Activity in InstalledActivities)
			{
				bDidSendStops = true;
				ActivityManagerSendAndWait(Device, Activity, "-e cmd stop");
			}

			// it can take up to 2 seconds for running servers to terminate so check if any binds to port are still active
			if (bDidSendStops && bWaitForStop)
			{
				// we will try ss first, but it may not be available
				bool bUSB;
				bool bWifi;
				string WifiAddress;

				if (GetListenStatus(Device, Port, out bUSB, out bWifi, out WifiAddress))
				{
					// wait for both to terminate (up to 10 seconds, then give up)
					long StartTime = DateTimeOffset.Now.ToUnixTimeSeconds();
					while (bUSB || bWifi)
					{
						GetListenStatus(Device, Port, out bUSB, out bWifi, out WifiAddress);
						if (DateTimeOffset.Now.ToUnixTimeSeconds() - StartTime > 9)
						{
							// there are still binds active that won't terminate
							return false;
						}
						Thread.Sleep(100);
					}
				}
				else
				{
					// command not available so will just need to wait 2 seconds to be sure
					Thread.Sleep(2000);
				}
			}

			return true;
		}

		public bool StartServer(string PackageName, string Token = "", string IPAddress = "127.0.0.1", bool bStopAnyServers = true)
		{
			// already connected?
			if (ClientSocket != null)
			{
				return true;
			}

			bool bIsUSB = (IPAddress == null || IPAddress == "" || IPAddress == "127.0.0.1");

			// deal with stopping all servers if requested
			if (bStopAnyServers)
			{
				if (!StopAnyServers(Device, ServerPort, true))
				{
					// there are still binds active that won't terminate so can't start a new one
					return false;
				}
			}

			/*
			// get a list of installed clients and verify package is available (up to 6 seconds)
			int tries = 60;
			List<string> InstalledReceivers = GetInstalledReceivers(Device);
			if (!InstalledReceivers.Contains(PackageName))
			{
				// if this was called right after installed the receiver may not be registered yet, wait and try again
				bool bFound = false;
				while (tries-- > 0)
				{
					Thread.Sleep(100);

					InstalledReceivers = GetInstalledReceivers(Device);
					if (InstalledReceivers.Contains(PackageName))
					{
						bFound = true;
						break;
					}
				}
				if (!bFound)
				{
					Logger.LogInformation("Did not find package with receiver");
					return false;
				}
			}
			*/

			// get a list of installed clients and verify package is available (up to 6 seconds)
			int tries = 60;
			List<string> InstalledActivities = GetInstalledActivities(Device);
			if (!InstalledActivities.Contains(PackageName))
			{
				// if this was called right after installed the activity may not be registered yet, wait and try again
				bool bFound = false;
				while (tries-- > 0)
				{
					Thread.Sleep(100);

					InstalledActivities = GetInstalledActivities(Device);
					if (InstalledActivities.Contains(PackageName))
					{
						bFound = true;
						break;
					}
				}
				if (!bFound)
				{
					Logger.LogInformation("Did not find package with activity");
					return false;
				}
			}

			// try to retrieve token if not provided
			if (Token == "")
			{
				// try asking AFS to write the file; we do this instead of just checking for file since an update could have changed the token
				ActivityManagerSendAndWait(Device, PackageName, "-e cmd init");

				// this will only work if debuggable - try to retrieve the file contents
				String CatResult = adb.Shell(Device, "run-as " + PackageName+" cat files/afs.cache");
				if (CatResult.Contains("not debuggable"))
				{
					Logger.LogInformation("Package is not debuggable; assuming token not required.");
				}
				else if (!CatResult.Contains("No such file or directory"))
				{
					Token = CatResult;
				}
				else
				{
					Logger.LogInformation("Did not retrieve a token; assuming not required.");
				}
			}

			// retries up to 8 seconds to start and see listener ready
			tries = 40;
			bool bUSB;
			bool bWifi;
			string WifiAddress;
			
			// only include -e Token if non-empty
			String StartRequestParams = "-e cmd start" + (Token == "" ? "" : " -e token " + Token) + " -ei port " + ServerPort;

			// sent start request (won't do anything if already started)
			ActivityManagerSendAndWait(Device, PackageName, StartRequestParams);

			// see if we can check listen status
			if (GetListenStatus(Device, ServerPort, out bUSB, out bWifi, out WifiAddress))
			{
				while (tries-- > 0)
				{
					// try to connect if requested server is listening
					if ((bUSB && bIsUSB) || (bWifi && !bIsUSB))
					{
						if (OpenConnection(IPAddress))
						{
							// remember if needed to reestablish connection for retries
							ActivePackageName = PackageName;
							ActiveToken = Token;
							ActiveIPAddress = IPAddress;

							return true;
						}
					}

					// wait before trying to again
					Thread.Sleep(200);

					// sent start request again (won't do anything if already started)
					ActivityManagerSendAndWait(Device, PackageName, StartRequestParams);

					GetListenStatus(Device, ServerPort, out bUSB, out bWifi, out WifiAddress);
				}

				Logger.LogInformation("Did not find a bind listener");
				return false;
			}

			// try 5 times, this is faster than looking response in logcat
			while (tries-- > 0)
			{
				// wait before trying to connect
				Thread.Sleep(250);

				if (OpenConnection(IPAddress))
				{
					// remember if needed to reestablish connection for retries
					ActivePackageName = PackageName;
					ActiveToken = Token;
					ActiveIPAddress = IPAddress;

					return true;
				}

				// sent start request again (won't do anything if already started)
				ActivityManagerSendAndWait(Device, PackageName, StartRequestParams);
			}

			Logger.LogInformation("Timed out on connection attempts");
			return false;
		}

		public void TerminateServer()
		{
			// try to connect if not already connected
			if (ClientSocket == null)
			{
				if (!OpenConnection())
				{
					return;
				}
			}

			if (ClientSocket != null)
			{
				try
				{
					int bytesSent = SocketSend(false, CommandPacket(Command_Terminate, 0));

					// close socket
					ClientSocket.Shutdown(SocketShutdown.Both);
					ClientSocket.Close();
					ClientSocket.Dispose();
					ClientSocket = null;
				}
				catch (SocketException se)
				{
					Logger.LogWarning("SocketException: {Arg0}", se.ToString());
				}
				catch (ObjectDisposedException)
				{
					// allow it
				}
				catch (Exception e)
				{
					Logger.LogError("Unexpected Exception: {Arg0}", e.ToString());
				}
			}
		}

		public void CloseConnection()
		{
			Stats_Stopwatch = null;
			if (ClientSocket != null)
			{
				if (!ClientSocket.Connected)
				{
					return;
				}

				try
				{
					int bytesSent = SocketSend(false, CommandPacket(Command_Close, 0));

					// close socket
					ClientSocket.Shutdown(SocketShutdown.Both);
					ClientSocket.Close();
					ClientSocket.Dispose();
					ClientSocket = null;
				}
				catch (SocketException se)
				{
					Logger.LogWarning("SocketException: {Arg0}", se.ToString());
				}
				catch (ObjectDisposedException)
				{
					// allow it
				}
				catch (Exception e)
				{
					Logger.LogError("Unexpected Exception: {Arg0}", e.ToString());
				}
			}

			if (HostPort != -1)
			{
				ForwardPort_Remove(Device, HostPort);
				HostPort = -1;
			}
		}

		public string Query(string Filepath, bool bStripEndingSlash = false)
		{
			string result = GetStringResult(Command_Query, Filepath);
			if (result != null && bStripEndingSlash && result.EndsWith("/"))
			{
				return result.Substring(0, result.Length - 1);
			}
			return result;
		}

		public string GetProp(string Filepath)
		{
			return GetStringResult(Command_GetProp, Filepath);
		}

		public bool SetBaseDir(string Filepath)
		{
			if (Filepath.EndsWith("/"))
			{
				Filepath = Filepath.Substring(0, Filepath.Length - 1);
			}
			LastBaseDir = Filepath;
			return GetBoolResult(Command_SetBaseDir, Filepath);
		}

		public bool DirExists(string Filepath)
		{
			return GetBoolResult(Command_DirExists, Filepath);
		}

		public string DirList(string Filepath)
		{
			return GetStringResult(Command_DirList, Filepath);
		}

		public string DirListFlat(string Filepath)
		{
			return GetStringResult(Command_DirListFlat, Filepath);
		}

		public bool DirCreate(string Filepath)
		{
			return GetBoolResult(Command_DirCreate, Filepath);
		}

		public bool DirDelete(string Filepath)
		{
			return GetBoolResult(Command_DirDelete, Filepath);
		}

		public bool DirDeleteRecurse(string Filepath)
		{
			return GetBoolResult(Command_DirDeleteRecurse, Filepath);
		}

		public bool FileExists(string Filepath)
		{
			return GetBoolResult(Command_FileExists, Filepath);
		}

		public bool FileDelete(string Filepath)
		{
			return GetBoolResult(Command_FileDelete, Filepath);
		}

		public bool FileCopy(string SourcePath, string DestPath)
		{
			return GetBoolResult(Command_FileCopy, SourcePath + "\0" + DestPath);
		}

		public bool FileMove(string SourcePath, string DestPath)
		{
			return GetBoolResult(Command_FileMove, SourcePath + "\0" + DestPath);
		}

		public bool FileRead(string SourcePath, string DestPath)
		{
			Boolean Result = false;

			Logger.LogInformation("FileRead {SourcePath}", SourcePath);

			if (ClientSocket == null)
			{
				return Result;
			}

			if (File.Exists(DestPath))
			{
				File.Delete(DestPath);
			}

			try
			{
				// deal with LastBaseDir substitution
				SourcePath = OptimizePath(SourcePath);

				byte[] paramMessage = Encoding.UTF8.GetBytes(SourcePath + "\0");
				long paramSize = paramMessage.Length;
				byte[] commandMessage = CommandPacket(Command_FileRead, paramSize);

				int bytesSent = SocketSend(false, commandMessage);
				bytesSent = SocketSend(false, paramMessage);

				long resultSize = ResultSize();
				long startSize = resultSize;
				if (resultSize >= 0)
				{
					int bytesRead;
					byte[] buffer = new byte[FILE_BUFFERSIZE];

					Stats_FilesRead++;

					try
					{
						using (FileStream fileStream = File.Open(DestPath, FileMode.Create, FileAccess.Write))
						using (BufferedStream bufferedStream = new BufferedStream(fileStream))
						{
							long remaining = resultSize;
							while (remaining > 0)
							{
								int readSize = remaining > FILE_BUFFERSIZE ? FILE_BUFFERSIZE : (int)remaining;
								bytesRead = SocketReceive(true, buffer, readSize);
								remaining -= bytesRead;
								bufferedStream.Write(buffer, 0, bytesRead);
								Stats_CompressedBytesReceived += bytesRead;
							}
							bufferedStream.Close();
							fileStream.Close();
						}
					}
					catch (Exception e)
					{
						Logger.LogError("Unexpected Exception: {Arg0}", e.ToString());
						// ignore received data
						long remainingSkip = resultSize;
						while (remainingSkip > 0)
						{
							int readSize = remainingSkip > FILE_BUFFERSIZE ? FILE_BUFFERSIZE : (int)remainingSkip;
							bytesRead = SocketReceive(true, buffer, readSize);
							remainingSkip -= bytesRead;
						}
						return Result;
					}

					Result = true;
				}
			}
			catch (Exception e)
			{
				Logger.LogError("Unexpected Exception: {Arg0}", e.ToString());
				CloseConnection();
			}
			return Result;
		}

		public string FileReadString(string SourcePath)
		{
			if (ClientSocket == null)
			{
				return null;
			}

			try
			{
				// deal with LastBaseDir substitution
				SourcePath = OptimizePath(SourcePath);

				{
					byte[] paramMessage = Encoding.UTF8.GetBytes(SourcePath + "\0");
					long paramSize = paramMessage.Length;
					byte[] commandMessage = CommandPacket(Command_FileRead, paramSize);

					SocketSend(false, commandMessage);
					SocketSend(false, paramMessage);
				}

				long resultSize = ResultSize();
				if (resultSize >= 0)
				{
					byte[] buffer = new byte[FILE_BUFFERSIZE];

					Stats_FilesRead++;

					try
					{
						MemoryStream stringStream = new MemoryStream();
						
						for (long remaining = resultSize; remaining > 0;)
						{
							int readSize = remaining > FILE_BUFFERSIZE ? FILE_BUFFERSIZE : (int)remaining;
							int bytesRead = SocketReceive(true, buffer, readSize);
							remaining -= bytesRead;
							stringStream.Write(buffer, 0, bytesRead);
							Stats_CompressedBytesReceived += bytesRead;
						}

						stringStream.Position = 0;
						return new StreamReader(stringStream).ReadToEnd();
					}
					catch (Exception e)
					{
						Logger.LogError("Unexpected Exception: {Arg0}", e.ToString());
						
						// ignore received data
						for (long remainingSkip = resultSize; remainingSkip > 0;)
						{
							int readSize = remainingSkip > FILE_BUFFERSIZE ? FILE_BUFFERSIZE : (int)remainingSkip;
							int bytesRead = SocketReceive(true, buffer, readSize);
							remainingSkip -= bytesRead;
						}
					}
				}
			}
			catch (Exception e)
			{
				Logger.LogError("Unexpected Exception: {Arg0}", e.ToString());
				
				CloseConnection();
			}
			
			return null;
		}
		
		private const int RetriesMax = 5;
		private const int RetryMillisecondsDelay = 5000;

		public bool FileWrite(string SourcePath, string DestPath, int bLog = 0)
		{
			if (ClientSocket == null)
			{
				return false;
			}

			// with no network errors, return the result of filesystem IO process
			// with network error, try reconnect
			// with reconnect error, retry reconnect
			// with reconnect success, try filesystem IO process
			int TryNum = 0;
			bool HasNetError = false;
			while (true)
			{
				if (!HasNetError)
				{
					(bool Result, HasNetError) = FileWriteInternal(SourcePath, DestPath, bLog);
					if (!HasNetError)
					{
						return Result;
					}
				}

				if (TryNum++ == RetriesMax)
				{
					Logger.LogInformation($"Too many retries writing {DestPath}; giving up");
					break;
				}

				Logger.LogInformation($"Retry {TryNum}/{RetriesMax} writing file {DestPath} because of network error");
				if (!ClientSocket.Connected)
				{
					try
					{
						ClientSocket.Close();
						ClientSocket.Dispose();
					}
					catch (Exception)
					{
					}
					ClientSocket = null;
				}
				Thread.Sleep(RetryMillisecondsDelay);

				bool bUSB;
				bool bWifi;
				string WifiAddress;
				if (GetListenStatus(Device, ServerPort, out bUSB, out bWifi, out WifiAddress))
				{
					bool bIsUSB = (ActiveIPAddress == null || ActiveIPAddress == "" || ActiveIPAddress == "127.0.0.1");
					if ((bUSB && bIsUSB) || (bWifi && !bIsUSB))
					{
						HasNetError = !OpenConnection();
						continue;
					}
					HasNetError = !StartServer(ActivePackageName, ActiveToken, ActiveIPAddress);
					continue;
				}
				HasNetError = !OpenConnection();
			}

			return false;
		}

		private (bool Result, bool HasNetworkError) FileWriteInternal_Normal(string SourcePath, string DestPath, int bLog = 0)
		{
			bool Result = false;
			bool HasNetError = false;
			try
			{
				using (FileStream fileStream = new FileStream(SourcePath, FileMode.Open, FileAccess.Read, FileShare.None, 4096, FileOptions.SequentialScan))
				{
					// deal with LastBaseDir substitution
					DestPath = OptimizePath(DestPath);
					if (bLog > 0)
					{
						Logger.LogInformation("{bLog}> Writing {DestPath}", bLog, DestPath);
					}

					long fileSize = fileStream.Length;
					Stats_FilesWrite++;

					int bytesSent;
					int bytesRead;
					long remaining = fileSize;

					byte[] paramMessage = Encoding.UTF8.GetBytes(DestPath + "\0");
					int headerSize = paramMessage.Length;
					long paramSize = headerSize + fileSize;

					if (BatchBuffer != null)
					{
						byte[] commandMessage = CommandPacket(Command_FileWrite, paramSize);
						bytesSent = SocketSend(false, commandMessage);
						bytesSent = SocketSend(false, paramMessage);

						// directly read into batch buffer for minimal copies
						while (remaining > 0)
						{
							int batchRemaining = BATCH_BUFFERSIZE - BatchBufferIndex;
							int chunkSize = (int)(BatchBufferIndex + remaining > BATCH_BUFFERSIZE ? BATCH_BUFFERSIZE - BatchBufferIndex : remaining);
							bytesRead = fileStream.Read(BatchBuffer, BatchBufferIndex, chunkSize);

							// need to do this here since SocketSend not called
							if (RecordStream != null)
							{
								RecordStream.Write(BatchBuffer, BatchBufferIndex, bytesRead);
							}

							Stats_CompressedBytesSent += bytesRead;
							Stats_TotalBytesSent += bytesRead;
							Stats_PayloadBytesSent += bytesRead;

							BatchBufferIndex += bytesRead;
							remaining -= bytesRead;

							if (BatchBufferIndex == BATCH_BUFFERSIZE)
							{
								Batch_Flush();
							}
						}
					}
					else
					{
						// legacy file write support
						// reduce the number of SocketSends if not batching
						byte[] commandMessage = new byte[8 + headerSize];
						commandMessage[0] = (byte)(Command_FileWrite & 255);
						commandMessage[1] = (byte)((Command_FileWrite >> 8) & 255);
						commandMessage[2] = (byte)(paramSize & 255);
						commandMessage[3] = (byte)((paramSize >> 8) & 255);
						commandMessage[4] = (byte)((paramSize >> 16) & 255);
						commandMessage[5] = (byte)((paramSize >> 24) & 255);
						commandMessage[6] = (byte)((paramSize >> 32) & 255);
						commandMessage[7] = (byte)((paramSize >> 40) & 255);
						Buffer.BlockCopy(paramMessage, 0, commandMessage, 8, headerSize);
						bytesSent = SocketSend(false, commandMessage);

						byte[] buffer = new byte[FILE_BUFFERSIZE];

						while ((bytesRead = fileStream.Read(buffer, 0, FILE_BUFFERSIZE)) != 0)
						{
							bytesSent = SocketSend(true, buffer, bytesRead);
							Stats_CompressedBytesSent += bytesSent;
						}
					}
					fileStream.Close();
				}
				Result = true;
			}
			catch (IOException)
			{
				// file not found
			}
			catch (SocketException e)
			{
				HasNetError = true;
				Logger.LogWarning("{bLog}> Network Exception: {Arg1}", bLog, e.ToString());
				CloseConnection();
			}
			catch (Exception e)
			{
				Logger.LogError("{bLog}> Unexpected Exception: {Arg1}", bLog, e.ToString());
				CloseConnection();
			}
			return (Result, HasNetError);
		}

		private (bool Result, bool HasNetworkError) FileWriteInternal(string SourcePath, string DestPath, int bLog = 0)
		{
			if (BatchBuffer != null || ActiveVersion < ServerVersion_Handshake)
			{
				return FileWriteInternal_Normal(SourcePath, DestPath, bLog);
			}

			bool Result = false;
			bool HasNetError = false;
			try
			{
				using (FileStream fileStream = new FileStream(SourcePath, FileMode.Open, FileAccess.Read, FileShare.None, 4096, FileOptions.SequentialScan))
				{
					// deal with LastBaseDir substitution
					DestPath = OptimizePath(DestPath);

					long fileSize = fileStream.Length;
					if (fileSize <= FILE_BUFFERSIZE)
					{
						// just start over as full transfer if smaller file
						fileStream.Close();
						return FileWriteInternal_Normal(SourcePath, DestPath, bLog);
					}

					if (bLog > 0)
					{
						Logger.LogInformation("{bLog}> Writing {DestPath}", bLog, DestPath);
					}

					// get timestamp of file to use for validating resume
					long Timestamp = File.GetLastWriteTimeUtc(SourcePath).ToBinary();

					int BytesRead;
					int BytesSent = 0;
					byte[] buffer = new byte[FILE_BUFFERSIZE];

					// send file size and timestamp
					byte[] paramMessage = Encoding.UTF8.GetBytes(DestPath + "\0");
					int headerSize = paramMessage.Length;
					long paramSize = headerSize + 8 + fileSize;

					// reduce the number of SocketSends
					byte[] commandMessage = new byte[8 + 8 + headerSize];
					commandMessage[0] = (byte)(Command_FileWriteHandshake & 255);
					commandMessage[1] = (byte)((Command_FileWriteHandshake >> 8) & 255);
					commandMessage[2] = (byte)(paramSize & 255);
					commandMessage[3] = (byte)((paramSize >> 8) & 255);
					commandMessage[4] = (byte)((paramSize >> 16) & 255);
					commandMessage[5] = (byte)((paramSize >> 24) & 255);
					commandMessage[6] = (byte)((paramSize >> 32) & 255);
					commandMessage[7] = (byte)((paramSize >> 40) & 255);
					commandMessage[8] = (byte)(Timestamp & 255);
					commandMessage[9] = (byte)((Timestamp >> 8) & 255);
					commandMessage[10] = (byte)((Timestamp >> 16) & 255);
					commandMessage[11] = (byte)((Timestamp >> 24) & 255);
					commandMessage[12] = (byte)((Timestamp >> 32) & 255);
					commandMessage[13] = (byte)((Timestamp >> 40) & 255);
					commandMessage[14] = (byte)((Timestamp >> 48) & 255);
					commandMessage[15] = (byte)((Timestamp >> 56) & 255);
					Buffer.BlockCopy(paramMessage, 0, commandMessage, 16, headerSize);
					int bytesSent = SocketSend(false, commandMessage);

					// wait for return (error, full file, or place to resume transfer)
					long resultSize = ResultSize();
					if (resultSize == -1)
					{
						// error with file
						Result = false;
					}
					else if (resultSize == 0)
					{
						// send full file
						Stats_FilesWrite++;
						while ((BytesRead = fileStream.Read(buffer, 0, FILE_BUFFERSIZE)) != 0)
						{
							BytesSent = SocketSend(true, buffer, BytesRead);
							Stats_CompressedBytesSent += BytesSent;
						}
						fileStream.Close();
						Result = GetStringResult() == "true";
					}
					else
					{
						// may already be the same size
						if (resultSize < fileSize)
						{
							// send from end of file on device (append)
							Stats_FilesWrite++;
							fileStream.Seek(resultSize, SeekOrigin.Begin);
							while ((BytesRead = fileStream.Read(buffer, 0, FILE_BUFFERSIZE)) != 0)
							{
								BytesSent = SocketSend(true, buffer, BytesRead);
								Stats_CompressedBytesSent += BytesSent;
							}
						}
						fileStream.Close();
						Result = GetStringResult() == "true";
					}
				}
			}
			catch (IOException)
			{
				// file not found
			}
			catch (SocketException e)
			{
				HasNetError = true;
				Logger.LogWarning("{bLog}> Network Exception: {Arg1}", bLog, e.ToString());
				CloseConnection();
			}
			catch (Exception e)
			{
				Logger.LogError("{bLog}> Unexpected Exception: {Arg1}", bLog, e.ToString());
				CloseConnection();
			}
			return (Result, HasNetError);
		}

		public bool FileWriteCompressed(string SourcePath, string DestPath, int bLog = 0)
		{
			Boolean Result = false;

			if (ClientSocket == null)
			{
				return Result;
			}

			try
			{
				using (FileStream fileStream = File.Open(SourcePath, FileMode.Open, FileAccess.Read))
				{
					long fileSize = fileStream.Length;
					if (fileSize < 1024)
					{
						fileStream.Close();
						return FileWrite(SourcePath, DestPath, bLog);
					}

					// deal with LastBaseDir substitution
					DestPath = OptimizePath(DestPath);
					if (bLog > 0)
					{
						Logger.LogInformation("{bLog}> Writing Compressed {DestPath}", bLog, DestPath);
					}

					Stats_FilesWrite++;
					int COMPRESS_BUFFERSIZE = 1024 * 1024;

					int bytesSent;
					int bytesRead;
					long remaining = fileSize;

					byte[] paramMessage = Encoding.UTF8.GetBytes("Z" + DestPath + "\0");
					int headerSize = paramMessage.Length;
					long paramSize = headerSize + fileSize;
					if (BatchBuffer != null)
					{
						byte[] commandMessage = CommandPacket(Command_FileWriteCompressed, paramSize);
						bytesSent = SocketSend(false, commandMessage);
						bytesSent = SocketSend(false, paramMessage);
					}
					else
					{
						// reduce the number of SocketSends if not batching
						byte[] commandMessage = new byte[8 + headerSize];
						commandMessage[0] = (byte)(Command_FileWriteCompressed & 255);
						commandMessage[1] = (byte)((Command_FileWriteCompressed >> 8) & 255);
						commandMessage[2] = (byte)(paramSize & 255);
						commandMessage[3] = (byte)((paramSize >> 8) & 255);
						commandMessage[4] = (byte)((paramSize >> 16) & 255);
						commandMessage[5] = (byte)((paramSize >> 24) & 255);
						commandMessage[6] = (byte)((paramSize >> 32) & 255);
						commandMessage[7] = (byte)((paramSize >> 40) & 255);
						Buffer.BlockCopy(paramMessage, 0, commandMessage, 8, headerSize);
						bytesSent = SocketSend(false, commandMessage);
					}

					byte[] buffer = new byte[COMPRESS_BUFFERSIZE + 3];
					byte[] header = new byte[3];

					// first part of buffer contains zero which flags COMPRESS_BUFFERSIZE uncompressed
					buffer[0] = 0;
					buffer[1] = 0;
					buffer[2] = 0;

					while ((bytesRead = fileStream.Read(buffer, 3, COMPRESS_BUFFERSIZE)) != 0)
					{
						Stats_CompressedBytesSent += bytesRead;

						var compressedStream = new MemoryStream();
						var zipStream = new GZipStream(compressedStream, CompressionLevel.Fastest);
						zipStream.Write(buffer, 3, bytesRead);
						zipStream.Close();
						byte[] compressedArray = compressedStream.ToArray();
						int compsize = compressedArray.Length;

						if (compsize < COMPRESS_BUFFERSIZE)
						{
							if (BatchBuffer != null)
							{
								// batching will do a block copy so better to do two calls
								header[0] = (byte)(compsize & 255);
								header[1] = (byte)((compsize >> 8) & 255);
								header[2] = (byte)((compsize >> 16) & 255);
								bytesSent = SocketSend(true, header);
								bytesSent = SocketSend(true, compressedArray);
							}
							else
							{
								// more efficient to send one packet
								byte[] compPacket = new byte[compsize + 3];
								compPacket[0] = (byte)(compsize & 255);
								compPacket[1] = (byte)((compsize >> 8) & 255);
								compPacket[2] = (byte)((compsize >> 16) & 255);
								Buffer.BlockCopy(compressedArray, 0, compPacket, 3, compsize);
								bytesSent = SocketSend(true, compPacket);
							}
						}
						else
						{
							bytesSent = SocketSend(true, buffer);
						}
					}
					fileStream.Close();

					/*
					int resultSize = (int)ResultSize();
					if (resultSize > 0)
					{
						byte[] message = new byte[resultSize];
						int bytesRecv = SocketReceive(false, message);

						string Outcome = Encoding.UTF8.GetString(message, 0, bytesRecv);
						Result = Outcome == "true";
					}
					*/
					Result = true;
				}
			}
			catch (IOException)
			{
				// file not found
				Result = false;
			}
			catch (Exception e)
			{
				Logger.LogError("{bLog}> Unexpected Exception: {Arg1}", bLog, e.ToString());
				CloseConnection();
			}
			return Result;
		}

		public bool FileWriteString(string Contents, string DestPath)
		{
			Boolean Result = false;

			if (ClientSocket == null)
			{
				return Result;
			}

			try
			{
				// deal with LastBaseDir substitution
				DestPath = OptimizePath(DestPath);

				byte[] paramMessage = Encoding.UTF8.GetBytes(DestPath + "\0");
				byte[] contentsMessage = Encoding.UTF8.GetBytes(Contents);
				long paramSize = paramMessage.Length + contentsMessage.Length;
				byte[] commandMessage = CommandPacket(Command_FileWrite, paramSize);

				Stats_FilesWrite++;

				int bytesSent = SocketSend(false, commandMessage);
				bytesSent = SocketSend(false, paramMessage);

				Stats_CompressedBytesSent += bytesSent;

				bytesSent = SocketSend(true, contentsMessage);

				/*				int resultSize = (int)ResultSize();
								if (resultSize > 0)
								{
									byte[] message = new byte[resultSize];
									int bytesRecv = SocketReceive(false, message);

									string Outcome = Encoding.UTF8.GetString(message, 0, bytesRecv);
									Result = Outcome == "true";
								}*/
				Result = true;
			}
			catch (Exception e)
			{
				Logger.LogError("Unexpected Exception: {Arg0}", e.ToString());
				CloseConnection();
			}
			return Result;
		}

		public bool PushFile(string InSource, string InDest, bool bCompress = false, int bLog = 0)
		{
			InSource = InSource.Replace("\\", "/");
			InDest = InDest.Replace("\\", "/");

			bool Result = false;
			if (bCompress)
			{
				Result = FileWriteCompressed(InSource, InDest, bLog);
			}
			else
			{
				Result = FileWrite(InSource, InDest, bLog);
			}
			if (!Result && bLog > 0)
			{
				Logger.LogInformation("{bLog}> Failed to copy {InSource} to {InDest}", bLog, InSource, InDest);
				return false;
			}

			/*
			if (bLog > 0)
			{
				Logger.LogInformation("{bLog}> Copied: {InSource}", bLog, InSource);
			}
			*/
			return true;
		}

		public bool PushDirectory(string InSource, string InDest, bool bCompress = false, int bLog = 0)
		{
			if (Directory.Exists(InSource))
			{
				foreach (string File in Directory.EnumerateFiles(InSource))
				{
					if (!PushFile(File, File.Replace(InSource, InDest), bCompress, bLog))
					{
						return false;
					}
				}
			}
			return true;
		}

		public bool PushDirectories(string InSource, string InDest, bool bCompress = false, int bLog = 0)
		{
			if (Directory.Exists(InSource))
			{
				foreach (string File in Directory.EnumerateFiles(InSource))
				{
					if (!PushFile(File, File.Replace(InSource, InDest), bCompress, bLog))
					{
						return false;
					}
				}

				foreach (string Directory in Directory.EnumerateDirectories(InSource))
				{
					if (!PushDirectories(Directory, Directory.Replace(InSource, InDest), bCompress, bLog))
					{
						return false;
					}
				}
			}
			return true;
		}

		private void AddDeployDirectories(ref List<KeyValuePair<string, string>> USBClientFiles, ref List<KeyValuePair<string, string>> WiFiClientFiles, ref bool bAlternate, string InSource, string InDest)
		{
			if (Directory.Exists(InSource))
			{
				foreach (string File in Directory.EnumerateFiles(InSource))
				{
					if (bAlternate)
					{
						WiFiClientFiles.Add(new KeyValuePair<string, string>(File, File.Replace(InSource, InDest)));
					}
					else
					{
						USBClientFiles.Add(new KeyValuePair<string, string>(File, File.Replace(InSource, InDest)));

					}
					bAlternate = !bAlternate;
				}

				foreach (string Directory in Directory.EnumerateDirectories(InSource))
				{
					AddDeployDirectories(ref USBClientFiles, ref WiFiClientFiles, ref bAlternate, Directory, Directory.Replace(InSource, InDest));
				}
			}
		}

		private bool bDeployThreadInterruptRequested = false;
		private bool bDeployThreadError = false;

		private void DeployThread(AndroidFileClient Client, ref List<KeyValuePair<string, string>> Files, bool bCompress, int bLog)
		{
			// Push all the WiFi client files
			foreach (KeyValuePair<string, string> Entry in Files)
			{
				if (!Client.PushFile(Entry.Key, Entry.Value, bCompress, bLog))
				{
					Volatile.Write(ref bDeployThreadError, true);
					break;
				}
				if (Volatile.Read(ref bDeployThreadInterruptRequested))
				{
					break;
				}
			}
		}

		private void AddSourceDirectories(ref List<string> Directories, string InSource, string InDest)
		{
			if (Directory.Exists(InSource))
			{
				foreach (string Directory in Directory.EnumerateDirectories(InSource))
				{
					AddSourceDirectories(ref Directories, Directory, Directory.Replace(InSource, InDest));
				}
				string Entry = InDest.Replace("\\", "/");
				int LastIndex = Directories.Count - 1;
				if (LastIndex < 0 || !Directories[LastIndex].StartsWith(Entry))
				{
					Directories.Add(Entry);
				}
			}
		}

		public bool Deploy(HashSet<string> EntriesToDeploy, string SourceDir, string DestDir, bool bCompress = false, bool bLog = false, bool bReportStats = true, AndroidFileClient WiFiClient = null)
		{
			bool Result = true;
			int logId = bLog ? 1 : 0;

			Stopwatch totalwatch = System.Diagnostics.Stopwatch.StartNew();
			Stopwatch stopwatch = System.Diagnostics.Stopwatch.StartNew();

			// pre-create the directories (send only deepest)
			List<string> Directories = new List<string>();
			foreach (string Entry in EntriesToDeploy)
			{
				string RemotePath = Entry.Replace(SourceDir, DestDir).Replace("\\", "/");

				FileAttributes attributes = File.GetAttributes(Entry);
				if ((attributes & FileAttributes.Directory) == FileAttributes.Directory)
				{
					AddSourceDirectories(ref Directories, Entry, RemotePath);
				}
				else
				{
					Directories.Add(Path.GetDirectoryName(RemotePath).Replace("\\", "/"));
				}
			}
			foreach (string Entry in Directories)
			{
				DirCreate(Entry);
			}

			stopwatch.Stop();
			var ElapsedMs = stopwatch.ElapsedMilliseconds;
			if (bReportStats)
			{
				Logger.LogInformation("Time to create directories:  {0:N3}s", (float)(ElapsedMs / 1000.0));
			}

			Stats_Clear();
			Batch_Start();

			stopwatch.Restart();

			if (WiFiClient != null)
			{
				WiFiClient.Stats_Clear(false);
				WiFiClient.Batch_Start();

				// collect all the files into two lists, one for each client
				List<KeyValuePair<string, string>> USBClientFiles = new List<KeyValuePair<string, string>>();
				List<KeyValuePair<string, string>> WiFiClientFiles = new List<KeyValuePair<string, string>>();
				bool bAlternate = false;

				foreach (string Entry in EntriesToDeploy)
				{
					string RemotePath = Entry.Replace(SourceDir, DestDir).Replace("\\", "/");

					FileAttributes attributes = File.GetAttributes(Entry);
					if ((attributes & FileAttributes.Directory) == FileAttributes.Directory)
					{
						AddDeployDirectories(ref USBClientFiles, ref WiFiClientFiles, ref bAlternate, Entry, RemotePath);
					}
					else
					{
						if (bAlternate)
						{
							WiFiClientFiles.Add(new KeyValuePair<string, string>(Entry, RemotePath));
						}
						else
						{
							USBClientFiles.Add(new KeyValuePair<string, string>(Entry, RemotePath));

						}
						bAlternate = !bAlternate;
					}
				}

				Thread WiFiDeployThread = new Thread(() => DeployThread(WiFiClient, ref WiFiClientFiles, bCompress, bLog ? 2 : 0))
				{
					Name = "WiFiDeployThread"
				};

				bDeployThreadInterruptRequested = false;
				bDeployThreadError = false;

				WiFiDeployThread.Start();

				// Push all the USB client files
				foreach (KeyValuePair<string, string> Entry in USBClientFiles)
				{
					if (!PushFile(Entry.Key, Entry.Value, bCompress, logId))
					{
						Volatile.Write(ref bDeployThreadInterruptRequested, true);
						Result = false;
						break;
					}
					if (Volatile.Read(ref bDeployThreadError))
						if (bDeployThreadError)
						{
							Result = false;
							break;
						}
				}

				// Wait for WiFi thread to finish
				WiFiDeployThread.Join();

				WiFiClient.Batch_Stop();
				Stats_Combine(WiFiClient);
			}
			else
			{
				// push all files on USB
				foreach (string Entry in EntriesToDeploy)
				{
					string RemotePath = Entry.Replace(SourceDir, DestDir).Replace("\\", "/");

					FileAttributes attributes = File.GetAttributes(Entry);
					if ((attributes & FileAttributes.Directory) == FileAttributes.Directory)
					{
						if (!PushDirectories(Entry, RemotePath, bCompress, logId))
						{
							Result = false;
							break;
						}
					}
					else
					{
						if (!PushFile(Entry, RemotePath, bCompress, logId))
						{
							Result = false;
							break;
						}
					}
				}
			}

			// time to batch (if buffer wasn't flushed)
			stopwatch.Stop();

			// this code will be removed if TimeBatching is false but will give an unreachable code warning
			#pragma warning disable
			if (TimeBatching)
			{
				if (bReportStats)
				{
					ElapsedMs = stopwatch.ElapsedMilliseconds;
					Logger.LogInformation("Time to build batch:  {0:N3}s", (float)(ElapsedMs / 1000.0));
				}
				Stats_Stopwatch.Stop();
				Stats_StartStopwatch();
			}
			#pragma warning restore

			// stop and flush batch buffer
			Batch_Stop();

			// this causes a sync so totalwatch is valid
			Query("^^");
			totalwatch.Stop();

			if (bReportStats)
			{
				Stats_Report();

				ElapsedMs = totalwatch.ElapsedMilliseconds;
				Logger.LogInformation("Total time to Deploy AFS:  {0:N3}s", (float)(ElapsedMs / 1000.0));
			}

			return Result;
		}

		/*
		public void TestOptimizePath()
		{
			Logger.LogInformation("{Arg0}", OptimizePath("/storage/emulated/0/Android/data/com.epicgames.SaveTest/files/UE4Game/SaveTest/Manifest_DebugFiles_Android.txt"));
			Logger.LogInformation("{Arg0}", OptimizePath("/storage/emulated/0/Android/data/com.epicgames.SaveTest/files/UE4Game/SaveTest/DataDrivenPlatformInfo.ini"));
			Logger.LogInformation("{Arg0}", OptimizePath("/storage/emulated/0/Android/data/com.epicgames.SaveTest/files/UE4Game/SaveTest/Engine/Config/Layouts/DefaultLayout.ini"));
			Logger.LogInformation("{Arg0}", OptimizePath("/storage/emulated/0/Android/data/com.epicgames.SaveTest/files/UE4Game/SaveTest/Engine/Content/ArtTools/RenderToTexture/Materials/Debug/M_Emissive_Color.uasset"));

			Logger.LogInformation("{Arg0}", OptimizePath("/storage/0/emulated/commandfile.txt"));
			Logger.LogInformation("{Arg0}", OptimizePath("/storage/0/emulated/123.txt"));
			Logger.LogInformation("{Arg0}", OptimizePath("/storage/0/emulated/game/123.txt"));
			Logger.LogInformation("{Arg0}", OptimizePath("/storage/0/emulated/every.txt"));

			Logger.LogInformation("{Arg0}", OptimizePath("^commandfile"));
			Logger.LogInformation("{Arg0}", OptimizePath("^commandfile"));

			Logger.LogInformation("{Arg0}", OptimizePath("^ext/z.txt"));
			Logger.LogInformation("{Arg0}", OptimizePath("^ext/abc.txt"));
			Logger.LogInformation("{Arg0}", OptimizePath("^ext/cde.txt"));
			Logger.LogInformation("{Arg0}", OptimizePath("^ext/1/abc.txt"));
			Logger.LogInformation("{Arg0}", OptimizePath("^ext/1/2/abc.txt"));
			Logger.LogInformation("{Arg0}", OptimizePath("^ext/xyz.txt"));
			Logger.LogInformation("{Arg0}", OptimizePath("^ext/1/2/stu.txt"));
			Logger.LogInformation("{Arg0}", OptimizePath("^ext/1/wer.txt"));
			Logger.LogInformation("{Arg0}", OptimizePath("^int/1/2/stu.txt"));
			Logger.LogInformation("{Arg0}", OptimizePath("^int/1/abc.txt"));
		}
		*/
	}
}
