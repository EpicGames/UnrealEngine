// Copyright Epic Games, Inc. All Rights Reserved.
using AutomationScripts;
using AutomationTool;
using EpicGames.Core;
using Microsoft.Extensions.Logging;
using System;
using System.Collections.Generic;
using System.Diagnostics;
using System.IO;
using System.IO.Compression;
using System.Linq;
using System.Text.Json;
using System.Text.RegularExpressions;
using System.Threading;
using UnrealBuildBase;
using UnrealBuildTool;

public class AndroidPlatform : Platform
{
	// Maximum allowed OBB size (1 GiB, 2 GiB or 4 GiB based on project settings)
	private const Int64 SmallOBBSizeAllowed = 1073741824;
	private const Int64 NormalOBBSizeAllowed = 2147483648;
	private const Int64 MaxOBBSizeAllowed = 4294967296;

	private const int DeployMaxParallelCommands = 6;

    private const string TargetAndroidLocation = "obb/";
	private const string TargetAndroidTemp = "/data/local/tmp/";

	private const string DefaultLaunchActivity = "com.epicgames.unreal.SplashActivity";

	public class AdbCreatedProcess : AutomationTool.IProcessResult
	{
		private readonly object StopSyncObject = new object();
		IProcessResult AdbLogProcess;
		string LogPath;
		string PackageName;
		string DeviceName;
		int LogFileProcessExitCode = 0;
		bool bStopped = false;

		public AdbCreatedProcess(
				IProcessResult InAdbLogProcess,
				string InLogPath,
				string InPackageName,
				string InDeviceName)
		{
			AdbLogProcess = InAdbLogProcess;
			LogPath = InLogPath;
			PackageName = InPackageName;
			DeviceName = InDeviceName;
			ProcessManager.AddProcess(this);
		}

		~AdbCreatedProcess()
		{
			ProcessManager.RemoveProcess(this);
		}

		public void StopProcess(bool KillDescendants = true)
		{
			lock (StopSyncObject)
			{
				if (!bStopped)
				{
					AndroidPlatform.RunAdbCommand(DeviceName, "shell am force-stop " + PackageName);
					if (!AdbLogProcess.HasExited)
					{
						AdbLogProcess.StopProcess(KillDescendants);
					}
					DumpDeviceOutputToLogFiles();
					bStopped = true;
				}
			}
		}

		public bool HasExited
		{
			get
			{
				if (!bStopped && (AdbLogProcess.HasExited || !IsPackageRunningOnDevice()))
				{
					StopProcess();
				}
				return bStopped;
			}
		}

		public string GetProcessName()
		{
			return String.Format("{0}@{1}", PackageName, DeviceName);
		}

		public void OnProcessExited()
		{
		}

		public void DisposeProcess()
		{
			AdbLogProcess.DisposeProcess();
		}

		public void StdOut(object sender, DataReceivedEventArgs e)
		{
		}

		public void StdErr(object sender, DataReceivedEventArgs e)
		{
		}

		public int ExitCode
		{
			get { return LogFileProcessExitCode; }
			set { LogFileProcessExitCode = value; }
		}

		public bool bExitCodeSuccess => ExitCode == 0;

		public string Output
		{
			get { return AdbLogProcess.Output; }
		}

		public Process ProcessObject
		{
			get { return AdbLogProcess.ProcessObject; }
		}

		public void WaitForExit()
		{
			while (!AdbLogProcess.HasExited && IsPackageRunningOnDevice())
			{
				Thread.Sleep(100);
			}
			StopProcess();
		}

		public FileReference WriteOutputToFile(string FileName)
		{
			return AdbLogProcess.WriteOutputToFile(FileName);
		}

		private bool IsPackageRunningOnDevice()
		{
			ERunOptions Options = ERunOptions.Default | ERunOptions.SpewIsVerbose | ERunOptions.NoLoggingOfRunCommand;
			IProcessResult Result = AndroidPlatform.RunAdbCommand(DeviceName, "shell ps", null, Options);
			string ProcessList = Result.Output;
			bool bIsProcessRunning = ProcessList.Contains(PackageName);
			return bIsProcessRunning;
		}

		private void DumpDeviceOutputToLogFiles()
		{
			string SanitizedDeviceName = DeviceName.Replace(":", "_");
			string LogFilename = Path.Combine(LogPath, "devicelog" + SanitizedDeviceName + ".log");
			string ServerLogFilename = Path.Combine(CmdEnv.LogFolder, "devicelog" + SanitizedDeviceName + ".log");
			ERunOptions Options = ERunOptions.Default & ~ERunOptions.AllowSpew;
			IProcessResult LogFileProcess = RunAdbCommand(DeviceName, "logcat -d", null, Options);
			string AllOutput = LogFileProcess.Output;
			File.WriteAllText(LogFilename, AllOutput);
			File.WriteAllText(ServerLogFilename, AllOutput);

			ExitCode = LogFileProcess.ExitCode;
		}
	}

	public AndroidPlatform()
		: base(UnrealTargetPlatform.Android)
	{

	}


	public override string[] GetCodeSpecifiedSdkVersions()
	{
		UEBuildPlatformSDK AndroidSDK = UEBuildPlatformSDK.GetSDKForPlatform("Android");

		return AndroidSDK != null ? new string[] { AndroidSDK.GetMainVersion() } : Array.Empty<string>();
	}

	// Android has a more complex sdk installation, so perform it manually
	public override bool InstallSDK(BuildCommand BuildCommand, ITurnkeyContext TurnkeyContext, DeviceInfo Device, bool bUnattended, bool bSdkAlreadyInstalled)
	{
		if (Device != null)
		{
			return base.InstallSDK(BuildCommand, TurnkeyContext, Device, bUnattended, bSdkAlreadyInstalled);
		}

		bool InstallCommandlineTools()
		{
			string SdkDir = GetSdkDir();
			if (Directory.Exists(SdkDir))
			{
				return true;
			}

			string OutputPath = TurnkeyContext.RetrieveFileSource("AndroidSDKCommandlineTools");
			if (OutputPath != null)
			{
				string SdkManager = Path.Combine(OutputPath, "cmdline-tools", "bin", "sdkmanager");
				if (!OperatingSystem.IsWindows())
				{
					// Mark executable before running
					File.SetUnixFileMode(SdkManager, File.GetUnixFileMode(SdkManager) | UnixFileMode.UserExecute | UnixFileMode.GroupExecute | UnixFileMode.OtherExecute);
				}
				else
				{
					SdkManager += ".bat";
				}

				// Use the JBR from Android Studio for the initial call to sdkmanager, if JAVA_HOME doesn't exist
				string JavaHome = Environment.GetEnvironmentVariable("JAVA_HOME");
				if (JavaHome == null)
				{
					string StudioExe = GetAndroidStudioExe();
					JavaHome = OperatingSystem.IsMacOS() ? Path.Combine(StudioExe, "Contents", "jbr", "Contents", "Home") : Path.Combine(Path.GetDirectoryName(Path.GetDirectoryName(StudioExe))!, "jbr");
				}
				
				int ExitCode;
				if (!OperatingSystem.IsWindows())
				{
					string BashCmd = $"export JAVA_HOME=\"{JavaHome}\" && \"{SdkManager}\" --sdk_root=\"{SdkDir}\" \"cmdline-tools;latest\"";
					string Command;
					string Arguments;
					if (OperatingSystem.IsMacOS())
					{
						Command = "osascript";
						Arguments = 
							"-e \"tell application \\\"Terminal\\\"\" " +
							"-e \"activate\" " +
							$"-e \"do script \\\"{BashCmd.Replace("\"", "\\\\\\\"")}; exit\\\"\" " +
							"-e \"set theTab to the result\" " +
							"-e \"repeat\" " +
							"-e \"delay 0.5\" " +
							"-e \"if not busy of theTab then exit repeat\" " +
							"-e \"end repeat\" " +
							"-e \"close (every window whose selected tab is theTab)\" " +
							"-e \"end tell\"";
					}
					else
					{
						Process Which;
						try
						{
							Which = Process.Start(new ProcessStartInfo { FileName = "which", Arguments = "gnome-terminal", RedirectStandardOutput = true, UseShellExecute = false });
							Which?.WaitForExit();
						}
						catch (Exception)
						{
							Which = null;
						}

						if (Which is { ExitCode: 0 })
						{
							Command = "gnome-terminal";
							Arguments = $"--wait -- bash -c \"{BashCmd.Replace("\"", "\\\"")}\"";
						}
						else
						{
							Command = "x-terminal-emulator";
							Arguments = $"-e bash -c \"{BashCmd.Replace("\"", "\\\"")}\"";
						}
					}
					
					ExitCode = TurnkeyContext.RunExternalCommand(Command, Arguments, false, bUnattended, false);
				}
				else
				{
					string BatchCmd = $"set \"JAVA_HOME={JavaHome}\" && \"{SdkManager}\" --sdk_root=\"{SdkDir}\" \"cmdline-tools;latest\"";
					string Arguments = $"/c {BatchCmd}";
					ExitCode = TurnkeyContext.RunExternalCommand("cmd.exe", Arguments, false, bUnattended, true);
				}

				if (ExitCode == 0)
				{
					return true;
				}
			}

			int Option = 2;
			while (Option == 2)
			{
				string Prompt = $"The Android Sdk directory was not found (expected to find it at '{SdkDir}'\n" +
					"Android Studio can install it for you, but you will need to manually perform some steps (if desired, you can get detailed help with option 2):\n" +
					"  - Wait for Android Studio to start, you will see an initial dialog asking how to proceed (called \"Welcome to Android Studio\")\n" +
					"  - Click the \"Configure\" dropdown in the bottom right, and select \"SDK Manager\"\n" +
					"  - Click on the \"SDK Tools\" tab near the top middle of the right pane\n" +
					"  - Check the box next to Android SDK Command-line Tools (latest)\n" +
					"  - Click OK in the bottom right\n" +
					"  - It will probably ask for you to accept a license - you MUST do this\n" +
					"  - Once installation has completed, close/quit Android Studio to continue\n";

				List<string> Options =
				[
					"Run Android Studio to install the Command Line Tools",
					"Get detailed step by step guide"
				];

				Option = TurnkeyContext.ReadInputInt(Prompt, Options, true, 1);
				if (Option == 0)
				{
					return false;
				}
				if (Option == 2)
				{
					// @todo: we need to redo the documentation DRAMATICALLY on here
					string URL = "https://docs.unrealengine.com/5.1/en-US/how-to-set-up-android-sdk-and-ndk-for-your-unreal-engine-development-environment/";
					Process.Start(new ProcessStartInfo { FileName = URL, UseShellExecute = true });
				}
			}

			if (HostPlatform.Platform == UnrealTargetPlatform.Mac)
			{
				TurnkeyContext.RunExternalCommand("open", $"-W \"{GetAndroidStudioExe()}\"", false, true, true);
			}
			else
			{
				TurnkeyContext.RunExternalCommand(GetAndroidStudioExe(), "", false, true, false);
			}

			return true;
		}

		if (!InstallCommandlineTools())
		{
			return false;
		}

		// run the Setup.bat in the engine, not coming from a normal FileSource

		string Command;
		if (HostPlatform.Platform == UnrealTargetPlatform.Win64)
		{
			Command = "$(EngineDir)/Extras/Android/SetupAndroid.bat";
		}
		else if (HostPlatform.Platform == UnrealTargetPlatform.Mac)
		{
			Command = "$(EngineDir)/Extras/Android/SetupAndroid.command";
		}
		else
		{
			Command = "$(EngineDir)/Extras/Android/SetupAndroid.sh";
		}

		// pull the desired version numbers to install
		UEBuildPlatformSDK AndroidSDK = UEBuildPlatformSDK.GetSDKForPlatform("Android");
		string PlatformsVersion = AndroidSDK.GetPlatformSpecificVersion("platforms");
		string BuildToolsVersion = AndroidSDK.GetPlatformSpecificVersion("build-tools");
		string CMakeVersion = AndroidSDK.GetPlatformSpecificVersion("cmake");
		string NDKVersion = AndroidSDK.GetPlatformSpecificVersion("ndk");

		string Params = $"{PlatformsVersion} {BuildToolsVersion} {CMakeVersion} {NDKVersion} -noninteractive";

		// because this may bring up a license acceptance message that needs the user to respond, so we make a new window
		int ExitCode = TurnkeyContext.RunExternalCommand(Command, Params, bRequiresPrivilegeElevation: false, bUnattended, bCreateWindow: true);
		return ExitCode == 0;
	}
	
	public override bool PostSDKSetup(ITurnkeyContext TurnkeyContext, bool bUnattended)
	{
		FileReference EmulatorConfigFile = FileReference.Combine(Unreal.EngineDirectory, "Platforms", "Android", "Config", "AndroidEmulator.json");
		
		if (!FileReference.Exists(EmulatorConfigFile))
		{
			EmulatorConfigFile = FileReference.Combine(Unreal.EngineDirectory, "Config", "Android", "AndroidEmulator.json");
			
			if (!FileReference.Exists(EmulatorConfigFile))
			{
				return true;
			}
		}

		Dictionary<string, string> EmulatorConfig = JsonSerializer.Deserialize<Dictionary<string, string>>(FileReference.ReadAllText(EmulatorConfigFile), new JsonSerializerOptions
		{
			AllowTrailingCommas = true,
			ReadCommentHandling = JsonCommentHandling.Skip
		});
		
		string CommandExtension = HostPlatform.Platform == UnrealTargetPlatform.Win64 ? "bat" : HostPlatform.Platform == UnrealTargetPlatform.Mac ? "command" : "sh";
		
		// run the SetupAndroidEmulator in the engine, not coming from a normal FileSource
		string Command = $"$(EngineDir)/Extras/Android/SetupAndroidEmulator.{CommandExtension}";
		string Params = $"{EmulatorConfig["AvdDevice"]} {EmulatorConfig["DeviceApiVersion"]} {EmulatorConfig["DeviceDataSize"]} {EmulatorConfig["DeviceRamSize"]} -noninteractive";
		
		// because this may bring up a license acceptance message that needs the user to respond, so we make a new window
		int ExitCode = TurnkeyContext.RunExternalCommand(Command, Params, bRequiresPrivilegeElevation: false, bUnattended, bCreateWindow: true);

		return ExitCode == 0;
	}

	private static string GetAndroidStudioExe()
	{
		if (OperatingSystem.IsLinux())
		{
			string UserHome = Environment.GetEnvironmentVariable("HOME");
			string AndroidStudioExe = Path.Combine(UserHome, "android-studio", "bin", "studio.sh");

			return AndroidStudioExe;
		}
		else if (OperatingSystem.IsMacOS())
		{

			string AndroidStudioExe = "/Applications/Android Studio.app";
			if (Directory.Exists(AndroidStudioExe))
			{
				return AndroidStudioExe;
			}

			string UserHome = Environment.GetEnvironmentVariable("HOME");
			AndroidStudioExe = Path.Combine(UserHome, "Applications", "Android Studio.app");

			return AndroidStudioExe;
		}

		Debug.Assert(OperatingSystem.IsWindows());

		string DefaultAndroidStudioInstallDir = Path.Combine(Environment.GetFolderPath(Environment.SpecialFolder.ProgramFiles), "Android", "Android Studio");
		string RegValue = Microsoft.Win32.Registry.GetValue("HKEY_LOCAL_MACHINE\\SOFTWARE\\Android Studio", "Path", null) as string;
		string AndroidStudioInstallDir = RegValue == null ? DefaultAndroidStudioInstallDir : RegValue;
  		// Some installs, like JetBrains Toolbox, may not place an entry in the registry so try an alternate location
		AndroidStudioInstallDir = Directory.Exists(AndroidStudioInstallDir) ? AndroidStudioInstallDir : Path.Combine(Environment.GetFolderPath(Environment.SpecialFolder.LocalApplicationData), "Programs", "Android Studio");
		return Path.Combine(AndroidStudioInstallDir, "bin", "studio64.exe");
	}

	private static string GetSdkDir()
	{
		string AndroidHome = Environment.GetEnvironmentVariable("ANDROID_HOME");
		if (!string.IsNullOrEmpty(AndroidHome) && Directory.Exists(AndroidHome))
		{
			return AndroidHome;
		}

		if (OperatingSystem.IsLinux())
		{
			string UserHome = Environment.GetEnvironmentVariable("HOME");
			string AndroidSdkPath = Path.Combine(UserHome, "Android", "Sdk");

			return AndroidSdkPath;
		}
		else if (OperatingSystem.IsMacOS())
		{
			string UserProfileDir = Environment.GetFolderPath(Environment.SpecialFolder.UserProfile);
			string BashProfilePath = Path.Combine(UserProfileDir, ".zshrc");
			if (!File.Exists(BashProfilePath))
			{
				// Fallbacks
				BashProfilePath = Path.Combine(UserProfileDir, ".bashrc");
				if (!File.Exists(BashProfilePath))
				{
					BashProfilePath = Path.Combine(UserProfileDir, ".bash_profile");
					Logger.LogWarning(".zshrc is missing. Falling back to .bash_profile.");
				}
				else
				{
					Logger.LogWarning(".zshrc is missing. Falling back to .bashrc.");
				}
			}
			if (File.Exists(BashProfilePath))
			{
				string[] BashProfileContents = File.ReadAllLines(BashProfilePath);

				// Walk backwards so we keep the last export setting instead of the first
				string SdkKey = "ANDROID_HOME";
				for (int LineIndex = BashProfileContents.Length - 1; LineIndex >= 0; --LineIndex)
				{
					if (BashProfileContents[LineIndex].StartsWith("export " + SdkKey + "="))
					{
						string PathVar = BashProfileContents[LineIndex].Split('=')[1].Replace("\"", "");
						return PathVar;
					}

				}
			}

			string UserHome = Environment.GetEnvironmentVariable("HOME");
			string AndroidSdkPath = Path.Combine(UserHome, "Library", "Android", "sdk");

			return AndroidSdkPath;
		}

		Debug.Assert(OperatingSystem.IsWindows());

		string DefaultSdkDir = Path.Combine(Environment.GetFolderPath(Environment.SpecialFolder.LocalApplicationData), "Android", "Sdk");
		string RegValue = Microsoft.Win32.Registry.GetValue("HKEY_LOCAL_MACHINE\\SOFTWARE\\Android Studio", "SdkPath", null) as string;
		return string.IsNullOrEmpty(RegValue) ? DefaultSdkDir : RegValue;
	}

	public override bool UpdateHostPrerequisites(BuildCommand Command, ITurnkeyContext TurnkeyContext, bool bVerifyOnly)
	{
		string AndroidStudioExe = GetAndroidStudioExe();

		bool bIsMac = (HostPlatform.Current.HostEditorPlatform == UnrealTargetPlatform.Mac);
		bool bHaveAndroidStudio = (bIsMac && Directory.Exists(AndroidStudioExe)) || 
					(!bIsMac && FileExists(AndroidStudioExe));

		// if we are only verifying, just return the status, and if it's installed, we are done!
		if (bVerifyOnly)
		{
			if (!bHaveAndroidStudio)
			{
				TurnkeyContext.ReportError("Android Studio is not installed correctly.");
			}
			return bHaveAndroidStudio;
		}

		if (!bHaveAndroidStudio)
		{
			TurnkeyContext.PauseForUser("Android Studio was not found on this machine, which is required to use Android. Download and install Android Studio?");

			// get AS installer
			string OutputPath = TurnkeyContext.RetrieveFileSource("AndroidStudio");

			// Unset some envvars in case autosdk ran - they will mess up the first run of Android Studio
			string[] Vars = new string[]
			{
					"ANDROID_HOME",
					"ANDROID_SDK_HOME",
					"JAVA_HOME",
					"NDKROOT",
					"NDK_ROOT",
					"ANDROID_NDK_ROOT",
					"ANDROID_SWT"
			};
			Array.ForEach(Vars, x => Environment.SetEnvironmentVariable(x, null));

			if (OutputPath == null)
			{
				TurnkeyContext.PauseForUser("Unable to find Android Studio installer. Please download and install Android Studio 2022.2.1 from https://developer.android.com/studio/archive to standard location before continuing.");
			}
			else
			{
				if (HostPlatform.Current.HostEditorPlatform == UnrealTargetPlatform.Linux)
				{
					// TODO finish GUI support for Linux here, otherwise this will throw. Using zenity
					// TurnkeyContext.PauseForUser("Running the Android Studio installer, and then Android Studio for first-time setup!\n\nChoose all default options unless you know what you are doing.");

					string UserHome = Environment.GetEnvironmentVariable("HOME");
					string Args = string.Format("-xf {0} -C {1}", OutputPath, UserHome);

					int ExitCode = TurnkeyContext.RunExternalCommand("/bin/tar", Args, false, true, true);

					if (ExitCode != 0)
					{
						TurnkeyContext.ReportError($"Android Studio installer failed. ExitCode = {ExitCode}");
						return false;
					}
				}
				else if (HostPlatform.Current.HostEditorPlatform == UnrealTargetPlatform.Mac)
				{

					string UserHome = Environment.GetEnvironmentVariable("HOME");
					string SourceApp = Path.Combine(OutputPath, "Android Studio.app");

					int ExitCode = TurnkeyContext.RunExternalCommand("/usr/bin/hdiutil", "attach " + OutputPath, false, true, true);

					if (ExitCode != 0)
					{
						TurnkeyContext.ReportError($"Android Studio installer failed. ExitCode = {ExitCode}");
						return false;
					}

					string AndroidStudioVolume = "";
					foreach (string Volume in Directory.GetDirectories("/Volumes"))

					{
						if (Volume.Contains("Android Studio"))
						{
							AndroidStudioVolume = Volume;
							break;
						}
					}
					if (AndroidStudioVolume == "")
					{
						TurnkeyContext.ReportError($"Android Studio installer failed. DMG did not mount");
						return false;
					}

					string SourcePath = Path.Combine(AndroidStudioVolume, "Android Studio.app");
					string DestPath = Path.Combine(UserHome, "Applications") + "/";
					if (SourcePath.Contains(" "))
					{
						SourcePath = "\"" + SourcePath + "\"";
					}
					if (DestPath.Contains(" "))
					{
						DestPath = "\"" + DestPath + "\"";
					}

					ExitCode = TurnkeyContext.RunExternalCommand("/bin/cp", "-R " + SourcePath + " " + DestPath, false, true, true);

					if (AndroidStudioVolume.Contains(" "))
					{
						AndroidStudioVolume = "\"" + AndroidStudioVolume + "\"";
					}
					int ExitCode2 = TurnkeyContext.RunExternalCommand("/usr/bin/hdiutil", "detach " + AndroidStudioVolume, false, true, true);

					// give error for cp, but can ignore detach failure
					if (ExitCode != 0)
					{
						TurnkeyContext.ReportError($"Android Studio installer failed. ExitCode = {ExitCode}");
						return false;
					}
				}
				else if (HostPlatform.Current.HostEditorPlatform == UnrealTargetPlatform.Win64)
				{
					// install AS with the /S switch
					int ExitCode = TurnkeyContext.RunExternalCommand(OutputPath, "/S", false, true, true);

					// AS installer returns 1223 even on success ("user canceled" even tho there's no UI to cancel it...) when running with /S
					if (ExitCode != 0 && ExitCode != 1223)
					{
						TurnkeyContext.ReportError($"Android Studio installer failed. ExitCode = {ExitCode}");
						return false;
					}
				}
				else
				{
					TurnkeyContext.ReportError($"Invalid host platform");
					return false;
				}
			}
		}

		// check to see if the installation worked. If so, continue on!
		AndroidStudioExe = GetAndroidStudioExe();

		bHaveAndroidStudio = (bIsMac && Directory.Exists(AndroidStudioExe)) || 
					(!bIsMac && FileExists(AndroidStudioExe));

		if (!bHaveAndroidStudio)
		{
			TurnkeyContext.ReportError("Android Studio is not installed correctly, after attempted installation.");
		}

		return bHaveAndroidStudio;
	}

	static string GetAvdHomePath()
	{
		return Environment.GetEnvironmentVariable("ANDROID_AVD_HOME") ?? Path.Combine(Environment.GetEnvironmentVariable("ANDROID_EMULATOR_HOME") ?? Environment.GetEnvironmentVariable("ANDROID_USER_HOME") ?? Path.Combine(Environment.GetFolderPath(Environment.SpecialFolder.UserProfile), ".android"), "avd");
	}
	
	public override DeviceInfo[] GetDevices()
	{
		List<DeviceInfo> Devices = new List<DeviceInfo>();
		
		List<string> ConnectedDevices;
		GetConnectedDevices(null, out ConnectedDevices);
		
		foreach (string ConnectedDeviceName in ConnectedDevices)
		{
			DeviceInfo CurrentDevice = new DeviceInfo(TargetPlatformType);
			// GetConnectedDevices returns valid device names with an '@' in front
			CurrentDevice.Name = ConnectedDeviceName.StartsWith("@") ? ConnectedDeviceName.Remove(0, 1) : ConnectedDeviceName; 
			
			CurrentDevice.Id = CurrentDevice.Name;
			CurrentDevice.bCanConnect = ConnectedDeviceName.StartsWith("@");
			
			// Instead, we return the SDK Version, and other parts of the code were adjusted
			string GetPropCommand = "shell getprop";
			string SDKVersionCommand = $"{GetPropCommand} ro.build.version.sdk";
			IProcessResult Result = RunAdbCommand(CurrentDevice.Name, SDKVersionCommand);
			if (Result.Output.Length > 0)
			{
				CurrentDevice.SoftwareVersion = Result.Output.Trim();
			}
						
			Devices.Add(CurrentDevice);
		}
		return Devices.ToArray();
	}
	
	public override DeviceInfo GetDeviceByName(string DeviceName)
	{
		if (DeviceName.StartsWith("avd-"))
		{
			return Directory.EnumerateFiles(GetAvdHomePath(), "*.ini").Select(Filename =>
			{
				string AvdName = "avd-" + Path.GetFileNameWithoutExtension(Filename);
				string Target = GetIniValue(Filename, "target");
				const string TargetPrefix = "android-";
				string SdkVersion = Target.StartsWith(TargetPrefix) ? Target[TargetPrefix.Length..] : "";
			
				return new DeviceInfo(TargetPlatformType)
				{
					Id = AvdName,
					Name = AvdName,
					bCanConnect = true,
					SoftwareVersion = SdkVersion.Length > 0 ? SdkVersion : null
				};
			}).FirstOrDefault(Device => string.Compare(Device.Id, DeviceName, true) == 0);
		}
		else
		{
			return base.GetDeviceByName(DeviceName);
		}
	}

	private static string GetSONameWithoutArchitecture(ProjectParams Params, string DecoratedExeName)
	{
		return Path.Combine(Path.GetDirectoryName(Params.GetProjectExeForPlatform(UnrealTargetPlatform.Android).ToString()), DecoratedExeName) + ".so";
	}

	private static string GetSOName(ProjectParams Params, string DecoratedExeName, UnrealArch? Architecture)
	{
		string ArchName = Architecture == null ? "" : "-" + Architecture.ToString();
		return Path.Combine(Path.GetDirectoryName(Params.GetProjectExeForPlatform(UnrealTargetPlatform.Android).ToString()), DecoratedExeName) + ArchName + ".so";
	}

	private static string GetFinalApkName(ProjectParams Params, string DecoratedExeName, bool bRenameUnrealGame, UnrealArch? Architecture)
	{
		string ProjectDir = Path.Combine(Path.GetDirectoryName(Path.GetFullPath(Params.RawProjectPath.FullName)), "Binaries/Android");

		if (Params.Prebuilt)
		{
			ProjectDir = Path.Combine(Params.BaseStageDirectory, "Android");
		}

		// Apk's go to project location, not necessarily where the .so is (content only packages need to output to their directory)
		string ArchName = Architecture == null ? "" : "-" + Architecture.ToString();
		string ApkName = Path.Combine(ProjectDir, $"{DecoratedExeName}{ArchName}.apk");

		// if the source binary was UnrealGame, handle using it or switching to project name
		if (Path.GetFileNameWithoutExtension(Params.GetProjectExeForPlatform(UnrealTargetPlatform.Android).ToString()) == "UnrealGame")
		{
			if (bRenameUnrealGame)
			{
				// replace UnrealGame with project name (only replace in the filename part)
				ApkName = Path.Combine(Path.GetDirectoryName(ApkName), Path.GetFileName(ApkName).Replace("UnrealGame", Params.ShortProjectName));
			}
			else
			{
				// if we want to use UE directly then use it from the engine directory not project directory
				ApkName = ApkName.Replace(ProjectDir, Path.Combine(CmdEnv.LocalRoot, "Engine/Binaries/Android"));
			}
		}

		return ApkName;
	}

	private static bool bHaveReadEngineVersion = false;
	private static string EngineMajorVersion = "4";
	private static string EngineMinorVersion = "0";
	private static string EnginePatchVersion = "0";

	#pragma warning disable CS0414
	private static string EngineChangelist = "0";

	private static string ReadEngineVersion(string EngineDirectory)
	{
		if (!bHaveReadEngineVersion)
		{
			string EngineVersionFile = Path.Combine(EngineDirectory, "Source", "Runtime", "Launch", "Resources", "Version.h");
			string[] EngineVersionLines = File.ReadAllLines(EngineVersionFile);
			for (int i = 0; i < EngineVersionLines.Length; ++i)
			{
				if (EngineVersionLines[i].StartsWith("#define ENGINE_MAJOR_VERSION"))
				{
					EngineMajorVersion = EngineVersionLines[i].Split('\t')[1].Trim(' ');
				}
				else if (EngineVersionLines[i].StartsWith("#define ENGINE_MINOR_VERSION"))
				{
					EngineMinorVersion = EngineVersionLines[i].Split('\t')[1].Trim(' ');
				}
				else if (EngineVersionLines[i].StartsWith("#define ENGINE_PATCH_VERSION"))
				{
					EnginePatchVersion = EngineVersionLines[i].Split('\t')[1].Trim(' ');
				}
				else if (EngineVersionLines[i].StartsWith("#define BUILT_FROM_CHANGELIST"))
				{
					EngineChangelist = EngineVersionLines[i].Split(new char[] { ' ', '\t' })[2].Trim(' ');
				}
			}

			bHaveReadEngineVersion = true;
		}

		return EngineMajorVersion + "." + EngineMinorVersion + "." + EnginePatchVersion;
	}

	#pragma warning restore CS0414


	private static string GetFinalSymbolizedSODirectory(string ApkName, DeploymentContext SC, UnrealArch Architecture)
	{
		string PackageVersion = GetPackageInfo(ApkName, SC, true);
		if (PackageVersion == null || PackageVersion.Length == 0)
		{
			throw new AutomationException(ExitCode.Error_FailureGettingPackageInfo, "Failed to get package version from " + ApkName);
		}

		return SC.ShortProjectName + "_Symbols_v" + PackageVersion + "/" + SC.ShortProjectName + Architecture;
	}

	private static string GetFinalObbName(string ApkName, DeploymentContext SC, bool bUseAppType = true)
	{
		// calculate the name for the .obb file
		string PackageName = GetPackageInfo(ApkName, SC, false);
		if (PackageName == null)
		{
			throw new AutomationException(ExitCode.Error_FailureGettingPackageInfo, "Failed to get package name from " + ApkName);
		}

		string PackageVersion = GetPackageInfo(ApkName, SC, true);
		if (PackageVersion == null || PackageVersion.Length == 0)
		{
			throw new AutomationException(ExitCode.Error_FailureGettingPackageInfo, "Failed to get package version from " + ApkName);
		}

		if (PackageVersion.Length > 0)
		{
			int IntVersion = int.Parse(PackageVersion);
			PackageVersion = IntVersion.ToString("0");
		}

		string AppType = bUseAppType ? GetMetaAppType() : "";
		if (AppType.Length > 0)
		{
			AppType += ".";
		}

		string ObbName = string.Format("main.{0}.{1}.{2}obb", PackageVersion, PackageName, AppType);

		// plop the .obb right next to the executable
		ObbName = Path.Combine(Path.GetDirectoryName(ApkName), ObbName);

		return ObbName;
	}

	private static string GetFinalPatchName(string ApkName, DeploymentContext SC, bool bUseAppType = true)
	{
		// calculate the name for the .obb file
		string PackageName = GetPackageInfo(ApkName, SC, false);
		if (PackageName == null)
		{
			throw new AutomationException(ExitCode.Error_FailureGettingPackageInfo, "Failed to get package name from " + ApkName);
		}

		string PackageVersion = GetPackageInfo(ApkName, SC, true);
		if (PackageVersion == null || PackageVersion.Length == 0)
		{
			throw new AutomationException(ExitCode.Error_FailureGettingPackageInfo, "Failed to get package version from " + ApkName);
		}

		if (PackageVersion.Length > 0)
		{
			int IntVersion = int.Parse(PackageVersion);
			PackageVersion = IntVersion.ToString("0");
		}

		string AppType = bUseAppType ? GetMetaAppType() : "";
		if (AppType.Length > 0)
		{
			AppType += ".";
		}

		string PatchName = string.Format("patch.{0}.{1}.{2}obb", PackageVersion, PackageName, AppType);

		// plop the .obb right next to the executable
		PatchName = Path.Combine(Path.GetDirectoryName(ApkName), PatchName);

		return PatchName;
	}

	private static string GetFinalOverflowName(string ApkName, DeploymentContext SC, int Index, bool bUseAppType = true)
	{
		// calculate the name for the .obb file
		string PackageName = GetPackageInfo(ApkName, SC, false);
		if (PackageName == null)
		{
			throw new AutomationException(ExitCode.Error_FailureGettingPackageInfo, "Failed to get package name from " + ApkName);
		}

		string PackageVersion = GetPackageInfo(ApkName, SC, true);
		if (PackageVersion == null || PackageVersion.Length == 0)
		{
			throw new AutomationException(ExitCode.Error_FailureGettingPackageInfo, "Failed to get package version from " + ApkName);
		}

		if (PackageVersion.Length > 0)
		{
			int IntVersion = int.Parse(PackageVersion);
			PackageVersion = IntVersion.ToString("0");
		}

		string AppType = bUseAppType ? GetMetaAppType() : "";
		if (AppType.Length > 0)
		{
			AppType += ".";
		}

		string OverflowName = string.Format("overflow{0}.{1}.{2}.{3}obb", Index, PackageVersion, PackageName, AppType);

		// plop the .obb right next to the executable
		OverflowName = Path.Combine(Path.GetDirectoryName(ApkName), OverflowName);

		return OverflowName;
	}


	public override string GetPlatformPakCommandLine(ProjectParams Params, DeploymentContext SC)
	{
		string PakParams = " -patchpaddingalign=0";

		string OodleDllPath = DirectoryReference.Combine(SC.ProjectRoot, "Binaries/ThirdParty/Oodle/Win64/UnrealPakPlugin.dll").FullName;
		if (File.Exists(OodleDllPath))
		{
			PakParams += String.Format(" -customcompressor=\"{0}\"", OodleDllPath);
		}

		return PakParams;
	}

	private static string GetDeviceObbName(string ApkName, DeploymentContext SC)
	{
        string ObbName = GetFinalObbName(ApkName, SC, false);
        string PackageName = GetPackageInfo(ApkName, SC, false);
        return TargetAndroidLocation + PackageName + "/" + Path.GetFileName(ObbName);
	}

	private static string GetDevicePatchName(string ApkName, DeploymentContext SC)
	{
		string PatchName = GetFinalPatchName(ApkName, SC, false);
		string PackageName = GetPackageInfo(ApkName, SC, false);
		return TargetAndroidLocation + PackageName + "/" + Path.GetFileName(PatchName);
	}

	private static string GetDeviceOverflowName(string ApkName, DeploymentContext SC, int Index)
	{
		string OverflowName = GetFinalOverflowName(ApkName, SC, Index, false);
		string PackageName = GetPackageInfo(ApkName, SC, false);
		return TargetAndroidLocation + PackageName + "/" + Path.GetFileName(OverflowName);
	}

	public static string GetStorageQueryCommand(bool bForcePC = false)
    {
		if (!bForcePC && !OperatingSystem.IsWindows())
		{
			return "shell 'echo $EXTERNAL_STORAGE'";
		}
		else
		{
			return "shell \"echo $EXTERNAL_STORAGE\"";
		}
    }

	enum EBatchType
	{
		Install,
		Uninstall,
		Symbolize,
	};
	private static string GetFinalBatchName(string ApkName, DeploymentContext SC, bool bNoOBBInstall, EBatchType BatchType, UnrealTargetPlatform Target)
	{
		string Extension = ".bat";
		if (Target == UnrealTargetPlatform.Linux || Target == UnrealTargetPlatform.LinuxArm64)
		{
			Extension = ".sh";
		}
		else if (Target == UnrealTargetPlatform.Mac)
		{
			Extension = ".command";
		}

		// Get the name of the APK to use for batch file
		string ExecutableName = Path.GetFileNameWithoutExtension(ApkName);
		
		switch(BatchType)
		{
			case EBatchType.Install:
			case EBatchType.Uninstall:
				return Path.Combine(Path.GetDirectoryName(ApkName), (BatchType == EBatchType.Uninstall ? "Uninstall_" : "Install_") + ExecutableName + (!bNoOBBInstall ? "" : "_NoOBBInstall") + Extension);
			case EBatchType.Symbolize:
				return Path.Combine(Path.GetDirectoryName(ApkName), "SymbolizeCrashDump_" + ExecutableName + Extension);
		}
		return "";
	}

	private UnrealArchitectures GetDeploymentArchitectures(ProjectParams Params, DeploymentContext SC)
	{
		if (Params.ClientArchitecture != null)
		{
			return Params.ClientArchitecture;
		}

		// use Receipt architectures if bIsCodeBasedProject (UnrealGame.target used if no code or plugin differences)
		if (SC.StageTargets[0].Receipt.TargetName != "UnrealGame")
		{
			Logger.LogInformation("GetDeploymentArchitectures: not UnrealGame");
			return SC.StageTargets[0].Receipt.Architectures;
		}
		
		// otherwise figure it out from project ini
		return UnrealArchitectureConfig.ForPlatform(UnrealTargetPlatform.Android).ActiveArchitectures(Params.RawProjectPath, null);
	}

	private List<string> CollectPluginDataPaths(DeploymentContext SC)
	{
		// collect plugin extra data paths from target receipts
		List<string> PluginExtras = new List<string>();
		foreach (StageTarget Target in SC.StageTargets)
		{
			TargetReceipt Receipt = Target.Receipt;
			var Results = Receipt.AdditionalProperties.Where(x => x.Name == "AndroidPlugin");
			foreach (var Property in Results)
			{
				// Keep only unique paths
				string PluginPath = Property.Value;
				if (PluginExtras.FirstOrDefault(x => x == PluginPath) == null)
				{
					PluginExtras.Add(PluginPath);
					Logger.LogInformation("AndroidPlugin: {PluginPath}", PluginPath);
				}
			}
		}
		return PluginExtras;
	}

	private bool UsingAndroidFileServer(ProjectParams Params, DeploymentContext SC, out bool bEnablePlugin, out string AFSToken, out bool bIsShipping, out bool bIncludeInShipping, out bool bAllowExternalStartInShipping)
	{
		FileReference RawProjectPath = SC != null ? SC.RawProjectPath : Params.RawProjectPath;
		UnrealTargetPlatform TargetPlatform = SC != null ? SC.StageTargetPlatform.PlatformType : Params.ClientTargetPlatforms[0].Type;
		UnrealTargetConfiguration TargetConfiguration = SC != null ? SC.StageTargetConfigurations[0] : Params.ClientConfigsToBuild[0];
		bIsShipping = TargetConfiguration == UnrealTargetConfiguration.Shipping;

		if (Params.ProjectTargets.Any(Target => Target.Rules.IsTestTarget))
		{
			bEnablePlugin = true;
			AFSToken = "";
			bIncludeInShipping = false;
			bAllowExternalStartInShipping = false;

			return true;
		}

		ConfigHierarchy Ini = ConfigCache.ReadHierarchy(ConfigHierarchyType.Engine, DirectoryReference.FromFile(RawProjectPath), TargetPlatform);
		if (!Ini.GetBool("/Script/AndroidFileServerEditor.AndroidFileServerRuntimeSettings", "bEnablePlugin", out bEnablePlugin))
		{
			bEnablePlugin = true;
		}
		if (!Ini.GetString("/Script/AndroidFileServerEditor.AndroidFileServerRuntimeSettings", "SecurityToken", out AFSToken))
		{
			AFSToken = "";
		}
		if (!Ini.GetBool("/Script/AndroidFileServerEditor.AndroidFileServerRuntimeSettings", "bIncludeInShipping", out bIncludeInShipping))
		{
			bIncludeInShipping = false;
		}
		if (!Ini.GetBool("/Script/AndroidFileServerEditor.AndroidFileServerRuntimeSettings", "bAllowExternalStartInShipping", out bAllowExternalStartInShipping))
		{
			bAllowExternalStartInShipping = false;
		}

		if (bIsShipping && !(bIncludeInShipping && bAllowExternalStartInShipping))
		{
			return false;
		}
		return bEnablePlugin;
	}

	enum EConnectionType
	{
		USBOnly,
		NetworkOnly,
		Combined
	}

	private EConnectionType GetAndroidFileServerNetworkConfig(DeploymentContext SC, out bool bUseCompression, out bool bLogFiles, out bool bReportStats, out bool bUseManualIPAddress, out string ManualIPAddress)
	{
		EConnectionType ConnectionType = EConnectionType.USBOnly;

		UnrealTargetConfiguration TargetConfiguration = SC.StageTargetConfigurations[0];
		ConfigHierarchy Ini = ConfigCache.ReadHierarchy(ConfigHierarchyType.Engine, DirectoryReference.FromFile(SC.RawProjectPath), SC.StageTargetPlatform.PlatformType);

		string ConnectionString = "";
		Ini.GetString("/Script/AndroidFileServerEditor.AndroidFileServerRuntimeSettings", "ConnectionType", out ConnectionString);
		switch (ConnectionString)
		{
			case "USBOnly":
				ConnectionType = EConnectionType.USBOnly;
				break;
			case "NetworkOnly":
				ConnectionType = EConnectionType.NetworkOnly;
				break;
			case "Combined":
				ConnectionType = EConnectionType.Combined;
				break;
			default:
				ConnectionType = EConnectionType.USBOnly;
				break;
		}

		if (!Ini.GetBool("/Script/AndroidFileServerEditor.AndroidFileServerRuntimeSettings", "bUseCompression", out bUseCompression))
		{
			bUseCompression = false;
		}
		if (!Ini.GetBool("/Script/AndroidFileServerEditor.AndroidFileServerRuntimeSettings", "bLogFiles", out bLogFiles))
		{
			bLogFiles = false;
		}
		if (!Ini.GetBool("/Script/AndroidFileServerEditor.AndroidFileServerRuntimeSettings", "bReportStats", out bReportStats))
		{
			bReportStats = false;
		}
		if (!Ini.GetBool("/Script/AndroidFileServerEditor.AndroidFileServerRuntimeSettings", "bUseManualIPAddress", out bUseManualIPAddress))
		{
			bUseManualIPAddress = false;
		}
		if (!Ini.GetString("/Script/AndroidFileServerEditor.AndroidFileServerRuntimeSettings", "ManualIPAddress", out ManualIPAddress))
		{
			ManualIPAddress = "127.0.0.1";
		}

		bool bAllowNetworkConnection = true;
		if (!Ini.GetBool("/Script/AndroidFileServerEditor.AndroidFileServerRuntimeSettings", "bAllowNetworkConnection", out bAllowNetworkConnection))
		{
			bAllowNetworkConnection = true;
		}
		if (!bAllowNetworkConnection && ConnectionType != EConnectionType.USBOnly)
		{
			Logger.LogWarning("AFS will only use USB connection due to network connection disabled");
			ConnectionType = EConnectionType.USBOnly;
		}

		return ConnectionType;
	}

	private bool BuildWithHiddenSymbolVisibility(DeploymentContext SC)
	{
		ConfigHierarchy Ini = ConfigCache.ReadHierarchy(ConfigHierarchyType.Engine, DirectoryReference.FromFile(SC.RawProjectPath), SC.StageTargetPlatform.PlatformType);
		return Ini.GetBool("/Script/AndroidRuntimeSettings.AndroidRuntimeSettings", "bBuildWithHiddenSymbolVisibility", out bool bBuild) && bBuild;
	}

	private bool GetSaveSymbols(DeploymentContext SC)
	{
		ConfigHierarchy Ini = ConfigCache.ReadHierarchy(ConfigHierarchyType.Engine, DirectoryReference.FromFile(SC.RawProjectPath), SC.StageTargetPlatform.PlatformType);
		bool bSave = false;
		return (Ini.GetBool("/Script/AndroidRuntimeSettings.AndroidRuntimeSettings", "bSaveSymbols", out bSave) && bSave);
	}

	private bool GetEnableBundle(DeploymentContext SC)
	{
		ConfigHierarchy Ini = ConfigCache.ReadHierarchy(ConfigHierarchyType.Engine, DirectoryReference.FromFile(SC.RawProjectPath), SC.StageTargetPlatform.PlatformType);
		bool bEnableBundle = false;
		return (Ini.GetBool("/Script/AndroidRuntimeSettings.AndroidRuntimeSettings", "bEnableBundle", out bEnableBundle) && bEnableBundle);
	}

	private bool GetEnableUniversalAPK(DeploymentContext SC)
	{
		ConfigHierarchy Ini = ConfigCache.ReadHierarchy(ConfigHierarchyType.Engine, DirectoryReference.FromFile(SC.RawProjectPath), SC.StageTargetPlatform.PlatformType);
		bool bEnableUniversalAPK = false;
		return (Ini.GetBool("/Script/AndroidRuntimeSettings.AndroidRuntimeSettings", "bEnableUniversalAPK", out bEnableUniversalAPK) && bEnableUniversalAPK);
	}

	private Int64 GetMaxOBBSizeAllowed(DeploymentContext SC)
	{
		ConfigHierarchy Ini = ConfigCache.ReadHierarchy(ConfigHierarchyType.Engine, DirectoryReference.FromFile(SC.RawProjectPath), SC.StageTargetPlatform.PlatformType);
		bool bForceSmallOBBFiles = false;
		bool bAllowLargeOBBFiles = false;
		Ini.GetBool("/Script/AndroidRuntimeSettings.AndroidRuntimeSettings", "bForceSmallOBBFiles", out bForceSmallOBBFiles);
		Ini.GetBool("/Script/AndroidRuntimeSettings.AndroidRuntimeSettings", "bAllowLargeOBBFiles", out bAllowLargeOBBFiles);
		return bForceSmallOBBFiles ? SmallOBBSizeAllowed : (bAllowLargeOBBFiles ? MaxOBBSizeAllowed : NormalOBBSizeAllowed);
	}

	private bool AllowPatchOBBFile(DeploymentContext SC)
	{
		ConfigHierarchy Ini = ConfigCache.ReadHierarchy(ConfigHierarchyType.Engine, DirectoryReference.FromFile(SC.RawProjectPath), SC.StageTargetPlatform.PlatformType);
		bool bAllowPatchOBBFile = false;
		Ini.GetBool("/Script/AndroidRuntimeSettings.AndroidRuntimeSettings", "bAllowPatchOBBFile", out bAllowPatchOBBFile);
		return bAllowPatchOBBFile;
	}

	private int AllowOverflowOBBFiles(DeploymentContext SC)
	{
		int FileLimit = 0;
		ConfigHierarchy Ini = ConfigCache.ReadHierarchy(ConfigHierarchyType.Engine, DirectoryReference.FromFile(SC.RawProjectPath), SC.StageTargetPlatform.PlatformType);
		bool bAllowOverflowOBBFiles = false;
		Ini.GetBool("/Script/AndroidRuntimeSettings.AndroidRuntimeSettings", "bAllowOverflowOBBFiles", out bAllowOverflowOBBFiles);
		if (bAllowOverflowOBBFiles)
		{
			FileLimit = 2;
			Ini.GetInt32("/Script/AndroidRuntimeSettings.AndroidRuntimeSettings", "OverflowOBBFileLimit", out FileLimit);
		}
		return FileLimit;
	}

	private bool CreateOBBFile(DeploymentContext SC, string OutputFilename, List<FileReference> FilesForObb)
	{
		Logger.LogInformation("Creating {OutputFilename} from {Arg1}", OutputFilename, SC.StageDirectory);
		try
		{
			using ZipArchive ObbFile = ZipFile.Open(OutputFilename, ZipArchiveMode.Create);
			ObbFile.Comment = $"{"1",10}";
			int ObbFileCount = 0;

			foreach (FileReference FileRef in FilesForObb)
			{
				string RelativePath = FileRef.MakeRelativeTo(SC.StageDirectory).Replace("\\", "/");
				ObbFile.CreateEntryFromFile_CrossPlatform(FileRef.FullName, RelativePath, CompressionLevel.NoCompression);
				ObbFileCount += 1;
				Logger.LogInformation("[{Count}/{Total}] Adding {File} to OBB", ObbFileCount, FilesForObb.Count, RelativePath);
			}
		}
		catch (Exception)
		{
			return false;
		}
		return true;
	}

	private bool UpdateObbStoreVersion(string Filename)
	{
		string Version = Path.GetFileNameWithoutExtension(Filename).Split('.')[1];

		try
		{
			using ZipArchive ObbFile = ZipFile.Open(Filename, ZipArchiveMode.Update);
			// Add the store version from the filename as a comment
			ObbFile.Comment = $"{Version,10}";
		}
		catch (Exception)
		{
			return false;
		}
		return true;
	}
	private class OverflowFileInfo
	{
		public OverflowFileInfo(List<FileReference> InFilesForOverflow, Int64 InOverflowObbSize)
		{
			FilesForOverflow = InFilesForOverflow;
			OverflowObbSize = InOverflowObbSize;
		}

		public List<FileReference> FilesForOverflow { get; set; }
		public Int64 OverflowObbSize { get; set; }
	}

	public override void Package(ProjectParams Params, DeploymentContext SC, int WorkingCL)
	{
		if (SC.StageTargetConfigurations.Count != 1)
		{
			throw new AutomationException(ExitCode.Error_OnlyOneTargetConfigurationSupported, "Android is currently only able to package one target configuration at a time, but StageTargetConfigurations contained {0} configurations", SC.StageTargetConfigurations.Count);
		}

		UnrealTargetConfiguration TargetConfiguration = SC.StageTargetConfigurations[0];

		UnrealArchitectures Architectures = GetDeploymentArchitectures(Params, SC);
		bool bMakeSeparateApks = UnrealBuildTool.AndroidExports.ShouldMakeSeparateApks();
		bool bBuildWithHiddenSymbolVisibility = BuildWithHiddenSymbolVisibility(SC);
		bool bSaveSymbols = GetSaveSymbols(SC);
		bool bEnableBundle = GetEnableBundle(SC);
		bool bEnableUniversalAPK = GetEnableUniversalAPK(SC);

		var Deploy = AndroidExports.CreateDeploymentHandler(Params.RawProjectPath, Params.ForcePackageData);
		bool bPackageDataInsideApk = Deploy.GetPackageDataInsideApk();

		bool bUseAFS = false;
		bool bUseAFSProject = false;

		bool bAFSEnablePlugin;
		string AFSToken;
		bool bIsShipping;
		bool bAFSIncludeInShipping;
		bool bAFSAllowExternalStartInShipping;
		UsingAndroidFileServer(Params, SC, out bAFSEnablePlugin, out AFSToken, out bIsShipping, out bAFSIncludeInShipping, out bAFSAllowExternalStartInShipping);

		if (bAFSEnablePlugin && !bPackageDataInsideApk)
		{
			bUseAFS = true;
			// AFSProject APK should be used if shipping and AFS wasn't included
			if (bIsShipping && !(bAFSIncludeInShipping && bAFSAllowExternalStartInShipping))
			{
				bUseAFSProject = true;
			}
		}

		string BaseApkName = GetFinalApkName(Params, SC.StageExecutables[0], true, Architecture:null);
		Logger.LogInformation("BaseApkName = {BaseApkName}", BaseApkName);

		// Create main OBB with entire contents of staging dir. This
		// includes any PAK files, movie files, etc.

		string LocalObbName = SC.StageDirectory.FullName+".obb";
		string LocalPatchName = SC.StageDirectory.FullName + ".patch.obb";
		string LocalOverflowNameTemplate = SC.StageDirectory.FullName + ".overflow{0}.obb";
		
		if (!Deploy.bNoObbs)
		{
			FileFilter ObbFileFilter = new FileFilter(FileFilterType.Include);
			ConfigHierarchy EngineIni = ConfigCache.ReadHierarchy(ConfigHierarchyType.Engine, DirectoryReference.FromFile(Params.RawProjectPath), UnrealTargetPlatform.Android);
			List<string> ObbFilters;
			EngineIni.GetArray("/Script/AndroidRuntimeSettings.AndroidRuntimeSettings", "ObbFilters", out ObbFilters);
			if (ObbFilters != null)
			{
				ObbFileFilter.AddRules(ObbFilters);
			}
			// Filter out dynamic libraries from obb
			ObbFileFilter.Exclude("*.so");

			List<FileReference> FilesForObb = new List<FileReference>();
			// Add staged Engine files
			{
				string EngineStageDirectoryPath = Path.Combine(SC.StageDirectory.FullName, "Engine");
				FilesForObb.AddRange(ObbFileFilter.ApplyToDirectory(new DirectoryReference(EngineStageDirectoryPath), true));
			}
			// Add staged project files
			{
				string ProjectStageDirectoryPath = Path.Combine(SC.StageDirectory.FullName, SC.ShortProjectName);
				FilesForObb.AddRange(ObbFileFilter.ApplyToDirectory(new DirectoryReference(ProjectStageDirectoryPath), true));
			}

			bool OBBNeedsUpdate = false;

			if (File.Exists(LocalObbName))
			{ 
				System.DateTime OBBTimeStamp = File.GetLastWriteTimeUtc(LocalObbName);
				foreach (FileReference FileToObb in FilesForObb)
				{
					System.DateTime FileTimeStamp = File.GetLastWriteTimeUtc(FileToObb.FullName);
					if (FileTimeStamp > OBBTimeStamp)
					{
						OBBNeedsUpdate = true;
						break;
					}
				}
			}
			else
			{
				OBBNeedsUpdate = true;
			}
			Int64 OBBSizeAllowed = GetMaxOBBSizeAllowed(SC);
			string LimitString = (OBBSizeAllowed < NormalOBBSizeAllowed) ? "1 GiB" : ((OBBSizeAllowed < MaxOBBSizeAllowed) ? "2 GiB" : "4 GiB");

			if (!OBBNeedsUpdate)
			{
				Logger.LogInformation("{Text}", "OBB is up to date: " + LocalObbName);
			}
			else
			{
				// Always delete the target OBB file if it exists
				if (File.Exists(LocalObbName))
				{
					File.Delete(LocalObbName);
				}

				// Always delete the target patch OBB file if it exists
				if (File.Exists(LocalPatchName))
				{
					File.Delete(LocalPatchName);
				}

				// Always delete all target overflow OBB files if they exists
				int OverflowIndex = 1;
				while (File.Exists(string.Format(LocalOverflowNameTemplate, OverflowIndex)))
				{
					File.Delete(string.Format(LocalOverflowNameTemplate, OverflowIndex));
					// move to next overflow
					OverflowIndex++;
				}

				List<FileReference> FilesToObb = FilesForObb;
				List<FileReference> FilesToPatch = new List<FileReference>();
				List<OverflowFileInfo> OverflowInfos = new List<OverflowFileInfo>();

				if (AllowPatchOBBFile(SC))
				{
					int AllowOverflowOBBLimit = AllowOverflowOBBFiles(SC);

					FilesToObb = new List<FileReference>();

					// Collect the filesize and place into Obb or Patch list
					Int64 StagingDirLength = SC.StageDirectory.FullName.Length;
					Int64 MinimumObbSize = 22 + 10;		// EOCD with comment (store version)
					Int64 MainObbSize = MinimumObbSize;
					Int64 PatchObbSize = MinimumObbSize;

					foreach (FileReference FileRef in FilesForObb)
					{
						FileInfo LocalFileInfo = new FileInfo(FileRef.FullName);
						Int64 LocalFileLength = LocalFileInfo.Length;
						Int64 FilenameLength = FileRef.FullName.Length - StagingDirLength - 1;

						Int64 LocalOverhead = (30 + FilenameLength + 36);       // local file descriptor
						Int64 GlobalOverhead = (46 + FilenameLength + 36);      // central directory cost
						Int64 FileRequirements = LocalFileLength + LocalOverhead + GlobalOverhead;

						if (MainObbSize + FileRequirements < OBBSizeAllowed)
						{
							FilesToObb.Add(FileRef);
							MainObbSize += FileRequirements;
						}
						else if (PatchObbSize + FileRequirements < OBBSizeAllowed)
						{
							FilesToPatch.Add(FileRef);
							PatchObbSize += FileRequirements;
						}
						else if (AllowOverflowOBBLimit > 0)
						{
							// find an overflow we can fit this file
							bool bFoundOverflow = false;
							foreach (OverflowFileInfo OverflowRef in OverflowInfos)
							{
								if (OverflowRef.OverflowObbSize + FileRequirements < OBBSizeAllowed)
								{
									OverflowRef.FilesForOverflow.Add(FileRef);
									OverflowRef.OverflowObbSize += FileRequirements;
									bFoundOverflow = true;
									break;
								}
							}

							if (!bFoundOverflow)
							{
								if (AllowOverflowOBBLimit >= OverflowInfos.Count)
								{
									// create a new overflow obb
									if (MinimumObbSize + FileRequirements < OBBSizeAllowed)
									{
										List<FileReference> NewObb = new List<FileReference>();
										NewObb.Add(FileRef);
										OverflowInfos.Add(new OverflowFileInfo(NewObb, MinimumObbSize + FileRequirements));
									}
									else
									{
										Logger.LogInformation("{Text}", "Failed to add " + FileRef.FullName + " to a new overflow as it is bigger than the allowed OBB size ");
										throw new AutomationException(ExitCode.Error_AndroidOBBError, "Stage Failed. Could not add {1} ({0} bytes) to an OBB as OBBs are limited to {2} bytes.", FileRequirements, FileRef.FullName, OBBSizeAllowed);
									}
								}
								else
								{
									Logger.LogInformation("{Text}", "Failed to add required overflow OBB: " + LocalObbName);
									throw new AutomationException(ExitCode.Error_AndroidOBBError, "Stage Failed. Overflow OBBs limited to a count of {0}. Contents are to big to fit.", OverflowInfos.Count);
								}
							}
						}
						else
						{
							// no room in either file and no overflows allowed
							Logger.LogInformation("{Text}", "Failed to build OBB: " + LocalObbName);
							throw new AutomationException(ExitCode.Error_AndroidOBBError, "Stage Failed. Could not build OBB {0}. The file may be too big to fit in an OBB ({1} limit and no overflows are permitted)", LocalObbName, LimitString);
						}
					}
				}

				// Now create the main OBB as a ZIP archive.
				if (!CreateOBBFile(SC, LocalObbName, FilesToObb))
				{
					Logger.LogInformation("{Text}", "Failed to build OBB: " + LocalObbName);
					throw new AutomationException(ExitCode.Error_AndroidOBBError, "Stage Failed. Could not build OBB {0}. The file may be too big to fit in an OBB ({1} limit)", LocalObbName, LimitString);
				}

				// Now create the patch OBB as a ZIP archive if required.
				if (FilesToPatch.Count() > 0)
				{
					if (!CreateOBBFile(SC, LocalPatchName, FilesToPatch))
					{
						Logger.LogInformation("{Text}", "Failed to build OBB: " + LocalPatchName);
						throw new AutomationException(ExitCode.Error_AndroidOBBError, "Stage Failed. Could not build OBB {0}. The file may be too big to fit in an OBB ({1} limit)", LocalPatchName, LimitString);
					}
				}

				OverflowIndex = 1;
				foreach (OverflowFileInfo OverflowRef in OverflowInfos)
				{
					if (OverflowRef.FilesForOverflow.Count() > 0)
					{
						string LocalOverflowName = string.Format(LocalOverflowNameTemplate, OverflowIndex);
						if (!CreateOBBFile(SC, LocalOverflowName, OverflowRef.FilesForOverflow))
						{
							Logger.LogInformation("{Text}", "Failed to build OBB: " + LocalOverflowName);
							throw new AutomationException(ExitCode.Error_AndroidOBBError, "Stage Failed. Could not build OBB {0}. The file may be too big to fit in an OBB ({1} limit)", LocalOverflowName, LimitString);
						}
					}
					OverflowIndex++;
				}
			}

			// make sure the OBB is <= 2GiB (or 4GiB if large OBB enabled)
			FileInfo OBBFileInfo = new FileInfo(LocalObbName);
			Int64 ObbFileLength = OBBFileInfo.Length;
			if (ObbFileLength > OBBSizeAllowed)
			{
				Logger.LogInformation("{Text}", "OBB exceeds " + LimitString + " limit: " + ObbFileLength + " bytes");
				throw new AutomationException(ExitCode.Error_AndroidOBBError, "Stage Failed. OBB {0} exceeds {1} limit)", LocalObbName, LimitString);
			}
		}

		// collect plugin extra data paths from target receipts
		Deploy.SetAndroidPluginData(Architectures, CollectPluginDataPaths(SC));

		ConfigHierarchy Ini = ConfigCache.ReadHierarchy(ConfigHierarchyType.Engine, DirectoryReference.FromFile(Params.RawProjectPath), UnrealTargetPlatform.Android);
		int MinSDKVersion;
		Ini.GetInt32("/Script/AndroidRuntimeSettings.AndroidRuntimeSettings", "MinSDKVersion", out MinSDKVersion);
		int TargetSDKVersion = MinSDKVersion;
		Ini.GetInt32("/Script/AndroidRuntimeSettings.AndroidRuntimeSettings", "TargetSDKVersion", out TargetSDKVersion);
		Logger.LogInformation("{Text}", "Target SDK Version " + TargetSDKVersion);
		bool bDisablePerfHarden = false;
        if (TargetConfiguration != UnrealTargetConfiguration.Shipping)
        {
			Ini.GetBool("/Script/AndroidRuntimeSettings.AndroidRuntimeSettings", "bEnableMaliPerfCounters", out bDisablePerfHarden);
		}

		foreach (UnrealArch Architecture in Architectures.Architectures)
		{
			string ApkName = GetFinalApkName(Params, SC.StageExecutables[0], true, bMakeSeparateApks ? Architecture : null);
			string ApkBareName = GetFinalApkName(Params, SC.StageExecutables[0], true, null);
			bool bHaveAPK = !bEnableBundle;     // do not have a standard APK if bundle enabled
			if (!SC.IsCodeBasedProject)
			{
				string UnrealSOName = GetFinalApkName(Params, SC.StageExecutables[0], false, bMakeSeparateApks ? Architecture : null);
                UnrealSOName = UnrealSOName.Replace(".apk", ".so");
                if (FileExists_NoExceptions(UnrealSOName) == false)
				{
					Logger.LogInformation("{Text}", "Failed to find game .so " + UnrealSOName);
                    throw new AutomationException(ExitCode.Error_MissingExecutable, "Stage Failed. Could not find .so {0}. You may need to build the UE project with your target configuration and platform.", UnrealSOName);
				}
			}
				
			TargetReceipt Receipt = SC.StageTargets[0].Receipt;
				              
			// when we make an embedded executable, all we do is output to libUnreal.so - we don't need to make an APK at all
			// however, we still let package go through to make the .obb file
			string CookFlavor = SC.FinalCookPlatform.IndexOf("_") > 0 ? SC.FinalCookPlatform.Substring(SC.FinalCookPlatform.IndexOf("_")) : "";
			if (!Params.Prebuilt)
			{
				string SOName = GetSONameWithoutArchitecture(Params, SC.StageExecutables[0]);
				bool bShouldCompileAsDll = Receipt.HasValueForAdditionalProperty("CompileAsDll", "true");
				if (bShouldCompileAsDll)
				{
					// MakeApk
					SOName = Receipt.BuildProducts[0].Path.FullName;

					// saving package info, which will allow 
					TargetType Type = TargetType.Game;
					if (CookFlavor.EndsWith("Client"))
					{
						Type = TargetType.Client;
					}
					else if (CookFlavor.EndsWith("Server"))
					{
						Type = TargetType.Server;
					}
					Logger.LogInformation("SavePackageInfo");
					Deploy.SavePackageInfo(Params.ShortProjectName, SC.ProjectRoot.FullName, Type, true);
				}
				Deploy.PrepForUATPackageOrDeploy(Params.RawProjectPath, Params.ShortProjectName, SC.ProjectRoot, SOName, SC.LocalRoot + "/Engine", Params.Distribution, CookFlavor, SC.StageTargets[0].Receipt.Configuration, false, bShouldCompileAsDll, SC.Archive);
			}

			// Create APK specific OBB in case we have a detached OBB.
			string DeviceObbName = "";
			string ObbName = "";
			string DevicePatchName = "";
			string PatchName = "";

			List<OverflowBatchInstallInfo> OverflowInfos = new List<OverflowBatchInstallInfo>();
			
			if (!Deploy.bNoObbs && !bPackageDataInsideApk)
			{
				DeviceObbName = GetDeviceObbName(ApkName, SC);
				ObbName = GetFinalObbName(ApkName, SC);
				CopyFile(LocalObbName, ObbName);

				// apply store version to OBB to make it unique for PlayStore upload
				UpdateObbStoreVersion(ObbName);

				if (File.Exists(LocalPatchName))
				{
					DevicePatchName = GetDevicePatchName(ApkName, SC);
					PatchName = GetFinalPatchName(ApkName, SC);
					CopyFile(LocalPatchName, PatchName);

					// apply store version to OBB to make it unique for PlayStore upload
					UpdateObbStoreVersion(PatchName);
				}

				int OverflowIndex = 1;
				while (File.Exists(string.Format(LocalOverflowNameTemplate, OverflowIndex)))
				{
					string DeviceOverflowName = GetDeviceOverflowName(ApkName, SC, OverflowIndex);
					string OverflowName = GetFinalOverflowName(ApkName, SC, OverflowIndex);

					OverflowInfos.Add(new OverflowBatchInstallInfo(OverflowIndex, DeviceOverflowName, OverflowName, false));
					CopyFile(string.Format(LocalOverflowNameTemplate, OverflowIndex), OverflowName);

					// apply store version to OBB to make it unique for PlayStore upload
					UpdateObbStoreVersion(OverflowName);

					OverflowIndex++;
				}
			}

			// check for optional universal apk
			string APKDirectory = Path.GetDirectoryName(ApkName);
			string APKNameWithoutExtension = Path.GetFileNameWithoutExtension(ApkName);
			string APKBareNameWithoutExtension = Path.GetFileNameWithoutExtension(ApkBareName);
			string UniversalApkName = Path.Combine(APKDirectory, APKNameWithoutExtension + "_universal.apk");
			bool bHaveUniversal = false;
			if (bEnableBundle && bEnableUniversalAPK)
			{
				if (FileExists(UniversalApkName))
				{
					bHaveUniversal = true;
				}
				else
				{
					UniversalApkName = Path.Combine(APKDirectory, APKBareNameWithoutExtension + "_universal.apk");
					if (FileExists(UniversalApkName))
					{
						bHaveUniversal = true;
					}
				}
			}

			//figure out which platforms we need to create install files for
			bool bNeedsPCInstall = false;
			bool bNeedsMacInstall = false;
			bool bNeedsLinuxInstall = false;
			GetPlatformInstallOptions(SC, out bNeedsPCInstall, out bNeedsMacInstall, out bNeedsLinuxInstall);

			//helper delegate to prevent code duplication but allow us access to all the local variables we need
			void CreateInstallFilesAction(UnrealTargetPlatform Target)
			{
				bool bIsPC = (Target == UnrealTargetPlatform.Win64);
				string LineEnding = bIsPC ? "\r\n" : "\n";
				// Write install batch file(s).
				string PackageName = GetPackageInfo(ApkName, SC, false);
				string BatchName = GetFinalBatchName(ApkName, SC, false, EBatchType.Install, Target);
				List<string> InstallBatchLines = GenerateInstallBatchFile(bPackageDataInsideApk, PackageName, ApkName, Params, ObbName, DeviceObbName, false, PatchName, DevicePatchName, false, OverflowInfos,
					bIsPC, Params.Distribution, TargetSDKVersion > 22, bDisablePerfHarden, bUseAFS, bUseAFSProject, AFSToken, Target);
				if (!Deploy.bNoObbs && bHaveAPK)
				{
					// make a batch file that can be used to install the .apk and .obb files
					File.WriteAllText(BatchName, string.Join(LineEnding, InstallBatchLines) + LineEnding);
				}
				// make a batch file that can be used to uninstall the .apk and .obb files
				string UninstallBatchName = GetFinalBatchName(ApkName, SC, false, EBatchType.Uninstall, Target);
				string[] UninstallBatchLines = GenerateUninstallBatchFile(bPackageDataInsideApk, PackageName, ApkName, Params, bIsPC, Target);
				if (bHaveAPK || bHaveUniversal)
				{
					File.WriteAllText(UninstallBatchName, string.Join(LineEnding, UninstallBatchLines) + LineEnding);
				}

				string UniversalBatchName = "";
				if (bHaveUniversal)
				{
					UniversalBatchName = GetFinalBatchName(UniversalApkName, SC, false, EBatchType.Install, Target);
					// make a batch file that can be used to install the .apk
					List<string> UniversalBatchLines = GenerateInstallBatchFile(bPackageDataInsideApk, PackageName, UniversalApkName, Params, ObbName, DeviceObbName, false, PatchName, DevicePatchName, false, OverflowInfos,
						bIsPC, Params.Distribution, TargetSDKVersion > 22, bDisablePerfHarden, bUseAFS, bUseAFSProject, AFSToken, Target);
					File.WriteAllText(UniversalBatchName, string.Join(LineEnding, UniversalBatchLines) + LineEnding);
				}

				string SymbolizeBatchName = GetFinalBatchName(ApkName, SC, false, EBatchType.Symbolize, Target);
				// Deprecated check of bBuildWithHiddenSymbolVisibility. Remove in 5.8+
				if (bBuildWithHiddenSymbolVisibility || bSaveSymbols)
				{
					string[] UniversalSymbolizeBatchLines = GenerateSymbolizeBatchFile(Params, PackageName, ApkName, SC, Architecture, bIsPC);
					File.WriteAllText(SymbolizeBatchName, string.Join(LineEnding, UniversalSymbolizeBatchLines) + LineEnding);
				}

				if (!OperatingSystem.IsWindows())
				{
					if (!Deploy.bNoObbs && bHaveAPK)
					{
						File.SetUnixFileMode(BatchName, File.GetUnixFileMode(BatchName) | UnixFileMode.UserExecute | UnixFileMode.GroupExecute | UnixFileMode.OtherExecute);
					}
					if (bHaveAPK || bHaveUniversal)
					{
						File.SetUnixFileMode(UninstallBatchName, File.GetUnixFileMode(UninstallBatchName) | UnixFileMode.UserExecute | UnixFileMode.GroupExecute | UnixFileMode.OtherExecute);
					}
					if (bHaveUniversal)
					{
						File.SetUnixFileMode(UniversalBatchName, File.GetUnixFileMode(UniversalBatchName) | UnixFileMode.UserExecute | UnixFileMode.GroupExecute | UnixFileMode.OtherExecute);
					}
					// Deprecated check of bBuildWithHiddenSymbolVisibility. Remove in 5.8+
					if (bBuildWithHiddenSymbolVisibility || bSaveSymbols)
					{
						File.SetUnixFileMode(SymbolizeBatchName, File.GetUnixFileMode(SymbolizeBatchName) | UnixFileMode.UserExecute | UnixFileMode.GroupExecute | UnixFileMode.OtherExecute);
					}
					//if(File.Exists(NoInstallBatchName)) 
					//{
					//    File.SetUnixFileMode(NoInstallBatchName, File.GetUnixFileMode(NoInstallBatchName) | UnixFileMode.UserExecute | UnixFileMode.GroupExecute | UnixFileMode.OtherExecute);
					//}
				}
			}

			if (bNeedsPCInstall)
			{
				CreateInstallFilesAction(UnrealTargetPlatform.Win64);
			}
			if (bNeedsMacInstall)
			{
				CreateInstallFilesAction(UnrealTargetPlatform.Mac);
			}
			if (bNeedsLinuxInstall)
			{
				CreateInstallFilesAction(UnrealTargetPlatform.Linux);
			}

			// If we aren't packaging data in the APK then lets write out a bat file to also let us test without the OBB
			// on the device.
			//String NoInstallBatchName = GetFinalBatchName(ApkName, Params, bMakeSeparateApks ? Architecture : "", bMakeSeparateApks ? GPUArchitecture : "", true, false);
			// if(!bPackageDataInsideApk)
			//{
			//    BatchLines = GenerateInstallBatchFile(bPackageDataInsideApk, PackageName, ApkName, Params, ObbName, DeviceObbName, true);
			//    File.WriteAllLines(NoInstallBatchName, BatchLines);
			//}
		}

		PrintRunTime();
	}

	string GetAFSExecutable(UnrealTargetPlatform Target)
	{
		return AndroidExports.GetAFSExecutable(Target, Logger);
	}

	private class OverflowBatchInstallInfo
	{
		public OverflowBatchInstallInfo(int InOverflowIndex, string InDeviceOverflowName, string InOverflowName, bool InNoOverflowInstall)
		{
			OverflowIndex = InOverflowIndex;
			DeviceOverflowName = InDeviceOverflowName;
			OverflowName = InOverflowName;
			bNoOverflowInstall = InNoOverflowInstall;
		}

		public int OverflowIndex { get; }
		public string DeviceOverflowName { get; }
		public string OverflowName { get; }
		public bool bNoOverflowInstall { get; }
	}

	private List<string> GenerateInstallBatchFile(bool bPackageDataInsideApk, string PackageName, string ApkName, ProjectParams Params, string ObbName, string DeviceObbName, bool bNoObbInstall,
		string PatchName, string DevicePatchName, bool bNoPatchInstall, List<OverflowBatchInstallInfo> OverflowInfo,
		bool bIsPC, bool bIsDistribution, bool bRequireRuntimeStoragePermission, bool bDisablePerfHarden, bool bUseAFS, bool bUseAFSProject, string AFSToken, UnrealTargetPlatform Target)
	{
		List<string> BatchLines = new List<string>();
		string ReadPermissionGrantCommand = "shell pm grant " + PackageName + " android.permission.READ_EXTERNAL_STORAGE";
		string WritePermissionGrantCommand = "shell pm grant " + PackageName + " android.permission.WRITE_EXTERNAL_STORAGE";
		string ForegroundPermissionGrantCommand = "shell pm grant " + PackageName + " android.permission.FOREGROUND_SERVICE";
		string ForegroundDataSyncPermissionGrantCommand = "shell pm grant " + PackageName + " android.permission.FOREGROUND_SERVICE_DATA_SYNC";
		string NotificationPermissionGrantCommand = "shell pm grant " + PackageName + " android.permission.POST_NOTIFICATIONS";
		string DisablePerfHardenCommand = "shell setprop security.perf_harden 0";

		string NullCmd = bIsPC ? " >nul 2>&1" : " >/dev/null 2>&1";

		// We don't grant runtime permission for distribution build on purpose since we will push the obb file to the folder that doesn't require runtime storage permission.
		// This way developer can catch permission issue if they try to save/load game file in folder that requires runtime storage permission.
		bool bNeedGrantStoragePermission = bRequireRuntimeStoragePermission && !bIsDistribution;
		bool bNeedGrantForegroundPermission = bUseAFS || bUseAFSProject;

		// We can't always push directly to Android/obb so uploads to Download then moves it
		bool bDontMoveOBB = bUseAFS ? true : bPackageDataInsideApk;

		bool bHavePatch = (PatchName != "");

		string AFSExecutable = GetAFSExecutable(Target);
		string AFSCommonArg = "-p " + PackageName;
		if (AFSToken != "")
		{
			AFSCommonArg += " -k " + AFSToken;
		}

		if (!bIsPC)
		{
			string APKInstallCommand = "$ADB $DEVICE install " + Path.GetFileName(ApkName);
			string APKReinstallCommand = "";

			// If it is a distribution build, push to $STORAGE/Android/obb folder instead of $STORAGE/obb folder.
			// Note that $STORAGE/Android/obb will be the folder that contains the obb if you download the app from playstore.
			string OBBInstallCommand = bNoObbInstall ? "\t$ADB $DEVICE shell 'rm -r $EXTERNAL_STORAGE/" + DeviceObbName + "'" : "\t$ADB $DEVICE push " + Path.GetFileName(ObbName) + " " + TargetAndroidTemp + DeviceObbName;
			string PatchInstallCommand = bNoPatchInstall ? "\t$ADB $DEVICE shell 'rm -r $EXTERNAL_STORAGE/" + DevicePatchName + "'" : "\t$ADB $DEVICE push " + Path.GetFileName(PatchName) + TargetAndroidTemp + DevicePatchName;

			List<string> OverflowInstallCommands = new List<string>();

			if (bUseAFS)
			{
				if (bUseAFSProject)
				{
					APKInstallCommand = "$ADB $DEVICE install AFS_" + Path.GetFileName(ApkName);
					APKReinstallCommand = "$ADB $DEVICE install -r " + Path.GetFileName(ApkName);
				}
				else
				{
					// stop the fileserver (not needed on reinstall above
					APKReinstallCommand = "$AFS $DEVICE " + AFSCommonArg + " stop-all";
				}
				string AFSCommand = "\t$AFS $DEVICE " + AFSCommonArg;
				OBBInstallCommand = bNoObbInstall ? AFSCommand + " deletefile '^mainobb'" : AFSCommand + " push " + Path.GetFileName(ObbName) + " '^mainobb'";
				PatchInstallCommand = bNoPatchInstall ? AFSCommand + " deletefile '^patchobb'" : AFSCommand + " push " + Path.GetFileName(PatchName) + " '^patchobb'";
				if (!bPackageDataInsideApk)
				{
					foreach (OverflowBatchInstallInfo Overflow in OverflowInfo)
					{
						string AFSOverflowName = string.Format("^overflow{0}obb", Overflow.OverflowIndex);
						OverflowInstallCommands.Add(Overflow.bNoOverflowInstall ? AFSCommand + " deletefile '" + AFSOverflowName + "'" : AFSCommand + " push " + Path.GetFileName(Overflow.OverflowName) + " '" + AFSOverflowName + "'");
					}
				}
			}
			else
			{
				if (!bPackageDataInsideApk)
				{
					foreach (OverflowBatchInstallInfo Overflow in OverflowInfo)
					{
						OverflowInstallCommands.Add(Overflow.bNoOverflowInstall ? "\t$ADB $DEVICE shell 'rm -r $EXTERNAL_STORAGE/" + Overflow.DeviceOverflowName + "'" : "\t$ADB $DEVICE push " + Path.GetFileName(Overflow.OverflowName) + " " + TargetAndroidTemp + Overflow.DeviceOverflowName);
					}
				}
			}

			Logger.LogInformation("Writing shell script for install with {Arg0}", bPackageDataInsideApk ? "data in APK" : "separate obb");
			BatchLines.AddRange(new string[] {
						"#!/bin/sh",
						"cd \"`dirname \"$0\"`\"",
						"AFS=./" + AFSExecutable,
						"ADB=adb",
						"if ! command -v -- \"$ADB\" >/dev/null 2>&1; then",
						"\tSTUDIO_SDK_PATH=$ANDROID_HOME",
						"\tif [ ! -d \"$STUDIO_SDK_PATH\" ]; then",
						"\t\tSTUDIO_SDK_PATH=" + (Target == UnrealTargetPlatform.Mac ? "~/Library/Android/sdk" : "$HOME/Android/Sdk"),
						"\t\tif [ ! -d \"$STUDIO_SDK_PATH\" ]; then",
						"\t\t\techo Unable to locate local Android SDK location.",
						"\t\t\texit 1",
						"\t\tfi",
						"\tfi",
						"\tADB=$STUDIO_SDK_PATH/platform-tools/adb",
						"fi",
						"DEVICE=",
						"if [ \"$1\" != \"\" ]; then DEVICE=\"-s $1\"; fi",
						"echo",
						"echo Uninstalling existing application. Failures here can almost always be ignored.",
						"$ADB $DEVICE uninstall " + PackageName,
						"echo",
						"echo Installing existing application. Failures here indicate a problem with the device \\(connection or storage permissions\\) and are fatal.",
						APKInstallCommand,
						"if [ $? -eq 0 ]; then",
						"\techo",
						"\t$ADB $DEVICE shell pm list packages " + PackageName,
						bNeedGrantForegroundPermission ? "\t$ADB $DEVICE " + ForegroundPermissionGrantCommand + NullCmd : "",
						bNeedGrantForegroundPermission ? "\t$ADB $DEVICE " + ForegroundDataSyncPermissionGrantCommand + NullCmd : "",
						bNeedGrantForegroundPermission ? "\t$ADB $DEVICE " + NotificationPermissionGrantCommand + NullCmd : "",
						"\techo",
						"\techo Removing old data. Failures here are usually fine - indicating the files were not on the device.",
						"\t$ADB $DEVICE shell 'rm -r $EXTERNAL_STORAGE/UnrealGame/" + Params.ShortProjectName + "'",
						"\t$ADB $DEVICE shell 'rm -r $EXTERNAL_STORAGE/UnrealGame/UECommandLine.txt" + "'",
						"\t$ADB $DEVICE shell 'rm -r $EXTERNAL_STORAGE/" + TargetAndroidLocation + PackageName + "'",
						"\t$ADB $DEVICE shell 'rm -r $EXTERNAL_STORAGE/Android/" + TargetAndroidLocation + PackageName + "'",
						"\t$ADB $DEVICE shell 'rm -r $EXTERNAL_STORAGE/Download/" + TargetAndroidLocation + PackageName + "'",
						bPackageDataInsideApk ? "" : "\techo",
						bPackageDataInsideApk ? "" : "\techo Installing new data. Failures here indicate storage problems \\(missing SD card or bad permissions\\) and are fatal.",
						bPackageDataInsideApk ? "" : "\tSTORAGE=$(echo \"`$ADB $DEVICE shell 'echo $EXTERNAL_STORAGE'`\" | cat -v | tr -d '^M')",
						bPackageDataInsideApk ? "" : OBBInstallCommand,
						bPackageDataInsideApk ? "if [ 1 ]; then" : "\tif [ $? -eq 0 ]; then",
						!bHavePatch ? "" : (bPackageDataInsideApk ? "" : PatchInstallCommand)});

			BatchLines.AddRange(OverflowInstallCommands);

			BatchLines.AddRange(new string[] {
						bDontMoveOBB ? "" : "\t\t$ADB $DEVICE shell mkdir $STORAGE/Android/" + TargetAndroidLocation + PackageName, // don't check for error since installing may create the obb directory
						bDontMoveOBB ? "" : "\t\t$ADB $DEVICE shell mv " + TargetAndroidTemp + TargetAndroidLocation + PackageName + " $STORAGE/Android/" + TargetAndroidLocation,
						bDontMoveOBB ? "" : "\t\t$ADB $DEVICE shell rm -r " + TargetAndroidTemp + TargetAndroidLocation,
						APKReinstallCommand,
						bNeedGrantStoragePermission ? "\techo Grant READ_EXTERNAL_STORAGE and WRITE_EXTERNAL_STORAGE to the apk for reading OBB or game file in external storage." : "",
						bNeedGrantStoragePermission ? "\t$ADB $DEVICE " + ReadPermissionGrantCommand + NullCmd : "",
						bNeedGrantStoragePermission ? "\t$ADB $DEVICE " + WritePermissionGrantCommand + NullCmd : "",
						bDisablePerfHarden ? "\t$ADB $DEVICE " + DisablePerfHardenCommand : "",
						"\t\techo",
						"\t\techo Installation successful",
						"\t\texit 0",
						"\tfi",
						"fi",
						"echo",
						"echo There was an error installing the game or the obb file. Look above for more info.",
						"echo",
						"echo Things to try:",
						"echo 'Check that the device (and only the device) is listed with \\\"$ADB devices\\\" from a command prompt.'",
						"echo Make sure all Developer options look normal on the device",
						"echo Check that the device has an SD card.",
						"exit 1"
					});
		}
		else
		{
			string APKInstallCommand = "%ADB% %DEVICE% install " + Path.GetFileName(ApkName);
			string APKReinstallCommand = "";

			string OBBInstallCommand = bNoObbInstall ? "%ADB% %DEVICE% shell rm -r %STORAGE%/" + DeviceObbName : "%ADB% %DEVICE% push " + Path.GetFileName(ObbName) + " " + TargetAndroidTemp + DeviceObbName;
			string PatchInstallCommand = bNoPatchInstall ? "%ADB% %DEVICE% shell rm -r %STORAGE%/" + DevicePatchName : "%ADB% %DEVICE% push " + Path.GetFileName(PatchName) + " " + TargetAndroidTemp + DevicePatchName;

			List<string> OverflowInstallCommands = new List<string>();

			if (bUseAFS)
			{
				if (bUseAFSProject)
				{
					APKInstallCommand = "%ADB% %DEVICE% install AFS_" + Path.GetFileName(ApkName);
					APKReinstallCommand = "%ADB% %DEVICE% install -r " + Path.GetFileName(ApkName);
				}
				else
				{
					// stop the fileserver (not needed on reinstall above
					APKReinstallCommand = "%AFS% %DEVICE% " + AFSCommonArg + " stop-all";
				}
				string AFSCommand = "\t%AFS% %DEVICE% " + AFSCommonArg;
				OBBInstallCommand = bNoObbInstall ? AFSCommand + " deletefile \"^mainobb\"" : AFSCommand + " push " + Path.GetFileName(ObbName) + " \"^mainobb\"";
				PatchInstallCommand = bNoPatchInstall ? AFSCommand + " deletefile \"^patchobb\"" : AFSCommand + " push " + Path.GetFileName(PatchName) + " \"^patchobb\"";

				if (!bPackageDataInsideApk)
				{
					foreach (OverflowBatchInstallInfo Overflow in OverflowInfo)
					{
						string AFSOverflowName = string.Format("^overflow{0}obb", Overflow.OverflowIndex);
						OverflowInstallCommands.Add(Overflow.bNoOverflowInstall ? AFSCommand + " deletefile \"" + AFSOverflowName + "\"" : AFSCommand + " push " + Path.GetFileName(Overflow.OverflowName) + " \"" + AFSOverflowName + "\"");
						OverflowInstallCommands.Add("if \"%ERRORLEVEL%\" NEQ \"0\" goto Error");
					}
				}

			}
			else
			{
				if (!bPackageDataInsideApk)
				{
					foreach (OverflowBatchInstallInfo Overflow in OverflowInfo)
					{
						OverflowInstallCommands.Add(Overflow.bNoOverflowInstall ? "%ADB% %DEVICE% shell rm -r %STORAGE%/" + Overflow.DeviceOverflowName : "%ADB% %DEVICE% push " + Path.GetFileName(Overflow.OverflowName) + " " + TargetAndroidTemp + Overflow.DeviceOverflowName);
						OverflowInstallCommands.Add("if \"%ERRORLEVEL%\" NEQ \"0\" goto Error");
					}
				}
			}

			Logger.LogInformation("Writing bat for install with {Arg0}", bPackageDataInsideApk ? "data in APK" : "separate OBB");
			BatchLines.AddRange(new string[] {
						"setlocal ENABLEEXTENSIONS",
						"setlocal ENABLEDELAYEDEXPANSION",
						"if NOT \"%UE_SDKS_ROOT%\"==\"\" (call %UE_SDKS_ROOT%\\HostWin64\\Android\\SetupEnvironmentVars.bat)",
						"set ADB=adb.exe",
						"where.exe /q %ADB%",
						"if /i \"%ERRORLEVEL%\" neq \"0\" (",
						"\tset STUDIO_SDK_PATH=%ANDROID_HOME%",
						"\tif not exist \"!STUDIO_SDK_PATH!\" (",
						"\t\tset STUDIO_SDK_PATH=",
						"\t\tfor /f \"tokens=2*\" %%A in ('reg.exe query \"HKLM\\SOFTWARE\\Android Studio\" /v \"SdkPath\"') do (set STUDIO_SDK_PATH=%%B)",
						"\t\tif not exist \"!STUDIO_SDK_PATH!\" (",
						"\t\t\tset STUDIO_SDK_PATH=%LOCALAPPDATA%\\Android\\Sdk",
						"\t\t\tif not exist \"!STUDIO_SDK_PATH!\" (",
						"\t\t\t\techo Unable to locate local Android SDK location.",
						"\t\t\t\texit /b 1",
						"\t\t\t)",
						"\t\t)",
						"\t)",
						"\tset ADB=!STUDIO_SDK_PATH!\\platform-tools\\adb.exe",
						")",
						"set AFS=.\\" + AFSExecutable.Replace("/", "\\"),
						"set DEVICE=",
						"if not \"%1\"==\"\" set DEVICE=-s %1",
						"for /f \"delims=\" %%A in ('%ADB% %DEVICE% " + GetStorageQueryCommand(true) +"') do @set STORAGE=%%A",
						"@echo.",
						"@echo Uninstalling existing application. Failures here can almost always be ignored.",
						"%ADB% %DEVICE% uninstall " + PackageName,
						"@echo.",
						"@echo Installing existing application. Failures here indicate a problem with the device (connection or storage permissions) and are fatal.",
						APKInstallCommand,
						"@if \"%ERRORLEVEL%\" NEQ \"0\" goto Error",
						"%ADB% %DEVICE% shell pm list packages " + PackageName,
						bNeedGrantForegroundPermission ? "%ADB% %DEVICE% " + ForegroundPermissionGrantCommand + NullCmd : "",
						bNeedGrantForegroundPermission ? "%ADB% %DEVICE% " + ForegroundDataSyncPermissionGrantCommand + NullCmd : "",
						bNeedGrantForegroundPermission ? "%ADB% %DEVICE% " + NotificationPermissionGrantCommand + NullCmd : "",
						"%ADB% %DEVICE% shell rm -r %STORAGE%/UnrealGame/" + Params.ShortProjectName,
						"%ADB% %DEVICE% shell rm -r %STORAGE%/UnrealGame/UECommandLine.txt", // we need to delete the commandline in UnrealGame or it will mess up loading
						"%ADB% %DEVICE% shell rm -r %STORAGE%/" + TargetAndroidLocation + PackageName,
						"%ADB% %DEVICE% shell rm -r %STORAGE%/Android/" + TargetAndroidLocation + PackageName,
						"%ADB% %DEVICE% shell rm -r %STORAGE%/Download/" + TargetAndroidLocation + PackageName,
						bPackageDataInsideApk ? "" : "@echo.",
						bPackageDataInsideApk ? "" : "@echo Installing new data. Failures here indicate storage problems (missing SD card or bad permissions) and are fatal.",
						bPackageDataInsideApk ? "" : OBBInstallCommand,
						bPackageDataInsideApk ? "" : "if \"%ERRORLEVEL%\" NEQ \"0\" goto Error",
						!bHavePatch ? "" : (bPackageDataInsideApk ? "" : PatchInstallCommand),
						!bHavePatch ? "" : (bPackageDataInsideApk ? "" : "if \"%ERRORLEVEL%\" NEQ \"0\" goto Error")});

			BatchLines.AddRange(OverflowInstallCommands);

			BatchLines.AddRange(new string[] {
						bDontMoveOBB ? "" : "%ADB% %DEVICE% shell mkdir %STORAGE%/Android/" + TargetAndroidLocation + PackageName, // don't check for error since installing may create the obb directory
						bDontMoveOBB ? "" : "%ADB% %DEVICE% shell mv " + TargetAndroidTemp + TargetAndroidLocation + PackageName + " %STORAGE%/Android/" + TargetAndroidLocation,
						bDontMoveOBB ? "" : "if \"%ERRORLEVEL%\" NEQ \"0\" goto Error",
						bDontMoveOBB ? "" : "%ADB% %DEVICE% shell rm -r " + TargetAndroidTemp + TargetAndroidLocation,
						APKReinstallCommand,
						bUseAFSProject ? "if \"%ERRORLEVEL%\" NEQ \"0\" goto Error" : "",
						"@echo.",
						bNeedGrantStoragePermission ? "@echo Grant READ_EXTERNAL_STORAGE and WRITE_EXTERNAL_STORAGE to the apk for reading OBB file or game file in external storage." : "",
						bNeedGrantStoragePermission ? "%ADB% %DEVICE% " + ReadPermissionGrantCommand + NullCmd : "",
						bNeedGrantStoragePermission ? "%ADB% %DEVICE% " + WritePermissionGrantCommand + NullCmd : "",
						bDisablePerfHarden ? "%ADB% %DEVICE% " + DisablePerfHardenCommand : "",
						"@echo.",
						"@echo Installation successful",
						"goto:eof",
						":Error",
						"@echo.",
						"@echo There was an error installing the game or the obb file. Look above for more info.",
						"@echo.",
						"@echo Things to try:",
						"@echo Check that the device (and only the device) is listed with \"%ADB$ devices\" from a command prompt.",
						"@echo Make sure all Developer options look normal on the device",
						"@echo Check that the device has an SD card.",
						"@pause"
					});
		}
		return BatchLines;
	}


	private string[] GenerateUninstallBatchFile(bool bPackageDataInsideApk, string PackageName, string ApkName, ProjectParams Params, bool bIsPC, UnrealTargetPlatform Target)
	{
		string[] BatchLines = null;

		if (!bIsPC)
		{
			Logger.LogInformation("Writing shell script for uninstall with {Arg0}", bPackageDataInsideApk ? "data in APK" : "separate obb");
			BatchLines = new string[] {
						"#!/bin/sh",
						"cd \"`dirname \"$0\"`\"",
						"ADB=adb",
						"if ! command -v -- \"$ADB\" >/dev/null 2>&1; then",
						"\tSTUDIO_SDK_PATH=$ANDROID_HOME",
						"\tif [ ! -d \"$STUDIO_SDK_PATH\" ]; then",
						"\t\tSTUDIO_SDK_PATH=" + (Target == UnrealTargetPlatform.Mac ? "~/Library/Android/sdk" : "$HOME/Android/Sdk"),
						"\t\tif [ ! -d \"$STUDIO_SDK_PATH\" ]; then",
						"\t\t\techo Unable to locate local Android SDK location.",
						"\t\t\texit 1",
						"\t\tfi",
						"\tfi",
						"\tADB=$STUDIO_SDK_PATH/platform-tools/adb",
						"fi",
						"DEVICE=",
						"if [ \"$1\" != \"\" ]; then DEVICE=\"-s $1\"; fi",
						"echo",
						"echo Uninstalling existing application. Failures here can almost always be ignored.",
						"$ADB $DEVICE uninstall " + PackageName,
						"echo",
						"echo Removing old data. Failures here are usually fine - indicating the files were not on the device.",
						"$ADB $DEVICE shell 'rm -r $EXTERNAL_STORAGE/UnrealGame/" + Params.ShortProjectName + "'",
						"$ADB $DEVICE shell 'rm -r $EXTERNAL_STORAGE/UnrealGame/UECommandLine.txt" + "'",
						"$ADB $DEVICE shell 'rm -r $EXTERNAL_STORAGE/" + TargetAndroidLocation + PackageName + "'",
						"$ADB $DEVICE shell 'rm -r $EXTERNAL_STORAGE/Android/" + TargetAndroidLocation + PackageName + "'",
						"echo",
						"echo Uninstall completed",
						"exit 0",
					};
		}
		else
		{
			Logger.LogInformation("Writing bat for uninstall with {Arg0}", bPackageDataInsideApk ? "data in APK" : "separate OBB");
			BatchLines = new string[] {
						"setlocal ENABLEEXTENSIONS",
						"setlocal ENABLEDELAYEDEXPANSION",
						"if NOT \"%UE_SDKS_ROOT%\"==\"\" (call %UE_SDKS_ROOT%\\HostWin64\\Android\\SetupEnvironmentVars.bat)",
						"set ADB=adb.exe",
						"where.exe /q %ADB%",
						"if /i \"%ERRORLEVEL%\" neq \"0\" (",
						"\tset STUDIO_SDK_PATH=%ANDROID_HOME%",
						"\tif not exist \"!STUDIO_SDK_PATH!\" (",
						"\t\tset STUDIO_SDK_PATH=",
						"\t\tfor /f \"tokens=2*\" %%A in ('reg.exe query \"HKLM\\SOFTWARE\\Android Studio\" /v \"SdkPath\"') do (set STUDIO_SDK_PATH=%%B)",
						"\t\tif not exist \"!STUDIO_SDK_PATH!\" (",
						"\t\t\tset STUDIO_SDK_PATH=%LOCALAPPDATA%\\Android\\Sdk",
						"\t\t\tif not exist \"!STUDIO_SDK_PATH!\" (",
						"\t\t\t\techo Unable to locate local Android SDK location.",
						"\t\t\t\texit /b 1",
						"\t\t\t)",
						"\t\t)",
						"\t)",
						"\tset ADB=!STUDIO_SDK_PATH!\\platform-tools\\adb.exe",
						")",
						"set DEVICE=",
						"if not \"%1\"==\"\" set DEVICE=-s %1",
						"for /f \"delims=\" %%A in ('%ADB% %DEVICE% " + GetStorageQueryCommand(true) +"') do @set STORAGE=%%A",
						"@echo.",
						"@echo Uninstalling existing application. Failures here can almost always be ignored.",
						"%ADB% %DEVICE% uninstall " + PackageName,
						"@echo.",
						"echo Removing old data. Failures here are usually fine - indicating the files were not on the device.",
						"%ADB% %DEVICE% shell rm -r %STORAGE%/UnrealGame/" + Params.ShortProjectName,
						"%ADB% %DEVICE% shell rm -r %STORAGE%/UnrealGame/UECommandLine.txt", // we need to delete the commandline in UnrealGame or it will mess up loading
						"%ADB% %DEVICE% shell rm -r %STORAGE%/" + TargetAndroidLocation + PackageName,
						"%ADB% %DEVICE% shell rm -r %STORAGE%/Android/" + TargetAndroidLocation + PackageName,
						"@echo.",
						"@echo Uninstall completed",
					};
		}
		return BatchLines;
	}

	private string[] GenerateSymbolizeBatchFile(ProjectParams Params, string PackageName, string ApkName, DeploymentContext SC, UnrealArch Architecture, bool bIsPC)
	{
		string[] BatchLines = null;

		if (!bIsPC)
		{
			Logger.LogInformation("Writing shell script for symbolize with {Arg0}", "data in APK" );
			BatchLines = new string[] {
				"#!/bin/sh",
				"if [ $# -eq 0 ]; then",
				 "echo \"Required argument missing, pass a dump of adb crash log.\"",
				 "exit 1",
				"fi",
				"if [ ! -d \"$NDKROOT\" ]; then",
				 "echo Unable to locate local NDK location.",
				 "exit 1",
				"fi",
				"cd \"`dirname \"$0\"`\"",
				"$NDKROOT/ndk-stack -sym " + GetFinalSymbolizedSODirectory(ApkName, SC, Architecture) + " -dump \"$1\" > " + Params.ShortProjectName + "_SymbolizedCallStackOutput.txt",
				"exit 0",
				};
		}
		else
		{
			Logger.LogInformation("Writing bat for symbolize");
			BatchLines = new string[] {
						"@echo off",
						"IF %1.==. GOTO NoArgs",
						"if \"%NDKROOT%\"==\"\" GOTO NoNDK",
						"setlocal",
						"",
						"%NDKROOT%\\ndk-stack.cmd -sym "+GetFinalSymbolizedSODirectory(ApkName, SC, Architecture)+" -dump \"%1\" > "+ Params.ShortProjectName+"_SymbolizedCallStackOutput.txt",
						"",
						"goto:eof",
						"",
						"",
						":NoArgs",
						"echo.",
						"echo Required argument missing, pass a dump of adb crash log. (SymboliseCallStackDump C:\\adbcrashlog.txt)",
						"pause",
						"",
						"goto:eof",
						"",
						":NoNDK",
						"echo.",
						"echo Unable to locate local NDK location.",
						"pause"
					};
		}
		return BatchLines;
	}

	public override void GetFilesToArchive(ProjectParams Params, DeploymentContext SC)
	{
		if (SC.StageTargetConfigurations.Count != 1)
		{
			throw new AutomationException(ExitCode.Error_OnlyOneTargetConfigurationSupported, "Android is currently only able to package one target configuration at a time, but StageTargetConfigurations contained {0} configurations", SC.StageTargetConfigurations.Count);
		}

		UnrealTargetConfiguration TargetConfiguration = SC.StageTargetConfigurations[0];
		UnrealArchitectures Architectures = GetDeploymentArchitectures(Params, SC);
		bool bMakeSeparateApks = UnrealBuildTool.AndroidExports.ShouldMakeSeparateApks();
		IAndroidDeploy Deploy = UnrealBuildTool.AndroidExports.CreateDeploymentHandler(Params.RawProjectPath, Params.ForcePackageData);
		bool bPackageDataInsideApk = Deploy.GetPackageDataInsideApk();
		bool bBundleEnabled = GetEnableBundle(SC);

		bool bAFSEnablePlugin;
		string AFSToken;
		bool bIsShipping;
		bool bAFSIncludeInShipping;
		bool bAFSAllowExternalStartInShipping;
		UsingAndroidFileServer(Params, SC, out bAFSEnablePlugin, out AFSToken, out bIsShipping, out bAFSIncludeInShipping, out bAFSAllowExternalStartInShipping);
		bool bUseAFS = bAFSEnablePlugin && !bPackageDataInsideApk;
		int AllowOverflowOBBLimit = AllowOverflowOBBFiles(SC);

		List<string> AddedObbFiles = new List<string>();
		foreach (UnrealArch Architecture in Architectures.Architectures)
		{
			string ApkBareName = GetFinalApkName(Params, SC.StageExecutables[0], true, null);
			string ApkName = GetFinalApkName(Params, SC.StageExecutables[0], true, bMakeSeparateApks ? Architecture : null);
			bool bHaveAPK = FileExists(ApkName);
			string ObbName = GetFinalObbName(ApkName, SC);
			string PatchName = GetFinalPatchName(ApkName, SC);
			bool bBuildWithHiddenSymbolVisibility = BuildWithHiddenSymbolVisibility(SC);
			bool bSaveSymbols = GetSaveSymbols(SC);

			string APKDirectory = Path.GetDirectoryName(ApkName);
			string APKNameWithoutExtension = Path.GetFileNameWithoutExtension(ApkName);
			string APKBareNameWithoutExtension = Path.GetFileNameWithoutExtension(ApkBareName);

			bool bHaveAAB = false;
			bool bHaveUniversal = false;

			string AppBundleName = Path.Combine(APKDirectory, APKNameWithoutExtension + ".aab");
			string APKSName = Path.Combine(APKDirectory, APKNameWithoutExtension + ".apks");
			string UniversalApkName = Path.Combine(APKDirectory, APKNameWithoutExtension + "_universal.apk");
			if (bBundleEnabled)
			{
				// copy optional app bundle if exists
				if (FileExists(AppBundleName))
				{
					bHaveAAB = true;
					SC.ArchiveFiles(APKDirectory, Path.GetFileName(AppBundleName));
				}
				else
				{
					AppBundleName = Path.Combine(APKDirectory, APKBareNameWithoutExtension + ".aab");
					if (FileExists(AppBundleName))
					{
						bHaveAAB = true;
						SC.ArchiveFiles(APKDirectory, Path.GetFileName(AppBundleName));
					}
				}

				// copy optional apks (zip of split apks) if exists
				if (FileExists(APKSName))
				{
					SC.ArchiveFiles(APKDirectory, Path.GetFileName(APKSName));
				}
				else
				{
					APKSName = Path.Combine(APKDirectory, APKBareNameWithoutExtension + ".apks");
					if (FileExists(APKSName))
					{
						SC.ArchiveFiles(APKDirectory, Path.GetFileName(APKSName));
					}
				}

				// copy optional universal apk if exists
				if (FileExists(UniversalApkName))
				{
					bHaveUniversal = true;
					SC.ArchiveFiles(APKDirectory, Path.GetFileName(UniversalApkName));
				}
				else
				{
					UniversalApkName = Path.Combine(APKDirectory, APKBareNameWithoutExtension + "_universal.apk");
					if (FileExists(UniversalApkName))
					{
						bHaveUniversal = true;
						SC.ArchiveFiles(APKDirectory, Path.GetFileName(UniversalApkName));
					}
				}
			}

			// copy optional AFSProject apk if exists
			string AFSApkName = Path.Combine(APKDirectory, "AFS_" + APKNameWithoutExtension + ".apk");
			if (FileExists(AFSApkName))
			{
				SC.ArchiveFiles(APKDirectory, Path.GetFileName(AFSApkName));
			}

			// add any other APKs with a prefix
			IEnumerable<string> files = Directory.EnumerateFiles(APKDirectory, "*_" + APKNameWithoutExtension + ".apk", SearchOption.TopDirectoryOnly);
			foreach (string filename in files)
			{
				if (filename != AFSApkName)
				{
					SC.ArchiveFiles(APKDirectory, Path.GetFileName(filename));
				}
			}

			// verify the files exist
			if (!FileExists(ApkName))
			{
				// still valid if we found an AAB
				if (!bHaveAAB)
				{
					throw new AutomationException(ExitCode.Error_AppNotFound, "ARCHIVE FAILED - {0} was not found", ApkName);
				}
			}
			else
			{
				SC.ArchiveFiles(Path.GetDirectoryName(ApkName), Path.GetFileName(ApkName));
			}

			if (!Deploy.bNoObbs && !bPackageDataInsideApk && !FileExists(ObbName))
			{
                throw new AutomationException(ExitCode.Error_ObbNotFound, "ARCHIVE FAILED - {0} was not found", ObbName);
			}

			// Deprecated check of bBuildWithHiddenSymbolVisibility. Remove in 5.8+
			if (bBuildWithHiddenSymbolVisibility || bSaveSymbols)
			{
				string SymbolizedSODirectory = GetFinalSymbolizedSODirectory(ApkName, SC, Architecture);
				string SymbolizedSOPath = Path.Combine(Path.Combine(Path.GetDirectoryName(ApkName), SymbolizedSODirectory), "libUnreal.so");
				if (!FileExists(SymbolizedSOPath))
				{
					throw new AutomationException(ExitCode.Error_SymbolizedSONotFound, "ARCHIVE FAILED - {0} was not found", SymbolizedSOPath);
				}
				// Add symbolized .so directory
				SC.ArchiveFiles(Path.GetDirectoryName(SymbolizedSOPath), Path.GetFileName(SymbolizedSOPath), true, null, SymbolizedSODirectory);

				// copy mapping.txt file if generated
				string SymbolizedBasePath = SymbolizedSODirectory.Substring(0, SymbolizedSODirectory.LastIndexOf("/"));
				string SymbolizedMappingFile = Path.Combine(Path.Combine(Path.GetDirectoryName(ApkName), SymbolizedBasePath), "mapping.txt");
				if (FileExists(SymbolizedMappingFile))
				{
					SC.ArchiveFiles(Path.GetDirectoryName(SymbolizedMappingFile), Path.GetFileName(SymbolizedMappingFile), false, null, SymbolizedBasePath);
				}
			}

			if (!Deploy.bNoObbs && !bPackageDataInsideApk)
			{
				// only add if not already in archive list
				if (!AddedObbFiles.Contains(ObbName))
				{
					AddedObbFiles.Add(ObbName);

					SC.ArchiveFiles(Path.GetDirectoryName(ObbName), Path.GetFileName(ObbName));
					if (FileExists(PatchName))
					{
						SC.ArchiveFiles(Path.GetDirectoryName(PatchName), Path.GetFileName(PatchName));
					}

					for (int Index = 1; Index <= AllowOverflowOBBLimit; Index++)
					{
						string OverflowName = GetFinalOverflowName(ApkName, SC, Index);
						if (FileExists(OverflowName))
						{
							SC.ArchiveFiles(Path.GetDirectoryName(OverflowName), Path.GetFileName(OverflowName));
						}
						else
						{
							break;
						}
					}
				}
			}

			// copy optional unprotected APK if exists
			string UnprotectedApkName = Path.Combine(APKDirectory, "unprotected_" + APKNameWithoutExtension + ".apk");
			if (FileExists(UnprotectedApkName))
			{
				SC.ArchiveFiles(APKDirectory, Path.GetFileName(UnprotectedApkName));
			}

			// copy optional logs directory if exists
			string LogsDirName = Path.Combine(APKDirectory, APKNameWithoutExtension + ".logs");
			if (DirectoryExists(LogsDirName))
			{
				SC.ArchiveFiles(LogsDirName);
			}

			bool bNeedsPCInstall = false;
			bool bNeedsMacInstall = false;
			bool bNeedsLinuxInstall = false;
			GetPlatformInstallOptions(SC, out bNeedsPCInstall, out bNeedsMacInstall, out bNeedsLinuxInstall);

			//helper delegate to prevent code duplication but allow us access to all the local variables we need
			var CreateBatchFilesAndArchiveAction = new Action<UnrealTargetPlatform>(Target =>
			{
				if (bHaveAPK)
				{
					string BatchName = GetFinalBatchName(ApkName, SC, false, EBatchType.Install, Target);
					SC.ArchiveFiles(Path.GetDirectoryName(BatchName), Path.GetFileName(BatchName));
				}
				if (bHaveAPK || bHaveUniversal)
				{
					string UninstallBatchName = GetFinalBatchName(ApkName, SC, false, EBatchType.Uninstall, Target);
					SC.ArchiveFiles(Path.GetDirectoryName(UninstallBatchName), Path.GetFileName(UninstallBatchName));
				}
				if (bHaveUniversal)
				{
					string UniversalBatchName = GetFinalBatchName(UniversalApkName, SC, false, EBatchType.Install, Target);
					SC.ArchiveFiles(Path.GetDirectoryName(UniversalBatchName), Path.GetFileName(UniversalBatchName));
				}

				// Deprecated check of bBuildWithHiddenSymbolVisibility. Remove in 5.8+
				if (bBuildWithHiddenSymbolVisibility || bSaveSymbols)
				{
					string SymbolizeBatchName = GetFinalBatchName(ApkName, SC, false, EBatchType.Symbolize, Target);
					SC.ArchiveFiles(Path.GetDirectoryName(SymbolizeBatchName), Path.GetFileName(SymbolizeBatchName));
				}
				if (bUseAFS && (bHaveAPK || bHaveUniversal))
				{
					SC.ArchiveFiles(Path.Combine(SC.EngineRoot.FullName, "Binaries", "DotNET", "Android", "UnrealAndroidFileTool"), GetAFSExecutable(Target));
				}
				//SC.ArchiveFiles(Path.GetDirectoryName(NoOBBBatchName), Path.GetFileName(NoOBBBatchName));
			}
			);

			//it's possible we will need both PC and Mac/Linux install files, do both
			if (bNeedsPCInstall)
			{
				CreateBatchFilesAndArchiveAction(UnrealTargetPlatform.Win64);
			}
			if (bNeedsMacInstall)
			{
				CreateBatchFilesAndArchiveAction(UnrealTargetPlatform.Mac);
			}
			if (bNeedsLinuxInstall)
			{
				CreateBatchFilesAndArchiveAction(UnrealTargetPlatform.Linux);
			}
		}
	}

	private void GetPlatformInstallOptions(DeploymentContext SC, out bool bNeedsPCInstall, out bool bNeedsMacInstall, out bool bNeedsLinuxInstall)
	{
		ConfigHierarchy Ini = ConfigCache.ReadHierarchy(ConfigHierarchyType.Engine, DirectoryReference.FromFile(SC.RawProjectPath), SC.StageTargetPlatform.PlatformType);
		bool bGenerateAllPlatformInstall = false;
		Ini.GetBool("/Script/AndroidRuntimeSettings.AndroidRuntimeSettings", "bCreateAllPlatformsInstall", out bGenerateAllPlatformInstall);

		bNeedsPCInstall = bNeedsMacInstall = bNeedsLinuxInstall = false;

		if (bGenerateAllPlatformInstall)
		{
			bNeedsPCInstall = bNeedsMacInstall = bNeedsLinuxInstall = true;
		}
		else
		{
			if (HostPlatform.Current.HostEditorPlatform == UnrealTargetPlatform.Mac)
			{
				bNeedsMacInstall = true;
			}
			else if (HostPlatform.Current.HostEditorPlatform == UnrealTargetPlatform.Linux)
			{
				bNeedsLinuxInstall = true;
			}
			else
			{
				bNeedsPCInstall = true;
			}
		}
	}

	private static string GetAdbCommandLine(string SerialNumber, string Args)
	{
	    if (string.IsNullOrEmpty(SerialNumber) == false)
		{
			SerialNumber = "-s " + SerialNumber;
		}

		return string.Format("{0} {1}", SerialNumber, Args);
	}
	private static string GetAFSCommandLine(string SerialNumber, string Args)
	{
	    if (string.IsNullOrEmpty(SerialNumber) == false)
		{
			SerialNumber = "-s " + SerialNumber;
		}

		return string.Format("{0} {1}", SerialNumber, Args);
	}

	static string LastSpewFilename = "";

	public static string ADBSpewFilter(string Message)
	{
		if (Message.StartsWith("[") && Message.Contains("%]"))
		{
			int LastIndex = Message.IndexOf(":");
			LastIndex = LastIndex == -1 ? Message.Length : LastIndex;

			if (Message.Length > 7)
			{
				string Filename = Message.Substring(7, LastIndex - 7);
				if (Filename == LastSpewFilename)
				{
					return null;
				}
				LastSpewFilename = Filename;
			}
			return Message;
		}
		return Message;
	}

	public static IProcessResult RunAdbCommand(ProjectParams Params, string SerialNumber, string Args, string Input = null, ERunOptions Options = ERunOptions.Default, bool bShouldLogCommand = false)
	{
		return RunAdbCommand(SerialNumber, Args, Input, Options, bShouldLogCommand);
	}

	private static IProcessResult RunAdbCommand(string SerialNumber, string Args, string Input = null, ERunOptions Options = ERunOptions.Default, bool bShouldLogCommand = false)
	{
		string AdbCommand = Environment.ExpandEnvironmentVariables("%ANDROID_HOME%/platform-tools/adb" + (OperatingSystem.IsWindows() ? ".exe" : ""));
		if (Options.HasFlag(ERunOptions.AllowSpew) || Options.HasFlag(ERunOptions.SpewIsVerbose))
		{
			LastSpewFilename = "";
			return Run(AdbCommand, GetAdbCommandLine(SerialNumber, Args), Input, Options, SpewFilterCallback: new ProcessResult.SpewFilterCallbackType(ADBSpewFilter));
		}
		return Run(AdbCommand, GetAdbCommandLine(SerialNumber, Args), Input, Options);
	}

	private string RunAndLogAdbCommand(ProjectParams Params, string SerialNumber, string Args, out int SuccessCode)
	{
		string AdbCommand = Environment.ExpandEnvironmentVariables("%ANDROID_HOME%/platform-tools/adb" + (OperatingSystem.IsWindows() ? ".exe" : ""));
		LastSpewFilename = "";
		return RunAndLog(CmdEnv, AdbCommand, GetAdbCommandLine(SerialNumber, Args), out SuccessCode, SpewFilterCallback: new ProcessResult.SpewFilterCallbackType(ADBSpewFilter));
	}

	public static IProcessResult RunAFSCommand(ProjectParams Params, string SerialNumber, string Args, string Input = null, ERunOptions Options = ERunOptions.Default, bool bShouldLogCommand = false)
	{
		return RunAFSCommand(SerialNumber, Args, Input, Options, bShouldLogCommand);
	}

	private static IProcessResult RunAFSCommand(string SerialNumber, string Args, string Input = null, ERunOptions Options = ERunOptions.Default, bool bShouldLogCommand = false)
	{
		string AFSExecutable = AndroidExports.GetAFSExecutable(BuildHostPlatform.Current.Platform, EpicGames.Core.Log.Logger);

		AFSExecutable = Path.Combine(Unreal.EngineDirectory.FullName, "Binaries", "DotNET", "Android", "UnrealAndroidFileTool", AFSExecutable);
		if (Options.HasFlag(ERunOptions.AllowSpew) || Options.HasFlag(ERunOptions.SpewIsVerbose))
		{
			LastSpewFilename = "";
			return Run(AFSExecutable, GetAFSCommandLine(SerialNumber, Args), Input, Options, SpewFilterCallback: new ProcessResult.SpewFilterCallbackType(ADBSpewFilter));
		}
		return Run(AFSExecutable, GetAFSCommandLine(SerialNumber, Args), Input, Options);
	}

	private string RunAndLogAFSCommand(ProjectParams Params, string SerialNumber, string Args, out int SuccessCode)
	{
		string AFSExecutable = AndroidExports.GetAFSExecutable(BuildHostPlatform.Current.Platform, EpicGames.Core.Log.Logger);
		AFSExecutable = Path.Combine(Unreal.EngineDirectory.FullName, "Binaries", "DotNET", "Android", "UnrealAndroidFileTool", AFSExecutable);
		LastSpewFilename = "";
		return RunAndLog(CmdEnv, AFSExecutable, GetAFSCommandLine(SerialNumber, Args), out SuccessCode, SpewFilterCallback: new ProcessResult.SpewFilterCallbackType(ADBSpewFilter));
	}

	public override void GetConnectedDevices(ProjectParams Params, out List<string> Devices)
	{
		Devices = new List<string>();
		IProcessResult Result = RunAdbCommand(Params, "", "devices");

		if (Result.Output.Length > 0)
		{
			string[] LogLines = Result.Output.Split(new char[] { '\n', '\r' });
			bool FoundList = false;
			for (int i = 0; i < LogLines.Length; ++i)
			{
				if (FoundList == false)
				{
					if (LogLines[i].StartsWith("List of devices attached"))
					{
						FoundList = true;
					}
					continue;
				}

				string[] DeviceLine = LogLines[i].Split(new char[] { '\t' });

				if (DeviceLine.Length == 2)
				{
					// the second param should be "device"
					// if it's not setup correctly it might be "unattached" or "powered off" or something like that
					// warning in that case
					if (DeviceLine[1] == "device")
					{
						Devices.Add("@" + DeviceLine[0]);
					}
					else
					{
						Logger.LogWarning("Device attached but in bad state {Arg0}:{Arg1}", DeviceLine[0], DeviceLine[1]);
					}
				}
			}
		}
	}

	/*
	private class TimeRegion : System.IDisposable
	{
		private System.DateTime StartTime { get; set; }

		private string Format { get; set; }

		private System.Collections.Generic.List<object> FormatArgs { get; set; }

		public TimeRegion(string format, params object[] format_args)
		{
			Format = format;
			FormatArgs = new List<object>(format_args);
			StartTime = DateTime.UtcNow;
		}

		public void Dispose()
		{
			double total_time = (DateTime.UtcNow - StartTime).TotalMilliseconds / 1000.0;
			FormatArgs.Insert(0, total_time);
			CommandUtils.Log(Format, FormatArgs.ToArray());
		}
	}
	*/

	private bool RetrieveDeployedManifestsAFS(ProjectParams Params, DeploymentContext SC, string DeviceName, out List<string> UFSManifests, out List<string> NonUFSManifests, string AFSToken)
	{
		UFSManifests = null;
		NonUFSManifests = null;

		UnrealArch? DeviceArchitecture = GetBestDeviceArchitecture(Params, DeviceName);
		string ApkName = GetFinalApkName(Params, SC.StageExecutables[0], true, DeviceArchitecture);
		string PackageName = GetPackageInfo(ApkName, SC, false);

		AndroidFileClient client = new AndroidFileClient(DeviceName);
		if (!client.OpenConnection())
		{
			if (PackageName == null || PackageName == "")
			{
				Logger.LogInformation("Retrieve Manifests: Unable to start file server without package name, ignoring manifests");
				return false;
			}
			Logger.LogInformation("Retrieve Manifests: Trying to start file server {PackageName}", PackageName);
			if (!client.StartServer(PackageName, AFSToken))
			{
				Logger.LogInformation("Retrieve Manifests: Failed to start server, ignoring manifests");
				return false;
			}
		}

		// verify connection to the correct server
		string DevicePackageName = client.Query("^packagename");
		if (DevicePackageName != PackageName)
		{
			if (PackageName == null || PackageName == "")
			{
				Logger.LogInformation("Retrieve Manifests: Unable to start file server without package name, ignoring manifests");
				client.CloseConnection();
				return false;
			}

			Logger.LogInformation("Connected to wrong server {DevicePackageName}, trying again", DevicePackageName);
			client.TerminateServer();

			Logger.LogInformation("Retrieve Manifests: Trying to start file server {PackageName}", PackageName);
			if (!client.StartServer(PackageName, AFSToken))
			{
				Logger.LogInformation("Retrieve Manifests: Failed to start server, ignoring manifests");
				return false;
			}
		}

		// Try retrieving the UFS files manifest files from the device
		string UFSManifestFileName = CombinePaths(SC.StageDirectory.FullName, SC.GetUFSDeployedManifestFileName(DeviceName));
		if (!client.FileRead("^project/" + SC.GetUFSDeployedManifestFileName(null), UFSManifestFileName))
		{
			return false;
		}

		// Try retrieving the non UFS files manifest files from the device
		string NonUFSManifestFileName = CombinePaths(SC.StageDirectory.FullName, SC.GetNonUFSDeployedManifestFileName(DeviceName));
		if (!client.FileRead("^project/" + SC.GetNonUFSDeployedManifestFileName(null), NonUFSManifestFileName))
		{
			return false;
		}

		client.CloseConnection();

		// Return the manifest files
		UFSManifests = new List<string>();
		UFSManifests.Add(UFSManifestFileName);
		NonUFSManifests = new List<string>();
		NonUFSManifests.Add(NonUFSManifestFileName);

		Logger.LogInformation("Retrieve Manifests: Success!!");

		return true;
	}

	public override bool RetrieveDeployedManifests(ProjectParams Params, DeploymentContext SC, string DeviceName, out List<string> UFSManifests, out List<string> NonUFSManifests)
	{
		DeviceName = GetSerialNumber(DeviceName);
		
		bool bAFSEnablePlugin;
		string AFSToken;
		bool bIsShipping;
		bool bAFSIncludeInShipping;
		bool bAFSAllowExternalStartInShipping;
		if (UsingAndroidFileServer(Params, SC, out bAFSEnablePlugin, out AFSToken, out bIsShipping, out bAFSIncludeInShipping, out bAFSAllowExternalStartInShipping))
		{
			return RetrieveDeployedManifestsAFS(Params, SC, DeviceName, out UFSManifests, out NonUFSManifests, AFSToken);
		}

		UFSManifests = null;
		NonUFSManifests = null;

		// Query the storage path from the device
		string DeviceStorageQueryCommand = GetStorageQueryCommand();
		IProcessResult StorageResult = RunAdbCommand(Params, DeviceName, DeviceStorageQueryCommand, null, ERunOptions.AppMustExist);
		String StorageLocation = StorageResult.Output.Trim();
		string RemoteDir = StorageLocation + "/UnrealGame/" + Params.ShortProjectName;

		// Try retrieving the UFS files manifest files from the device
		string RetrievedUFSManifestFileName = CombinePaths(SC.StageDirectory.FullName, $"Retrieved_{SC.GetUFSDeployedManifestFileName(DeviceName)}");
		IProcessResult UFSResult = RunAdbCommand(Params, DeviceName, " pull " + RemoteDir + "/" + SC.GetUFSDeployedManifestFileName(null) + " \"" + RetrievedUFSManifestFileName + "\"", null, ERunOptions.AppMustExist);
		if (!(UFSResult.Output.Contains("bytes") || UFSResult.Output.Contains("[100%]")))
		{
			Logger.LogWarning("Failed retrieving UFS Manifest: {Arg0}", UFSResult.Output);
			return false;
		}

		// Try retrieving the non UFS files manifest files from the device
		string RetrievedNonUFSManifestFileName = CombinePaths(SC.StageDirectory.FullName, $"Retrieved_{SC.GetNonUFSDeployedManifestFileName(DeviceName)}");
		IProcessResult NonUFSResult = RunAdbCommand(Params, DeviceName, " pull " + RemoteDir + "/" + SC.GetNonUFSDeployedManifestFileName(null) + " \"" + RetrievedNonUFSManifestFileName + "\"", null, ERunOptions.AppMustExist);
		if (!(NonUFSResult.Output.Contains("bytes") || NonUFSResult.Output.Contains("[100%]")))
		{
			Logger.LogWarning("Failed retrieving NonUFS Manifest: {Arg0}", NonUFSResult.Output);
			// Did not retrieve both so delete one we did retrieve
			File.Delete(RetrievedUFSManifestFileName);
			return false;
		}

		// Return the manifest files
		UFSManifests = new List<string>();
		UFSManifests.Add(RetrievedUFSManifestFileName);
		NonUFSManifests = new List<string>();
		NonUFSManifests.Add(RetrievedNonUFSManifestFileName);

		return true;
	}

	internal class LongestFirst : IComparer<string>
	{
		public int Compare(string a, string b)
		{
			if (a.Length == b.Length) return a.CompareTo(b);
			else return b.Length - a.Length;
		}
	}

	// Returns a filename from "adb shell ls -RF" output
	// or null if the input line is a directory.
	private string GetFileNameFromListing(string SingleLine)
	{
		if (SingleLine.StartsWith("- ")) // file on Samsung
			return SingleLine.Substring(2);
		else if (SingleLine.StartsWith("d ")) // directory on Samsung
			return null;
		else if (SingleLine.EndsWith("/")) // directory on Google
			return null;
		else // undecorated = file on Google
		{
			return SingleLine;
		}
	}

	private bool WasBuiltWithDontBundleLibrariesInAPK(string FinalSOName)
	{
		// The existence of the Push_..._so.bat file signifies a DontBundleLibrariesInAPK build
		return File.Exists(Path.Combine(Path.GetDirectoryName(FinalSOName)!, "Push_" + Path.GetFileNameWithoutExtension(FinalSOName) + "_so.bat"));
	}

	private void DeployAndroidFileServer(ProjectParams Params, DeploymentContext SC, string AFSToken)
	{
		ConfigHierarchy Ini = ConfigCache.ReadHierarchy(ConfigHierarchyType.Engine, DirectoryReference.FromFile(Params.RawProjectPath), UnrealTargetPlatform.Android);
		Ini.GetBool("/Script/AndroidRuntimeSettings.AndroidRuntimeSettings", "bEnableMaliPerfCounters", out bool bDisablePerfHarden);
		IAndroidDeploy Deploy = AndroidExports.CreateDeploymentHandler(Params.RawProjectPath, Params.ForcePackageData);
		bool bDontBundleLibrariesInAPK = Deploy.GetDontBundleLibrariesInAPK(SC.StageTargets[0].Receipt.Configuration, SC.Archive, true);

		bool bUseCompression;
		bool bLogFiles;
		bool bReportStats;
		bool bUseManualIPAddress;
		string ManualIPAddress;
		EConnectionType ConnectionType = GetAndroidFileServerNetworkConfig(SC, out bUseCompression, out bLogFiles, out bReportStats, out bUseManualIPAddress, out ManualIPAddress);

		AndroidFileClient.OptimalADB adb = new AndroidFileClient.OptimalADB();
		int AllowOverflowOBBLimit = AllowOverflowOBBFiles(SC);

		foreach (string DeviceName in Params.DeviceNames.Select(DeviceName => GetSerialNumber(DeviceName)))
		{
			UnrealArch? DeviceArchitecture = GetBestDeviceArchitecture(Params, DeviceName);
			string ApkName = GetFinalApkName(Params, SC.StageExecutables[0], true, DeviceArchitecture);
			string FinalSOName = GetSOName(Params, SC.StageExecutables[0], DeviceArchitecture);

			// make sure APK is up to date (this is fast if so)
			if (!Params.Prebuilt && !Deploy.bFromMSBuild)
			{
				string CookFlavor = SC.FinalCookPlatform.IndexOf("_") > 0 ? SC.FinalCookPlatform.Substring(SC.FinalCookPlatform.IndexOf("_")) : "";
				string SOName = GetSONameWithoutArchitecture(Params, SC.StageExecutables[0]);
				Deploy.SetAndroidPluginData(GetDeploymentArchitectures(Params, SC), CollectPluginDataPaths(SC));
				Deploy.PrepForUATPackageOrDeploy(Params.RawProjectPath, Params.ShortProjectName, SC.ProjectRoot, SOName, SC.LocalRoot + "/Engine", Params.Distribution, CookFlavor, SC.StageTargets[0].Receipt.Configuration, !SC.Archive, false, SC.Archive);
			}

			// now we can use the apk to get more info
			string PackageName = GetPackageInfo(ApkName, SC, false);

			// start up AFS connection (allowed to fail here.. may not be a server installed yet)
			AndroidFileClient client = new AndroidFileClient(DeviceName);
			AndroidFileClient client2 = null;
			string IPAddress = (ConnectionType == EConnectionType.NetworkOnly && bUseManualIPAddress) ? ManualIPAddress : "127.0.0.1";
			if (!client.OpenConnection(IPAddress))
			{
				if (!client.StartServer(PackageName, AFSToken, IPAddress))
				{
				}
			}

			// Try to get IP address from device if we connected
			string DeviceIPAddress = client.Query("^ip");

			// Setup the OBB name and add the storage path (queried from the device) to it
			string QueryStorageResult = adb.Shell(DeviceName, "echo $EXTERNAL_STORAGE");
			string ExternalStorage = QueryStorageResult.Trim();		// "mnt/sdcard"
			string StorageLocation = ExternalStorage + "/Android";	// "mnt/sdcard/Android"
			string DeviceObbName = StorageLocation + "/" + GetDeviceObbName(ApkName, SC);
			string DevicePatchName = StorageLocation + "/" + GetDevicePatchName(ApkName, SC);
			string RemoteDir = client.Query("^project", true);
			if (RemoteDir == null)
			{
				RemoteDir = StorageLocation + "/data/" + PackageName + "/files/UnrealGame/" + Params.ShortProjectName;
			}
			string ExtFiles = StorageLocation + "/data/" + PackageName + "/files";

			if (bDisablePerfHarden)
			{
				adb.Shell(DeviceName, "setprop security.perf_harden 0");
			}

			// remove any main or patch OBB file which would override deployed data (note: not the same as later delete or deploy)
			{
				string DeviceOldObbName = ExternalStorage + "/" + GetDeviceObbName(ApkName, SC);
				string DeviceOldPatchName = ExternalStorage + "/" + GetDevicePatchName(ApkName, SC);

				adb.Shell(DeviceName, "rm " + DeviceOldObbName);
				adb.Shell(DeviceName, "rm " + DeviceOldPatchName);
			}

			// close connection to since uninstall/reinstall will reset server
			client.CloseConnection();

			// determine if APK out of date
			string APKLastUpdateTime = new FileInfo(ApkName).LastWriteTime.ToString();
			bool bNeedAPKInstall = true;
			bool bFreshInstall = false;
			bool bFastDeploy = false;
			if (Params.IterativeDeploy && !Deploy.bFromMSBuild)
			{
				// Check for apk installed with this package name on the device
				String InstalledResult = adb.Shell(DeviceName, "pm list packages " + PackageName);
				if (InstalledResult.Contains(PackageName))
				{
					Logger.LogInformation("{PackageName} already installed!", PackageName);
					// already installed so enable --fast-deploy option if need to update apk
					bFastDeploy = true;

					// See if apk is up to date on device
					InstalledResult = adb.Shell(DeviceName, "cat " + ExtFiles + "/APKFileStamp.txt");
					if (InstalledResult.StartsWith("APK: "))
					{
						Logger.LogInformation("Found APKFileStamp.txt! {InstalledResult}", InstalledResult);
						if (InstalledResult.Substring(5).Trim() == APKLastUpdateTime)
							bNeedAPKInstall = false;

						if (InstalledResult.Substring(5).Trim() != APKLastUpdateTime)
						{
							Logger.LogInformation("{Arg0} != {APKLastUpdateTime}", InstalledResult.Substring(5).Trim(), APKLastUpdateTime);
						}

						// Stop the previously running copy (uninstall/install did this before)
						InstalledResult = adb.Shell(DeviceName, "am force-stop " + PackageName);
						if (InstalledResult.Contains("Error"))
						{
							// force-stop not supported (Android < 3.0) so check if package is actually running
							// Note: cannot use grep here since it may not be installed on device
							InstalledResult = adb.Shell(DeviceName, "ps");
							if (InstalledResult.Contains(PackageName))
							{
								// it is actually running so use the slow way to kill it (uninstall and reinstall)
								bNeedAPKInstall = true;
							}
						}
					}
				}
				else
				{
					// If not already installed we must do a full deploy
					bFreshInstall = true;
				}
			}

			// install new APK if needed
			if (bNeedAPKInstall)
			{
				if (!Deploy.bFromMSBuild)
				{
					// try reinstall the apk to preserve data
					int SuccessCode = 0;
					string InstallCommandline = "install -r " + (bFastDeploy ? "--fastdeploy \"" : "\"") + ApkName + "\"";
					string InstallOutput = RunAndLogAdbCommand(Params, DeviceName, InstallCommandline, out SuccessCode);
					int FailureIndex = InstallOutput.IndexOf("Failure");
					// adb install doesn't always return an error code on failure, and instead prints "Failure", followed by an error code.
					if (SuccessCode != 0 || FailureIndex != -1)
					{
						string ErrorMessage = string.Format("Installation of apk '{0}' failed", ApkName);
						if (FailureIndex != -1)
						{
							string FailureString = InstallOutput.Substring(FailureIndex + 7).Trim();
							if (FailureString != "")
							{
								ErrorMessage += ": " + FailureString;
							}
						}
						if (ErrorMessage.Contains("OLDER_SDK"))
						{
							Logger.LogError("minSdkVersion is higher than Android version installed on device, possibly due to NDK API Level");
							throw new AutomationException(ExitCode.Error_AppInstallFailed, ErrorMessage);
						}

						// try uninstalling an old app with the same identifier.
						// NOTE: uninstall -k will preserve data/cache.. consider using instead?
						bFreshInstall = true;
						SuccessCode = 0;
						adb.Shell(DeviceName, "pm uninstall " + PackageName);

						// install the apk
						InstallCommandline = "install \"" + ApkName + "\"";
						InstallOutput = RunAndLogAdbCommand(Params, DeviceName, InstallCommandline, out SuccessCode);
						FailureIndex = InstallOutput.IndexOf("Failure");

						// adb install doesn't always return an error code on failure, and instead prints "Failure", followed by an error code.
						if (SuccessCode != 0 || FailureIndex != -1)
						{
							ErrorMessage = string.Format("Installation of apk '{0}' failed", ApkName);
							if (FailureIndex != -1)
							{
								string FailureString = InstallOutput.Substring(FailureIndex + 7).Trim();
								if (FailureString != "")
								{
									ErrorMessage += ": " + FailureString;
								}
							}
							if (ErrorMessage.Contains("OLDER_SDK"))
							{
								Logger.LogError("minSdkVersion is higher than Android version installed on device, possibly due to NDK API Level");
							}
							throw new AutomationException(ExitCode.Error_AppInstallFailed, ErrorMessage);
						}
					}
				}

				// giving EXTERNAL_STORAGE_WRITE permission to the apk for API23+
				// without this permission apk can't access to the assets put into the device
				string ReadPermissionCommandLine = "pm grant " + PackageName + " android.permission.READ_EXTERNAL_STORAGE";
				string WritePermissionCommandLine = "pm grant " + PackageName + " android.permission.WRITE_EXTERNAL_STORAGE";
				adb.Shell(DeviceName, ReadPermissionCommandLine);
				adb.Shell(DeviceName, WritePermissionCommandLine);

				// grant permission for the foreground service to start (otherwise a security violation on Android 28+)
				string ForegroundPermissionGrantCommand = "pm grant " + PackageName + " android.permission.FOREGROUND_SERVICE";
				string ForegroundDataSyncPermissionGrantCommand = "pm grant " + PackageName + " android.permission.FOREGROUND_SERVICE_DATA_SYNC";
				string NotificationsGrantCommand = "pm grant " + PackageName + " android.permission.POST_NOTIFICATIONS";
				adb.Shell(DeviceName, ForegroundPermissionGrantCommand);
				adb.Shell(DeviceName, ForegroundDataSyncPermissionGrantCommand);
				adb.Shell(DeviceName, NotificationsGrantCommand);

				// time for receivers to be registered by pm after install
				Thread.Sleep(350);
			}

			// reopen file server connection (either USB or Network)
			IPAddress = (ConnectionType == EConnectionType.NetworkOnly) ? (bUseManualIPAddress ? ManualIPAddress : (DeviceIPAddress != null ? DeviceIPAddress : "127.0.0.1")) : "127.0.0.1";
			Logger.LogInformation("Attempting to connect to file server [{Arg0}]", (IPAddress == "127.0.0.1" ? "USB" : IPAddress));
			if (!client.OpenConnection(IPAddress))
			{
				Logger.LogInformation("Not connected, attempting to start file server");
				if (!client.StartServer(PackageName, AFSToken, IPAddress))
				{
					// try one more time with longer delay
					Logger.LogInformation("Trying again");
					Thread.Sleep(1000);
					if (!client.StartServer(PackageName, AFSToken, IPAddress))
					{
						Logger.LogWarning("Failed to start Android file server for {PackageName}, skipping deploy for {DeviceName}", PackageName, DeviceName);
						continue;
					}
				}
			}
			else
			{
				// remember for retries
				client.SetActive(PackageName, AFSToken, IPAddress);
			}

			// verify we connected to the right server
			string DevicePackageName = client.Query("^packagename");
			if (DevicePackageName != PackageName)
			{
				Logger.LogInformation("Connected to wrong server {DevicePackageName}, trying again", DevicePackageName);
				client.TerminateServer();

				Logger.LogInformation("Trying to start file server {PackageName}", PackageName);
				if (!client.StartServer(PackageName, AFSToken, IPAddress))
				{
					// try one more time with longer delay
					Logger.LogInformation("Trying again");
					Thread.Sleep(1000);
					if (!client.StartServer(PackageName, AFSToken, IPAddress))
					{
						Logger.LogWarning("Failed to start Android file server for {PackageName}, skipping deploy for {DeviceName}", PackageName, DeviceName);
						continue;
					}
				}
			}

			if (ConnectionType == EConnectionType.Combined)
			{
				IPAddress = (bUseManualIPAddress ? ManualIPAddress : client.Query("^ip"));
				client2 = new AndroidFileClient(DeviceName);

				Logger.LogInformation("Attempting to connect to file server [{IPAddress}]", IPAddress);
				if (!client2.OpenConnection(IPAddress))
				{
					Logger.LogInformation("Not connected, attempting to start file server");
					if (!client2.StartServer(PackageName, AFSToken, IPAddress, false))
					{
						Logger.LogWarning("Failed to start Android file server for {PackageName}, only using one connection for {DeviceName}", PackageName, DeviceName);
						client2 = null;
					}
				}

				// verify we connected to the right server
				DevicePackageName = client2.Query("^packagename");
				if (DevicePackageName != PackageName)
				{
					Logger.LogInformation("Connected to wrong server {DevicePackageName} for [{IPAddress}], not using network", DevicePackageName, IPAddress);
					client2.CloseConnection();
					client2 = null;
				}
			}

			// get RemoteDir again (should be valid now after install / restart)
			RemoteDir = client.Query("^project", true);

			// write new timestamp for APK (do it here since RemoteDir now available)
			if (bNeedAPKInstall && !Deploy.bFromMSBuild)
			{
				client.FileWriteString("APK: " + APKLastUpdateTime + "\n", "^ext/APKFileStamp.txt");
			}

			if (bDontBundleLibrariesInAPK && WasBuiltWithDontBundleLibrariesInAPK(FinalSOName))
			{
				string FinalSONameStripped = Path.Combine(Path.GetDirectoryName(FinalSOName), Path.GetFileNameWithoutExtension(FinalSOName) + "-stripped" + Path.GetExtension(FinalSOName));

				long NewTimestamp = ((DateTimeOffset)File.GetLastWriteTimeUtc(FinalSONameStripped)).ToUnixTimeMilliseconds();
				
				if (!long.TryParse(client.FileReadString("^int/libUnreal.so.timestamp") ?? "", out long OldTimestamp) || OldTimestamp != NewTimestamp)
				{
					Logger.LogInformation("libUnreal.so changed...pushing to device");
					client.PushFile(FinalSONameStripped, "^int/libUnreal.so");
					client.FileWriteString(NewTimestamp.ToString(), "^int/libUnreal.so.timestamp");
				}
			}
			else
			{
				client.FileDelete("^int/libUnreal.so.timestamp");
				client.FileDelete("^int/libUnreal.so");
			}

			if (Deploy.bFromMSBuild)
			{
				// terminate server and close AFS connections
				if (client != null)
				{
					client.TerminateServer();
					client.CloseConnection();
				}
				if (client2 != null)
				{
					client2.TerminateServer();
					client2.CloseConnection();
				}
				continue;
			}

			// update the uecommandline.txt
			// update and deploy uecommandline.txt
			// always delete the existing commandline text file, so it doesn't reuse an old one
			FileReference IntermediateCmdLineFile = FileReference.Combine(SC.StageDirectory, "UECommandLine.txt");
			Project.WriteStageCommandline(IntermediateCmdLineFile, Params, SC);

			// copy files to device if we were staging
			if (SC.Stage)
			{
				HashSet<string> EntriesToDeploy = new HashSet<string>();

				// Fresh install always needs full deploy
				if (Params.IterativeDeploy && !bFreshInstall)
				{
					// always send UECommandLine.txt (it was written above after delta checks applied)
					EntriesToDeploy.Add(IntermediateCmdLineFile.FullName);

					// Add non UFS files if any to deploy
					String NonUFSManifestPath = SC.GetNonUFSDeploymentDeltaPath(DeviceName);
					if (File.Exists(NonUFSManifestPath))
					{
						string NonUFSFiles = File.ReadAllText(NonUFSManifestPath);
						foreach (string Filename in NonUFSFiles.Split('\n'))
						{
							if (!string.IsNullOrEmpty(Filename) && !string.IsNullOrWhiteSpace(Filename))
							{
								EntriesToDeploy.Add(CombinePaths(SC.StageDirectory.FullName, Filename.Trim()));
							}
						}
					}

					// Add UFS files if any to deploy
					String UFSManifestPath = SC.GetUFSDeploymentDeltaPath(DeviceName);
					if (File.Exists(UFSManifestPath))
					{
						string UFSFiles = File.ReadAllText(UFSManifestPath);
						foreach (string Filename in UFSFiles.Split('\n'))
						{
							if (!string.IsNullOrEmpty(Filename) && !string.IsNullOrWhiteSpace(Filename))
							{
								EntriesToDeploy.Add(CombinePaths(SC.StageDirectory.FullName, Filename.Trim()));
							}
						}
					}

					// For now, if too many files may be better to just push them all
					if (EntriesToDeploy.Count > 500)
					{
						// make sure device is at a clean state
						client.DirDeleteRecurse(RemoteDir);

						EntriesToDeploy.Clear();
						EntriesToDeploy.TrimExcess();
						EntriesToDeploy.Add(SC.StageDirectory.FullName);
					}
					else
					{
						// Discover & remove any files on device that are not in staging

						// get listing of remote directory from device
						string CommandResult = client.DirListFlat(RemoteDir);

						if (CommandResult == null)
						{
							Logger.LogWarning("Failed to read remote dir: {RemoteDir}", RemoteDir);
							RemoteDir = client.Query("^project", true);
							CommandResult = client.DirListFlat(RemoteDir);
							if (CommandResult == null)
							{
								Logger.LogWarning("Failed to read remote dir again: {RemoteDir}", RemoteDir);
							}
						}

						{
							// listing output is of the form
							// [Samsung]                 [Google]
							//
							// RemoteDir/RestOfPath:     RemoteDir/RestOfPath:
							// - File1.png               File1.png
							// - File2.txt               File2.txt
							// d SubDir1                 SubDir1/
							// d SubDir2                 Subdir2/
							//
							// RemoteDir/RestOfPath/SubDir1:

							HashSet<string> DirsToDeleteFromDevice = new HashSet<string>();
							List<string> FilesToDeleteFromDevice = new List<string>();

							using (var reader = new StringReader(CommandResult))
							{
								string ProjectSaved = Params.ShortProjectName + "/Saved";
								string ProjectConfig = Params.ShortProjectName + "/Config";
								const string EngineSaved = "Engine/Saved"; // is this safe to use, or should we use SC.EngineRoot.GetDirectoryName()?
								const string EngineConfig = "Engine/Config";
								Logger.LogWarning("Excluding {ProjectSaved} {ProjectConfig} {EngineSaved} {EngineConfig} from clean during deployment.", ProjectSaved, ProjectConfig, EngineSaved, EngineConfig);

								string CurrentDir = "";
								bool SkipFiles = false;
								for (string Line = reader.ReadLine(); Line != null; Line = reader.ReadLine())
								{
									if (String.IsNullOrWhiteSpace(Line))
									{
										continue; // ignore blank lines
									}

									if (Line.EndsWith(":"))
									{
										// RemoteDir/RestOfPath:
										//      keep ^--------^
										CurrentDir = Line.Substring(RemoteDir.Length + 1, Math.Max(0, Line.Length - RemoteDir.Length - 2));
										// Max is there for the case of base "RemoteDir:" --> ""

										// We want to keep config & logs between deployments.
										if (CurrentDir.StartsWith(ProjectSaved) || CurrentDir.StartsWith(ProjectConfig) || CurrentDir.StartsWith(EngineSaved) || CurrentDir.StartsWith(EngineConfig))
										{
											SkipFiles = true;
											continue;
										}

										bool DirExistsInStagingArea = Directory.Exists(Path.Combine(SC.StageDirectory.FullName, CurrentDir));
										if (DirExistsInStagingArea)
										{
											SkipFiles = false;
										}
										else
										{
											// delete directory from device
											SkipFiles = true;
											DirsToDeleteFromDevice.Add(CurrentDir);
										}
									}
									else
									{
										if (SkipFiles)
										{
											continue;
										}

										string FileName = GetFileNameFromListing(Line);
										if (FileName != null)
										{
											bool FileExistsInStagingArea = File.Exists(Path.Combine(SC.StageDirectory.FullName, CurrentDir, FileName));
											if (FileExistsInStagingArea)
											{
												// keep or overwrite
											}
											else if (FileName == "APKFileStamp.txt")
											{
												// keep it
											}
											else
											{
												// delete file from device
												string FilePath = CurrentDir.Length == 0 ? FileName : (CurrentDir + "/" + FileName); // use / for Android target, no matter the development system
												Logger.LogWarning("Deleting {FilePath} from device; not found in staging area", FilePath);
												FilesToDeleteFromDevice.Add(FilePath);
											}
										}
										// We ignore subdirs here as each will have its own "RemoteDir/CurrentDir/SubDir:" entry.
									}
								}
							}

							// delete directories
							foreach (var DirToDelete in DirsToDeleteFromDevice)
							{
								// if a whole tree is to be deleted, don't spend extra commands deleting its branches
								int FinalSlash = DirToDelete.LastIndexOf('/');
								string ParentDir = FinalSlash >= 0 ? DirToDelete.Substring(0, FinalSlash) : "";
								bool ParentMarkedForDeletion = DirsToDeleteFromDevice.Contains(ParentDir);
								if (!ParentMarkedForDeletion)
								{
									Logger.LogWarning("Deleting {DirToDelete} and its contents from device; not found in staging area", DirToDelete);
									client.DirDeleteRecurse(RemoteDir + "/" + DirToDelete);
								}
							}

							// delete loose files
							foreach (var FileToDelete in FilesToDeleteFromDevice)
							{
								client.FileDelete(RemoteDir + "/" + FileToDelete);
							}
						}
					}
				}
				else
				{
					// make sure device is at a clean state
					client.DirDeleteRecurse(RemoteDir);

					// Copy UFS files..
					string[] Files = Directory.GetFiles(SC.StageDirectory.FullName, "*", SearchOption.AllDirectories);
					System.Array.Sort(Files);

					// Find all the files we exclude from copying. And include
					// the directories we need to individually copy.
					HashSet<string> ExcludedFiles = new HashSet<string>();
					SortedSet<string> IndividualCopyDirectories
						= new SortedSet<string>((IComparer<string>)new LongestFirst());
					foreach (string Filename in Files)
					{
						bool Exclude = false;
						// Don't push the apk, we install it
						Exclude |= Path.GetExtension(Filename).Equals(".apk", StringComparison.InvariantCultureIgnoreCase);
						// For excluded files we add the parent dirs to our
						// tracking of stuff to individually copy.
						if (Exclude)
						{
							ExcludedFiles.Add(Filename);
							// We include all directories up to the stage root in having
							// to individually copy the files.
							for (string FileDirectory = Path.GetDirectoryName(Filename);
								!FileDirectory.Equals(SC.StageDirectory);
								FileDirectory = Path.GetDirectoryName(FileDirectory))
							{
								if (!IndividualCopyDirectories.Contains(FileDirectory))
								{
									IndividualCopyDirectories.Add(FileDirectory);
								}
							}
							if (!IndividualCopyDirectories.Contains(SC.StageDirectory.FullName))
							{
								IndividualCopyDirectories.Add(SC.StageDirectory.FullName);
							}
						}
					}

					// The directories are sorted above in "deepest" first. We can
					// therefore start copying those individual dirs which will
					// recreate the tree. As the subtrees will get copied at each
					// possible individual level.
					foreach (string DirectoryName in IndividualCopyDirectories)
					{
						string[] Entries
							= Directory.GetFileSystemEntries(DirectoryName, "*", SearchOption.TopDirectoryOnly);
						foreach (string Entry in Entries)
						{
							// We avoid excluded files and the individual copy dirs
							// (the individual copy dirs will get handled as we iterate).
							if (ExcludedFiles.Contains(Entry) || IndividualCopyDirectories.Contains(Entry))
							{
								continue;
							}
							else
							{
								EntriesToDeploy.Add(Entry);
							}
						}
					}

					if (EntriesToDeploy.Count == 0)
					{
						EntriesToDeploy.Add(SC.StageDirectory.FullName);
					}
				}

				// delete the .obb file, since it will cause nothing we deploy to be used
				client.FileDelete(DeviceObbName);
				client.FileDelete(DevicePatchName);

				// delete existing overflow files on device, one more than the number we have to ensure we do not try to mount an old extra one left on device
				for (int Index = 1; Index <= AllowOverflowOBBLimit; Index++)
				{
					string DeviceOverflowName = StorageLocation + "/" + GetDeviceOverflowName(ApkName, SC, Index); 
					client.FileDelete(DeviceOverflowName);

					// stop if there is no staged overflow of this index
					string OverflowPath = Path.Combine(SC.StageDirectory.FullName, GetFinalOverflowName(ApkName, SC, Index));
					if (!File.Exists(OverflowPath))
					{
						break;
					}
				}

				// We now have a minimal set of file & dir entries we need
				// to deploy. Files we deploy will get individually copied
				// and dirs will get the tree copies by default (that's
				// what ADB does, too).
				Logger.LogInformation("Deploying files using AFS");
				string SourceDir = SC.StageDirectory.FullName;
				client.Deploy(EntriesToDeploy, SourceDir, RemoteDir, bUseCompression, bLogFiles, bReportStats, client2);
			}
			else if (SC.Archive)
			{
				// deploy the obb if there is one
				string ObbPath = Path.Combine(SC.StageDirectory.FullName, GetFinalObbName(ApkName, SC));
				if (File.Exists(ObbPath))
				{
					client.FileWrite(ObbPath, DeviceObbName);
				}

				// deploy the patch if there is one
				string PatchPath = Path.Combine(SC.StageDirectory.FullName, GetFinalPatchName(ApkName, SC));
				if (File.Exists(PatchPath))
				{
					client.FileWrite(PatchPath, DevicePatchName);
				}

				for (int Index = 1; Index <= AllowOverflowOBBLimit; Index++)
				{
					string OverflowPath = Path.Combine(SC.StageDirectory.FullName, GetFinalOverflowName(ApkName, SC, Index));
					if (File.Exists(OverflowPath))
					{
						string DeviceOverflowName = StorageLocation + "/" + GetDeviceOverflowName(ApkName, SC, Index);
						client.FileWrite(OverflowPath, DeviceOverflowName);
					}
					else
					{
						break;
					}
				}
			}
			else
			{
				// cache some strings
				string RemoteFilename = IntermediateCmdLineFile.FullName.Replace(SC.StageDirectory.FullName, RemoteDir).Replace("\\", "/");
				client.FileWrite(IntermediateCmdLineFile.FullName, RemoteFilename);
			}

			// terminate server and close AFS connections
			if (client != null)
			{
				client.TerminateServer();
				client.CloseConnection();
			}
			if (client2 != null)
			{
				client2.TerminateServer();
				client2.CloseConnection();
			}
		}
	}

	private void DeployADB(ProjectParams Params, DeploymentContext SC)
    {
		ConfigHierarchy Ini = ConfigCache.ReadHierarchy(ConfigHierarchyType.Engine, DirectoryReference.FromFile(Params.RawProjectPath), UnrealTargetPlatform.Android);
		Ini.GetBool("/Script/AndroidRuntimeSettings.AndroidRuntimeSettings", "bEnableMaliPerfCounters", out bool bDisablePerfHarden);
		IAndroidDeploy Deploy = AndroidExports.CreateDeploymentHandler(Params.RawProjectPath, Params.ForcePackageData);
		bool bDontBundleLibrariesInAPK = Deploy.GetDontBundleLibrariesInAPK(SC.StageTargets[0].Receipt.Configuration, SC.Archive, true);
		int AllowOverflowOBBLimit = AllowOverflowOBBFiles(SC);

		foreach (string DeviceName in Params.DeviceNames.Select(DeviceName => GetSerialNumber(DeviceName)))
		{
            UnrealArch? DeviceArchitecture = GetBestDeviceArchitecture(Params, DeviceName);
            string ApkName = GetFinalApkName(Params, SC.StageExecutables[0], true, DeviceArchitecture);
            string FinalSOName = GetSOName(Params, SC.StageExecutables[0], DeviceArchitecture);

            // make sure APK is up to date (this is fast if so)
            if (!Params.Prebuilt && !Deploy.bFromMSBuild)
            {
                string CookFlavor = SC.FinalCookPlatform.IndexOf("_") > 0 ? SC.FinalCookPlatform.Substring(SC.FinalCookPlatform.IndexOf("_")) : "";
				string SOName = GetSONameWithoutArchitecture(Params, SC.StageExecutables[0]);
				Deploy.SetAndroidPluginData(GetDeploymentArchitectures(Params, SC), CollectPluginDataPaths(SC));
				Deploy.PrepForUATPackageOrDeploy(Params.RawProjectPath, Params.ShortProjectName, SC.ProjectRoot, SOName, SC.LocalRoot + "/Engine", Params.Distribution, CookFlavor, SC.StageTargets[0].Receipt.Configuration, !SC.Archive, false, SC.Archive);
            }

            // now we can use the apk to get more info
            string PackageName = GetPackageInfo(ApkName, SC, false);

            // Setup the OBB name and add the storage path (queried from the device) to it
            string DeviceStorageQueryCommand = GetStorageQueryCommand();
            IProcessResult Result = RunAdbCommand(Params, DeviceName, DeviceStorageQueryCommand, null, ERunOptions.AppMustExist);
            String StorageLocation = Result.Output.Trim(); // "/mnt/sdcard";
            string DeviceObbName = StorageLocation + "/" + GetDeviceObbName(ApkName, SC);
			string DevicePatchName = StorageLocation + "/" + GetDevicePatchName(ApkName, SC);
			string RemoteDir = StorageLocation + "/UnrealGame/" + Params.ShortProjectName;

			if (bDisablePerfHarden)
			{
				RunAdbCommand(Params, DeviceName, "shell setprop security.perf_harden 0");
			}

			// remove any main or patch OBB file which would override deployed data (note: not the same as later delete or deploy)
			{
				string DeviceOldObbName = StorageLocation + "/Android/" + GetDeviceObbName(ApkName, SC);
				string DeviceOldPatchName = StorageLocation + "/Android/" + GetDevicePatchName(ApkName, SC);

				RunAdbCommand(Params, DeviceName, "shell rm " + DeviceOldObbName);
				RunAdbCommand(Params, DeviceName, "shell rm " + DeviceOldPatchName);
			}

            // determine if APK out of date
            string APKLastUpdateTime = new FileInfo(ApkName).LastWriteTime.ToString();
            bool bNeedAPKInstall = true;
            if (Params.IterativeDeploy && !Deploy.bFromMSBuild)
            {
                // Check for apk installed with this package name on the device
                IProcessResult InstalledResult = RunAdbCommand(Params, DeviceName, "shell pm list packages " + PackageName, null, ERunOptions.AppMustExist);
                if (InstalledResult.Output.Contains(PackageName))
                {
                    // See if apk is up to date on device
                    InstalledResult = RunAdbCommand(Params, DeviceName, "shell cat " + RemoteDir + "/APKFileStamp.txt", null, ERunOptions.AppMustExist);
                    if (InstalledResult.Output.StartsWith("APK: "))
                    {
                        if (InstalledResult.Output.Substring(5).Trim() == APKLastUpdateTime)
                            bNeedAPKInstall = false;

                        // Stop the previously running copy (uninstall/install did this before)
                        InstalledResult = RunAdbCommand(Params, DeviceName, "shell am force-stop " + PackageName, null, ERunOptions.AppMustExist);
                        if (InstalledResult.Output.Contains("Error"))
                        {
                            // force-stop not supported (Android < 3.0) so check if package is actually running
                            // Note: cannot use grep here since it may not be installed on device
                            InstalledResult = RunAdbCommand(Params, DeviceName, "shell ps", null, ERunOptions.AppMustExist);
                            if (InstalledResult.Output.Contains(PackageName))
                            {
                                // it is actually running so use the slow way to kill it (uninstall and reinstall)
                                bNeedAPKInstall = true;
                            }

                        }
                    }
                }
            }

            // install new APK if needed
            if (bNeedAPKInstall && !Deploy.bFromMSBuild)
            {
                // try uninstalling an old app with the same identifier.
                int SuccessCode = 0;
                string UninstallCommandline = "uninstall " + PackageName;
                RunAndLogAdbCommand(Params, DeviceName, UninstallCommandline, out SuccessCode);

                // install the apk
                string InstallCommandline = "install \"" + ApkName + "\"";
                string InstallOutput = RunAndLogAdbCommand(Params, DeviceName, InstallCommandline, out SuccessCode);
                int FailureIndex = InstallOutput.IndexOf("Failure");

                // adb install doesn't always return an error code on failure, and instead prints "Failure", followed by an error code.
                if (SuccessCode != 0 || FailureIndex != -1)
                {
                    string ErrorMessage = string.Format("Installation of apk '{0}' failed", ApkName);
                    if (FailureIndex != -1)
                    {
                        string FailureString = InstallOutput.Substring(FailureIndex + 7).Trim();
                        if (FailureString != "")
                        {
                            ErrorMessage += ": " + FailureString;
                        }
                    }
                    if (ErrorMessage.Contains("OLDER_SDK"))
                    {
                        Logger.LogError("minSdkVersion is higher than Android version installed on device, possibly due to NDK API Level");
                    }
                    throw new AutomationException(ExitCode.Error_AppInstallFailed, ErrorMessage);
                }
                else
                {
                    // giving EXTERNAL_STORAGE_WRITE permission to the apk for API23+
                    // without this permission apk can't access to the assets put into the device
                    string ReadPermissionCommandLine = "shell pm grant " + PackageName + " android.permission.READ_EXTERNAL_STORAGE";
                    string WritePermissionCommandLine = "shell pm grant " + PackageName + " android.permission.WRITE_EXTERNAL_STORAGE";
                    RunAndLogAdbCommand(Params, DeviceName, ReadPermissionCommandLine, out SuccessCode);
                    RunAndLogAdbCommand(Params, DeviceName, WritePermissionCommandLine, out SuccessCode);
                }
            }

			if (bDontBundleLibrariesInAPK && WasBuiltWithDontBundleLibrariesInAPK(FinalSOName))
			{
				string FinalSOPathStripped = Path.Combine(Path.GetDirectoryName(FinalSOName), Path.GetFileNameWithoutExtension(FinalSOName) + "-stripped" + Path.GetExtension(FinalSOName));
				string FinalSOFileNameStripped = Path.GetFileName(FinalSOPathStripped);
				
				long NewTimestamp = ((DateTimeOffset)File.GetLastWriteTimeUtc(FinalSOPathStripped)).ToUnixTimeMilliseconds();
				
				if (!long.TryParse(RunAdbCommand(Params, DeviceName, "shell cat ./files/libUnreal.so.timestamp", null, ERunOptions.AppMustExist).Output, out long OldTimestamp) || OldTimestamp != NewTimestamp)
				{
					int SuccessCode = 0;
					string PushSO = $"push -z lz4 \"{FinalSOPathStripped}\" /data/local/tmp/";
					string MkdirDestDirSO = $"shell run-as {PackageName} mkdir -p ./files";
					string CopySO = $"shell run-as {PackageName} cp \"/data/local/tmp/{FinalSOFileNameStripped}\" ./files/libUnreal.so";
					string DeleteSO = $"shell rm \"/data/local/tmp/{FinalSOFileNameStripped}\"";
					RunAndLogAdbCommand(Params, DeviceName, PushSO, out SuccessCode);
					RunAndLogAdbCommand(Params, DeviceName, MkdirDestDirSO, out SuccessCode);
					RunAndLogAdbCommand(Params, DeviceName, CopySO, out SuccessCode);
					RunAndLogAdbCommand(Params, DeviceName, DeleteSO, out SuccessCode);

					if (SuccessCode != 0)
					{
						string ErrorMessage = $"Installation of '{PackageName}' failed due to failing to push libUnreal.so outside of {ApkName}";
						throw new AutomationException(ExitCode.Error_AppInstallFailed, ErrorMessage);
					}
					
					RunAndLogAdbCommand(Params, DeviceName, $"shell \"echo '{NewTimestamp}' > ./files/libUnreal.so.timestamp\"", out SuccessCode);
				}
			}
			else
			{
				int SuccessCode;
				RunAndLogAdbCommand(Params, DeviceName, "shell rm ./files/libUnreal.so.timestamp", out SuccessCode);
				RunAndLogAdbCommand(Params, DeviceName, "shell rm ./files/libUnreal.so", out SuccessCode);
			}

			if (Deploy.bFromMSBuild)
			{
				continue;
			}

            // update the uecommandline.txt
            // update and deploy uecommandline.txt
            // always delete the existing commandline text file, so it doesn't reuse an old one
            FileReference IntermediateCmdLineFile = FileReference.Combine(SC.StageDirectory, "UECommandLine.txt");
            Project.WriteStageCommandline(IntermediateCmdLineFile, Params, SC);

			// copy files to device if we were staging
			if (SC.Stage)
			{
				// cache some strings
				string BaseCommandline = "push";

				HashSet<string> EntriesToDeploy = new HashSet<string>();

				if (Params.IterativeDeploy)
				{
					// always send UECommandLine.txt (it was written above after delta checks applied)
					EntriesToDeploy.Add(IntermediateCmdLineFile.FullName);

					// Add non UFS files if any to deploy
					String NonUFSManifestPath = SC.GetNonUFSDeploymentDeltaPath(DeviceName);
					if (File.Exists(NonUFSManifestPath))
					{
						string NonUFSFiles = File.ReadAllText(NonUFSManifestPath);
						foreach (string Filename in NonUFSFiles.Split('\n'))
						{
							if (!string.IsNullOrEmpty(Filename) && !string.IsNullOrWhiteSpace(Filename))
							{
								EntriesToDeploy.Add(CombinePaths(SC.StageDirectory.FullName, Filename.Trim()));
							}
						}
					}

					// Add UFS files if any to deploy
					String UFSManifestPath = SC.GetUFSDeploymentDeltaPath(DeviceName);
					if (File.Exists(UFSManifestPath))
					{
						string UFSFiles = File.ReadAllText(UFSManifestPath);
						foreach (string Filename in UFSFiles.Split('\n'))
						{
							if (!string.IsNullOrEmpty(Filename) && !string.IsNullOrWhiteSpace(Filename))
							{
								EntriesToDeploy.Add(CombinePaths(SC.StageDirectory.FullName, Filename.Trim()));
							}
						}
					}

					// For now, if too many files may be better to just push them all
					if (EntriesToDeploy.Count > 500)
					{
						// make sure device is at a clean state
						RunAdbCommand(Params, DeviceName, "shell rm -r " + RemoteDir);

						EntriesToDeploy.Clear();
						EntriesToDeploy.TrimExcess();
						EntriesToDeploy.Add(SC.StageDirectory.FullName);
					}
					else
					{
						// Discover & remove any files on device that are not in staging

						// get listing of remote directory from device
						string Commandline = "shell ls -RF1 " + RemoteDir;
						var CommandResult = RunAdbCommand(Params, DeviceName, Commandline, null, ERunOptions.AppMustExist);
						// CommandResult.ExitCode is adb shell's exit code, not ls exit code, which is what we need.
						// Check output for error message instead.
						if (CommandResult.Output.StartsWith("ls: "))
						{
							// list command failed, try simpler options
							Commandline = "shell ls -RF " + RemoteDir;
							CommandResult = RunAdbCommand(Params, DeviceName, Commandline, null, ERunOptions.AppMustExist);
						}

						if (CommandResult.Output.StartsWith("ls: "))
						{
							// list command failed, so clean the remote dir instead of selectively deleting files
							RunAdbCommand(Params, DeviceName, "shell rm -r " + RemoteDir);
						}
						else
						{
							// listing output is of the form
							// [Samsung]                 [Google]
							//
							// RemoteDir/RestOfPath:     RemoteDir/RestOfPath:
							// - File1.png               File1.png
							// - File2.txt               File2.txt
							// d SubDir1                 SubDir1/
							// d SubDir2                 Subdir2/
							//
							// RemoteDir/RestOfPath/SubDir1:

							HashSet<string> DirsToDeleteFromDevice = new HashSet<string>();
							List<string> FilesToDeleteFromDevice = new List<string>();

							using (var reader = new StringReader(CommandResult.Output))
							{
								string ProjectSaved = Params.ShortProjectName + "/Saved";
								string ProjectConfig = Params.ShortProjectName + "/Config";
								const string EngineSaved = "Engine/Saved"; // is this safe to use, or should we use SC.EngineRoot.GetDirectoryName()?
								const string EngineConfig = "Engine/Config";
								Logger.LogWarning("Excluding {ProjectSaved} {ProjectConfig} {EngineSaved} {EngineConfig} from clean during deployment.", ProjectSaved, ProjectConfig, EngineSaved, EngineConfig);

								string CurrentDir = "";
								bool SkipFiles = false;
								for (string Line = reader.ReadLine(); Line != null; Line = reader.ReadLine())
								{
									if (String.IsNullOrWhiteSpace(Line))
									{
										continue; // ignore blank lines
									}

									if (Line.EndsWith(":"))
									{
										// RemoteDir/RestOfPath:
										//      keep ^--------^
										CurrentDir = Line.Substring(RemoteDir.Length + 1, Math.Max(0, Line.Length - RemoteDir.Length - 2));
										// Max is there for the case of base "RemoteDir:" --> ""

										// We want to keep config & logs between deployments.
										if (CurrentDir.StartsWith(ProjectSaved) || CurrentDir.StartsWith(ProjectConfig) || CurrentDir.StartsWith(EngineSaved) || CurrentDir.StartsWith(EngineConfig))
										{
											SkipFiles = true;
											continue;
										}

										bool DirExistsInStagingArea = Directory.Exists(Path.Combine(SC.StageDirectory.FullName, CurrentDir));
										if (DirExistsInStagingArea)
										{
											SkipFiles = false;
										}
										else
										{
											// delete directory from device
											SkipFiles = true;
											DirsToDeleteFromDevice.Add(CurrentDir);
										}
									}
									else
									{
										if (SkipFiles)
										{
											continue;
										}

										string FileName = GetFileNameFromListing(Line);
										if (FileName != null)
										{
											bool FileExistsInStagingArea = File.Exists(Path.Combine(SC.StageDirectory.FullName, CurrentDir, FileName));
											if (FileExistsInStagingArea)
											{
												// keep or overwrite
											}
											else if (FileName == "APKFileStamp.txt")
											{
												// keep it
											}
											else
											{
												// delete file from device
												string FilePath = CurrentDir.Length == 0 ? FileName : (CurrentDir + "/" + FileName); // use / for Android target, no matter the development system
												Logger.LogWarning("Deleting {FilePath} from device; not found in staging area", FilePath);
												FilesToDeleteFromDevice.Add(FilePath);
											}
										}
										// We ignore subdirs here as each will have its own "RemoteDir/CurrentDir/SubDir:" entry.
									}
								}
							}

							// delete directories
							foreach (var DirToDelete in DirsToDeleteFromDevice)
							{
								// if a whole tree is to be deleted, don't spend extra commands deleting its branches
								int FinalSlash = DirToDelete.LastIndexOf('/');
								string ParentDir = FinalSlash >= 0 ? DirToDelete.Substring(0, FinalSlash) : "";
								bool ParentMarkedForDeletion = DirsToDeleteFromDevice.Contains(ParentDir);
								if (!ParentMarkedForDeletion)
								{
									Logger.LogWarning("Deleting {DirToDelete} and its contents from device; not found in staging area", DirToDelete);
									RunAdbCommand(Params, DeviceName, "shell rm -r " + RemoteDir + "/" + DirToDelete);
								}
							}

							// delete loose files
							if (FilesToDeleteFromDevice.Count > 0)
							{
								// delete all stray files with one command
								Commandline = String.Format("shell cd {0}; rm ", RemoteDir);
								RunAdbCommand(Params, DeviceName, Commandline + String.Join(" ", FilesToDeleteFromDevice));
							}
						}
					}
				}
				else
				{
					// make sure device is at a clean state
					RunAdbCommand(Params, DeviceName, "shell rm -r " + RemoteDir);

					// Copy UFS files..
					string[] Files = Directory.GetFiles(SC.StageDirectory.FullName, "*", SearchOption.AllDirectories);
					System.Array.Sort(Files);

					// Find all the files we exclude from copying. And include
					// the directories we need to individually copy.
					HashSet<string> ExcludedFiles = new HashSet<string>();
					SortedSet<string> IndividualCopyDirectories
						= new SortedSet<string>((IComparer<string>)new LongestFirst());
					foreach (string Filename in Files)
					{
						bool Exclude = false;
						// Don't push the apk, we install it
						Exclude |= Path.GetExtension(Filename).Equals(".apk", StringComparison.InvariantCultureIgnoreCase);
						// For excluded files we add the parent dirs to our
						// tracking of stuff to individually copy.
						if (Exclude)
						{
							ExcludedFiles.Add(Filename);
							// We include all directories up to the stage root in having
							// to individually copy the files.
							for (string FileDirectory = Path.GetDirectoryName(Filename);
								!FileDirectory.Equals(SC.StageDirectory);
								FileDirectory = Path.GetDirectoryName(FileDirectory))
							{
								if (!IndividualCopyDirectories.Contains(FileDirectory))
								{
									IndividualCopyDirectories.Add(FileDirectory);
								}
							}
							if (!IndividualCopyDirectories.Contains(SC.StageDirectory.FullName))
							{
								IndividualCopyDirectories.Add(SC.StageDirectory.FullName);
							}
						}
					}

					// The directories are sorted above in "deepest" first. We can
					// therefore start copying those individual dirs which will
					// recreate the tree. As the subtrees will get copied at each
					// possible individual level.
					foreach (string DirectoryName in IndividualCopyDirectories)
					{
						string[] Entries
							= Directory.GetFileSystemEntries(DirectoryName, "*", SearchOption.TopDirectoryOnly);
						foreach (string Entry in Entries)
						{
							// We avoid excluded files and the individual copy dirs
							// (the individual copy dirs will get handled as we iterate).
							if (ExcludedFiles.Contains(Entry) || IndividualCopyDirectories.Contains(Entry))
							{
								continue;
							}
							else
							{
								EntriesToDeploy.Add(Entry);
							}
						}
					}

					if (EntriesToDeploy.Count == 0)
					{
						EntriesToDeploy.Add(SC.StageDirectory.FullName);
					}
				}

				// We now have a minimal set of file & dir entries we need
				// to deploy. Files we deploy will get individually copied
				// and dirs will get the tree copies by default (that's
				// what ADB does).
				HashSet<IProcessResult> DeployCommands = new HashSet<IProcessResult>();
				foreach (string Entry in EntriesToDeploy)
				{
					string FinalRemoteDir = RemoteDir;
					string RemotePath = Entry.Replace(SC.StageDirectory.FullName, FinalRemoteDir).Replace("\\", "/");
					string Commandline = string.Format("{0} \"{1}\" \"{2}\"", BaseCommandline, Entry, RemotePath);
					// We run deploy commands in parallel to maximize the connection
					// throughput.
					DeployCommands.Add(
						RunAdbCommand(Params, DeviceName, Commandline, null,
							ERunOptions.Default | ERunOptions.NoWaitForExit));
					// But we limit the parallel commands to avoid overwhelming
					// memory resources.
					if (DeployCommands.Count == DeployMaxParallelCommands)
					{
						while (DeployCommands.Count > DeployMaxParallelCommands / 2)
						{
							Thread.Sleep(1);
							DeployCommands.RemoveWhere(
								delegate (IProcessResult r)
								{
									return r.HasExited;
								});
						}
					}
				}
				foreach (IProcessResult deploy_result in DeployCommands)
				{
					deploy_result.WaitForExit();
				}

				// delete the .obb file, since it will cause nothing we just deployed to be used
				RunAdbCommand(Params, DeviceName, "shell rm " + DeviceObbName);
				RunAdbCommand(Params, DeviceName, "shell rm " + DevicePatchName);

				// delete existing overflow files on device, one more than the number we have to ensure we do not try to mount an old extra one left on device
				for (int Index = 1; Index <= AllowOverflowOBBLimit; Index++)
				{
					string DeviceOverflowName = StorageLocation + "/" + GetDeviceOverflowName(ApkName, SC, Index);
					RunAdbCommand(Params, DeviceName, "shell rm " + DeviceOverflowName);

					string OverflowPath = Path.Combine(SC.StageDirectory.FullName, GetFinalOverflowName(ApkName, SC, Index));
					if (!File.Exists(OverflowPath))
					{
						break;
					}
				}
			}
			else if (SC.Archive)
            {
                // deploy the obb if there is one
                string ObbPath = Path.Combine(SC.StageDirectory.FullName, GetFinalObbName(ApkName, SC));
                if (File.Exists(ObbPath))
                {
                    // cache some strings
                    string BaseCommandline = "push";

                    string Commandline = string.Format("{0} \"{1}\" \"{2}\"", BaseCommandline, ObbPath, DeviceObbName);
                    RunAdbCommand(Params, DeviceName, Commandline);
                }

				// deploy the patch if there is one
				string PatchPath = Path.Combine(SC.StageDirectory.FullName, GetFinalPatchName(ApkName, SC));
				if (File.Exists(PatchPath))
				{
					// cache some strings
					string BaseCommandline = "push";

					string Commandline = string.Format("{0} \"{1}\" \"{2}\"", BaseCommandline, PatchPath, DevicePatchName);
					RunAdbCommand(Params, DeviceName, Commandline);
				}

				// deploy the overflows
				for (int Index = 1; Index <= AllowOverflowOBBLimit; Index++)
				{
					string OverflowPath = Path.Combine(SC.StageDirectory.FullName, GetFinalOverflowName(ApkName, SC, Index));
					if (File.Exists(OverflowPath))
					{
						// cache some strings
						string BaseCommandline = "push";
						string DeviceOverflowName = StorageLocation + "/" + GetDeviceOverflowName(ApkName, SC, Index);

						string Commandline = string.Format("{0} \"{1}\" \"{2}\"", BaseCommandline, OverflowPath, DeviceOverflowName);
						RunAdbCommand(Params, DeviceName, Commandline);
					}
					else
					{
						break;
					}
				}
			}
			else
            {
                // cache some strings
                string BaseCommandline = "push";

                string FinalRemoteDir = RemoteDir;
                /*
			    // handle the special case of the UECommandline.txt when using content only game (UnrealGame)
			    if (!Params.IsCodeBasedProject)
			    {
				    FinalRemoteDir = "/mnt/sdcard/UnrealGame";
			    }
			    */

                string RemoteFilename = IntermediateCmdLineFile.FullName.Replace(SC.StageDirectory.FullName, FinalRemoteDir).Replace("\\", "/");
                string Commandline = string.Format("{0} \"{1}\" \"{2}\"", BaseCommandline, IntermediateCmdLineFile, RemoteFilename);
                RunAdbCommand(Params, DeviceName, Commandline);
            }

            // write new timestamp for APK (do it here since RemoteDir will now exist)
            if (bNeedAPKInstall)
            {
                int SuccessCode = 0;
                RunAndLogAdbCommand(Params, DeviceName, "shell \"echo 'APK: " + APKLastUpdateTime + "' > " + RemoteDir + "/APKFileStamp.txt\"", out SuccessCode);
            }
        }
    }

	public override void Deploy(ProjectParams Params, DeploymentContext SC)
	{
		bool bAFSEnablePlugin;
		string AFSToken;
		bool bIsShipping;
		bool bAFSIncludeInShipping;
		bool bAFSAllowExternalStartInShipping;

		// Pick the proper deploy method
		if (UsingAndroidFileServer(Params, SC, out bAFSEnablePlugin, out AFSToken, out bIsShipping, out bAFSIncludeInShipping, out bAFSAllowExternalStartInShipping))
		{
			DeployAndroidFileServer(Params, SC, AFSToken);
		}
		else
		{
			DeployADB(Params, SC);
		}
	}

	/** Internal usage for GetPackageName */
	private static string PackageLine = null;
	private static Mutex PackageInfoMutex = new Mutex();
	private static string LaunchableActivityLine = null;
	private static string MetaAppTypeLine = null;
	private static Dictionary<string,string> MetaDataMap = null;

	/** Run an external exe (and capture the output), given the exe path and the commandline. */
	public static string GetPackageInfo(string ApkName, bool bRetrieveVersionCode)
	{
		string ReturnValue = null;

		// we expect there to be one, so use the first one
		string AaptPath = GetAaptPath();

		PackageInfoMutex.WaitOne();

		if (File.Exists(ApkName))
		{
			var ExeInfo = new ProcessStartInfo(AaptPath, "dump --include-meta-data badging \"" + ApkName + "\"");
			ExeInfo.UseShellExecute = false;
			ExeInfo.RedirectStandardOutput = true;
			using (var GameProcess = Process.Start(ExeInfo))
			{
				PackageLine = null;
				LaunchableActivityLine = null;
				MetaAppTypeLine = null;
				MetaDataMap = null;
				GameProcess.BeginOutputReadLine();
				GameProcess.OutputDataReceived += ParsePackageName;
				GameProcess.WaitForExit();
			}

			PackageInfoMutex.ReleaseMutex();
			
			if (PackageLine != null)
			{
				// the line should look like: package: name='com.epicgames.qagame' versionCode='1' versionName='1.0'
				string[] Tokens = PackageLine.Split("'".ToCharArray());
				int TokenIndex = bRetrieveVersionCode ? 3 : 1;
				if (Tokens.Length >= TokenIndex + 1)
				{
					ReturnValue = Tokens[TokenIndex];
				}
			}
			Logger.LogInformation("GetPackageInfo ReturnValue: {ReturnValue}", ReturnValue);
		}

		return ReturnValue;
	}

	public static string GetPackageInfo(string ApkName, DeploymentContext SC, bool bRetrieveVersionCode)
	{
		string ReturnValue = GetPackageInfo(ApkName, bRetrieveVersionCode);

		if (ReturnValue == null || ReturnValue.Length == 0)
		{
			// If APK does not exist or we cant find package info in apk use the packageInfo file
			ReturnValue = GetPackageInfoFromInfoFile(ApkName, SC, bRetrieveVersionCode);
		}

		return ReturnValue;
	}

	/** Lookup package info in packageInfo.txt file in same directory as the APK would have been */
	private static string GetPackageInfoFromInfoFile(string ApkName, DeploymentContext SC, bool bRetrieveVersionCode)
	{
		string ReturnValue = null;
		String PackageInfoPath = Path.Combine(Path.GetDirectoryName(ApkName), "packageInfo.txt");
		Boolean fileExists = File.Exists(PackageInfoPath);
		if (fileExists)
		{
			string[] Lines = File.ReadAllLines(PackageInfoPath);
			int LineIndex = bRetrieveVersionCode ? 1 : 0;
			Logger.LogInformation("packageInfo line index: {LineIndex}", LineIndex);
			if (Lines.Length >= 2)
			{
				ReturnValue = Lines[LineIndex];
			}
			// parse extra info that the aapt-based method got
			MetaAppTypeLine = Lines[3];
		}

		if (bRetrieveVersionCode)
		{
			int StoreVersion = 1;
			int.TryParse(ReturnValue, out StoreVersion);

			int StoreVersionOffset = 0;
			ConfigHierarchy Ini = ConfigCache.ReadHierarchy(ConfigHierarchyType.Engine, DirectoryReference.FromFile(SC.RawProjectPath), SC.StageTargetPlatform.PlatformType);
			if (ApkName.Contains("-arm64-"))
			{
				Ini.GetInt32("/Script/AndroidRuntimeSettings.AndroidRuntimeSettings", "StoreVersionOffsetArm64", out StoreVersionOffset);
			}
			else if (ApkName.Contains("-x64-"))
			{
				Ini.GetInt32("/Script/AndroidRuntimeSettings.AndroidRuntimeSettings", "StoreVersionOffsetX8664", out StoreVersionOffset);
			}
			StoreVersion += StoreVersionOffset;
			ReturnValue = StoreVersion.ToString("0");
		}

		Logger.LogInformation("packageInfo.txt file exists: {fileExists}", fileExists);
		Logger.LogInformation("packageInfo return MetaAppTypeLine: {MetaAppTypeLine}", MetaAppTypeLine);
		Logger.LogInformation("packageInfo return value: {ReturnValue}", ReturnValue);

		return ReturnValue;
	}

	/** Returns the launch activity name to launch (must call GetPackageInfo first), returns "com.epicgames.unreal.SplashActivity" default if not found */
	public static string GetLaunchableActivityName()
	{
		string ReturnValue = DefaultLaunchActivity;
		if (LaunchableActivityLine != null)
		{
			// the line should look like: launchable-activity: name='com.epicgames.unreal.SplashActivity'  label='TappyChicken' icon=''
			string[] Tokens = LaunchableActivityLine.Split("'".ToCharArray());
			if (Tokens.Length >= 2)
			{
				ReturnValue = Tokens[1];
			}
		}
		return ReturnValue;
	}

	public static string GetLaunchableActivityName(string ApkName)
	{
		if (!string.IsNullOrEmpty(ApkName) && LaunchableActivityLine == null)
		{
			GetPackageInfo(ApkName, false);
		}
		return GetLaunchableActivityName();
	}

	/** Returns the app type from the packaged APK metadata, returns "" if not found */
	public static string GetMetaAppType()
	{
		string ReturnValue = "";
		if (MetaAppTypeLine != null)
		{
			// the line should look like: meta-data: name='com.epicgames.unreal.GameActivity.AppType' value='Client'
			string[] Tokens = MetaAppTypeLine.Split("'".ToCharArray());
			if (Tokens.Length >= 4)
			{
				ReturnValue = Tokens[3];
			}
		}
		return ReturnValue;
	}

	public static string GetMetadataValue(string MetadataKey)
	{
		if(MetaDataMap != null)
		{
			string MetadataValue;
			if ( MetaDataMap.TryGetValue(MetadataKey, out MetadataValue) )
			{
				return MetadataValue;
			}
		}
		return null;
	}

	/** Simple function to pipe output asynchronously */
	private static void ParsePackageName(object Sender, DataReceivedEventArgs Event)
	{
		// DataReceivedEventHandler is fired with a null string when the output stream is closed.  We don't want to
		// print anything for that event.
		if (!String.IsNullOrEmpty(Event.Data))
		{
			string Line = Event.Data;
			if (PackageLine == null)
			{
				if (Line.StartsWith("package:"))
				{
					PackageLine = Line;
				}
			}
			if (LaunchableActivityLine == null)
			{
				if (Line.StartsWith("launchable-activity:"))
				{
					LaunchableActivityLine = Line;
				}
			}
			if (MetaAppTypeLine == null)
			{
				if (Line.StartsWith("meta-data: name='com.epicgames.unreal.GameActivity.AppType'"))
				{
					MetaAppTypeLine = Line;
				}
			}
			if(Line.StartsWith("meta-data: name='com.epicgames.unreal.GameActivity"))
			{
				// We expect the meta-data string to be in the format of " meta-data: name='...' value='...' "
				Match MetaDataMatch = Regex.Match(Line, @"meta-data: name='com.epicgames.unreal.GameActivity.(.*?)'.*?'(.*?)'");
				if(MetaDataMatch.Groups.Count == 3)
				{
					if (MetaDataMap == null)
					{
						MetaDataMap = new Dictionary<string, string>();
					}
					try
					{
						MetaDataMap.Add(MetaDataMatch.Groups[1].Value, MetaDataMatch.Groups[2].Value);
					}
					catch (Exception ex)
					{
						Logger.LogWarning("{Text}", @"Ignoring duplicate package metadata entry '"+Line+"\'.\n" + ex.ToString());
					}
				}
				else
				{
					Logger.LogWarning("{Text}", "Unexpected layout of package metadata: " + Line);
				}
			}
		}
	}

	static private string CachedAaptPath = null;
	static private string LastAndroidHomePath = null;

	private static uint GetRevisionValue(string VersionString)
	{
		// read up to 4 sections (ie. 20.0.3.5), first section most significant
		// each section assumed to be 0 to 255 range
		uint Value = 0;
		try
		{
			string[] Sections= VersionString.Split(".".ToCharArray());
			Value |= (Sections.Length > 0) ? (uint.Parse(Sections[0]) << 24) : 0;
			Value |= (Sections.Length > 1) ? (uint.Parse(Sections[1]) << 16) : 0;
			Value |= (Sections.Length > 2) ? (uint.Parse(Sections[2]) <<  8) : 0;
			Value |= (Sections.Length > 3) ?  uint.Parse(Sections[3])        : 0;
		}
		catch (Exception)
		{
			// ignore poorly formed version
		}
		return Value;
	}	

	private static string GetAaptPath()
	{
		// return cached path if ANDROID_HOME has not changed
        string HomePath = Environment.ExpandEnvironmentVariables("%ANDROID_HOME%");
		if (CachedAaptPath != null && LastAndroidHomePath == HomePath)
		{
			return CachedAaptPath;
		}

		// get a list of the directories in build-tools.. may be more than one set installed (or none which is bad)
		string[] Subdirs = Directory.GetDirectories(Path.Combine(HomePath, "build-tools"));
        if (Subdirs.Length == 0)
        {
            throw new AutomationException(ExitCode.Error_AndroidBuildToolsPathNotFound, "Failed to find %ANDROID_HOME%/build-tools subdirectory. Run SDK manager and install build-tools.");
        }

		// valid directories will have a source.properties with the Pkg.Revision (there is no guarantee we can use the directory name as revision)
		string BestToolPath = null;
		uint BestVersion = 0;
		foreach (string CandidateDir in Subdirs)
		{
			string AaptFilename = Path.Combine(CandidateDir, OperatingSystem.IsWindows() ? "aapt.exe" : "aapt");
			uint RevisionValue = 0;

			if (File.Exists(AaptFilename))
			{
				string SourcePropFilename = Path.Combine(CandidateDir, "source.properties");
				if (File.Exists(SourcePropFilename))
				{
					string[] PropertyContents = File.ReadAllLines(SourcePropFilename);
					foreach (string PropertyLine in PropertyContents)
					{
						if (PropertyLine.StartsWith("Pkg.Revision="))
						{
							RevisionValue = GetRevisionValue(PropertyLine.Substring(13));
							break;
						}
					}
				}
			}

			// remember it if newer version or haven't found one yet
			if (RevisionValue > BestVersion || BestToolPath == null)
			{
				BestVersion = RevisionValue;
				BestToolPath = AaptFilename;
			}
		}

		if (BestToolPath == null)
		{
            throw new AutomationException(ExitCode.Error_AndroidBuildToolsPathNotFound, "Failed to find %ANDROID_HOME%/build-tools subdirectory with aapt. Run SDK manager and install build-tools.");
		}

		CachedAaptPath = BestToolPath;
		LastAndroidHomePath = HomePath;

		Logger.LogInformation("Using this aapt: {CachedAaptPath}", CachedAaptPath);

		return CachedAaptPath;
	}
	
	static string GetIniValue(string Path, string Key)
	{
		try
		{
			using StreamReader Reader = File.OpenText(Path);
					
			for (;;)
			{
				string Line = Reader.ReadLine();

				if (Line == null)
				{
					break;
				}

				string[] Parts = Line.Split('=', 2);

				if (Parts[0].Trim() == Key)
				{
					return Parts[1].Trim();
				}
			}
		}
		catch
		{
		}

		return null;
	}

	static string GetSerialNumber(string DeviceName)
	{
		if (DeviceName.StartsWith("avd-"))
		{
			string AvdName = DeviceName[4..];

			(string, string) QuerySerialNumber()
			{
				return RunAdbCommand(null, "devices", null, ERunOptions.AppMustExist).Output.Split(Environment.NewLine, StringSplitOptions.RemoveEmptyEntries).Skip(1).Where(Line => Line.StartsWith("emulator-")).Select(Line => Line.Split('\t', 2)).Where(Parts =>
				{
					IProcessResult Result = RunAdbCommand(Parts[0], "emu avd name", null, ERunOptions.AppMustExist);
					return Result.bExitCodeSuccess && Result.Output[0..Result.Output.IndexOf(Environment.NewLine, StringComparison.Ordinal)] == AvdName;
				}).Select(Parts => (Parts[0], Parts[1])).FirstOrDefault();
			}

			(string SerialNumber, string Status) = QuerySerialNumber();

			if (SerialNumber == null)
			{
				using Process Process = Process.Start(new ProcessStartInfo(Environment.ExpandEnvironmentVariables("%ANDROID_HOME%/emulator/emulator"), ["-avd", AvdName])	{ UseShellExecute = false, CreateNoWindow = true, WindowStyle = ProcessWindowStyle.Hidden })!;
				
				for (uint RetryCount = 60;; --RetryCount)
				{
					bool bExited = Process.WaitForExit(1000);
						
					(SerialNumber, Status) = QuerySerialNumber();

					if (bExited || SerialNumber != null || RetryCount == 0)
					{
						break;
					}
				}
			}
			
			string QueryStatus()
			{
				return RunAdbCommand(null, "devices", null, ERunOptions.AppMustExist).Output.Split(Environment.NewLine, StringSplitOptions.RemoveEmptyEntries).Skip(1).Select(Line => Line.Split('\t', 2)).Where(Parts => Parts[0] == SerialNumber).Select(Parts => Parts[1]).FirstOrDefault();
			}
			
			for (uint RetryCount = 60; Status != "device"; --RetryCount)
			{
				Logger.LogInformation("Device status for '{DeviceName}': {Status}", DeviceName, Status);
			
				if (string.IsNullOrEmpty(Status) || RetryCount == 0)
				{
					throw new AutomationException(ExitCode.Error_UnknownLaunchFailure, "Unable to run because couldn't connect to device '{0}'", DeviceName);
				}
				
				Thread.Sleep(1000);

				Status = QueryStatus();
			}

			return SerialNumber;
		}
		else
		{
			return DeviceName;
		}
	}

	private UnrealArch? GetBestDeviceArchitecture(ProjectParams Params, string DeviceName)
	{
		bool bMakeSeparateApks = UnrealBuildTool.AndroidExports.ShouldMakeSeparateApks();
		// if we are joining all .so's into a single .apk, there's no need to find the best one - there is no other one
		if (!bMakeSeparateApks)
		{
			return null;
		}

		// @todo get TargetName?
		UnrealArchitectures AppArchitectures = Params.ClientArchitecture ?? UnrealArchitectureConfig.ForPlatform(UnrealTargetPlatform.Android).ActiveArchitectures(Params.RawProjectPath, null);

		// ask the device
		IProcessResult ABIResult = RunAdbCommand(Params, DeviceName, " shell getprop ro.product.cpu.abi", null, ERunOptions.AppMustExist);

		// the output is just the architecture
		UnrealArch DeviceArch = UnrealBuildTool.AndroidExports.GetUnrealArch(ABIResult.Output.Trim());

		// if the architecture wasn't built, look for a backup
		if (!AppArchitectures.Contains(DeviceArch))
		{
			// Houdini emulation can run arm64 on intel
			if (DeviceArch == UnrealArch.X64)
			{
				DeviceArch = UnrealArch.Arm64;
			}
		}

		// if after the fallbacks, we still don't have it, we can't continue
		if (!AppArchitectures.Contains(DeviceArch))
		{
            throw new AutomationException(ExitCode.Error_NoApkSuitableForArchitecture, "Unable to run because you don't have an apk that is usable on {0}. Looked for {1}", DeviceName, DeviceArch);
		}

		return DeviceArch;
	}

	private bool DeployClientCmdLineAFS(ProjectParams Params, string DeviceName, string PackageName, string AFSToken, string ClientCmdLine)
	{
		AndroidFileClient client = new AndroidFileClient(DeviceName);
		if (!client.OpenConnection())
		{
			Logger.LogInformation("DeployClientCmdLine: Trying to start file server {PackageName}", PackageName);
			if (!client.StartServer(PackageName, AFSToken))
			{
				Logger.LogInformation("DeployClientCmdLine: Failed to start server {PackageName}, ignoring client command line", PackageName);
				return false;
			}
		}

		// verify connection to the correct server
		string DevicePackageName = client.Query("^packagename");
		if (DevicePackageName != PackageName)
		{
			Logger.LogInformation("DeployClientCmdLine: Connected to wrong server {DevicePackageName}, trying again", DevicePackageName);
			client.TerminateServer();

			Logger.LogInformation("DeployClientCmdLine: Trying to start file server {PackageName}", PackageName);
			if (!client.StartServer(PackageName, AFSToken))
			{
				Logger.LogInformation("DeployClientCmdLine: Failed to start server {PackageName}, ignoring client command line", PackageName);
				return false;
			}
		}

		Logger.LogInformation("Writing ClientCmdLine to remote ^commandfile: {ClientCmdLine}", ClientCmdLine);
		client.FileWriteString(ClientCmdLine, "^commandfile");
		client.TerminateServer();
		client.CloseConnection();
		return true;
	}

	private bool DeployClientCmdLineADB(ProjectParams Params, string DeviceName, string PackageName, string ClientCmdLine)
	{
		string DeviceStorageQueryCommand = GetStorageQueryCommand();
		IProcessResult StorageResult = RunAdbCommand(Params, DeviceName, DeviceStorageQueryCommand, null, ERunOptions.AppMustExist);
		string StorageLocation = StorageResult.Output.Trim();
		string RemoteDir = StorageLocation + "/UnrealGame/" + Params.ShortProjectName;
		string ClientCmdLineTmpFile = Path.GetTempFileName();
		string ClientCmdLineRemoteFile = RemoteDir + "/UECommandLine.txt";
		File.WriteAllText(ClientCmdLineTmpFile, ClientCmdLine);
		Logger.LogInformation("Pushing ClientCmdLine to remote file {ClientCmdLineRemoteFile}: {ClientCmdLine}", ClientCmdLineRemoteFile, ClientCmdLine);
		RunAdbCommand(DeviceName, String.Format("push {0} {1}", ClientCmdLineTmpFile, ClientCmdLineRemoteFile));
		File.Delete(ClientCmdLineTmpFile);
		return true;
	}

	public override void ModifyFileHostAddresses(List<string> HostAddresses)
	{
		HostAddresses.Insert(0, "127.0.0.1");
	}

	public override IProcessResult RunClient(ERunOptions ClientRunFlags, string ClientApp, string ClientCmdLine, ProjectParams Params)
	{
		IProcessResult Result = null;

		string LogPath = Path.Combine(Params.BaseStageDirectory, "Android\\logs");
		Directory.CreateDirectory(LogPath);

		foreach (string DeviceName in Params.DeviceNames.Select(DeviceName => GetSerialNumber(DeviceName)))
		{
			//get the package name and save that
			UnrealArch? DeviceArchitecture = GetBestDeviceArchitecture(Params, DeviceName);

			//strip off the device, GPU architecture and extension (.so)
			int DashIndex = ClientApp.LastIndexOf("-");
			if (DashIndex >= 0)
			{
				ClientApp = ClientApp.Substring(0, DashIndex);
			}

			string ApkName = GetFinalApkName(Params, Path.GetFileNameWithoutExtension(ClientApp), true, DeviceArchitecture);

			if (!File.Exists(ApkName))
			{
				throw new AutomationException(ExitCode.Error_AppNotFound, "Failed to find application " + ApkName);
			}

			// run aapt to get the name of the intent
			string PackageName = GetPackageInfo(ApkName, false);
			if (PackageName == null)
			{
				throw new AutomationException(ExitCode.Error_FailureGettingPackageInfo, "Failed to get package name from " + ClientApp);
			}

			var canReadClientCmdLineViaAmStart = Params.ClientConfigsToBuild.Count > 0 &&
			                                     Params.ClientConfigsToBuild.First() != UnrealTargetConfiguration.Shipping;

			// push ClientCmdLine args as a file to the device to override the stage/apk command line if we can't push it via am start
			if (!canReadClientCmdLineViaAmStart)
			{
				bool bAFSEnablePlugin;
				string AFSToken;
				bool bIsShipping;
				bool bAFSIncludeInShipping;
				bool bAFSAllowExternalStartInShipping;
				if (UsingAndroidFileServer(Params, null, out bAFSEnablePlugin, out AFSToken, out bIsShipping, out bAFSIncludeInShipping, out bAFSAllowExternalStartInShipping))
				{
					DeployClientCmdLineAFS(Params, DeviceName, PackageName, AFSToken, ClientCmdLine);
				}
				else
				{
					DeployClientCmdLineADB(Params, DeviceName, PackageName, ClientCmdLine);
				}
			}

			// Message back to the Unreal Editor to correctly set the app id for each device
			Logger.LogInformation("Running Package@Device:{PackageName}@{DeviceName}", PackageName, DeviceName);

			// clear the log for the device
			RunAdbCommand(DeviceName, "logcat -c");

			// start the app on device!
			var CommandLine = "shell am start -n " + PackageName + "/" + GetLaunchableActivityName();
			if (canReadClientCmdLineViaAmStart)
			{
				var ToScapeChars = new string[]{" ", "(", ")", "`", "$", "%", "&"};
				
				var ClientSessionCmdLineEscaped = ClientCmdLine.Replace("\"", "\\\\\\\"");
				foreach( var ToScape in ToScapeChars )
				{
					ClientSessionCmdLineEscaped = ClientSessionCmdLineEscaped.Replace(ToScape, "\\" + ToScape);
				}
				CommandLine += " --es cmdline \"" + ClientSessionCmdLineEscaped + "\"";
			}
			RunAdbCommand(DeviceName, CommandLine);

			// wait before getting the process list with "adb shell ps" from AdbCreatedProcess
			// on some devices the list is not yet ready
			Thread.Sleep(2000);

			// Start logging process and return immediately.
			// Stdout from the title is continuosly emitted to stdout in UAT.
			// When process is done the AdbCreatedProcess wrapper will save the output to the log directories
			Result = RunAdbCommand(DeviceName, "logcat -s UE debug Debug DEBUG", null, ClientRunFlags | ERunOptions.NoWaitForExit);
			Result = new AdbCreatedProcess(Result, LogPath, PackageName, DeviceName);
		}

		return Result;
	}

	public override void GetFilesToDeployOrStage(ProjectParams Params, DeploymentContext SC)
	{
		// Add any Android shader cache files
		DirectoryReference ProjectShaderDir = DirectoryReference.Combine(Params.RawProjectPath.Directory, "Build", "ShaderCaches", "Android");
		if(DirectoryReference.Exists(ProjectShaderDir))
		{
			SC.StageFiles(StagedFileType.UFS, ProjectShaderDir, StageFilesSearch.AllDirectories);
		}
	}

    /// <summary>
    /// Gets cook platform name for this platform.
    /// </summary>
    /// <returns>Cook platform string.</returns>
    public override string GetCookPlatform(bool bDedicatedServer, bool bIsClientOnly)
	{
		return bIsClientOnly ? "AndroidClient" : "Android";
	}

	public override bool DeployLowerCaseFilenames(StagedFileType FileType)
	{
		return false;
	}

	public override string LocalPathToTargetPath(string LocalPath, string LocalRoot)
	{
		return LocalPath.Replace("\\", "/").Replace(LocalRoot, "../../..");
	}

	public override bool IsSupported { get { return true; } }

	public override PakType RequiresPak(ProjectParams Params)
	{
		// if packaging is enabled, always create a pak, otherwise use the Params.Pak value
		return Params.Package ? PakType.Always : PakType.DontCare;
	}
    public override bool SupportsMultiDeviceDeploy
    {
        get
        {
            return true;
        }
    }

    /*
        public override bool RequiresPackageToDeploy
        {
            get { return true; }
        }
    */

	public override List<string> GetDebugFileExtensions()
	{
		return new List<string> { };
	}

	public override void StripSymbols(FileReference SourceFile, FileReference TargetFile)
	{
		AndroidExports.StripSymbols(SourceFile, TargetFile, Log.Logger);
	}

	public override void StripAllSymbols(FileReference SourceFile, FileReference TargetFile)
	{
		AndroidExports.StripSymbols(SourceFile, TargetFile, Log.Logger, true);
	}
}

public class AndroidPlatformMulti : AndroidPlatform
{
    public override string GetCookPlatform(bool bDedicatedServer, bool bIsClientOnly)
    {
		return bIsClientOnly ? "Android_MultiClient" : "Android_Multi";
    }
    public override TargetPlatformDescriptor GetTargetPlatformDescriptor()
    {
        return new TargetPlatformDescriptor(TargetPlatformType, "Multi");
    }
}

public class AndroidPlatformDXT : AndroidPlatform
{
    public override string GetCookPlatform(bool bDedicatedServer, bool bIsClientOnly)
    {
		return bIsClientOnly ? "Android_DXTClient" : "Android_DXT";
    }

    public override TargetPlatformDescriptor GetTargetPlatformDescriptor()
    {
        return new TargetPlatformDescriptor(TargetPlatformType, "DXT");
    }
}
public class AndroidPlatformETC2 : AndroidPlatform
{
    public override string GetCookPlatform(bool bDedicatedServer, bool bIsClientOnly)
    {
		return bIsClientOnly ? "Android_ETC2Client" : "Android_ETC2";
    }
    public override TargetPlatformDescriptor GetTargetPlatformDescriptor()
    {
        return new TargetPlatformDescriptor(TargetPlatformType, "ETC2");
    }
}

public class AndroidPlatformASTC : AndroidPlatform
{
    public override string GetCookPlatform(bool bDedicatedServer, bool bIsClientOnly)
    {
		return bIsClientOnly ? "Android_ASTCClient" : "Android_ASTC";
    }
    public override TargetPlatformDescriptor GetTargetPlatformDescriptor()
    {
        return new TargetPlatformDescriptor(TargetPlatformType, "ASTC");
    }
}

public class AndroidPlatformOpenXR : AndroidPlatform
{
	public override string GetCookPlatform(bool bDedicatedServer, bool bIsClientOnly)
	{
		return bIsClientOnly ? "Android_OpenXRClient" : "Android_OpenXR";
	}
	public override TargetPlatformDescriptor GetTargetPlatformDescriptor()
	{
		return new TargetPlatformDescriptor(TargetPlatformType, "OpenXR");
	}
}
