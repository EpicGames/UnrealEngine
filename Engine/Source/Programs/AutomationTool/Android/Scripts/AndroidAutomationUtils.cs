// Copyright Epic Games, Inc. All Rights Reserved.

using AutomationTool;
using EpicGames.Core;
using Microsoft.Extensions.Logging;
using System;
using System.IO;
using System.IO.Compression;
using UnrealBuildBase;
using UnrealBuildTool;

public class ReplaceAndroidExecutable : BuildCommand
{
	public override ExitCode Execute()
	{
		FileReference ProjectFile = new (ParseParamValue("ProjectFile", null));
		FileReference Executable = new (ParseParamValue("Executable", null));
		FileReference SourceApk = new (ParseParamValue("SourceApk", null));
		FileReference TargetApk = new (ParseParamValue("TargetApk", null));
		string Architecture = ParseParamValue("Architecture", null);

		if (!FileReference.Exists(ProjectFile))
		{
			throw new AutomationException("The Project file '{0}' does not exist", ProjectFile);
		}
		if (!FileReference.Exists(Executable))
		{
			throw new AutomationException("The executable '{0}' to use as a replacement does not exist", Executable);
		}
		if (!FileReference.Exists(SourceApk))
		{
			throw new AutomationException("The source apk '{0}' does not exist", SourceApk);
		}
		if (string.IsNullOrEmpty(Architecture))
		{
			throw new AutomationException("The architecture was null or empty");
		}

		string RelativeSO = $"lib/{Architecture}/libUnreal.so";

		Logger.LogInformation("Replacing shared object file...");
		AndroidExports.ReplaceSO(SourceApk, TargetApk, Executable, RelativeSO, Logger);

		Logger.LogInformation("Signing apk...");
		AndroidExports.SignDebugApk(ProjectFile, TargetApk, TargetApk, Logger);

		return ExitCode.Success;
	}
}