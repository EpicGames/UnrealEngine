<Project Sdk="Microsoft.NET.Sdk">
  <Import Project="..\..\Shared\UnrealEngine.csproj.props" />
  
  <PropertyGroup>
    <OutputType>Library</OutputType>
    <GenerateAssemblyInfo>false</GenerateAssemblyInfo>
    <GenerateTargetFrameworkAttribute>false</GenerateTargetFrameworkAttribute>
    <RootNamespace>Apple.Automation</RootNamespace>
    <AssemblyName>Apple.Automation</AssemblyName>
    <WarningsNotAsErrors>612,618</WarningsNotAsErrors>
    <OutputPath Condition="'$(ArtifactsPath)' == ''">..\..\..\..\Binaries\DotNET\AutomationTool\AutomationScripts\Platforms\Apple</OutputPath>
    <TreatWarningsAsErrors>true</TreatWarningsAsErrors>
  </PropertyGroup>
  <ItemGroup>
    <Compile Include="..\..\Shared\MetaData.cs">
      <Link>Properties\MetaData.cs</Link>
    </Compile>
  </ItemGroup>
  <ItemGroup>

	<PackageReference Include="Microsoft.IdentityModel.Tokens" Version="8.17.0" />
	<PackageReference Include="System.IdentityModel.Tokens.Jwt" Version="8.17.0" />
	<Reference Include="BouncyCastle.Crypto, Version=1.8.3.0, Culture=neutral, PublicKeyToken=0e99375e54769942">
		<HintPath>..\..\..\..\Binaries\ThirdParty\IOS\BouncyCastle.Crypto.dll</HintPath>
	</Reference>
	<PackageReference Include="Portable.BouncyCastle" Version="1.9.0" />

    <ProjectReference Include="..\..\Shared\EpicGames.Core\EpicGames.Core.csproj" PrivateAssets="All"><Private>false</Private></ProjectReference>
    <ProjectReference Include="..\..\UnrealBuildTool\UnrealBuildTool.csproj" PrivateAssets="All"><Private>false</Private></ProjectReference>
    <ProjectReference Include="..\AutomationUtils\AutomationUtils.Automation.csproj" PrivateAssets="All"><Private>false</Private></ProjectReference>
  </ItemGroup>
</Project>