// Copyright Epic Games, Inc. All Rights Reserved.
using System;
using System.Linq;
using System.Text;
using System.IO;
using AutomationTool;
using UnrealBuildTool;
using EpicGames.Core;
using UnrealBuildBase;
using System.Diagnostics;
using System.Collections.Generic;
using Microsoft.Extensions.Logging;
using System.Xml.Linq;

public abstract class ApplePlatform : Platform
{
	public ApplePlatform(UnrealTargetPlatform TargetPlatform)
		: base(TargetPlatform)
	{
	}

	#region SDK

	public override bool GetSDKInstallCommand(out string Command, out string Params, ref bool bRequiresPrivilegeElevation, ref bool bCreateWindow, ITurnkeyContext TurnkeyContext)
	{
		Command = "";
		Params = "";

		if (UnrealBuildTool.BuildHostPlatform.Current.Platform == UnrealTargetPlatform.Mac)
		{
			TurnkeyContext.Log("Moving your original Xcode application from /Applications to the Trash, and unzipping the new version into /Applications");

			// put current Xcode in the trash, and unzip a new one. Xcode in the dock will have to be fixed up tho!
			Command = "osascript";
			Params =
				" -e \"try\"" +
				" -e   \"tell application \\\"Finder\\\" to delete POSIX file \\\"/Applications/Xcode.app\\\"\"" +
				" -e \"end try\"" +
				" -e \"do shell script \\\"cd /Applications; xip --expand $(CopyOutputPath);\\\"\"" +
				" -e \"try\"" +
				" -e   \"do shell script \\\"xcode-select -s /Applications/Xcode.app; xcode-select --install; xcodebuild -license accept; xcodebuild -runFirstLaunch\\\" with administrator privileges\"" +
				" -e \"end try\"";
		}
		else if (UnrealBuildTool.BuildHostPlatform.Current.Platform == UnrealTargetPlatform.Win64)
		{

			TurnkeyContext.Log("Uninstalling old iTunes and preparing the new one to be installed.");

			Command = "$(EngineDir)/Extras/iTunes/Install_iTunes.bat";
			Params = "$(CopyOutputPath)";
		}
		return true;
	}

	public override bool OnSDKInstallComplete(int ExitCode, ITurnkeyContext TurnkeyContext, DeviceInfo Device)
	{
		if (Device == null)
		{
			if (ExitCode == 0)
			{
				if (UnrealBuildTool.BuildHostPlatform.Current.Platform == UnrealTargetPlatform.Mac)
				{
					TurnkeyContext.PauseForUser("If you had Xcode in your Dock, you will need to remove it and add the new one (even though it was in the same location). macOS follows the move to the Trash for the Dock icon");
				}
			}
		}

		return ExitCode == 0;
	}

	#endregion

	public override void PreBuildAgenda(UnrealBuild Build, UnrealBuild.BuildAgenda Agenda, ProjectParams Params)
	{
		if (!AppleExports.CreatingAppOnWindows(Params.RawProjectPath))
		{
			// any building before we stage should not pull the stage dir into the .app, and we will unset it below when we build again after staging
			Environment.SetEnvironmentVariable("UE_SKIP_STAGEDDATA_SYNC", "1", EnvironmentVariableTarget.Process);
		}
		
		base.PreBuildAgenda(Build, Agenda, Params);
	}

	private string MakeContentOnlyTargetName(TargetReceipt Target, string GameName)
	{
		// for a BP project, with a uproject, where the TargetName is one of the Unreal targets, the xcode project actually makes
		// a <ProjectName><Type> scheme/target, so we need to use it for the -SingleTarget param, and the scheme we xcodebuild
		// note: this must match the code in ProjectFileGenerator.cs / AddProjectsForAllTargets, like this:
		// ProjectFile = FindOrAddProject(GetProjectLocation($"{ProjectName}{EngineTarget.TargetRules!.Type}"), ContentOnlyGameProject.Directory, ...
		// in the "if (bAllowContentOnlyProjects)" block
		return $"{GameName}{Target.TargetType}";
	}

	public override void PostStagingFileCopy(ProjectParams Params, DeploymentContext SC)
	{
		// If we are cooking a DLC, we should never be staging executables so we can early out
		if (Params.HasDLCName)
		{
			Logger.LogInformation("Skipping PostStagingFileCopy because we are cooking a DLC plugin");
			return;
		}

		// staging will put binaries into Staged/<game>/Binaries/<platform> and they aren't needed, and when we pull this into a .app, it 
		// messes with the resulting .app. So, we remove the game binary now (leaving in helper .app's and raw .dylibs, etc)
		// they come from BuildProducts, and we could maybe remove from that list, but it could cause issues with Horde/buildmachines
		// program binaries usually live under the Engine directory, and there isn't a great way to know if this is a project program or not
		// a more programmatic solution is being investigated
		string RootDirName = "Engine";
		if (Params.IsCodeBasedProject && SC.StageTargets[0].Receipt.TargetType != TargetType.Program)
		{
			RootDirName = SC.ShortProjectName;
		}
		if (SC.StageExecutables.Any())
		{
			FileReference BinaryPath = FileReference.Combine(SC.StageDirectory, RootDirName, "Binaries", SC.PlatformDir, SC.StageExecutables[0]);
			InternalUtils.SafeDeleteFile(BinaryPath.FullName, true);
			// this may leave the binaries directory empty, but the Mac needs the Binaries/Mac dir to exist (see FMacPlatformProcess::BaseDir())
			// so plop a file down in it's place
			if (SC.StageTargetPlatform.PlatformType == UnrealTargetPlatform.Mac)
			{
				Directory.CreateDirectory(BinaryPath.Directory.FullName);
				File.WriteAllText(Path.Combine(BinaryPath.Directory.FullName, ".binariesdir"), "");
			}

			DirectoryReference AppPath = new DirectoryReference(BinaryPath.FullName + ".app");
			InternalUtils.SafeDeleteDirectory(AppPath.FullName, true);
		}

		if (!AppleExports.CreatingAppOnWindows(Params.RawProjectPath))
		{
			// now reset the envvar so the following build will process the staged data
			Environment.SetEnvironmentVariable("UE_SKIP_STAGEDDATA_SYNC", "0", EnvironmentVariableTarget.Process);

			foreach (TargetReceipt Target in SC.StageTargets.Select(x => x.Receipt))
			{
				string ExtraOptions =
					// fiddle with some envvars to redirect the .app into root of Staging directory, without going under any build subdirectories
					$"SYMROOT=\"{SC.StageDirectory.ParentDirectory}\" " +
					$"EFFECTIVE_PLATFORM_NAME={SC.StageDirectory.GetDirectoryName()} " +
					// set where the stage directory is
					$"UE_OVERRIDE_STAGE_DIR=\"{SC.StageDirectory}\"";

				if (Params.XcodeBuildOptions != null)
				{
					ExtraOptions += $" {Params.XcodeBuildOptions}";
				}

				string TargetName = Target.TargetName;
				if (!Params.IsCodeBasedProject)
				{
					if (Params.RawProjectPath != null)
					{
						TargetName = MakeContentOnlyTargetName(Target, Params.ShortProjectName);
					}
				}

				AppleExports.BuildWithStubXcodeProject(SC.RawProjectPath, Target.Platform, Target.Architectures, Target.Configuration, TargetName, AppleExports.XcodeBuildMode.Stage, Logger, ExtraOptions, DestinationIds:Params.DeviceNames);
			}
		}
	}

	private DirectoryReference GetArchivePath(StageTarget Target, DeploymentContext SC)
	{
		string TargetName = Target.Receipt.TargetName;
		if (!SC.IsCodeBasedProject)
		{
			if (SC.RawProjectPath != null)
			{
				TargetName = MakeContentOnlyTargetName(Target.Receipt, SC.ShortProjectName);
			}
		}

		// find the most recent archive for this target (based on name of target, this ignores Development vs Shipping, but 
		// since Distribution is meant only for Shipping it's ok
		string ArchiveName = AppleExports.MakeBinaryFileName(TargetName, "-", Target.Receipt.Platform, UnrealTargetConfiguration.Development,
		 	Target.Receipt.Architectures, UnrealTargetConfiguration.Development, null);

		if (ArchiveName != TargetName)
		{
			// i don't know why we called MakeBinaryFileName - with Dev it seems to just return the TargetName passed in
			throw new AutomationException($"GetArchivePath changed name of Target from {TargetName} to {ArchiveName} - tell Josh Adams");
		}

		return AppleExports.FindLatestXcArchive(TargetName);
	}

	private DirectoryReference GetFinalAppPath(StageTarget Target, DeploymentContext SC)
	{
		// content only projects have to not use the executable, but make a path to a .app 
		if (!SC.IsCodeBasedProject)
		{
			string AppBundleName = AppleExports.MakeBinaryFileName(SC.ShortProjectName, "-", Target.Receipt.Platform, Target.Receipt.Configuration, 
				Target.Receipt.Architectures, UnrealTargetConfiguration.Development, ".app");

			return DirectoryReference.Combine(SC.ProjectRoot, "Binaries", Target.Receipt.Platform.ToString(), AppBundleName);
		}

		DirectoryReference AppDir;
		// get the executable from the receipt
		FileReference Executable = Target.Receipt.BuildProducts.First(x => x.Type == BuildProductType.Executable).Path;

		// if we are in a .app, then use that
		if (Executable.FullName.Contains(".app/"))
		{
			AppDir = Executable.Directory;
			// go up until we find the .app - Mac and IOS are different, so this handles both cases
			while (AppDir.HasExtension(".app") == false)
			{
				AppDir = AppDir.ParentDirectory;
			}
		}
		// otherwise use the .app next to the executable
		else
		{
			AppDir = new DirectoryReference(Executable.FullName + ".app");
		}

		return AppDir;
	}

	public override void Package(ProjectParams Params, DeploymentContext SC, int WorkingCL)
	{
		if (AppleExports.CreatingAppOnWindows(Params.RawProjectPath))
		{
			PrintRunTime();
			return;
		}

		// Strip any .DS_Store files from the staged tree before xcodebuild assembles the .app/.xcarchive.
		if (DirectoryReference.Exists(SC.StageDirectory))
		{
			foreach (string DSStorePath in Directory.EnumerateFiles(SC.StageDirectory.FullName, ".DS_Store", SearchOption.AllDirectories))
			{
				Logger.LogInformation("Removing .DS_Store file from stage: {File}", DSStorePath);
				File.Delete(DSStorePath);
			}
		}

		foreach (StageTarget Target in SC.StageTargets)
		{
			TargetReceipt Receipt = Target.Receipt;
			string TargetName = Receipt.TargetName;

			string ExtraOptions =
				// set where the stage directory is
				$"UE_OVERRIDE_STAGE_DIR=\"{SC.StageDirectory}\"";
			if (!Params.IsCodeBasedProject)
			{
				if (!Params.Distribution)
				{
					// override where the .app will be located and named
					ExtraOptions += $" SYMROOT=\"{SC.ProjectRoot}/Binaries\"";
				}
				
				if (Params.RawProjectPath != null)
				{
					TargetName = MakeContentOnlyTargetName(Receipt, Params.ShortProjectName);
				}
			}

			// if we aren't going to archive right after this to another location, and then archivedirectory specifies a .xcarchive extension
			// then we want to archive directly to that location instead of the standard ~/Library/Developer/Xcode/Archives. This is useful for
			// build machines that want to make a .xcarchive and export/upload it, without needing to hunt down latest, or dirty up the Library dir
			string XcArchivePath = "";
			if (Params.Distribution && Params.Archive == false && Params.ArchiveDirectoryParam.EndsWith(".xcarchive"))
			{
				XcArchivePath = Params.ArchiveDirectoryParam;
				ExtraOptions += $" -archivePath \"{XcArchivePath}\"";
			}

			if (Params.XcodeBuildOptions != null)
			{
				ExtraOptions += $" {Params.XcodeBuildOptions}";
			}

			// if we we packaging for distrbution, we will create a .xcarchive which can be used to submit to app stores, or exported for other distribution methods
			// the archive will be created in the standard Archives location accessible via Xcode. Using -archive will copy it out into
			// the specified location for use as needed
			AppleExports.XcodeBuildMode BuildMode = Params.Distribution ? AppleExports.XcodeBuildMode.Distribute : AppleExports.XcodeBuildMode.Package;
			if (AppleExports.BuildWithStubXcodeProject(Params.RawProjectPath, Receipt.Platform, Receipt.Architectures, Receipt.Configuration, TargetName, BuildMode, Logger, ExtraOptions, DestinationIds:Params.DeviceNames) == 0)
			{
				Logger.LogInformation("==================================================================================================================================================");
				if (Params.Distribution)
				{
					if (XcArchivePath == "")
					{
						XcArchivePath = AppleExports.FindLatestXcArchive(TargetName)?.FullName;
						if (XcArchivePath == null)
						{
							Logger.LogError("Unable to discover a xcarchive for {TargetName} in ~/Library/Developer/Xcode/Archives.", TargetName);
							Logger.LogError("This is unexpected since the command succeeded");
						}
						else
						{
							Logger.LogInformation("Created '{XcArchivePath}', which can be seen in Xcode's Organizer window.", XcArchivePath);
							Logger.LogInformation("You may use this to validate and prepare for various distribution methods");
						}
					}
					else
					{
						Logger.LogInformation("Created '{XcArchivePath}' using an overriden location - this will _not_ appear in Xcode's Organizer window.", XcArchivePath);
						Logger.LogInformation("You may use this to validate and prepare for various distribution methods");
					}
				}
				else
				{
					Logger.LogInformation("Finalized {App} for running fully self-contained", GetFinalAppPath(Target, SC));
				}

				Logger.LogInformation("==================================================================================================================================================");
			}

		}

		// package up the program, potentially with an installer for Mac
		PrintRunTime();
	}

	public override void GetFilesToArchive(ProjectParams Params, DeploymentContext SC)
	{
		if (AppleExports.CreatingAppOnWindows(Params.RawProjectPath))
		{
			base.GetFilesToArchive(Params, SC);
			return;
		}

		Logger.LogInformation("staging targets: '{tagets}', '{configs}'", string.Join(", ", Params.ClientCookedTargets),
			string.Join(", ", SC.StageTargetConfigurations));
		foreach (StageTarget Target in SC.StageTargets)
		{
			//// copy the xcarchive and any exported files
			//string ExportMode = "Development";
			//DirectoryReference ExportPath = GetExportPath(ExportMode, SC);

			// distribution mode we want to archive the .xcarchive that was created during Package
			DirectoryReference ArchiveSource;
			DirectoryReference ArchiveDestination = null;
			if (Params.Distribution)
			{
				// find the most recent .xcarchive in the Xcode archives library
				ArchiveSource = GetArchivePath(Target, SC);

				if (ArchiveSource == null)
				{
					Logger.LogError("Unable to find a .xcarchive in Xcode's Library to archive to {ArchiveDir}", SC.ArchiveDirectory);
					return;
				}

				if (SC.ArchiveDirectory.HasExtension(".xcarchive"))
				{
					ArchiveDestination = SC.ArchiveDirectory;
					if (DirectoryReference.Exists(ArchiveDestination))
					{
						DirectoryReference.Delete(ArchiveDestination);
					}
				}
			}
			else
			{
				ArchiveSource = GetFinalAppPath(Target, SC);
				if (!DirectoryReference.Exists(ArchiveSource))
				{
					Logger.LogError("Unable to find the expected application ({App}) for achiving", ArchiveSource);
					return;
				}
			}

			if (ArchiveDestination == null)
			{
				ArchiveDestination = DirectoryReference.Combine(SC.ArchiveDirectory, ArchiveSource.GetDirectoryName());
			}
			
			Logger.LogInformation("==================================================================================================================================================");
			Logger.LogInformation("Copying {Type} package {ArchiveSource} to archive directory {ArchiveDir}", Params.Distribution ? "Distribution" : "Development", ArchiveSource, ArchiveDestination);
			Logger.LogInformation("==================================================================================================================================================");

			Utils.RunLocalProcessAndReturnStdOut("/usr/bin/env", $"ditto \"{ArchiveSource}\" \"{ArchiveDestination}\"", null);
		}
	}
}
