// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Linq;
using System.IO;
using System.Collections.Generic;
using EpicGames.Core;
using AutomationTool;
using System.Threading;
using UnrealBuildBase;
using Microsoft.Extensions.Logging;
using UnrealBuildTool;
using System.IdentityModel.Tokens.Jwt;
using Org.BouncyCastle.Crypto.Parameters;
using System.Security.Cryptography;
using Microsoft.IdentityModel.Tokens;
using OpenTracing.Tag;
using EpicGames.Horde.Storage;
using Google.Protobuf.WellKnownTypes;
using System.Net.Http;
using System.Xml.Linq;
using System.Net.Http.Headers;
using System.Text.Json;
using System.Text.RegularExpressions;
using System.Threading.Tasks;
using System.Net.Http.Json;
using Org.BouncyCastle.Asn1;

class ExportAppleBuild : BuildCommand
{
	public override ExitCode Execute()
	{
		FileReference ProjectFile = ParseProjectParam();
		if (ProjectFile == null)
		{
			Logger.LogError("Project not specified, of found, with -project param");
			return ExitCode.Error_Arguments;
		}

		string PlistParam = ParseParamValue("Plist");
		string XcArchivePathParam = ParseParamValue("XcArchive");
		string LatestXcArchiveParam = ParseParamValue("LatestXcArchiveForTarget");
		string LatestXcArchiveSearch = ParseParamValue("LatestXcArchiveSearchPath");
		string ExportPath = ParseParamValue("ExportPath");
		string WhatsNew = ParseParamValue("WhatsNew");
		bool bUpload = !ParseParam("SkipUpload");
		bool bCheckVersion = ParseParam("CheckVersion");
		string VersionConflictSigningIdentity = ParseParamValue("VersionConflictSigningIdentity");
		string VersionConflictIncrement = ParseParamValue("VersionConflictIncrement");
		string ExportAuthOverride = ParseParamValue("ExportAuthOverride");
		string CheckDelay = ParseParamValue("CheckDelay");
		string CheckCount = ParseParamValue("CheckCount");

		DirectoryReference XcArchive;
		if (!string.IsNullOrEmpty(LatestXcArchiveParam))
		{
			// find the latest archive for the given project name
			DirectoryReference SearchPath = null;
			if (!string.IsNullOrEmpty(LatestXcArchiveSearch))
			{
				SearchPath = new DirectoryReference(LatestXcArchiveSearch);
			}
			XcArchive = AppleExports.FindLatestXcArchive(LatestXcArchiveParam, SearchPath);
		}
		else
		{
			XcArchive = new DirectoryReference(XcArchivePathParam);
		}

		if (XcArchive == null || !DirectoryReference.Exists(XcArchive))
		{
			Logger.LogError("No XCArchive found, with -XCArchive or -LatestXCArchiveForTarget params");
			return ExitCode.Error_Arguments;
		}

		TokenClient = new JwtTokenClient(ProjectFile);

		if (bCheckVersion)
		{
			if (CheckVersion(TokenClient, ref XcArchive, VersionConflictSigningIdentity, VersionConflictIncrement) == false)
			{
				return ExitCode.Error_Arguments;
			}
		}

		if (bUpload)
		{
			string PlistPath;
			if (!string.IsNullOrEmpty(PlistParam) && PlistParam[0] == '/')
			{
				PlistPath = PlistParam;
			}
			else
			{
				// look in project and engine for named plist
				PlistPath = Path.Combine(ProjectFile.Directory.FullName, "Build", "Xcode", PlistParam + ".plist");
				if (!File.Exists(PlistPath))
				{
					PlistPath = Path.Combine(Unreal.EngineDirectory.FullName, "Build", "Xcode", PlistParam + ".plist");
				}
			}

			if (!File.Exists(PlistPath))
			{
				Logger.LogError("No plist options file found, with -plist param");
				return ExitCode.Error_Arguments;
			}

			Logger.LogInformation("Project: {Project}, XCArchive: {XC}, Plist {Plist}", ProjectFile, XcArchive, PlistPath);

			// build commandline
			string CommandLine = $"-exportArchive";
			CommandLine += $" -archivePath \"{XcArchive}\"";
			CommandLine += $" -exportOptionsPlist \"{PlistPath}\"";
			CommandLine += $" -allowProvisioningUpdates";
			if (!string.IsNullOrEmpty(ExportAuthOverride))
			{
				Logger.LogInformation("Auth Overrides: {Overrides}", ExportAuthOverride);
				CommandLine += " " + ExportAuthOverride;
			}
			else
			{
				CommandLine += AppleExports.GetXcodeBuildAuthOptions(ProjectFile);
			}

			if (!string.IsNullOrEmpty(ExportPath))
			{
				CommandLine += $" -exportPath {ExportPath}";
			}

			Logger.LogInformation($"Running 'xcodebuild {CommandLine}'...");

			int Return;
			string Output = Utils.RunLocalProcessAndReturnStdOut("/usr/bin/xcodebuild", CommandLine, null, out Return);
			Logger.LogInformation(Output);

			if (Return != 0)
			{
				return (ExitCode)Return;
			}
		}

		if (!string.IsNullOrEmpty(WhatsNew))
		{
			int Delay = 30;
			if (!string.IsNullOrEmpty(CheckDelay))
			{
				int Result = 0;
				if (Int32.TryParse(CheckDelay, out Result))
				{
					Delay = Int32.Max(Int32.Min(Result, 300), 15);
				}
			}
			int Tries = 30;
			if (!string.IsNullOrEmpty(CheckCount))
			{
				int Result = 0;
				if (Int32.TryParse(CheckCount, out Result))
				{
					Tries = Int32.Max(Int32.Min(Result, 100), 5);
				}
			}

			// get some info from the .xcarchive that we just uploaded
			string BundleId, AppVersion, BundleVersion, CommandLine;
			GetInfoFromXcArchive(XcArchive, out BundleId, out AppVersion, out BundleVersion, out CommandLine);

			Logger.LogInformation("Setting What's New for {BundleId}, app version {Version}, build version {BundleVersion}, commandline {CommandLine}, final whatsnew = {WhatsNew}", BundleId, AppVersion, BundleVersion, CommandLine, WhatsNew);

			// replace some variables
			WhatsNew = WhatsNew.Replace("@appversion", AppVersion);
			WhatsNew = WhatsNew.Replace("@buildversion", BundleVersion);
			WhatsNew = WhatsNew.Replace("@bundleid", BundleId);
			WhatsNew = WhatsNew.Replace("@uecommandline", CommandLine);
			WhatsNew = WhatsNew.Replace("@commandline", CommandLine);

			UpdateWhatsNew(TokenClient, BundleId, AppVersion, BundleVersion, WhatsNew, Delay, Tries);
		}

		return (ExitCode)0;
	}


	record Links(string self, string related);
	record Relationship(Links links);

	record AppInfo(string type, string id);
	record AppResponse(List<AppInfo> data);


	record BuildRelationships(Relationship preReleaseVersion);
	record BuildAttributes(string version, string processingState, string uploadedDate);
	record BuildInfo(string type, string id, BuildAttributes attributes, BuildRelationships relationships);
	record BuildsResponse(List<BuildInfo> data);

	record VersionAttributes(string version, string platform);
	record VersionInfo(string id, VersionAttributes attributes);
	record VersionResponse(VersionInfo data);
	record VersionsResponse(List<VersionInfo> data);

	record BuildsWithVersionsResponse(List<BuildInfo> data, List<VersionInfo> included);

	record PagedLinks(string next);
	record PagedBuildsResponse(List<BuildInfo> data, PagedLinks links);

	record LocalizationAttributes(string whatsNew);
	record LocalizationInfo(string type, string id, LocalizationAttributes attributes);
	record LocalizationResponse(List<LocalizationInfo> data);
	record LocalizationUpdate(LocalizationInfo data);

	record LocalizationRelationship(string id, string type);
	record LocalizationRelationshipsData(LocalizationRelationship data);
	record LocalizationRelationships(LocalizationRelationshipsData build);
	record LocalizationCreateAttributes(string locale, string whatsNew);
	record LocalizationCreateInfo(LocalizationCreateAttributes attributes, LocalizationRelationships relationships, string type);
	record LocalizationCreateRequest(LocalizationCreateInfo data);


	static CancellationToken CancellationToken = new();

	// shared client object
	JwtTokenClient TokenClient = null;

	private async Task<TResponse> GetAsync<TResponse>(string RelativePath, CancellationToken CancellationToken = default)
	{
		TResponse Response = await TokenClient.Client.GetFromJsonAsync<TResponse>(RelativePath, CancellationToken);
		return Response ?? throw new InvalidCastException($"Expected non-null response from GET to {RelativePath}");
	}

	string GetAppId(string BundleId)
	{
		AppResponse Response = GetAsync<AppResponse>($"apps?filter[bundleId]={BundleId}&fields[apps]=name", CancellationToken).Result;
		return Response.data[0].id;
	}

	string GetBuildId(string AppId, string PreReleaseVersion, string BuildVersion)
	{
		int Limit = 1;
		string SubURL = $"builds?filter[app]={AppId}&filter[version]={BuildVersion}&sort=-uploadedDate&limit={Limit}";

		bool bHasProcessedAll = false;
		while (!bHasProcessedAll)
		{
			PagedBuildsResponse Response = GetAsync<PagedBuildsResponse>(SubURL, CancellationToken).Result;
			foreach (BuildInfo Build in Response.data)
			{
				// follow link to preReleaseVersion
				SubURL = Build.relationships.preReleaseVersion.links.related.Replace(TokenClient.Client.BaseAddress.ToString(), "");
				VersionResponse VersionResponse = GetAsync<VersionResponse>(SubURL, CancellationToken).Result;

				// if it matches, return this Build!
				if (VersionResponse.data.attributes.version == PreReleaseVersion)
				{
					return Build.id;
				}
			}

			if (Response.links.next == null)
			{
				bHasProcessedAll = true;
			}
			else
			{
				// get suburl for next page
				SubURL = Response.links.next.Replace(TokenClient.Client.BaseAddress.ToString(), "");
			}
		}

		// at this point, if no build was found, then it's not been uploaded yet
		return null;
	}


	string GetLocalizationId(string BuildId)
	{
		string SubURL = $"builds/{BuildId}/betaBuildLocalizations";
		LocalizationResponse Response = GetAsync<LocalizationResponse>(SubURL, CancellationToken).Result;

		if (Response.data.Count == 0)
		{
			Console.WriteLine("no current beta build localized data, will create new english data");
			return null;
		}
		Console.WriteLine("current whats new is: {0}", Response.data[0].attributes.whatsNew);

		return Response.data[0].id;
	}

	// return true if the version doesn't exist - which means it can be uploaded
	bool CheckVersion(JwtTokenClient TokenClient, ref DirectoryReference XcArchive, string SigningIdentity, string Increment)
	{
		// get some info from the .xcarchive
		string BundleId, AppVersion, BundleVersion;
		GetInfoFromXcArchive(XcArchive, out BundleId, out AppVersion, out BundleVersion, out _);
		
		Logger.LogInformation("Checking for version >= {Version}", BundleVersion);
		Version BundleVersionNumber = new Version(BundleVersion);

		int Attempts = 0;
		while (Attempts < 5)
		{
			if (Attempts > 0)
			{
				Thread.Sleep(30 * 1000);
			}
			Attempts++;

			try
			{
				string AppId = GetAppId(BundleId);

				// get prerelease version id
				string SubURL = $"preReleaseVersions?filter[app]={AppId}&filter[version]={AppVersion}&fields[preReleaseVersions]=version";
				VersionsResponse VersionResponse = GetAsync<VersionsResponse>(SubURL, CancellationToken).Result;
				// if the app version doesn't even exist, then the bundle verrsion can't exist in it!
				if (VersionResponse.data.Count == 0)
				{
					return true;
				}

				Version MaxVersion = new Version(0, 0);

				// now walk over all builds until we find one >= the bundle version
				string VersionId = VersionResponse.data[0].id;
				SubURL = $"preReleaseVersions/{VersionId}/builds?limit=5&fields[builds]=version";
				bool bHasProcessedAll = false;
				while (!bHasProcessedAll)
				{
					PagedBuildsResponse Response = GetAsync<PagedBuildsResponse>(SubURL, CancellationToken).Result;

					foreach (BuildInfo Build in Response.data)
					{
						string VersionString = Build.attributes.version;
						// append .0.0 if needed
						if (!VersionString.Contains("."))
						{
							VersionString += ".0.0";
						}

						Version Version = new Version(VersionString);
						if (Version > MaxVersion)
						{
							MaxVersion = Version;
						}
					}

					if (Response.links.next == null)
					{
						bHasProcessedAll = true;
					}
					else
					{
						// get suburl for next page
						SubURL = Response.links.next.Replace(TokenClient.Client.BaseAddress.ToString(), "");
					}
				}

				if (MaxVersion >= BundleVersionNumber)
				{
					if (SigningIdentity != null && Increment != null)
					{
						Version IncrementVersion = new Version(Increment);
						if (IncrementVersion == new Version(0, 0, 0))
						{
							throw new Exception($"Invalid version increment {Increment}");
						}

						string TempDir = Path.GetTempFileName();
						File.Delete(TempDir);
						Directory.CreateDirectory(TempDir);

						FileReference Script = FileReference.Combine(Unreal.EngineDirectory, "Build/BatchFiles/Mac/ResignApp.sh");
						string FixedXcArchive = TempDir + "/FixedVersion.xcarchive";
						Version FixedVersion = new Version(MaxVersion.Major + IncrementVersion.Major, MaxVersion.Minor + IncrementVersion.Minor, MaxVersion.Build + IncrementVersion.Build);
						string Options = $"{Script} -s \"{XcArchive}\" -d \"{FixedXcArchive}\" -i \"{SigningIdentity}\" -bv {FixedVersion}";

						if (MaxVersion == BundleVersionNumber)
						{
							Logger.LogInformation("A build exists with same version that is in the .xcarchive ({BundleVersion}). Making a new\n" +
							                      ".xcarchive ({FixedXcArchive}) with a version using the supplied increment. The new version is {FixedVersion}.",
								BundleVersion, FixedXcArchive, FixedVersion);
						}
						else
						{
							Logger.LogInformation("A build exists with LARGER version ({MaxVersion}) that is in the .xcarchive ({BundleVersion})\n" +
								"which would cause the uploaded build to be auto-incremented. Making a new .xcarchive ({FixedXcArchive})\n" +
								"with a incremented FROM THE LARGER VERSION (not the version in your .xcarchive!). The new version is {FixedVersion}.",
								MaxVersion, BundleVersion, FixedXcArchive, FixedVersion);
						}

						Logger.LogInformation("Running: {Script} {Options}...", "/usr/bin/env", Options);
						int ExitCode;
						string Output = Utils.RunLocalProcessAndReturnStdOut("/usr/bin/env", Options, out ExitCode);
						if (ExitCode != 0)
						{
							Logger.LogError("{Output}", Output);
							return false;
						}

						Logger.LogInformation("{Output}", Output);

						// replace the xcarchive with the fixed version
						Logger.LogInformation("Overwriting xcarchive: {XcArchive} with updated version", XcArchive.FullName);
						Directory.Delete(XcArchive.FullName, true);
						Directory.Move(FixedXcArchive, XcArchive.FullName);

						// we know that this xcarchive has a version that is not in the appstore, so return true 
					}
					else
					{
						Version PossibleVersion = new Version(MaxVersion.Major, MaxVersion.Minor + 1, MaxVersion.Build);

						Logger.LogError("A build exists with same or larger version ({MaxVersion}) than is in the .xcarchive ({BundleVersion}). Either make a new build,\n" +
							"or modify the build version in the .xcarchive with a command like:\n" +
							"   Engine/Build/BatchFiles/Mac/ResignApp.sh -s {XcArchive} -d UpdatedVersion.xcarchive -i <signing identity> -bv {NewVersion}\n" +
							"and then upload that .xcarchive",
							MaxVersion, BundleVersion, XcArchive, PossibleVersion);

						return false;
					}

				}
				// if we get here, there is either no newer version, or we have successfully re-versioned the .app
				return true;
			}
			catch (AggregateException Ex)
			{
				Ex.Handle(x => 
					{
						if (x is HttpRequestException)
						{
							Logger.LogWarning("Got an exception while checking for existing version numbers: '{Msg}'. Will try again...", x.Message);
							return true;
						}
						return false;
					}
				);

			}
		}

		return false;
	}


	bool CreateWhatsNew(string BuildId, string Text)
	{
		LocalizationRelationship Relationship = new(id: BuildId, type:"builds");
		LocalizationRelationshipsData RelationshipData = new(data: Relationship);
		LocalizationRelationships Relationships = new(build: RelationshipData);
		LocalizationCreateAttributes Attributes = new(locale:"en-US", whatsNew:Text);
		LocalizationCreateInfo Info = new(type:"betaBuildLocalizations", attributes:Attributes, relationships:Relationships);
		LocalizationCreateRequest LocUpdate = new(data: Info);

		StringContent JsonContent = new StringContent(JsonSerializer.Serialize<LocalizationCreateRequest>(LocUpdate), System.Text.Encoding.UTF8, "application/json");

		string SubURL = $"betaBuildLocalizations";
		HttpResponseMessage Response = TokenClient.Client.PostAsync(SubURL, JsonContent).Result;

		if (!Response.IsSuccessStatusCode)
		{
			Logger.LogError("Create Localization Data failed!: StatusCode {Code}\n{Response}", Response.StatusCode, Response.Content.ReadAsStringAsync().Result);
			return false;
		}

		return true;
	}

	bool SetWhatsNew(string LocId, string Text)
	{
		LocalizationAttributes Attributes = new(whatsNew: Text);
		LocalizationInfo Info = new(type: "betaBuildLocalizations", id: LocId, attributes: Attributes);
		LocalizationUpdate LocUpdate = new(data: Info);

		StringContent JsonContent = new StringContent(JsonSerializer.Serialize<LocalizationUpdate>(LocUpdate), System.Text.Encoding.UTF8, "application/json");

		string SubURL = $"betaBuildLocalizations/{LocId}";
		HttpResponseMessage Response = TokenClient.Client.PatchAsync(SubURL, JsonContent).Result;

		if (!Response.IsSuccessStatusCode)
		{
			Logger.LogError("Patch Localization Data failed!: StatusCode {Code}\n{Response}", Response.StatusCode, Response.Content.ReadAsStringAsync().Result);
			return false;
		}

		return true;
	}

	bool UpdateWhatsNew(JwtTokenClient TokenClient, string BundleId, string PreReleaseVersion, string BundleVersion, string Text, int Delay = 30, int Tries = 30)
	{
		string BuildId = null;
		while (BuildId == null && Tries-- > 0)
		{
			try
			{
				string AppId = GetAppId(BundleId);

				BuildId = GetBuildId(AppId, PreReleaseVersion, BundleVersion);
			}
			catch (AggregateException Ex)
			{
				Ex.Handle(x => 
					{
						if (x is HttpRequestException)
						{
							Logger.LogWarning("Got an exception while checking for looking for uploaded version: '{Msg}'. Will try again...", Ex.Message);
							return true;
						}
						return false;
					}
				);
			}

			if (BuildId == null)
			{
				Logger.LogInformation("Build not found yet, waiting and will try again...");
				Thread.Sleep(Delay * 1000);
			}
		}

		if (BuildId == null)
		{
			Logger.LogWarning("Unable to find build for BundleId: {BundleId}, App Version {PreReleaseVersion}, Build Version {BundleVersion}. It could be due to:\n" + 
				"  * The build was never actually uploaded to AppStoreConnect to TestFlight\n" + 
				"  * A later version was already there, and Apple changed the version number automatically. Run again with -CheckVersion -SkipUpload to check\n" + 
				"  * There was a problem with the app. Check your email (or others on your team) to see if Apple rejected the app after uploading",
				BundleId, PreReleaseVersion, BundleVersion);
			return false;
		}
		// in the case that we don't use "most-recent" build, then we are looking for exact build version match to bundle version
		string BuildVersion = BundleVersion;

		// string BuildVersion;
		// string BuildId = GetMostRecentBuildId(AppId, PreReleaseVersion, out BuildVersion);
		// if (BuildId == null)
		// {
		// 	Logger.LogError("Unable to find a recent build for version {Version} of {BundleId}", PreReleaseVersion, BundleId);
		// 	return false;
		// }

		Logger.LogInformation("Found a build with version: {BuildVersion} [id is {Id}]", BuildVersion, BuildId);

		int Attempts = 0;
		while (Attempts < 5)
		{
			if (Attempts > 0)
			{
				Thread.Sleep(30 * 1000);
			}
			Attempts++;

			try
			{
				string LocId = GetLocalizationId(BuildId);	

				bool bUpdateSucceeded;
				if (LocId == null)
				{
					bUpdateSucceeded = CreateWhatsNew(BuildId, Text);
				}
				else 
				{
					bUpdateSucceeded = SetWhatsNew(LocId, Text);
				}
				if (bUpdateSucceeded)
				{
					GetLocalizationId(BuildId);
					return true;
				}
				return false;
			}
			catch (AggregateException Ex)
			{
				Ex.Handle(x => 
					{
						Logger.LogWarning("Got an unknown exception ('{Exception}') while updating the WhatsNew of the build. Will try again...", Ex.Message);
						return true;
					}
				);
				return false;
			}
		}

		return false;
	}

	static void GetInfoFromXcArchive(DirectoryReference XcArchive, out string BundleId, out string AppVersion, out string BundleVersion, out string CommandLine)
	{
		BundleId = null;
		AppVersion = null;
		BundleVersion = null;
		FileReference Plist = FileReference.Combine(XcArchive, "Info.plist");

		string[] Lines = FileReference.ReadAllLines(Plist);
		bool bMatchBundleId = false;
		bool bMatchAppVersion = false;
		bool bMatchBundleVersion = false;
		foreach (string Line in Lines)
		{
			if (bMatchBundleId || bMatchAppVersion || bMatchBundleVersion)
			{
				Match Match = Regex.Match(Line, @"\<string\>(.*)\<\/string\>");
				if (Match.Success)
				{
					if (bMatchBundleId)
					{
						BundleId = Match.Groups[1].Value;
					}
					else if (bMatchAppVersion)
					{
						AppVersion = Match.Groups[1].Value;
					}
					else if (bMatchBundleVersion)
					{
						BundleVersion = Match.Groups[1].Value;
					}
				}
				bMatchBundleId = bMatchAppVersion = bMatchBundleVersion = false;
			}
			else if (Line.Contains("CFBundleIdentifier"))
			{
				bMatchBundleId = true;
			}
			else if (Line.Contains("CFBundleShortVersionString"))
			{
				bMatchAppVersion = true;
			}
			else if (Line.Contains("CFBundleVersion"))
			{
				bMatchBundleVersion = true;
			}
		}

		if (BundleId == null || AppVersion == null || BundleVersion == null)
		{
			throw new Exception($"Failed to get bundle ID and versions from {Plist}");
		}

		CommandLine = "";
		// now find the uecommandline.txt - we don't know the name of the .app inside the .xcarchive, but there should only be one
		DirectoryReference ProductsDir = DirectoryReference.Combine(XcArchive, "Products", "Applications");
		foreach (DirectoryReference AppDir in DirectoryReference.EnumerateDirectories(ProductsDir, "*.app"))
		{
			FileReference CmdLineFile = FileReference.Combine(AppDir, "uecommandline.txt");
			if (FileReference.Exists(CmdLineFile))
			{
				if (CommandLine != "")
				{
					throw new Exception("Multiple .app's in the .xcarchive. This is unexpected");
				}
				CommandLine = File.ReadAllText(CmdLineFile.FullName).Trim();
			}
		}
	}
}

class JwtTokenClient
{
	public JwtTokenClient(FileReference ProjectFile)
	{
		projectFile = ProjectFile;
		
		client = new HttpClient();
		client.BaseAddress = new Uri("https://api.appstoreconnect.apple.com/v1/");
	}
	
	public HttpClient Client
	{
		get
		{
			CheckExpiration();
			return client;
		}
	}
	
	private DateTime expirationDate = DateTime.MinValue;
	private string token = "";
	private FileReference projectFile;
	private HttpClient client;

	void CheckExpiration()
	{
		if (token == "" || DateTime.UtcNow.AddSeconds(30) > expirationDate)
		{
			CreateJWTToken();

			// update the auth header
			client.DefaultRequestHeaders.Authorization = new AuthenticationHeaderValue("Bearer", token);
		}
	}
	
	void CreateJWTToken()
	{
		// @todo: refactor the AppleExports.GetXcodeBuildAuthOptions() code so that we can get this info in a shared way
		ConfigHierarchy SharedPlatformIni = ConfigCache.ReadHierarchy(ConfigHierarchyType.Engine, projectFile?.Directory, UnrealTargetPlatform.Mac);
		string IssuerID, KeyID, KeyPath;
		string P8Path;
		if (SharedPlatformIni.TryGetValue("/Script/MacTargetPlatform.XcodeProjectSettings", "AppStoreConnectIssuerID", out IssuerID) &&
			SharedPlatformIni.TryGetValue("/Script/MacTargetPlatform.XcodeProjectSettings", "AppStoreConnectKeyID", out KeyID) &&
			SharedPlatformIni.TryGetValue("/Script/MacTargetPlatform.XcodeProjectSettings", "AppStoreConnectKeyPath", out KeyPath))
		{
			P8Path = AppleExports.ConvertFilePath(projectFile?.Directory, KeyPath).FullName;
		}
		else
		{
			throw new Exception("Checking the version, or setting WhatsNew requires the AppStoreConnect API, which needs these config properties:\n" +
				"  [/Script/MacTargetPlatform.XcodeProjectSettings]\n" +
				"  AppStoreConnectIssuerID\n" +
				"  AppStoreConnectKeyID\n" +
				"  AppStoreConnectKeyPath\n" + 
				"to be set correctly in your Engine config files. Note that you don't need to set bUseAppStoreConnect to true for this to work");
		}


		// since we can't use Cng apis on mac (or linux) we have to use BouncyCastle to convert the .p8 text to a
		// ECDsa key, using some crazy elliptical cryptography black magic
		ECDsa Key;
		using (TextReader P8Reader = File.OpenText(P8Path))
		{
			ECPrivateKeyParameters ECPrivateKeyParameters = (ECPrivateKeyParameters)new Org.BouncyCastle.OpenSsl.PemReader(P8Reader).ReadObject();

            Org.BouncyCastle.Math.EC.ECPoint Q = ECPrivateKeyParameters.Parameters.G.Multiply(ECPrivateKeyParameters.D).Normalize();
			ECParameters MsEcp = new ECParameters {
				Curve = ECCurve.NamedCurves.nistP256, 
				Q = { 
					X = Q.AffineXCoord.GetEncoded(),
					Y = Q.AffineYCoord.GetEncoded()
				}, 
				D = ECPrivateKeyParameters.D.ToByteArrayUnsigned()
			};
			Key = ECDsa.Create(MsEcp);
		}

		// set up the header with the parmas that AppStoreConnect needs (kid, typ and alg)
		ECDsaSecurityKey SecurityKey = new ECDsaSecurityKey(Key) { KeyId = KeyID };
		SigningCredentials Credentials = new SigningCredentials(SecurityKey, SecurityAlgorithms.EcdsaSha256);
		JwtHeader Header = new JwtHeader(Credentials);
		
		// set expiration date to check Token validity
		expirationDate = DateTime.UtcNow.AddMinutes(19);
		
		// set up the payload manually with the body that AppStoreConnect needs (aud, exp, and iss - iat and scope are optional)
		// @todo: use scope to limit the token's access
		JwtPayload Payload = new JwtPayload
		{
			{ "aud", "appstoreconnect-v1" },
			{ "exp", (long)(expirationDate - DateTime.UnixEpoch).TotalSeconds },
			{ "iss", IssuerID }
		};

		// make a token and output as string
		JwtSecurityToken JwtToken = new JwtSecurityToken(Header, Payload);
		token = new JwtSecurityTokenHandler().WriteToken(JwtToken);

		// When debugging things with curl, you'll need this token, but we don't write it by default since it could technically 
		// be used to hack ASC within 20 minutes
//		Console.WriteLine("Auth header:\n  Authorization: Bearer {0}", token);
	}
}
