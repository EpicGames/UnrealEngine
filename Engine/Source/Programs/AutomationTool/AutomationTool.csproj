<Project Sdk="Microsoft.NET.Sdk">
  <Import Project="..\Shared\UnrealEngine.csproj.props" />

  <Choose>
    <When Condition="'$(IsWindows)' == 'true'">
      <PropertyGroup>
        <TargetFramework>net10.0-windows</TargetFramework>
        <UseWindowsForms>true</UseWindowsForms>
      </PropertyGroup>
    </When>
    <Otherwise>
      <PropertyGroup>
        <TargetFramework>net10.0</TargetFramework>
      </PropertyGroup>
    </Otherwise>
  </Choose>
  
  <PropertyGroup>
    <Configuration Condition=" '$(Configuration)' == '' ">Development</Configuration>
    <OutputType>Exe</OutputType>
    <GenerateAssemblyInfo>false</GenerateAssemblyInfo>
    <Configurations>Debug;Release;Development;Analyze</Configurations>
    <ApplicationManifest>app.manifest</ApplicationManifest>
    <RootNamespace>AutomationTool</RootNamespace>
    <AssemblyName>AutomationTool</AssemblyName>
    <WarningsNotAsErrors>612,618</WarningsNotAsErrors>
    <NoWarn>$(NoWarn);NETSDK1206</NoWarn>
    <EnableDefaultItems>false</EnableDefaultItems>
    <AppendRuntimeIdentifierToOutputPath>false</AppendRuntimeIdentifierToOutputPath>
    <SatelliteResourceLanguages>en</SatelliteResourceLanguages> <!-- remove non english resource languages -->
    <DebugType>pdbonly</DebugType>
    <DebugType Condition="'$(IsWindows)' == 'true' And '$(IsArm64)' == 'true'">portable</DebugType>
    <AllowUnsafeBlocks>true</AllowUnsafeBlocks> <!-- required for FileMatcher support code in MSBuild subdir -->
  </PropertyGroup>
  
  <PropertyGroup Condition=" '$(Configuration)|$(Platform)' == 'Development|AnyCPU' ">
    <DefineConstants>$(DefineConstants);TRACE</DefineConstants>
    <Optimize>true</Optimize>
  </PropertyGroup>
  <PropertyGroup Condition=" '$(Configuration)|$(Platform)' == 'Release|AnyCPU' ">
    <DefineConstants>$(DefineConstants);TRACE</DefineConstants>
    <Optimize>true</Optimize>
  </PropertyGroup>
  <PropertyGroup Condition=" '$(Configuration)|$(Platform)' == 'Debug|AnyCPU' ">
    <DefineConstants>$(DefineConstants);DEBUG;TRACE</DefineConstants>
    <DebugSymbols>true</DebugSymbols>
    <DebugType>full</DebugType>
  </PropertyGroup>
  <PropertyGroup Condition=" '$(Configuration)|$(Platform)' == 'Analyze|AnyCPU' ">
    <RunAnalyzersDuringBuild>True</RunAnalyzersDuringBuild>
    <TreatWarningsAsErrors>false</TreatWarningsAsErrors>
    <GenerateDocumentationFile>True</GenerateDocumentationFile>
    <DocumentationFile></DocumentationFile>
  </PropertyGroup>

  <ItemGroup>
    <Reference Include="fastJSON">
      <HintPath>..\..\..\Binaries\ThirdParty\fastJSON\netstandard2.0\fastJSON.dll</HintPath>
      <Private>true</Private>
    </Reference>
  </ItemGroup>
  <Choose>
    <When Condition="'$(IsWindows)'=='true'">
      <PropertyGroup>
        <!-- Reset output type back to a console application as the import of WindowsDesktop sets it to a windows executable -->
        <OutputType>Exe</OutputType>
      </PropertyGroup>
    </When>
  </Choose>

  <!-- This set of dependencies should be kept as small as possible -->
  <ItemGroup>
    <Compile Include="..\Shared\MetaData.cs">
      <Link>Properties\MetaData.cs</Link>
    </Compile>
    <Compile Include="Program.cs" />
    <Compile Include="Properties\AssemblyInfo.cs" />
    <None Include="app.manifest">
      <SubType>Designer</SubType>
    </None>
    <None Include="..\..\..\Intermediate\ProjectFiles\AutomationTool.csproj.References" />
  </ItemGroup>

  <ItemGroup>
    <PackageReference Include="AWSSDK.S3" Version="3.7.501.14" />
    <PackageReference Include="AWSSDK.SecurityToken" Version="3.7.501.14" />
    <PackageReference Include="Magick.NET-Q16-HDRI-AnyCPU" Version="14.13.1" />
    <PackageReference Include="Microsoft.Build" Version="18.4.0" ExcludeAssets="runtime" PrivateAssets="all">
      <IncludeAssets>compile; build; native; contentfiles; analyzers; buildtransitive</IncludeAssets>
    </PackageReference>
    <PackageReference Include="Microsoft.Build.Locator" Version="1.11.2" />
    <!-- Added as a temporary workaround to running from visual studio where the system version of dotnet is run. 
         Adding a loader context resolves many of the issues, but it still fails when MSBuild tries to write the json -->
    <PackageReference Include="Newtonsoft.Json" Version="13.0.4" />
  </ItemGroup>
  
  <!-- This set of dependencies should be kept as small as possible -->
  <ItemGroup>
    <ProjectReference Include="..\Shared\EpicGames.Core\EpicGames.Core.csproj" PrivateAssets="all" />
    <ProjectReference Include="..\Shared\EpicGames.Build\EpicGames.Build.csproj" PrivateAssets="all" />
    <ProjectReference Include="..\UnrealBuildTool\UnrealBuildTool.csproj" PrivateAssets="all" />
  </ItemGroup>

  <!-- Various batch files and shell scripts that build or run AutomationTool -->
  <ItemGroup>
    <None Include="../../../Build/BatchFiles/**/*.*">
      <Link>$([System.Text.RegularExpressions.Regex]::Replace(%(FullPath), '^.+?[\\/]BatchFiles[\\/]', 'BatchFiles/'))</Link>
    </None>
  </ItemGroup>
</Project>