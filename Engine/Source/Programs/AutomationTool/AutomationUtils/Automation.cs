// Copyright Epic Games, Inc. All Rights Reserved.
using System;
using System.Collections.Generic;
using System.Linq;
using System.Text;
using System.IO;
using System.Reflection;
using System.Diagnostics;
using UnrealBuildTool;
using EpicGames.Core;
using EpicGames.Horde;
using OpenTracing.Util;
using UnrealBuildBase;
using System.Threading.Tasks;
using AutomationUtils;
using EpicGames.LogMatchers;
using EpicGames.ProjectStore;
using Microsoft.Extensions.Logging;
using Microsoft.Extensions.DependencyInjection;

using static AutomationTool.CommandUtils;

namespace AutomationTool
{

    public static class Automation
	{
		static ILogger Logger => Log.Logger;

		/// <summary>
		/// Keep a persistent reference to the delegate for handling Ctrl-C events. Since it's passed to non-managed code, we have to prevent it from being garbage collected.
		/// </summary>
		static ProcessManager.CtrlHandlerDelegate CtrlHandlerDelegateInstance = CtrlHandler;

        static bool CtrlHandler(CtrlTypes EventType)
		{
			Domain_ProcessExit(null, null);
			if (EventType == CtrlTypes.CTRL_C_EVENT)
			{
				// Force exit
				Environment.Exit(3);
			}			
			return true;
		}

		static void Domain_ProcessExit(object sender, EventArgs e)
		{
			// Kill all spawned processes (Console instead of Log because logging is closed at this time anyway)
			if (ShouldKillProcesses && OperatingSystem.IsWindows())
			{			
				ProcessManager.KillAll();
			}
			Trace.Close();
		}

		/// <summary>
		/// Main method.
		/// </summary>
		/// <param name="AutomationToolCommandLine">Command line</param>
		/// <param name="ScriptModuleAssemblies"></param>
		public static async Task<ExitCode> ProcessAsync(ParsedCommandLine AutomationToolCommandLine, HashSet<FileReference> ScriptModuleAssemblies)
		{
			GlobalCommandLine.Initialize(AutomationToolCommandLine);

			// Hook up exit callbacks
			AppDomain Domain = AppDomain.CurrentDomain;
			Domain.ProcessExit += Domain_ProcessExit;
			Domain.DomainUnload += Domain_ProcessExit;
			HostPlatform.Current.SetConsoleCtrlHandler(CtrlHandlerDelegateInstance);

			try
			{
				IsBuildMachine = GlobalCommandLine.BuildMachine;
				if (!IsBuildMachine)
				{
					int Value;
					if (int.TryParse(Environment.GetEnvironmentVariable("IsBuildMachine"), out Value) && Value != 0)
					{
						IsBuildMachine = true;
					}
				}

				Logger.LogDebug("IsBuildMachine={IsBuildMachine}", IsBuildMachine);
				Environment.SetEnvironmentVariable("IsBuildMachine", IsBuildMachine ? "1" : "0");

				// Register all the log event matchers
				Log.EventParser.AddMatchers(LogMatcherCollections.AutomationUtilsMatchers);
				Log.EventParser.AddMatchers(LogMatcherCollections.UBTMatchers);

				// Get the path to the telemetry file, if present
				string TelemetryFile = GlobalCommandLine.TelemetryPath;
				JsonTracer Tracer = JsonTracer.TryRegisterAsGlobalTracer();

				// should we kill processes on exit
				ShouldKillProcesses = !GlobalCommandLine.NoKill;
				Logger.LogDebug("ShouldKillProcesses={ShouldKillProcesses}", ShouldKillProcesses);

				if (AutomationToolCommandLine.CommandsToExecute.Count == 0)
				{
					if (GlobalCommandLine.Help)
					{
						DisplayHelp(AutomationToolCommandLine.GlobalParameters);
						return ExitCode.Success;
					}
					if (GlobalCommandLine.CompileOnly)
					{
						return ExitCode.Success;
					}
				}

				// Disable AutoSDKs if specified on the command line
				if (GlobalCommandLine.NoAutoSDK)
				{
					PlatformExports.PreventAutoSDKSwitching();
				}

				// Setup environment
				Logger.LogDebug("Setting up command environment.");
				CommandUtils.InitCommandEnvironment();

				// Create the log file, and flush the startup listener to it
				LogUtils.AddLogFileListener(new DirectoryReference(CommandUtils.CmdEnv.LogFolder), new DirectoryReference(CommandUtils.CmdEnv.FinalLogFolder));
				if (LogUtils.FinalLogFileName != LogUtils.LogFileName)
				{
					Logger.LogInformation("Final log location: {Location}", LogUtils.FinalLogFileName);
				}
				
				// Initialize UBT
				if (!UnrealBuildTool.PlatformExports.Initialize(Environment.GetCommandLineArgs(), Log.Logger))
				{
					Logger.LogInformation("Failed to initialize UBT");
					return ExitCode.Error_Unknown;
				}
				
				// Clean rules folders up
				if (!CommandUtils.CmdEnv.IsChildInstance)
				{
					ProjectUtils.CleanupFolders();
				}

				// Compile scripts.
				using (GlobalTracer.Instance.BuildSpan("ScriptLoad").StartActive())
				{
					ScriptManager.LoadScriptAssemblies(ScriptModuleAssemblies, Logger);
				}

				if (GlobalCommandLine.List)
				{
					ListAvailableCommands(ScriptManager.Commands);
					return ExitCode.Success;
				}

				if (GlobalCommandLine.Help)
				{
					DisplayHelp(AutomationToolCommandLine.CommandsToExecute, ScriptManager.Commands);
					return ExitCode.Success;
				}

				// Enable or disable P4 support
				CommandUtils.InitP4Support(AutomationToolCommandLine.CommandsToExecute, ScriptManager.Commands);
				if (CommandUtils.P4Enabled)
				{
					Logger.LogDebug("Setting up Perforce environment.");
					using (GlobalTracer.Instance.BuildSpan("InitP4").StartActive())
					{
						CommandUtils.InitP4Environment();
						CommandUtils.InitDefaultP4Connection();
					}
				}

				try
				{
					// Find and execute commands.
					ExitCode Result = await ExecuteAsync(AutomationToolCommandLine.CommandsToExecute, ScriptManager.Commands);
					if (TelemetryFile != null)
					{
						Directory.CreateDirectory(Path.GetDirectoryName(TelemetryFile));
						CommandUtils.Telemetry.Write(TelemetryFile);
					}

					return Result;
				}
				finally
				{
					// Flush any timing data
					TraceSpan.Flush();
					
					if (Tracer != null)
					{
						Tracer.Flush();
					}
				}
			}
			catch (AutomationException Ex)
			{
				// Output the message in the desired format
				if (Ex.OutputFormat == AutomationExceptionOutputFormat.Silent)
				{
					Logger.LogDebug("{Arg0}", ExceptionUtils.FormatExceptionDetails(Ex));
				}
				else if (Ex.OutputFormat == AutomationExceptionOutputFormat.Minimal)
				{
					Logger.LogInformation(Ex, "{Message}", Ex.ToString().Replace("\n", "\n  "));
					Logger.LogDebug(Ex, "{Details}", ExceptionUtils.FormatExceptionDetails(Ex));
				}
				else if (Ex.OutputFormat == AutomationExceptionOutputFormat.MinimalError)
				{
					Logger.LogError(Ex, "{Message}", Ex.ToString().Replace("\n", "\n  "));
					Logger.LogDebug(Ex, "{Details}", ExceptionUtils.FormatExceptionDetails(Ex));
				}
				else
				{
					Log.WriteException(Ex, LogUtils.FinalLogFileName);
				}

				// Take the exit code from the exception
				return Ex.ErrorCode;
			}
			catch (Exception Ex)
			{
				// Use a default exit code
				Log.WriteException(Ex, LogUtils.FinalLogFileName);
				return ExitCode.Error_Unknown;
			}
			finally
			{
				// In all cases, do necessary shut down stuff, but don't let any additional exceptions leak out while trying to shut down.

				// Make sure there's no directories on the stack.
				NoThrow(() => CommandUtils.ClearDirStack(), "Clear Dir Stack");

				// Try to kill process before app domain exits to leave the other KillAll call to extreme edge cases
				NoThrow(() =>
				{
					if (ShouldKillProcesses && OperatingSystem.IsWindows()) ProcessManager.KillAll();
				}, "Kill All Processes");
			}
		}

		/// <summary>
		/// Wraps an action in an exception block.
		/// Ensures individual actions can be performed and exceptions won't prevent further actions from being executed.
		/// Useful for shutdown code where shutdown may be in several stages and it's important that all stages get a chance to run.
		/// </summary>
		/// <param name="Action"></param>
		/// <param name="ActionDesc"></param>
		private static void NoThrow(System.Action Action, string ActionDesc)
        {
            try
            {
                Action();
            }
            catch (Exception Ex)
            {
                Logger.LogError(Ex, "Exception performing nothrow action \"{ActionDesc}\": {Exception}", ActionDesc, ExceptionUtils.FormatException(Ex));
            }
        }

		/// <summary>
		/// Execute commands specified in the command line.
		/// </summary>
		/// <param name="CommandsToExecute"></param>
		/// <param name="Commands"></param>
		public static async Task<ExitCode> ExecuteAsync(List<CommandInfo> CommandsToExecute, Dictionary<string, Type> Commands)
		{
			ServiceCollection serviceCollection = new ServiceCollection();
			serviceCollection.AddLogging(builder => builder.AddEpicDefault());
			serviceCollection.AddHorde(options => options.AllowAuthPrompt = !CommandUtils.IsBuildMachine);

			serviceCollection.AddScoped<BuildCommandContainer>();
			serviceCollection.AddCloudStorage((CloudStorageOptions options) =>
			{
				BuildCommandContainer currentBuildCommand = CommandUtils.ServiceScope.ServiceProvider.GetService<BuildCommandContainer>();
				
				string projectFile = currentBuildCommand.CurrentCommand.ParseParamValue("Project");
				options.ReadFromIni(projectFile != null ? new FileReference(projectFile) : null /* no project file specified, only engine inis will be applied */);

				options.AllowAuthPrompt = !CommandUtils.IsBuildMachine;
				options.FindExecutablesFromUnrealRoot();
				options.EchoZenCliToStdout = true;

				options.ReadFromEnv();
				options.OnProcessCreated += process => ProcessManager.AddProcess(ZenUATProcessUtil.GetUATProcessStub(process));
				options.OnProcessExited += process => ProcessManager.RemoveProcess(ZenUATProcessUtil.GetUATProcessStub(process));
			});

			await using ServiceProvider serviceProvider = serviceCollection.BuildServiceProvider();
			try
			{
				CommandUtils.ServiceProvider = serviceProvider;

				Logger.LogInformation("Executing commands...");
				for (int CommandIndex = 0; CommandIndex < CommandsToExecute.Count; ++CommandIndex)
				{
					CommandInfo CommandInfo = CommandsToExecute[CommandIndex];
					Logger.LogDebug("Attempting to execute {CommandInfo}", CommandInfo.ToString());
					Type CommandType;
					if (!Commands.TryGetValue(CommandInfo.CommandName, out CommandType))
					{
						throw new AutomationException("Failed to find command {0}", CommandInfo.CommandName);
					}

					// each command is considered a "request" in the DI framework e.g. owns scoped providers
					// allows for systems that require per command configuration
					using IServiceScope serviceScope = serviceProvider.CreateScope();
					CommandUtils.ServiceScope = serviceScope;

					BuildCommand Command = (BuildCommand)Activator.CreateInstance(CommandType);

					// make the current build command available in the scope
					BuildCommandContainer buildCommandContainer = serviceScope.ServiceProvider.GetService<BuildCommandContainer>();
					buildCommandContainer.CurrentCommand = Command;

					Command.Params = CommandInfo.Arguments.ToArray();
					try
					{
						ExitCode Result = await Command.ExecuteAsync();
						if (Result != ExitCode.Success)
						{
							return Result;
						}
						Logger.LogInformation("BUILD SUCCESSFUL");
					}
					finally
					{
						// dispose of the class if necessary
						IDisposable CommandDisposable = Command as IDisposable;
						if (CommandDisposable != null)
						{
							CommandDisposable.Dispose();
						}
					}

					// Make sure there's no directories on the stack.
					CommandUtils.ClearDirStack();
				}
				return ExitCode.Success;
			}
			finally
			{
				CommandUtils.ServiceProvider = null!;
				CommandUtils.ServiceScope = null!;
			}
		}

		/// <summary>
		/// Display help for the specified commands (to execute)
		/// </summary>
		/// <param name="CommandsToExecute">List of commands specified in the command line.</param>
		/// <param name="Commands">All discovered command objects.</param>
		private static void DisplayHelp(List<CommandInfo> CommandsToExecute, Dictionary<string, Type> Commands)
		{
			for (int CommandIndex = 0; CommandIndex < CommandsToExecute.Count; ++CommandIndex)
			{
				CommandInfo CommandInfo = CommandsToExecute[CommandIndex];
				Type CommandType;
				if (Commands.TryGetValue(CommandInfo.CommandName, out CommandType) == false)
				{
					Logger.LogError("Help: Failed to find command {CommandName}", CommandInfo.CommandName);
				}
				else
				{
					CommandUtils.Help(CommandType);
				}
			}
		}

		/// <summary>
		/// Display AutomationTool help.
		/// </summary>
		private static void DisplayHelp(Dictionary<string, string> ParamDict)
		{
			HelpUtils.PrintHelp("Automation Help:",
@"Executes scripted commands

AutomationTool [-verbose] [-Help|-List|-CompileOnly] [-p4] Command0 [-Arg0 -Arg1 -Arg2 ...] Command1 [-Arg0 -Arg1 ...] Command2 [-Arg0 ...] CommandN ... [EnvVar0=MyValue0 ... EnvVarN=MyValueN]",
				ParamDict.ToList());
			CommandUtils.LogHelp(typeof(Automation));
		}

		/// <summary>
		/// List all available commands.
		/// </summary>
		/// <param name="Commands">All vailable commands.</param>
		private static void ListAvailableCommands(Dictionary<string, Type> Commands)
		{
			string Message = Environment.NewLine;
			Message += "Available commands:" + Environment.NewLine;
			string AssemblyName = "";
			foreach (KeyValuePair<string, Type> AvailableCommand in 
				Commands.OrderBy(Command => Command.Value.Assembly.GetName().Name).ThenBy(Command => Command.Key))
			{
				string NewAssemblyName = AvailableCommand.Value.Assembly.GetName().Name;
				if (!String.Equals(AssemblyName, NewAssemblyName))
				{
					AssemblyName = NewAssemblyName;
					Message += $"  {AssemblyName}:\n";
				}
				Message += String.Format($"    {AvailableCommand.Key}\n");
			}
			
			Logger.LogInformation("{Text}", Message);
		}

		/// <summary>
		/// True if this process is running on a build machine, false if locally.
		/// </summary>
		/// <remarks>
		/// The reason one this property exists in Automation class and not BuildEnvironment is that
		/// it's required long before BuildEnvironment is initialized.
		/// </remarks>
		public static bool IsBuildMachine
		{
			get
			{
				if (!bIsBuildMachine.HasValue)
				{
					throw new AutomationException("Trying to access IsBuildMachine property before it was initialized.");
				}
				return (bool)bIsBuildMachine;				
			}
			private set
			{
				bIsBuildMachine = value;
			}
		}
		private static bool? bIsBuildMachine;

		public static bool ShouldKillProcesses
		{
			get
			{
				return bShouldKillProcesses;
			}
			private set
			{
				bShouldKillProcesses = value;
			}
		}
		private static bool bShouldKillProcesses = true;
	}

	public class BuildCommandContainer
	{
		public BuildCommand CurrentCommand { get; set; }
	}


	// This class turns stringly-typed commandline option values into named compile-time values
	public class GlobalCommandLine
	{
		// Descriptions for these command line options may be found in AutomationTool/Program.cs

		public static void Initialize(ParsedCommandLine AutomationToolCommandLine)
		{
			BuildMachine = AutomationToolCommandLine.IsSetGlobal("-BuildMachine");
			NoKill = AutomationToolCommandLine.IsSetGlobal("-NoKill");
			Help = AutomationToolCommandLine.IsSetGlobal("-Help");
			NoAutoSDK = AutomationToolCommandLine.IsSetGlobal("-NoAutoSDK");
			List = AutomationToolCommandLine.IsSetGlobal("-List");
			CompileOnly = AutomationToolCommandLine.IsSetGlobal("-CompileOnly");
			TelemetryPath = (string)AutomationToolCommandLine.GetValueUnchecked("-Telemetry");
			Verbose = AutomationToolCommandLine.IsSetGlobal("-Verbose");
			AllowStdOutLogVerbosity = AutomationToolCommandLine.IsSetGlobal("-AllowStdOutLogVerbosity");
			UTF8Output = AutomationToolCommandLine.IsSetGlobal("-UTF8Output");
			UseLocalBuildStorage = AutomationToolCommandLine.IsSetGlobal("-UseLocalBuildStorage");
			Submit = AutomationToolCommandLine.IsSetGlobal("-Submit");
			NoSubmit = AutomationToolCommandLine.IsSetGlobal("-NoSubmit");
			P4 = AutomationToolCommandLine.IsSetGlobal("-P4");
			NoP4 = AutomationToolCommandLine.IsSetGlobal("-NoP4");

			int WaitTimeMs;
			if (int.TryParse((string)AutomationToolCommandLine.GetValueUnchecked("-WaitForStdStreams"), out WaitTimeMs))
			{
				WaitForStdStreams = WaitTimeMs;
			}
		}

		// Using Nullable bools here to ensure that Initialize() has been called before the members are accesed.
		// Accessing the value before it has been set will cause an InvalidOperationException to be thrown.
		// Private setters ensure values are set by Initialize()

		private static bool? bBuildMachine;
		public static bool BuildMachine
		{
			get => bBuildMachine.Value; 
			private set { bBuildMachine = value; }
		}

		private static bool? bNoKill;
		public static bool NoKill 
		{ 
			get => bNoKill.Value; 
			private set { bNoKill = value; } 
		}

		private static bool? bHelp;
		public static bool Help
		{
			get => bHelp.Value; 
			private set { bHelp = value; }
		}

		private static bool? bNoAutoSDK;
		public static bool NoAutoSDK
		{
			get => bNoAutoSDK.Value; 
			private set { bNoAutoSDK = value; }
		}

		private static bool? bList;
		public static bool List
		{
			get => bList.Value; 
			private set { bList = value; }
		}
		
		private static bool? bCompileOnly;
		public static bool CompileOnly
		{
			get => bCompileOnly.Value; 
			private set { bCompileOnly = value; }
		}
		
		private static bool? bVerbose;
		public static bool Verbose
		{
			get => bVerbose.Value; 
			private set { bVerbose = value; }
		}
		
		private static bool? bAllowStdOutLogVerbosity;
		public static bool AllowStdOutLogVerbosity
		{
			get => bAllowStdOutLogVerbosity.Value; 
			private set { bAllowStdOutLogVerbosity = value; }
		}
		
		private static bool? bUTF8Output;
		public static bool UTF8Output
		{
			get => bUTF8Output.Value; 
			private set { bUTF8Output = value; }
		}
		
		private static bool? bUseLocalBuildStorage;
		public static bool UseLocalBuildStorage
		{
			get => bUseLocalBuildStorage.Value; 
			private set { bUseLocalBuildStorage = value; }
		}
		
		private static bool? bSubmit;
		public static bool Submit
		{
			get => bSubmit.Value; 
			private set { bSubmit = value; }
		}
		
		private static bool? bNoSubmit;
		public static bool NoSubmit
		{
			get => bNoSubmit.Value; 
			private set { bNoSubmit = value; }
		}
		
		private static bool? bP4;
		public static bool P4
		{
			get => bP4.Value; 
			private set { bP4 = value; }
		}
		
		private static bool? bNoP4;
		public static bool NoP4
		{
			get => bNoP4.Value; 
			private set { bNoP4 = value; }
		}
		
		public static string TelemetryPath
		{
			get; 
			private set;
		}

		public static int WaitForStdStreams
		{
			get;
			private set;
		} = -1;
	}
}
