// Copyright Epic Games, Inc. All Rights Reserved.
using System;
using System.Collections.Generic;
using System.Linq;
using System.Text;
using UnrealBuildBase;

namespace AutomationTool
{
	/// <summary>
	/// How the exception should be output
	/// </summary>
	public enum AutomationExceptionOutputFormat
	{
		/// <summary>
		/// Do not output any message; just output the exception details to the log.
		/// </summary>
		Silent,

		/// <summary>
		/// Output a brief summary to the console, and the exception details to the log
		/// </summary>
		Minimal,

		/// <summary>
		/// Output a brief summary to the console as an error, and the exception details to the log
		/// </summary>
		MinimalError,

		/// <summary>
		/// Treat the exception as an error
		/// </summary>
		Error
	}

    /// <summary>
    /// Exception class used by the AutomationTool to throw exceptions. Allows setting an exit code that will be passed to the entry routine to return to the system on program exit.
    /// If no exit code is given, Error_Unkonwn is used.
    /// </summary>
    public class AutomationException : System.Exception
	{
        public ExitCode ErrorCode = ExitCode.Error_Unknown;
		public AutomationExceptionOutputFormat OutputFormat = AutomationExceptionOutputFormat.Error;

		public AutomationException(string Msg)
			:base(Msg)
		{
		}

        public AutomationException(ExitCode ErrorCode, string Msg)
            : base(Msg)
        {
            this.ErrorCode = ErrorCode;
        }

        public AutomationException(Exception InnerException, string Format, params object[] Args)
			: base(string.Format(Format, Args), InnerException)
		{
		}

        public AutomationException(ExitCode ErrorCode, Exception InnerException, string Format, params object[] Args)
            : base(string.Format(Format, Args), InnerException)
        {
            this.ErrorCode = ErrorCode;
        }

        public AutomationException(string Format, params object[] Args)
			: base(string.Format(Format, Args))
		{
		}

        public AutomationException(ExitCode ErrorCode, string Format, params object[] Args)
            : base(string.Format(Format, Args)) 
        {
            this.ErrorCode = ErrorCode;
        }

		public override string ToString()
		{
			return Message;
		}
	}
}
