<Project Sdk="Microsoft.NET.Sdk">
  <Import Project="..\..\Shared\UnrealEngine.csproj.props" />
  
  <PropertyGroup>
    <OutputType>Library</OutputType>
    <RootNamespace>AutomationUtils</RootNamespace>
    <AssemblyName>AutomationUtils.Automation</AssemblyName>
    <GenerateAssemblyInfo>false</GenerateAssemblyInfo>
    <GenerateTargetFrameworkAttribute>false</GenerateTargetFrameworkAttribute>
    <TreatWarningsAsErrors>true</TreatWarningsAsErrors>
    <CopyLocalLockFileAssemblies>true</CopyLocalLockFileAssemblies>
  </PropertyGroup>
  <PropertyGroup Condition="'$(EpicGamesMsBuild)' == 'true'">
    <OutputPath Condition="'$(ArtifactsPath)' == ''">..\..\..\..\Binaries\DotNET\AutomationTool\AutomationUtils</OutputPath>
  </PropertyGroup>

  <!-- The following will glob all platform extensions' AutomationUtils source
  files and include them in the build. They will also appear in VisualStudio -->
  <ItemGroup>
    <Compile Include="../../../../Platforms/*/Source/Programs/AutomationTool/AutomationUtils/**/*.cs">
      <!-- RecursiveDir metadata expands to [PlatformName]/Source/Programs/[etc]. The Replace() will
      replace everything from the first slash to the end leaving just the platform name. Should the
      Replace() fail then RecursiveDir is still an agreeable value -->
      <Link>Platform/$([System.Text.RegularExpressions.Regex]::Replace(%(Compile.RecursiveDir), [\\/].+$, ``))/%(Compile.FileName).cs</Link>
    </Compile>
  </ItemGroup>
  
  <ItemGroup>
    <ProjectReference Include="..\..\Shared\EpicGames.Core\EpicGames.Core.csproj" PrivateAssets="All"><Private>true</Private></ProjectReference>
    <ProjectReference Include="..\..\Shared\EpicGames.Build\EpicGames.Build.csproj" PrivateAssets="All"> <Private>true</Private> </ProjectReference>
    <ProjectReference Include="..\..\Shared\EpicGames.Horde\EpicGames.Horde.csproj" PrivateAssets="All">
      <Private>true</Private>
    </ProjectReference>
    <ProjectReference Include="..\..\Shared\EpicGames.LogMatchers\EpicGames.LogMatchers.csproj" />
    <ProjectReference Include="..\..\Shared\EpicGames.Perforce\EpicGames.Perforce.csproj" />
    <ProjectReference Include="..\..\Shared\EpicGames.ProjectStore\EpicGames.ProjectStore.csproj" PrivateAssets="All" />
    <ProjectReference Include="..\..\UnrealBuildTool\UnrealBuildTool.csproj" PrivateAssets="All"><Private>true</Private></ProjectReference>
  </ItemGroup>
  <ItemGroup>
    <PackageReference Include="Magick.NET-Q16-HDRI-AnyCPU" Version="14.13.1" PrivateAssets="all" />
    <PackageReference Include="System.Drawing.Common" Version="10.0.6" />
    <!--Required by Ionic -->
    <PackageReference Include="System.Security.Permissions" Version="10.0.6" />

    <PackageReference Include="Google.Apis.Drive.v3" Version="1.33.1.1232" PrivateAssets="all" />
    <PackageReference Include="Google.Apis.Sheets.v4" Version="1.33.1.1240" PrivateAssets="all" />
  </ItemGroup>
</Project>