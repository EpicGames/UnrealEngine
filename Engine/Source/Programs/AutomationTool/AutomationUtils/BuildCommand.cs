// Copyright Epic Games, Inc. All Rights Reserved.
using System;
using System.Collections.Generic;
using System.Linq;
using System.Text;
using System.Threading.Tasks;
using EpicGames.Core;
using UnrealBuildBase;
using Microsoft.Extensions.Logging;
using System.Diagnostics.CodeAnalysis;

#nullable enable

namespace AutomationTool
{
	/// <summary>
	/// Base class for buildcommands.
	/// </summary>
	public abstract class BuildCommand : CommandUtils
	{
		/// <summary>
		/// Command line parameters for this command (empty by non-null by default)
		/// </summary>
		public string[] Params
		{
			get { return _commandLineParams; }
			set { _commandLineParams = value; }
		}

		private string[] _commandLineParams = new string[0];

		/// <summary>
		/// Parses the command's Params list for a parameter and returns whether it is defined or not.
		/// </summary>
		/// <param name="Param">Param to check for.</param>
		/// <returns>True if param was found, false otherwise.</returns>
		public bool ParseParam(string Param)
		{
			return ParseParam(Params, Param);
		}

		/// <summary>
		/// Parses the command's Params list for a parameter and reads its value. 
		/// Ex. ParseParamValue(Args, "map=")
		/// </summary>
		/// <param name="Param">Param to read its value.</param>
		/// <param name="Default"></param>
		/// <param name="ObsoleteParam"></param>
		/// <returns>Returns the value or Default if the parameter was not found.</returns>
		[return: NotNullIfNotNull("Default")]
		public string? ParseParamValue(string Param, string? Default = null, string? ObsoleteParam = null)
		{
			string ParamValue = ParseParamValue(Params, Param, null);

			if (ObsoleteParam != null)
			{
				string ObsoleteParamValue = ParseParamValue(Params, ObsoleteParam, null);

				if (ObsoleteParamValue != null)
				{
					if (ParamValue == null)
					{
						Logger.LogWarning("Param name \"{ObsoleteParam}\" is deprecated, use \"{Param}\" instead.", ObsoleteParam, Param);
					}
					else
					{
						Logger.LogWarning("Deprecated param name \"{ObsoleteParam}\" was ignored because \"{Param}\" was set.", ObsoleteParam, Param);
					}
				}

			}

			return ParamValue ?? Default;
		}

		/// <summary>
		/// Parses the command's Params list for a property set in the format set:&lt;Param&gt;=&lt;Value&gt; and reads its value. 
		/// Ex. ParseSetPropertyValue("projectName", "") will return "Lyra" for commandline "BuildGraph -set:ProjectName=Lyra"
		/// </summary>
		/// <param name="propertyName">PropertyName to read its value.</param>
		/// <param name="Default">Value to return if the PropertyName is not found.</param>
		/// <returns>Returns the value or Default if the parameter was not found.</returns>
		[return: NotNullIfNotNull("Default")]
		public string? ParseSetPropertyValue(string propertyName, string? Default = null)
		{
			return ParseParamValue(Params, $"set:{propertyName}", Default);
		}

		/// <summary>
		/// Parses an argument.
		/// </summary>
		/// <param name="Param"></param>
		/// <returns></returns>
		public string? ParseOptionalStringParam(string Param)
		{
			return ParseParamValue(Param, null);
		}

		/// <summary>
		/// Parses an argument. Throws an exception if the parameter is not specified.
		/// </summary>
		/// <param name="Param">Name of the argument</param>
		/// <returns>Value of the argument</returns>
		public string ParseRequiredStringParam(string Param)
		{
			string? value = ParseOptionalStringParam(Param);
			if(value == null)
			{
				throw new AutomationException("Missing -{0}=... parameter", Param);
			}
			return value;
		}

		/// <summary>
		/// Parses an file reference argument.
		/// </summary>
		/// <param name="Param">Name of the argument</param>
		/// <returns>Value of the argument</returns>
		public FileReference? ParseOptionalFileReferenceParam(string Param)
		{
			string? stringValue = ParseParamValue(Param);
			if(stringValue == null)
			{
				return null;
			}
			else
			{
				return new FileReference(stringValue);
			}
		}

		/// <summary>
		/// Parses an file reference argument. Throws an exception if the parameter is not specified.
		/// </summary>
		/// <param name="Param">Name of the argument</param>
		/// <returns>Value of the argument</returns>
		public FileReference ParseRequiredFileReferenceParam(string Param)
		{
			FileReference? value = ParseOptionalFileReferenceParam(Param);
			if(value == null)
			{
				throw new AutomationException("Missing -{0}=... parameter", Param);
			}
			return value;
		}

		/// <summary>
		/// Parses a directory reference argument.
		/// </summary>
		/// <param name="Param">Name of the argument</param>
		/// <returns>Value of the argument</returns>
		public DirectoryReference? ParseOptionalDirectoryReferenceParam(string Param)
		{
			string? stringValue = ParseOptionalStringParam(Param);
			if(stringValue == null)
			{
				return null;
			}
			else
			{
				return new DirectoryReference(stringValue);
			}
		}

		/// <summary>
		/// Parses a directory reference argument. Throws an exception if the parameter is not specified.
		/// </summary>
		/// <param name="Param">Name of the argument</param>
		/// <returns>Value of the argument</returns>
		public DirectoryReference ParseRequiredDirectoryReferenceParam(string Param)
		{
			DirectoryReference? value = ParseOptionalDirectoryReferenceParam(Param);
			if(value == null)
			{
				throw new AutomationException("Missing -{0}=... parameter", Param);
			}
			return value;
		}

		/// <summary>
		/// Parses an argument as an enum.
		/// </summary>
		/// <param name="Param">Name of the parameter to read.</param>
		/// <returns>Returns the value that was parsed.</returns>
		public Nullable<T> ParseOptionalEnumParam<T>(string Param) where T : struct
		{
			string? valueString = ParseParamValue(Param);
			if(valueString == null)
			{
				return null;
			}
			else
			{
				T Value;
				if(!Enum.TryParse<T>(valueString, out Value))
				{
					throw new AutomationException("'{0}' is not a valid value for {1}", valueString, typeof(T).Name);
				}
				return Value;
			}
		}

		/// <summary>
		/// Parses an argument as an enum. Throws an exception if the parameter is not specified.
		/// </summary>
		/// <param name="Param">Name of the parameter to read.</param>
		/// <returns>Returns the value that was parsed.</returns>
		public T ParseRequiredEnumParamEnum<T>(string Param) where T : struct
		{
			Nullable<T> value = ParseOptionalEnumParam<T>(Param);
			if(!value.HasValue)
			{
				throw new AutomationException("Missing -{0}=... parameter", Param);
			}
			return value.Value;
		}

		/// <summary>
		/// Parses the argument list for any number of parameters.
		/// </summary>
		/// <param name="Param">Param to read its value.</param>
		/// <returns>Returns an array of values for this parameter (or an empty array if one was not found.</returns>
		public string[] ParseParamValues(string Param)
		{
			return ParseParamValues(Params, Param);
		}

		/// <summary>
		/// Parses the command's Params list for a parameter and reads its value. 
		/// Ex. ParseParamValue(Args, "map=")
		/// </summary>
		/// <param name="Param">Param to read its value.</param>
		/// <param name="Default"></param>
		/// <returns>Returns the value or Default if the parameter was not found.</returns>
		public bool ParseParamBool(string Param, bool Default = false)
		{
			string boolValue = ParseParamValue(Params, Param, Default.ToString());
			return bool.Parse(boolValue);
		}

		/// <summary>
		/// Parses the command's Params list for a parameter and reads its value. 
		/// Ex. ParseParamValue(Args, "map=")
		/// </summary>
		/// <param name="Param">Param to read its value.</param>
		/// <param name="Default"></param>
		/// <returns>Returns the value or Default if the parameter was not found.</returns>
		public int ParseParamInt(string Param, int Default = 0)
		{
			string num = ParseParamValue(Params, Param, Default.ToString());
			return int.Parse(num);
		}

		/// <summary>
		/// Parses the command's Params list for a parameter and reads its value. 
		/// Ex. ParseParamValue(Args, "map=")
		/// </summary>
		/// <param name="Param">Param to read its value.</param>
		/// <returns>Returns the value or Default if the parameter was not found.</returns>
		public int? ParseParamNullableInt(string Param)
		{
			string value = ParseParamValue(Params, Param, null);
			if(value == null)
			{
				return null;
			}
			else
			{
				return int.Parse(value);
			}
		}
		/// <summary>
		/// Parses project name string into a FileReference. Can be "Game" or "Game.uproject" or "Path/To/Game.uproject"
		/// </summary>
		/// <param name="originalProjectName">In project string to parse</param>
		/// <returns>FileReference to uproject</returns>
		public FileReference? ParseProjectString(string originalProjectName)
		{
			FileReference? projectFullPath = null;

			if (string.IsNullOrEmpty(originalProjectName))
			{
				return null;
			}

			string projectName = originalProjectName;
			projectName = projectName.Trim(new char[] { '\"' });
			if (projectName.IndexOfAny(new char[] { '\\', '/' }) < 0)
			{
				projectName = CombinePaths(CmdEnv.LocalRoot, projectName, projectName + ".uproject");
			}
			else if (!FileExists_NoExceptions(projectName))
			{
				projectName = CombinePaths(CmdEnv.LocalRoot, projectName);
			}
			if (FileExists_NoExceptions(projectName))
			{
				projectFullPath = new FileReference(projectName);
			}
			else
			{
				BranchInfo Branch = new BranchInfo();
				BranchInfo.BranchUProject GameProj = Branch.FindGame(originalProjectName);
				if (GameProj != null)
				{
					projectFullPath = GameProj.FilePath;
				}
			}

			return projectFullPath;
		}

		/// <summary>
		/// Attempts to parse the project param. Returns null if no project is found, and throws if the specified project does not exist on disk.
		/// </summary>
		/// <returns>FileReference to uproject</returns>
		public FileReference? ParseProjectParam()
		{
			FileReference? projectFullPath = null;

			bool bForeign = ParseParam("foreign");
			bool bForeignCode = ParseParam("foreigncode");
			if (bForeign)
			{
				string destSample = ParseParamValue("DestSample", "CopiedHoverShip");
				string dest = ParseParamValue("ForeignDest", CombinePaths(@"C:\testue\foreign\", destSample + "_ _Dir"));
				projectFullPath = new FileReference(CombinePaths(dest, destSample + ".uproject"));
			}
			else if (bForeignCode)
			{
				string destSample = ParseParamValue("DestSample", "PlatformerGame");
				string dest = ParseParamValue("ForeignDest", CombinePaths(@"C:\testue\foreign\", destSample + "_ _Dir"));
				projectFullPath = new FileReference(CombinePaths(dest, destSample + ".uproject"));
			}
			else
			{
				string originalProjectName = ParseParamValue("project", "");

				if (string.IsNullOrEmpty(originalProjectName))
				{
					return null;
				}

				projectFullPath = ParseProjectString(originalProjectName);

				if (projectFullPath == null || !FileExists_NoExceptions(projectFullPath.FullName))
				{
					throw new AutomationException("Could not find a project file {0}.", originalProjectName);
				}
			}

			return projectFullPath;
		}

		/// <summary>
		/// Searches for project's .uproject filepath, user-friendly name, and directory of .uproject filepath, from
		/// commandline arguments (including -project= and -set:Project???= parameters). Caller should handle
		/// possiblity that some fields are valid and others are null.
		/// </summary>
		public void ParseProjectIdParams(out FileReference? projectFile, out string? projectName,
			out DirectoryReference? projectDir)
		{
			FileReference? projectFullPath = ParseProjectParam();
			if (projectFullPath != null)
			{
				projectFile = projectFullPath;
				projectName = projectFullPath.GetFileNameWithoutExtension();
				projectDir = projectFullPath.Directory;
			}
			else
			{
				projectFile = null;
				projectName = ParseSetPropertyValue("projectName", null);
				projectDir = null;

				string? projectFileString = ParseSetPropertyValue("projectFile", null);
				if (projectFileString != null)
				{
					projectFile = ParseProjectString(projectFileString);
					if (projectFile != null)
					{
						projectDir = projectFile.Directory;
					}
				}
			}
		}

		/// <summary>
		/// Checks that all of the required params are present, throws an exception if not
		/// </summary>
		/// <param name="args"></param>
		public void CheckParamsArePresent(params string[] args)
		{
			List<string> missingParams = new List<string>();
			foreach (string arg in args)
			{
				if (ParseParamValue(arg, null) == null)
				{
					missingParams.Add(arg);
				}
			}

			if (missingParams.Count > 0)
			{
				throw new AutomationException("Params {0} are missing but required. Required params are {1}", string.Join(",", missingParams), string.Join(",", args));
			}
		}

		/// <summary>
		/// Build command entry point.  Throws AutomationExceptions on failure.
		/// </summary>
		public virtual void ExecuteBuild()
		{
			throw new AutomationException("Either Execute() or ExecuteBuild() should be implemented for {0}", GetType().Name);
		}

		/// <summary>
		/// Command entry point.
		/// </summary>
		public virtual ExitCode Execute()
		{
			ExecuteBuild();
			return ExitCode.Success;
		}

		/// <summary>
		/// Async command entry point.
		/// </summary>
		public virtual Task<ExitCode> ExecuteAsync()
		{
			return Task.FromResult(Execute());
		}

		/// <summary>
		/// Executes a new command as a child of another command.
		/// </summary>
		/// <typeparam name="T"></typeparam>
		/// <param name="ParentCommand"></param>
		public static ExitCode Execute<T>(BuildCommand ParentCommand) where T : BuildCommand, new()
		{
			T Command = new T();
			if (ParentCommand != null)
			{
				Command.Params = ParentCommand.Params;
			}
			return Command.Execute();
		}
	}
}
