// Copyright Epic Games, Inc. All Rights Reserved.
using System;
using System.Collections.Generic;
using System.Text;
using System.IO;
using OpenTracing;
using OpenTracing.Util;

namespace AutomationTool
{

	public partial class CommandUtils
	{
		/// <summary>
		/// Runs msbuild.exe with the specified arguments. Automatically creates a logfile. When 
		/// no LogName is specified, the executable name is used as logfile base name.
		/// </summary>
		/// <param name="Env">BuildEnvironment to use.</param>
		/// <param name="Project">Path to the project to build.</param>
		/// <param name="Arguments">Arguments to pass to msbuild.exe.</param>
		/// <param name="LogName">Optional logfile name.</param>
		public static void MsBuild(CommandEnvironment Env, string Project, string Arguments, string LogName)
		{
			if (String.IsNullOrEmpty(Env.FrameworkMsbuildPath))
			{
				throw new AutomationException("Unable to find msbuild.exe at: \"{0}\"", Env.FrameworkMsbuildPath);
			}
			if (!FileExists(Project))
			{
				throw new AutomationException("Project {0} does not exist!", Project);
			}
			string RunArguments = MakePathSafeToUseWithCommandLine(Project);
			if (!String.IsNullOrEmpty(Arguments))
			{
				RunArguments += " " + Arguments;
			}

			RunAndLog(Env, Env.FrameworkMsbuildPath, RunArguments, LogName);
		}

		/// <summary>
		/// Builds a Visual Studio solution with MsBuild (using msbuild.exe rather than devenv.com can help circumvent issues with expired Visual Studio licenses).
		/// Automatically creates a logfile. When no LogName is specified, the executable name is used as logfile base name.
		/// </summary>
		/// <param name="Env">BuildEnvironment to use.</param>
		/// <param name="SolutionFile">Path to the solution file</param>
		/// <param name="BuildConfig">Configuration to build.</param>
		/// <param name="BuildPlatform">Platform to build.</param>
		/// <param name="LogName">Optional logfile name.</param>
		public static void BuildSolution(CommandEnvironment Env, string SolutionFile, string BuildConfig, string BuildPlatform, string LogName = null)
		{
			if (!FileExists(SolutionFile))
			{
				throw new AutomationException(String.Format("Unabled to build Solution {0}. Solution file not found.", SolutionFile));
			}
			if (String.IsNullOrEmpty(Env.FrameworkMsbuildPath))
			{
				throw new AutomationException("Unable to find msbuild.exe at: \"{0}\"", Env.FrameworkMsbuildPath);
			}
			string CmdLine = String.Format("\"{0}\" /m /t:Build /p:Configuration=\"{1}\" /p:Platform=\"{2}\" /verbosity:minimal /nologo", SolutionFile, BuildConfig, BuildPlatform);
			using (IScope Scope = GlobalTracer.Instance.BuildSpan("Compile").StartActive())
			{
				Scope.Span.SetTag("solution", Path.GetFileName(SolutionFile));
				Scope.Span.SetTag("platform", "WinC#");
				Scope.Span.SetTag("config", BuildConfig);
				RunAndLog(Env, Env.FrameworkMsbuildPath, CmdLine, LogName);
			}
		}

		/// <summary>
		/// Builds a CSharp project with msbuild.exe. Automatically creates a logfile. When 
		/// no LogName is specified, the executable name is used as logfile base name.
		/// </summary>
		/// <param name="Env">BuildEnvironment to use.</param>
		/// <param name="ProjectFile">Path to the project file. Must be a .csproj file.</param>
		/// <param name="BuildConfig">Configuration to build.</param>
		/// <param name="LogName">Optional logfile name.</param>
		public static void BuildCSharpProject(CommandEnvironment Env, string ProjectFile, string BuildConfig = "Development", string LogName = null)
		{
			if (!ProjectFile.EndsWith(".csproj"))
			{
				throw new AutomationException(String.Format("Unabled to build Project {0}. Not a valid .csproj file.", ProjectFile));
			}
			if (!FileExists(ProjectFile))
			{
				throw new AutomationException(String.Format("Unabled to build Project {0}. Project file not found.", ProjectFile));
			}

			string CmdLine = String.Format(@"/verbosity:minimal /nologo /target:Build /property:Configuration={0} /property:Platform=AnyCPU", BuildConfig);
			MsBuild(Env, ProjectFile, CmdLine, LogName);
		}

		///
		/// <summary>
		/// returns true if this is a linux executable using some awful conventions
		/// <param name="Filename">Filename</param>
		/// </summary>
		public static bool IsProbablyALinuxExe(string Filename)
		{
			return
				Path.GetExtension(Filename) == ".sh" ||
				((
					CommandUtils.CombinePaths(Filename).ToLower().Contains(CommandUtils.CombinePaths("Binaries", "Linux").ToLower()) ||
					CommandUtils.CombinePaths(Filename).ToLower().Contains(CommandUtils.CombinePaths("Binaries", "LinuxArm64").ToLower()) ||
					CommandUtils.CombinePaths(Filename).ToLower().Contains(CommandUtils.CombinePaths("Binaries", "ThirdParty").ToLower())
				) && (Path.GetExtension(Filename) == "" || Path.GetExtension(Filename) == "."));
		}

		/// <summary>
		/// returns true if this is a mac executable using some awful conventions
		/// <param name="Filename">Filename</param>
		/// </summary>
		public static bool IsProbablyAMacOrIOSExe(string Filename)
		{
			return
				Path.GetExtension(Filename) == ".sh" ||
				Path.GetExtension(Filename) == ".command" ||
				((
					CommandUtils.CombinePaths(Filename).ToLower().Contains(CommandUtils.CombinePaths("Binaries", "Mac").ToLower()) ||
					CommandUtils.CombinePaths(Filename).ToLower().Contains(CommandUtils.CombinePaths("Binaries", "IOS").ToLower()) ||
					CommandUtils.CombinePaths(Filename).ToLower().Contains(CommandUtils.CombinePaths("Binaries", "ThirdParty").ToLower()) ||
					CommandUtils.CombinePaths(PathSeparator.Slash, Filename).ToLower().Contains(".app/Contents/MacOS".ToLower())
				) && (Path.GetExtension(Filename) == "" || Path.GetExtension(Filename) == "."));
		}

		/// <summary>
		/// Sets an executable bit for Unix executables and adds read permission for all users plus write permission for owner.
		/// <param name="Filename">Filename</param>
		/// </summary>
		public static void FixUnixFilePermissions(string Filename)
		{
			string Permissions = IsProbablyAMacOrIOSExe(Filename) || IsProbablyALinuxExe(Filename) ? "0755" : "0644";
			string CommandString = string.Format("-c 'chmod {0} \"{1}\"'", Permissions, Filename.Replace("'", "'\"'\"'"));
			IProcessResult Result = CommandUtils.Run("sh", CommandString, Options:ERunOptions.SpewIsVerbose);
			if (Result.ExitCode != 0)
			{
				// Try simply invoking chmod directly.
				IProcessResult DirectResult = CommandUtils.Run("chmod", string.Format("{0} \"{1}\"", Permissions, Filename.Replace("'", "'\"'\"'")));
				if (DirectResult.ExitCode != 0)
				{
					throw new AutomationException(String.Format("Command 'sh {0}' failed to chmod \"{1}\" (Exit Code {2}): {3}", CommandString, Filename, Result.ExitCode, Result.Output));
				}
			}
		}
	}

}
