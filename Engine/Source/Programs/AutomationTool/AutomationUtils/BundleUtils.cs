// Copyright Epic Games, Inc. All Rights Reserved.
using AutomationTool;
using EpicGames.Core;
using EpicGames.Horde.Projects;
using EpicGames.MCP.Automation;
using EpicGames.UHT.Utils;
using Microsoft.Extensions.Logging;
using System;
using System.Collections.Generic;
using System.Linq;
using System.Reflection.Metadata;
using System.Text;
using System.Text.RegularExpressions;
using System.Threading.Tasks;
using UnrealBuildTool;

namespace AutomationUtils.Automation
{
	public class BundleUtils
	{
		public class BundleSettings
		{
			public string Name { get; set; }
			public List<string> Tags { get; set; }
			public List<string> Dependencies { get; set; }
			public List<string> FileRegex { get; set; }
			public List<string> Files { get; set; }
			public List<string> Subsets { get; set; }
			public bool bContainsShaderLibrary { get; set; }
			public int Order { get; set; }
			public bool	UseChunkDBs { get; set; }
			public bool UseDetailedInstallSizes { get; set; } // default is true
			public string ExecFileName { get; set; } // TODO: We never used this.  Clean this up.
			public string CloudAppNameOverride { get; set; }			
			public List<string> VirtualBundleChildren { get; set; } // Bundles with children should never get files assigned to them, but they still generate chunkdb inis and such.
			public List<string> VirtualBundleParents { get; set; } // These are the virtual bundles we belong to. We have to be in the entry for the virtual bundle, and they have to be in our entry, to try and avoid mistakes.
		}

		public class BundleSubsetSettings
		{
			public string Name { get; set; }
			public List<string> Tags { get; set; }
			public List<string> FileRegex { get; set; }
		}

		public static ConfigHierarchy LoadConfigHierarchy(DirectoryReference ProjectDir, UnrealTargetPlatform Platform, string CustomConfig)
		{
			ConfigHierarchy BundleConfig = ConfigCache.ReadHierarchy(ConfigHierarchyType.InstallBundle, ProjectDir, Platform, CustomConfig);
			return BundleConfig;
		}

		public static void LoadBundleConfig(ConfigHierarchy BundleConfig, out IReadOnlyDictionary<string, BundleSettings> Bundles)
		{
			LoadBundleConfig<BundleSettings>(BundleConfig, out Bundles, delegate (BundleSettings Settings, ConfigHierarchy BundleConfig, string Section) { });
		}

		public static void LoadBundleConfig(DirectoryReference ProjectDir, UnrealTargetPlatform Platform, string CustomConfig, out IReadOnlyDictionary<string, BundleSettings> Bundles)
		{
			LoadBundleConfig<BundleSettings>(ProjectDir, Platform, CustomConfig, out Bundles, delegate (BundleSettings Settings, ConfigHierarchy BundleConfig, string Section) { });
		}

		public static void LoadBundleConfig<TPlatformBundleSettings>(ConfigHierarchy BundleConfig,
			out IReadOnlyDictionary<string, TPlatformBundleSettings> Bundles, 
			Action<TPlatformBundleSettings, ConfigHierarchy, string> GetPlatformSettings) 
			where TPlatformBundleSettings : BundleSettings, new()
		{
			List<TPlatformBundleSettings> Results = new List<TPlatformBundleSettings>();

			const string BundleDefinitionPrefix = "InstallBundleDefinition ";

			foreach (string SectionName in BundleConfig.SectionNames)
			{
				if (!SectionName.StartsWith(BundleDefinitionPrefix))
				{
					continue;
				}

				TPlatformBundleSettings Bundle = new TPlatformBundleSettings();
				Bundle.Name = SectionName.Substring(BundleDefinitionPrefix.Length);
				{
					int Order;
					if(BundleConfig.GetInt32(SectionName, "Order", out Order))
					{
						Bundle.Order = Order;
					}
					else
					{
						Bundle.Order = int.MaxValue;
					}
				}
				{
					List<string> Tags;
					if (BundleConfig.GetArray(SectionName, "Tags", out Tags))
					{
						Bundle.Tags = Tags;
					}
					else 
					{
						Bundle.Tags = new List<string>(); 
					}
				}
				{
					List<string> Aliases;
					if (BundleConfig.GetArray(SectionName, "VirtualChildren", out Aliases))
					{
						if (Bundle.Name.Contains("virtual", StringComparison.OrdinalIgnoreCase) == false)
						{
							throw new AutomationException("Bundle name {0} doesn't contain keyword Virtual in the name but has virtual children", Bundle.Name);
						}
						Bundle.VirtualBundleChildren = Aliases;
						Console.WriteLine($"{Bundle.Name} has children: {string.Join(",", Aliases)}");
					}
					else 
					{
						Bundle.VirtualBundleChildren = new List<string>(); 
					}
				}
				{
					List<string> VirtualParents;
					if (BundleConfig.GetArray(SectionName, "VirtualParents", out VirtualParents))
					{
						foreach (string VirtualParent in VirtualParents)
						{
							if (VirtualParent.Contains("virtual", StringComparison.OrdinalIgnoreCase) == false)
							{
								throw new AutomationException("Bundle name {0} has a non virtual virtual parent (must contain Virtual in the name): {1}", Bundle.Name, VirtualParent);
							}
						}
						Bundle.VirtualBundleParents = VirtualParents;
						Console.WriteLine($"{Bundle.Name} has parents: {string.Join(",", VirtualParents)}");
					}
					else 
					{
						Bundle.VirtualBundleParents = new List<string>(); 
					}
				}
				{
					List<string> Dependencies;
					if (BundleConfig.GetArray(SectionName, "Dependencies", out Dependencies))
					{
						Bundle.Dependencies = Dependencies;
					}
					else
					{
						Bundle.Dependencies = new List<string>();
					}
				}
				{
					List<string> FileRegex;
					if (BundleConfig.GetArray(SectionName, "FileRegex", out FileRegex))
					{
						Bundle.FileRegex = FileRegex;
					}
					else
					{
						Bundle.FileRegex = new List<string>();
					}
				}
				{
					List<string> Files;
					if (BundleConfig.GetArray(SectionName, "Files", out Files))
					{
						Bundle.Files = Files;
					}
					else
					{
						Bundle.Files = new List<string>();
					}
				}
				{
					List<string> Subsets;
					if (BundleConfig.GetArray(SectionName, "Subsets", out Subsets))
					{
						Bundle.Subsets = Subsets;
					}
					else
					{
						Bundle.Subsets = new List<string>();
					}
				}
				{
					bool bContainsShaderLibrary;
					if (BundleConfig.GetBool(SectionName, "ContainsShaderLibrary", out bContainsShaderLibrary))
					{
						Bundle.bContainsShaderLibrary = bContainsShaderLibrary;
					}
					else 
					{
						Bundle.bContainsShaderLibrary = false;
					}
				}

				{
					bool bUseChunkDBs;
					if (!BundleConfig.GetBool(SectionName, "UseChunkDBs", out bUseChunkDBs))
					{
						Bundle.UseChunkDBs = false;
					}
					else
					{
						Bundle.UseChunkDBs = bUseChunkDBs;
					}
					{
						bool bUseDetailedInstallSizes;
						if (!BundleConfig.GetBool(SectionName, "UseDetailedInstallSizes", out bUseDetailedInstallSizes))
						{
							Bundle.UseDetailedInstallSizes = true; // default to true
						}
						else
						{
							Bundle.UseDetailedInstallSizes = bUseDetailedInstallSizes;
						}
					}

				}
				{
					if (BundleConfig.GetString(SectionName, "CloudAppNameOverride", out string CloudAppNameOverride))
					{
						Bundle.CloudAppNameOverride = CloudAppNameOverride;
					}
					else 
					{
						Bundle.CloudAppNameOverride = "";
					}
				}
				GetPlatformSettings(Bundle, BundleConfig, BundleDefinitionPrefix + Bundle.Name);

				Results.Add(Bundle);
			}

			// Use OrderBy and not Sort because OrderBy is stable
			Bundles = Results.OrderBy(b => b.Order).ToDictionary(b => b.Name, b => b);
			
						
			// Iterate over the bundles and ensure that all virtual bundles have a two way mapping to try and prevent
			// changes making busted patch sizes.
			foreach (TPlatformBundleSettings Bundle in Bundles.Values)
			{
				foreach (string VirtualBundleChild in Bundle.VirtualBundleChildren)
				{
					TPlatformBundleSettings ChildBundle;
					if (!Bundles.TryGetValue(VirtualBundleChild, out ChildBundle))
					{
						throw new AutomationException("Virtual bundle {0} references child bundle that doesn't exist: {1}", Bundle.Name, VirtualBundleChild);
					}
					bool bFound = false;
					Console.WriteLine($"{Bundle.Name} Checking parents for {ChildBundle.Name}: {string.Join(",", ChildBundle.VirtualBundleParents)}");
					foreach (string VirtualBundleParent in ChildBundle.VirtualBundleParents)
					{
						if (Bundle.Name.Equals(VirtualBundleParent, StringComparison.OrdinalIgnoreCase))
						{
							bFound =  true;
							break;
						}
					}
					if (!bFound)
					{
						throw new AutomationException("Virtual bundle {0} references child bundle that doesn't reference it back: {1}", Bundle.Name, VirtualBundleChild);
					}
				}
			}
		}

		public static void LoadBundleConfig<TPlatformBundleSettings>(DirectoryReference ProjectDir, UnrealTargetPlatform Platform, string CustomConfig,
			out IReadOnlyDictionary<string, TPlatformBundleSettings> Bundles,
			Action<TPlatformBundleSettings, ConfigHierarchy, string> GetPlatformSettings)
			where TPlatformBundleSettings : BundleSettings, new()
		{
			ConfigHierarchy BundleConfig = LoadConfigHierarchy(ProjectDir, Platform, CustomConfig);
			LoadBundleConfig(BundleConfig, out Bundles, GetPlatformSettings);
		}

		public static void LoadBundleSubsetConfig(ConfigHierarchy BundleConfig, out IReadOnlyDictionary<string, BundleSubsetSettings> BundlesSubsets)
		{
			Dictionary<string, BundleSubsetSettings> Results = new Dictionary<string, BundleSubsetSettings>();

			const string BundleDefinitionSubsetPrefix = "InstallBundleSubsetDefinition ";

			foreach (string SectionName in BundleConfig.SectionNames)
			{
				if (!SectionName.StartsWith(BundleDefinitionSubsetPrefix))
				{
					continue;
				}

				BundleSubsetSettings Settings = new BundleSubsetSettings();
				Settings.Name = SectionName.Substring(BundleDefinitionSubsetPrefix.Length);

				{
					List<string> Tags;
					if (BundleConfig.GetArray(SectionName, "Tags", out Tags))
					{
						Settings.Tags = Tags;
					}
					else
					{
						Settings.Tags = new List<string>();
					}
				}
				{
					List<string> FileRegex;
					if (BundleConfig.GetArray(SectionName, "FileRegex", out FileRegex))
					{
						Settings.FileRegex = FileRegex;
					}
					else
					{
						Settings.FileRegex = new List<string>();
					}
				}

				Results.Add(Settings.Name, Settings);
			}

			BundlesSubsets = Results;
		}

		public static void LoadBundleSubsetConfig(DirectoryReference ProjectDir, UnrealTargetPlatform Platform, string CustomConfig,
			out IReadOnlyDictionary<string, BundleSubsetSettings> BundlesSubsets)
		{
			ConfigHierarchy BundleConfig = LoadConfigHierarchy(ProjectDir, Platform, CustomConfig);
			LoadBundleSubsetConfig(BundleConfig, out BundlesSubsets);
		}

		public static TPlatformBundleSettings MatchBundleSettings<TPlatformBundleSettings>(
			string FileName, IReadOnlyDictionary<string, TPlatformBundleSettings> InstallBundles) where TPlatformBundleSettings : BundleSettings
		{
			// Try to find a matching chunk regex or exact filename
			foreach (TPlatformBundleSettings Bundle in InstallBundles.Values)
			{
				foreach (string RegexString in Bundle.FileRegex)
				{
					if (Regex.Match(FileName, RegexString, RegexOptions.IgnoreCase).Success)
					{
						return Bundle;
					}
				}
				foreach(string FileString in Bundle.Files)
				{
					if (FileString.Equals(FileName, StringComparison.OrdinalIgnoreCase))
					{
						return Bundle;
					}
				}
			}

			return null;
		}

		public static IList<BundleSubsetSettings> MatchBundleSubsetSettings(BundleSettings Bundle, string FileName,
			IReadOnlyDictionary<string, BundleSubsetSettings> InstallBundleSubsets)
		{
			List<BundleSubsetSettings> Results = new List<BundleSubsetSettings>();

			// Find matching subsets in order specified by bundle config
			foreach (string Subset in Bundle.Subsets)
			{
				if (InstallBundleSubsets.TryGetValue(Subset, out BundleSubsetSettings SubsetSettings))
				{
					foreach (string RegexString in SubsetSettings.FileRegex)
					{
						if (Regex.Match(FileName, RegexString, RegexOptions.IgnoreCase).Success)
						{
							Results.Add(SubsetSettings);
							break;
						}
					}
				}
			}

			return Results;
		}

		public static IEnumerable<string> GetCloudAppsForBundles<TPlatformBundleSettings>(IReadOnlyDictionary<string, TPlatformBundleSettings> InstallBundles) where TPlatformBundleSettings : BundleSettings
		{
			// Gather all bundle clouda app name overrides
			HashSet<string> CloudAppNames = new HashSet<string>();
			foreach (TPlatformBundleSettings Bundle in InstallBundles.Values)
			{
				CloudAppNames.Add(Bundle.CloudAppNameOverride);
			}
			return CloudAppNames;
		}

		public static IEnumerable<TPlatformBundleSettings> GetBundleDependencies<TPlatformBundleSettings>(
			TPlatformBundleSettings Bundle, IReadOnlyDictionary<string, TPlatformBundleSettings> InstallBundles) where TPlatformBundleSettings : BundleSettings
		{
			Dictionary<string, TPlatformBundleSettings> FoundDependencies = new Dictionary<string, TPlatformBundleSettings>();

			List<TPlatformBundleSettings> CurrentDependencies = new List<TPlatformBundleSettings>();
			CurrentDependencies.Add(Bundle);

			while (CurrentDependencies.Count > 0)
			{
				List<TPlatformBundleSettings> NextDependencies = new List<TPlatformBundleSettings>();
				foreach (TPlatformBundleSettings Dep in CurrentDependencies)
				{
					if (FoundDependencies.ContainsKey(Dep.Name))
						continue;
					
					FoundDependencies.Add(Dep.Name, Dep);

					foreach (var NextDepName in Dep.Dependencies)
					{
						if (InstallBundles.TryGetValue(NextDepName, out TPlatformBundleSettings NextDep))
						{
							NextDependencies.Add(NextDep);
						}
					}
				}
				CurrentDependencies = NextDependencies;
			}

			return FoundDependencies.Values;
		}

		public static bool HasPlatformBundleSource(DirectoryReference ProjectDir, UnrealTargetPlatform Platform, string CustomConfig)
		{
			ConfigHierarchy BundleConfig = ConfigCache.ReadHierarchy(ConfigHierarchyType.InstallBundle, ProjectDir, Platform, CustomConfig);
			return 
				BundleConfig.GetArray("InstallBundleManager.BundleSources", "DefaultBundleSources", out List<string> InstallBundleSources) && 
				InstallBundleSources.Contains("Platform");
		}

		public static bool HasPlatformDLCBundleSource(DirectoryReference ProjectDir, UnrealTargetPlatform Platform, string CustomConfig)
		{
			ConfigHierarchy BundleConfig = ConfigCache.ReadHierarchy(ConfigHierarchyType.InstallBundle, ProjectDir, Platform, CustomConfig);
			if (BundleConfig.GetArray("InstallBundleManager.BundleSources", "DefaultBundleSources", out List<string> InstallBundleSources))
			{
				foreach (string Source in InstallBundleSources)
				{
					// Support sources that are derived from PlatformDLC, such as MyGamePlatformDLC, provided they are
					// named with a prefix.
					if (Source.EndsWith("PlatformDLC"))
					{
						return true;
					}
				}
			}

			return false;
		}

		// Searches through all tagged files in a manifest, and determines virtual bundle tags from any files with multiple tags
		//	Virtual bundles should always contain 'virtual' in their name
		public static List<string> InferVirtualBundleTagsFromManifest(BuildPatchToolBase.ManifestMetadataOutput ManifestContents)
		{
			List<string> VirtualBundleTags = new List<string>();
			if (ManifestContents != null)
			{
				foreach (BuildPatchToolBase.ManifestMetadataOutput.ManifestMetadataOutputFile TaggedFile in ManifestContents.Files)
				{
					if (TaggedFile.Tags.Count > 1)
					{
						foreach (string TagName in TaggedFile.Tags)
						{
							if (TagName.Contains("virtual", StringComparison.OrdinalIgnoreCase))
							{
								VirtualBundleTags.AddUnique(TagName);
							}
						}
					}
				}
			}
			return VirtualBundleTags;
		}

		// Loads the metadata for a given manifest and returns any virtual bundle tags inferred from its contents
		public static List<string> InferVirtualBundleTagsFromManifest(string ManifestPath, BuildPatchToolBase.ToolVersion BPTVersion = BuildPatchToolBase.ToolVersion.Live)
		{
			List<string> VirtualBundleTags = new List<string>();
			if (!string.IsNullOrEmpty(ManifestPath))
			{
				BuildPatchToolBase.ManifestMetadataOptions MetadataOptions = new BuildPatchToolBase.ManifestMetadataOptions
				{
					SourceManifestFilename = ManifestPath
				};
				BuildPatchToolBase.ManifestMetadataOutput ManifestMetadataOutput = null;
				BuildPatchToolBase.Get().Execute(MetadataOptions, out ManifestMetadataOutput, BPTVersion);
				if (ManifestMetadataOutput != null)
				{
					VirtualBundleTags = InferVirtualBundleTagsFromManifest(ManifestMetadataOutput);
				}
			}
			return VirtualBundleTags;
		}

		public static List<string> InferNonVirtualBundleTagsFromManifest(BuildPatchToolBase.ManifestMetadataOutput ManifestContents)
		{
			List<string> NonVirtualTags = new List<string>();
			if (ManifestContents != null)
			{
				List<string> VirtualBundleTags = InferVirtualBundleTagsFromManifest(ManifestContents);

				// We expect very few virtual bundles (like 1) so doing this n^2 isn't terrible.
				foreach (string Tag in ManifestContents.FileTagList)
				{
					if (!VirtualBundleTags.Contains(Tag))
					{
						NonVirtualTags.Add(Tag);
					}
				}
			}
			return NonVirtualTags;
		}

		// Loads the metadata for a given manifest and returns any NON virtual bundle tags inferred from its contents.
		public static List<string> InferNonVirtualBundleTagsFromManifest(string ManifestPath, BuildPatchToolBase.ToolVersion BPTVersion = BuildPatchToolBase.ToolVersion.Live)
		{
			List<string> NonVirtualTags = new List<string>();
			if (!string.IsNullOrEmpty(ManifestPath))
			{
				BuildPatchToolBase.ManifestMetadataOptions MetadataOptions = new BuildPatchToolBase.ManifestMetadataOptions
				{
					SourceManifestFilename = ManifestPath
				};
				BuildPatchToolBase.ManifestMetadataOutput ManifestMetadataOutput = null;
				BuildPatchToolBase.Get().Execute(MetadataOptions, out ManifestMetadataOutput, BPTVersion);
				NonVirtualTags = InferNonVirtualBundleTagsFromManifest(ManifestMetadataOutput);
			}
			return NonVirtualTags;
		}

		public static HashSet<string> InferVirtualBundleChildTagsFromManifest(string VirtualBundleTag, BuildPatchToolBase.ManifestMetadataOutput ManifestContents)
		{
			HashSet<string> ChildTags = [];
			if (ManifestContents != null)
			{
				foreach (BuildPatchToolBase.ManifestMetadataOutput.ManifestMetadataOutputFile TaggedFile in ManifestContents.Files)
				{
					if(TaggedFile.Tags.Exists(Tag => Tag.Equals(VirtualBundleTag, StringComparison.OrdinalIgnoreCase)))
					{
						// There should be only one 'base' bundle tag for each file in a virtual bundle, but multiple may occurr if a child bundle has subsets
						List<string> NonVirtualBundleTags = TaggedFile.Tags.FindAll(Tag => !Tag.Contains("virtual", StringComparison.OrdinalIgnoreCase));
						foreach (string ChildTag in NonVirtualBundleTags)
						{
							ChildTags.Add(ChildTag);
						}

						if(NonVirtualBundleTags.Count == 0)
						{
							Console.WriteLine($"Couldn't find the base tag when inferring virtual bundle children for file {TaggedFile.Filename}");
						}
						else if(NonVirtualBundleTags.Count > 1)
						{
							Console.WriteLine($"Found multiple base tags when inferring virtual bundle children for file {TaggedFile.Filename}: Base: {NonVirtualBundleTags[0]}, Found {NonVirtualBundleTags.Count}");
						}
					}
				}
			}
			return ChildTags;
		}
	}
}
