// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Linq;
using AutomationTool;
using EpicGames.Core;
using EpicGames.ProjectStore;
using JetBrains.Annotations;
using UnrealBuildTool;

namespace AutomationUtils
{
	public static class CloudStorageOptionsExtensions
	{
		/// <summary>
		/// Attempt to initialize options from inis
		/// </summary>
		/// <param name="projectPath"></param>
		public static void ReadFromIni(this CloudStorageOptions options, [CanBeNull] FileReference projectPath)
		{
			Dictionary<UnrealTargetPlatform, ConfigHierarchy> engineConfigs = new Dictionary<UnrealTargetPlatform, ConfigHierarchy>();
			Dictionary<UnrealTargetPlatform, ConfigHierarchy> gameConfigs = new Dictionary<UnrealTargetPlatform, ConfigHierarchy>();
			foreach (UnrealTargetPlatform targetPlatformType in UnrealTargetPlatform.GetValidPlatforms())
			{
				DirectoryReference projectDir = projectPath?.Directory;

				ConfigHierarchy engineConfig = ConfigCache.ReadHierarchy(ConfigHierarchyType.Engine, projectDir, targetPlatformType);
				engineConfigs.Add(targetPlatformType, engineConfig);

				ConfigHierarchy gameConfig = ConfigCache.ReadHierarchy(ConfigHierarchyType.Game, projectDir, targetPlatformType); 
				gameConfigs.Add(targetPlatformType, gameConfig);
			}

			ConfigHierarchy config = engineConfigs[HostPlatform.Current.HostEditorPlatform];
			bool foundConfig = config.TryGetValueGeneric("StorageServers", "Cloud", out IniCloudConfiguration foundCloudConfig);

			if (!foundConfig)
			{
				return;
			}

			string host = foundCloudConfig.Host;
			if (host.Contains(';', StringComparison.OrdinalIgnoreCase))
			{
				// if it's a list pick the first element
				host = host.Split(";").First();
			}
			options.Host = host;
			options.NamespaceId = foundCloudConfig.BuildsNamespace;
			options.BaselineBranch = foundCloudConfig.BuildsBaselineBranch;
		}

		/// <summary>
		/// Attempt to initialize options from environment variables
		/// </summary>
		public static void ReadFromEnv(this CloudStorageOptions options)
		{
			string ToEnvKey(string env)
			{
				if (RuntimePlatform.Current == RuntimePlatform.Type.Windows)
				{
					return env;
				}
				else
				{
					return env.Replace('-', '_');
				}
			}

			string SetFromEnv(string envVar, string defaultValue)
			{
				string key = ToEnvKey(envVar);
				if (!String.IsNullOrEmpty(Environment.GetEnvironmentVariable(key)))
				{
					return Environment.GetEnvironmentVariable(key);
				}

				return defaultValue;
			}

			options.Host = SetFromEnv("UE-CloudPublishHost", options.Host);
			options.NamespaceId = SetFromEnv("UE-CloudPublishNamespace", options.NamespaceId);
			options.BaselineBranch = SetFromEnv("UE-CloudPublishBaselineBranch", options.BaselineBranch);
		}

		struct IniCloudConfiguration
		{
#pragma warning disable IDE1006 // Static analysis wants these to be named differently, but they must be named the same as the config file properties
			public string Host = String.Empty;
			public string BuildsNamespace = String.Empty;
			public string BuildsBaselineBranch = String.Empty;
#pragma warning restore IDE1006

			public IniCloudConfiguration() {}
		}
	}
}
