// Copyright Epic Games, Inc. All Rights Reserved.
using System;
using System.Collections.Generic;
using System.IO;
using System.Text;
using System.Reflection;
using Microsoft.Win32;
using System.Diagnostics;
using EpicGames.Core;
using System.Text.RegularExpressions;
using UnrealBuildBase;
using Microsoft.Extensions.Logging;
using UnrealBuildTool;

namespace AutomationTool
{
	/// <summary>
	/// Defines the environment variable names that will be used to setup the environment.
	/// </summary>
	public static class EnvVarNames
	{
		// Command Environment
		public const string LocalRoot = "uebp_LOCAL_ROOT";
		public const string LogFolder = "uebp_LogFolder";
		public const string FinalLogFolder = "uebp_FinalLogFolder";
		public const string CSVFile = "uebp_CSVFile";
		public const string EngineSavedFolder = "uebp_EngineSavedFolder";
		public const string MacMallocNanoZone = "MallocNanoZone";
		public const string DisableStartupMutex = "uebp_UATMutexNoWait";
		public const string IsChildInstance = "uebp_UATChildInstance";

		// Perforce Environment
		public const string P4Port = "uebp_PORT";
		public const string ClientRoot = "uebp_CLIENT_ROOT";
		public const string Changelist = "uebp_CL";
		public const string CodeChangelist = "uebp_CodeCL";
		public const string User = "uebp_USER";
		public const string Client = "uebp_CLIENT";
		public const string BuildRootP4 = "uebp_BuildRoot_P4";
		public const string BuildRootEscaped = "uebp_BuildRoot_Escaped";
		public const string P4Password = "uebp_PASS";
	}


	/// <summary>
	/// Environment to allow access to commonly used environment variables.
	/// </summary>
	public class CommandEnvironment
	{
		static ILogger Logger => Log.Logger;

		/// <summary>
		/// Path to a file we know to always exist under the Unreal root directory.
		/// </summary>
		public static readonly string KnownFileRelativeToRoot = @"Engine/Config/BaseEngine.ini";

		public string LocalRoot { get; protected set; }
		public string EngineSavedFolder { get; protected set; }
		public string LogFolder { get; protected set; }
		public string FinalLogFolder { get; protected set; }
		public string CSVFile { get; protected set; }
		public string RobocopyExe { get; protected set; }
		public string MountExe { get; protected set; }
		public string CmdExe { get; protected set; }
		public string AutomationToolDll { get; protected set; }
		public string TimestampAsString { get; protected set; }
		public bool HasCapabilityToCompile { get; protected set; }
		public string FrameworkMsbuildPath { get; protected set; }
		public string DotnetMsbuildPath { get; protected set; }
		public string MallocNanoZone { get; protected set; }
		public bool IsChildInstance { get; protected set; }

		/// <summary>
		/// Initializes the environment.
		/// </summary>
		internal CommandEnvironment()
		{
			// Get the path to AutomationTool.dll
			AutomationToolDll = Assembly.GetEntryAssembly().GetOriginalLocation();

			if (!CommandUtils.FileExists(AutomationToolDll))
			{
				throw new AutomationException("Could not find AutomationTool.dll. Reflection indicated it was here: {0}", AutomationToolDll);
			}

			// Find the root directory (containing the Engine folder)
			LocalRoot = CommandUtils.GetEnvVar(EnvVarNames.LocalRoot);
			if(String.IsNullOrEmpty(LocalRoot))
			{
				LocalRoot = CommandUtils.ConvertSeparators(PathSeparator.Slash, Unreal.RootDirectory.FullName);
				CommandUtils.ConditionallySetEnvVar(EnvVarNames.LocalRoot, LocalRoot);
			}

			string SavedPath = CommandUtils.GetEnvVar(EnvVarNames.EngineSavedFolder);
			if (String.IsNullOrEmpty(SavedPath))
			{
				SavedPath = CommandUtils.CombinePaths(PathSeparator.Slash, LocalRoot, "Engine", "Programs", "AutomationTool", "Saved");
				CommandUtils.SetEnvVar(EnvVarNames.EngineSavedFolder, SavedPath);
			}

			EngineSavedFolder = CommandUtils.GetEnvVar(EnvVarNames.EngineSavedFolder);
            CSVFile = CommandUtils.GetEnvVar(EnvVarNames.CSVFile);

			LogFolder = CommandUtils.GetEnvVar(EnvVarNames.LogFolder);
			if (String.IsNullOrEmpty(LogFolder))
			{
				if (Unreal.IsEngineInstalled())
				{
					LogFolder = GetInstalledLogFolder();
				}
				else
				{
					LogFolder = CommandUtils.CombinePaths(PathSeparator.Slash, EngineSavedFolder, "Logs");
				}
				CommandUtils.SetEnvVar(EnvVarNames.LogFolder, LogFolder);
			}

			// clear the logfolder if we're the only running instance
			if (ProcessSingleton.IsSoleInstance)
			{
				ClearLogFolder(LogFolder);
			}

			FinalLogFolder = CommandUtils.GetEnvVar(EnvVarNames.FinalLogFolder);
			if(String.IsNullOrEmpty(FinalLogFolder))
			{
				FinalLogFolder = LogFolder;
				CommandUtils.SetEnvVar(EnvVarNames.FinalLogFolder, FinalLogFolder);
			}

			RobocopyExe = GetSystemExePath("robocopy.exe");
			MountExe = GetSystemExePath("mount.exe");
			CmdExe = OperatingSystem.IsWindows() ? GetSystemExePath("cmd.exe") : "/bin/sh";
			MallocNanoZone = "0";
			CommandUtils.SetEnvVar(EnvVarNames.MacMallocNanoZone, MallocNanoZone);

			int IsChildInstanceInt;
			int.TryParse(CommandUtils.GetEnvVar(EnvVarNames.IsChildInstance, "0"), out IsChildInstanceInt);
			IsChildInstance = (IsChildInstanceInt != 0);

			if (IsChildInstance)
			{
				Logger.LogInformation("AutomationTool is running as a child instance ({Name}={Value})", EnvVarNames.IsChildInstance, IsChildInstanceInt.ToString());
			}

			// Setup the timestamp string
			DateTime LocalTime = DateTime.Now;

			string TimeStamp = LocalTime.Year + "-"
						+ LocalTime.Month.ToString("00") + "-"
						+ LocalTime.Day.ToString("00") + "_"
						+ LocalTime.Hour.ToString("00") + "."
						+ LocalTime.Minute.ToString("00") + "."
						+ LocalTime.Second.ToString("00");

			TimestampAsString = TimeStamp;

			SetupBuildEnvironment();

			LogSettings();
		}

		/// <summary>
		/// Returns the path to an executable in the System Directory.
		/// To help support running 32-bit assemblies on a 64-bit operating system, if the executable
		/// can't be found in System32, we also search Sysnative.
		/// </summary>
		/// <param name="ExeName">The name of the executable to find</param>
		/// <returns>The path to the executable within the system folder</returns>
		string GetSystemExePath(string ExeName)
		{
			var Result = CommandUtils.CombinePaths(Environment.SystemDirectory, ExeName);
			if (!CommandUtils.FileExists(Result))
			{
				// Use Regex.Replace so we can do a case-insensitive replacement of System32
				var SysNativeDirectory = Regex.Replace(Environment.SystemDirectory, "System32", "Sysnative", RegexOptions.IgnoreCase);
				var SysNativeExe = CommandUtils.CombinePaths(SysNativeDirectory, ExeName);
				if (CommandUtils.FileExists(SysNativeExe))
				{
					Result = SysNativeExe;
				}
			}
			return Result;
		}

		void LogSettings()
		{
			Logger.LogDebug("Command Environment settings:");
			Logger.LogDebug("CmdExe={CmdExe}", CmdExe);
			Logger.LogDebug("EngineSavedFolder={EngineSavedFolder}", EngineSavedFolder);
			Logger.LogDebug("HasCapabilityToCompile={HasCapabilityToCompile}", HasCapabilityToCompile);
			Logger.LogDebug("LocalRoot={LocalRoot}", LocalRoot);
			Logger.LogDebug("LogFolder={LogFolder}", LogFolder);
			Logger.LogDebug("MountExe={MountExe}", MountExe);
			Logger.LogDebug("FrameworkMsbuildExe={FrameworkMsbuildPath}", FrameworkMsbuildPath);
			Logger.LogDebug("DotnetMsbuildExe={DotnetMsbuildPath}", DotnetMsbuildPath);
			Logger.LogDebug("RobocopyExe={RobocopyExe}", RobocopyExe);
			Logger.LogDebug("TimestampAsString={TimestampAsString}", TimestampAsString);
			Logger.LogDebug("AutomationToolDll={AutomationToolDll}", AutomationToolDll);
		}

		/// <summary>
		/// Initializes build environemnt: finds the path to msbuild.exe
		/// </summary>
		void SetupBuildEnvironment()
		{
			// Assume we have the capability co compile.
			HasCapabilityToCompile = true;

			try
			{
				FrameworkMsbuildPath = HostPlatform.Current.GetFrameworkMsbuildExe();
			}
			catch (Exception Ex)
			{
				Log.WriteLine(LogEventType.Log, Ex.Message);
				Log.WriteLine(LogEventType.Log, "Assuming no compilation capability for NET Framework projects.");
				HasCapabilityToCompile = false;
				FrameworkMsbuildPath = "";
			}
			Logger.LogDebug("CompilationEvironment.HasCapabilityToCompile={HasCapabilityToCompile}", HasCapabilityToCompile);
			Logger.LogDebug("CompilationEvironment.FrameworkMsbuildExe={FrameworkMsbuildPath}", FrameworkMsbuildPath);

			DotnetMsbuildPath = Unreal.DotnetPath.FullName;
			Logger.LogDebug("CompilationEvironment.DotnetMsbuildExe={DotnetMsbuildPath}", DotnetMsbuildPath);
		}

		/// <summary>
		/// Finds the root path of the branch by looking in progressively higher folder locations until a file known to exist in the branch is found.
		/// </summary>
		/// <param name="UATLocation">Location of the currently executing assembly.</param>
		/// <returns></returns>
		private static string RootFromUATLocation(string UATLocation)
		{
			// pick a known file in the depot and try to build a relative path to it. This will tell us where the root path to the branch is.
			var KnownFilePathFromRoot = CommandEnvironment.KnownFileRelativeToRoot;
			var CurrentPath = CommandUtils.ConvertSeparators(PathSeparator.Slash, Path.GetDirectoryName(UATLocation));
			var UATPathRoot = CommandUtils.CombinePaths(Path.GetPathRoot(CurrentPath), KnownFilePathFromRoot);

			// walk parent dirs until we find the file or hit the path root, which means we aren't in a known location.
			while (true)
			{
				var PathToCheck = Path.GetFullPath(CommandUtils.CombinePaths(CurrentPath, KnownFilePathFromRoot));
				Logger.LogDebug("Checking for {PathToCheck}", PathToCheck);
				if (!File.Exists(PathToCheck))
				{
					var LastSeparatorIndex = CurrentPath.LastIndexOf('/');
					if (String.Compare(PathToCheck, UATPathRoot, true) == 0 || LastSeparatorIndex < 0)
					{
						throw new AutomationException("{0} does not appear to exist at a valid location with the branch, so the local root cannot be determined.", UATLocation);
					}
					
					// Step back one directory
					CurrentPath = CurrentPath.Substring(0, LastSeparatorIndex);
				}
				else
				{
					return CurrentPath;
				}
			}
		}

		/// <summary>
		/// Gets the log folder used when running from installed build.
		/// </summary>
		/// <returns></returns>
		private string GetInstalledLogFolder()
		{
			var LocalRootPath = CommandUtils.GetEnvVar(EnvVarNames.LocalRoot);
			var LocalRootEscaped = CommandUtils.ConvertSeparators(PathSeparator.Slash, LocalRootPath.Replace(":", ""));
			if (LocalRootEscaped[LocalRootEscaped.Length - 1] == '/')
			{
				LocalRootEscaped = LocalRootEscaped.Substring(0, LocalRootEscaped.Length - 1);
			}
			LocalRootEscaped = CommandUtils.EscapePath(LocalRootEscaped);
			return CommandUtils.CombinePaths(PathSeparator.Slash, HostPlatform.Current.LocalBuildsLogFolder, LocalRootEscaped);
		}

		/// <summary>
		/// Clears out the contents of the given log folder
		/// </summary>
		/// <param name="LogFolder">The log folder to clear</param>
		private static void ClearLogFolder(string LogFolder)
		{
			try
			{
				if (Directory.Exists(LogFolder))
				{
					CommandUtils.DeleteDirectoryContents(LogFolder);

					// Since the directory existed and might have been empty, we need to make sure
					// we can actually write to it, so create and delete a temporary file.
					var RandomFilename = Path.GetRandomFileName();
					var RandomFilePath = CommandUtils.CombinePaths(LogFolder, RandomFilename);
					File.WriteAllText(RandomFilePath, RandomFilename);
					File.Delete(RandomFilePath);
				}
				else
				{
					Directory.CreateDirectory(LogFolder);
				}
			}
			catch(Exception Ex)
			{
				throw new AutomationException(Ex, "Unable to clear log folder ({0})", LogFolder);
			}
		}
	}
}
