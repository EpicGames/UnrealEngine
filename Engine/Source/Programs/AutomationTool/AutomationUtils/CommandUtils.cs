// Copyright Epic Games, Inc. All Rights Reserved.

using EpicGames.Core;
using JetBrains.Annotations;
using Microsoft.Extensions.Logging;
using System;
using System.Collections;
using System.Collections.Generic;
using System.Diagnostics;
using System.IO;
using System.IO.Compression;
using System.Linq;
using System.Reflection;
using System.Runtime.Serialization;
using System.Runtime.Versioning;
using System.Text;
using System.Threading;
using System.Threading.Tasks;
using System.Xml;
using System.Xml.Serialization;
using Microsoft.Extensions.DependencyInjection;
using UnrealBuildBase;
using UnrealBuildTool;

using static AutomationTool.CommandUtils;

namespace AutomationTool
{
	/// <summary>
	/// Wrapper around List with support for multi parameter constructor, i.e:
	///   var Maps = new ParamList&lt;string&gt;("Map1", "Map2");
	/// </summary>
	/// <typeparam name="T"></typeparam>
	public class ParamList<T> : List<T>
	{
		public ParamList(params T[] Args)
		{
			AddRange(Args);
		}

		public ParamList(ICollection<T> Collection)
			: base(Collection != null ? Collection : new T[] { })
		{
		}

		public override string ToString()
		{
			var Text = "";
			for (int Index = 0; Index < Count; ++Index)
			{
				if (Index > 0)
				{
					Text += ", ";
				}
				Text += this[Index].ToString();
			}
			return Text;
		}
	}

	public enum PathSeparator
	{
		Default = 0,
		Slash,
		Backslash,
		Depot,
		Local
	}

	/// <summary>
	/// Delegate to override the copy operation.
	/// <returns>Return true if the copy was handled. Otherwise false to fallback on the built in copy operation</returns>
	/// </summary>
	public delegate bool OverrideCopyDelegate(ILogger Logger, string SourceName, string TargetName);

	/// <summary>
	/// Base utility function for script commands.
	/// </summary>
	public partial class CommandUtils
	{
		static private CommandEnvironment CmdEnvironment;

		/// <summary>
		/// BuildEnvironment to use for this buildcommand. This is initialized by InitCommandEnvironment. As soon
		/// as the script execution in ExecuteBuild begins, the BuildEnv is set up and ready to use.
		/// </summary>
		static public CommandEnvironment CmdEnv
		{
			get
			{
				if (CmdEnvironment == null)
				{
					throw new AutomationException("Attempt to use CommandEnvironment before it was initialized.");
				}
				return CmdEnvironment;
			}
		}

		/// <summary>
		/// Initializes build environment. If the build command needs a specific env-var mapping or
		/// has an extended BuildEnvironment, it must implement this method accordingly.
		/// </summary>
		/// <returns>Initialized and ready to use BuildEnvironment</returns>
		static internal void InitCommandEnvironment()
		{
			CmdEnvironment = new CommandEnvironment();
		}

		/// <summary>
		/// Provides access to the structured logging interface
		/// </summary>
		public static ILogger Logger => Log.Logger;

		/// <summary>
		/// Shared service provider instance, constructed on startup
		/// </summary>
		public static IServiceProvider ServiceProvider { get; set; } = null!;

		
		/// <summary>
		/// Access to scoped services which are created per command
		/// </summary>
		public static IServiceScope ServiceScope { get; set; } = null!;

		/// <summary>
		/// Global activity source for UAT. Can be used to add trace data and additional context to logs.
		/// </summary>
		public static ActivitySource ActivitySource => CreateActivitySource();

		static ActivityListener _activityListener;

		static ActivitySource CreateActivitySource()
		{
			ActivitySource activitySource = new ActivitySource("AutomationTool");

			_activityListener = new ActivityListener();
			_activityListener.ShouldListenTo = x => x == activitySource;
			_activityListener.Sample = (ref ActivityCreationOptions<ActivityContext> x) => ActivitySamplingResult.AllData;
			ActivitySource.AddActivityListener(_activityListener);

			return activitySource;
		}

		/// <summary>
		/// Writes formatted text to log (with LogEventType.Console).
		/// </summary>
		/// <param name="Format">Format string</param>
		/// <param name="Args">Parameters</param>
		[StringFormatMethod("Format")]
		[Obsolete("Use Logger.LogInformation with a message template instead; see https://tinyurl.com/bp96bk2r.", false)]
		public static void LogInformation(string Format, params object[] Args)
		{
			EpicGames.Core.Log.WriteLine(EpicGames.Core.LogEventType.Console, Format, Args);
		}

		/// <summary>
		/// Writes formatted text to log (with LogEventType.Console).
		/// </summary>
		/// <param name="Message">Text</param>
		[Obsolete("Use Logger.LogInformation with a message template instead; see https://tinyurl.com/bp96bk2r.", false)]
		public static void LogInformation(string Message)
		{
			EpicGames.Core.Log.WriteLine(EpicGames.Core.LogEventType.Console, Message);
		}

		/// <summary>
		/// Writes formatted text to log (with LogEventType.Error).
		/// </summary>
		/// <param name="Format">Format string</param>
		/// <param name="Args">Parameters</param>
		[StringFormatMethod("Format")]
		[Obsolete("Use Logger.LogError with a message template instead; see https://tinyurl.com/bp96bk2r.", false)]
		public static void LogError(string Format, params object[] Args)
		{
			EpicGames.Core.Log.WriteLine(EpicGames.Core.LogEventType.Error, Format, Args);
		}

		/// <summary>
		/// Writes formatted text to log (with LogEventType.Error).
		/// </summary>
		/// <param name="Message">Text</param>
		[Obsolete("Use Logger.LogError with a message template instead; see https://tinyurl.com/bp96bk2r.", false)]
		public static void LogError(string Message)
		{
			EpicGames.Core.Log.WriteLine(EpicGames.Core.LogEventType.Error, Message);
		}

		/// <summary>
		/// Writes formatted text to log (with LogEventType.Warning).
		/// </summary>
		/// <param name="Format">Format string</param>
		/// <param name="Args">Parameters</param>
		[StringFormatMethod("Format")]
		[Obsolete("Use Logger.LogWarning with a message template instead; see https://tinyurl.com/bp96bk2r.", false)]
		public static void LogWarning(string Format, params object[] Args)
		{
			EpicGames.Core.Log.WriteLine(EpicGames.Core.LogEventType.Warning, Format, Args);
		}

		/// <summary>
		/// Writes a message to log (with LogEventType.Warning).
		/// </summary>
		/// <param name="Message">Text</param>
		[Obsolete("Use Logger.LogWarning with a message template instead; see https://tinyurl.com/bp96bk2r.", false)]
		public static void LogWarning(string Message)
		{
			EpicGames.Core.Log.WriteLine(EpicGames.Core.LogEventType.Warning, Message);
		}

		/// <summary>
		/// Writes formatted text to log (with LogEventType.Verbose).
		/// </summary>
		/// <param name="Format">Format string</param>
		/// <param name="Args">Arguments</param>
		[StringFormatMethod("Format")]
		[Obsolete("Use Logger.LogDebug with a message template instead; see https://tinyurl.com/bp96bk2r.", false)]
		public static void LogVerbose(string Format, params object[] Args)
		{
			EpicGames.Core.Log.WriteLine(EpicGames.Core.LogEventType.Verbose, Format, Args);
		}

		/// <summary>
		/// Writes formatted text to log (with LogEventType.Verbose).
		/// </summary>
		/// <param name="Message">Text</param>
		[Obsolete("Use Logger.LogDebug with a message template instead; see https://tinyurl.com/bp96bk2r.", false)]
		public static void LogVerbose(string Message)
		{
			EpicGames.Core.Log.WriteLine(EpicGames.Core.LogEventType.Verbose, Message);
		}

		/// <summary>
		/// Writes formatted text to log (with LogEventType.VeryVerbose).
		/// </summary>
		/// <param name="Format">Format string</param>
		/// <param name="Args">Arguments</param>
		[StringFormatMethod("Format")]
		[Obsolete("Use Logger.LogTrace with a message template instead; see https://tinyurl.com/bp96bk2r.", false)]
		public static void LogVeryVerbose(string Format, params object[] Args)
		{
			EpicGames.Core.Log.WriteLine(EpicGames.Core.LogEventType.VeryVerbose, Format, Args);
		}

		/// <summary>
		/// Writes formatted text to log (with LogEventType.VeryVerbose).
		/// </summary>
		/// <param name="Message">Text</param>
		[Obsolete("Use Logger.LogTrace with a message template instead; see https://tinyurl.com/bp96bk2r.", false)]
		public static void LogVeryVerbose(string Message)
		{
			EpicGames.Core.Log.WriteLine(EpicGames.Core.LogEventType.VeryVerbose, Message);
		}

		/// <summary>
		/// Writes formatted text to log (with LogEventType.Log).
		/// </summary>
		/// <param name="Format">Format string</param>
		/// <param name="Args">Arguments</param>
		[StringFormatMethod("Format")]
		[Obsolete("Use Logger.LogDebug with a message template instead; see https://tinyurl.com/bp96bk2r.", false)]
		public static void LogLog(string Format, params object[] Args)
		{
			EpicGames.Core.Log.WriteLine(EpicGames.Core.LogEventType.Log, Format, Args);
		}

		/// <summary>
		/// Writes formatted text to log (with LogEventType.Log).
		/// </summary>
		/// <param name="Message">Text</param>
		[Obsolete("Use Logger.LogDebug with a message template instead; see https://tinyurl.com/bp96bk2r.", false)]
		public static void LogLog(string Message)
		{
			EpicGames.Core.Log.WriteLine(LogEventType.Log, Message);
		}

		/// <summary>
		/// Writes formatted text to log.
		/// </summary>
		/// <param name="Verbosity">Verbosity</param>
		/// <param name="Format">Format string</param>
		/// <param name="Args">Arguments</param>
		[StringFormatMethod("Format")]
		public static void LogWithVerbosity(EpicGames.Core.LogEventType Verbosity, string Format, params object[] Args)
		{
			EpicGames.Core.Log.WriteLine(Verbosity, Format, Args);
		}

		/// <summary>
		/// Writes formatted text to log.
		/// </summary>
		/// <param name="Verbosity">Verbosity</param>
		/// <param name="Message">Text</param>
		public static void LogWithVerbosity(EpicGames.Core.LogEventType Verbosity, string Message)
		{
			EpicGames.Core.Log.WriteLine(Verbosity, Message);
		}

		/// <summary>
		/// Dumps exception to log.
		/// </summary>
		/// <param name="Verbosity">Verbosity</param>
		/// <param name="Ex">Exception</param>
		public static void LogWithVerbosity(EpicGames.Core.LogEventType Verbosity, Exception Ex)
		{
			EpicGames.Core.Log.WriteLine(Verbosity, LogUtils.FormatException(Ex));
		}

		public static void LogPushProgress(bool bShowProgress, int Numerator, int Denominator)
		{
			if (bShowProgress)
			{
				Logger.LogInformation("[@progress push {Numerator}/{Denominator} skipline]", Numerator, Denominator);
			}
		}

		public static void LogPopProgress(bool bShowProgress)
		{
			if (bShowProgress)
			{
				Logger.LogInformation("[@progress pop skipline]");
			}
		}

		public static void LogIncrementProgress(bool bShowProgress, int Numerator, int Denominator)
		{
			if (bShowProgress)
			{
				Logger.LogInformation("[@progress increment {Numerator}/{Denominator} skipline]", Numerator, Denominator);
			}
		}

		[StringFormatMethod("Format")]
		public static void LogSetProgress(bool bShowProgress, string Format, params string[] Args)
		{
			if (bShowProgress)
			{
				Logger.LogInformation("[@progress '{Message}' skipline]", String.Format(Format, Args));
			}
		}

		[StringFormatMethod("Format")]
		public static void LogSetProgress(bool bShowProgress, int Numerator, int Denominator, string Format, params string[] Args)
		{
			if (bShowProgress)
			{
				Logger.LogInformation("[@progress {Numerator}/{Denominator} '{Arg2}' skipline]", Numerator, Denominator, String.Format(Format, Args));
			}
		}

		/// <summary>
		/// Finds files in specified paths. 
		/// </summary>
		/// <param name="SearchPattern">Pattern</param>
		/// <param name="Recursive">Recursive search</param>
		/// <param name="PathToSearch">Paths to search</param>
		/// <returns>An array of files found in the specified paths</returns>
		public static string[] FindFiles(string SearchPattern, bool Recursive, string PathToSearch)
		{
			List<string> FoundFiles = new List<string>();

			var NormalizedPath = ConvertSeparators(PathSeparator.Default, PathToSearch);
			if (DirectoryExists(NormalizedPath))
			{
				var FoundInPath = InternalUtils.SafeFindFiles(NormalizedPath, SearchPattern, Recursive);
				if (FoundInPath == null)
				{
					throw new AutomationException(String.Format("Failed to find files in '{0}'", NormalizedPath));
				}
				FoundFiles.AddRange(FoundInPath);
			}

			return FoundFiles.ToArray();
		}

		/// <summary>
		/// Finds files in specified paths. 
		/// </summary>
		/// <param name="SearchPattern">Pattern</param>
		/// <param name="Recursive">Recursive search</param>
		/// <param name="PathToSearch">Paths to search</param>
		/// <returns>An array of files found in the specified paths</returns>
		public static FileReference[] FindFiles(string SearchPattern, bool Recursive, DirectoryReference PathToSearch)
		{
			return FindFiles(SearchPattern, Recursive, PathToSearch.FullName).Select(x => new FileReference(x)).ToArray();
		}

		/// <summary>
		/// Finds files in specified paths. 
		/// </summary>
		/// <param name="SearchPattern">Pattern</param>
		/// <param name="Recursive">Recursive search</param>
		/// <param name="PathToSearch">Paths to search</param>
		/// <returns>An array of files found in the specified paths</returns>
		public static string[] FindFiles_NoExceptions(string SearchPattern, bool Recursive, string PathToSearch)
		{
			List<string> FoundFiles = new List<string>();

			var NormalizedPath = ConvertSeparators(PathSeparator.Default, PathToSearch);
			if (DirectoryExists(NormalizedPath))
			{
				var FoundInPath = InternalUtils.SafeFindFiles(NormalizedPath, SearchPattern, Recursive);
				if (FoundInPath != null)
				{
					FoundFiles.AddRange(FoundInPath);
				}
			}

			return FoundFiles.ToArray();
		}
		/// <summary>
		/// Finds files in specified paths. 
		/// </summary>
		/// <param name="bQuiet">When true, logging is suppressed.</param>
		/// <param name="SearchPattern">Pattern</param>
		/// <param name="Recursive">Recursive search</param>
		/// <param name="PathToSearch">Paths to search</param>
		/// <returns>An array of files found in the specified paths</returns>
		public static string[] FindFiles_NoExceptions(bool bQuiet, string SearchPattern, bool Recursive, string PathToSearch)
		{
			List<string> FoundFiles = new List<string>();

			var NormalizedPath = ConvertSeparators(PathSeparator.Default, PathToSearch);
			if (DirectoryExists(NormalizedPath))
			{
				var FoundInPath = InternalUtils.SafeFindFiles(NormalizedPath, SearchPattern, Recursive, bQuiet);
				if (FoundInPath != null)
				{
					FoundFiles.AddRange(FoundInPath);
				}
			}

			return FoundFiles.ToArray();
		}
		/// <summary>
		/// Finds files in specified paths. 
		/// </summary>
		/// <param name="bQuiet">When true, logging is suppressed.</param>
		/// <param name="SearchPattern">Pattern</param>
		/// <param name="Recursive">Recursive search</param>
		/// <param name="PathToSearch">Paths to search</param>
		/// <returns>An array of files found in the specified paths</returns>
		public static string[] FindDirectories(bool bQuiet, string SearchPattern, bool Recursive, string PathToSearch)
		{
			List<string> FoundDirs = new List<string>();

			var NormalizedPath = ConvertSeparators(PathSeparator.Default, PathToSearch);
			if (DirectoryExists(NormalizedPath))
			{
				var FoundInPath = InternalUtils.SafeFindDirectories(NormalizedPath, SearchPattern, Recursive, bQuiet);
				if (FoundInPath == null)
				{
					throw new AutomationException(String.Format("Failed to find directories in '{0}'", NormalizedPath));
				}
				FoundDirs.AddRange(FoundInPath);
			}

			return FoundDirs.ToArray();
		}
		/// <summary>
		/// Finds Directories in specified paths. 
		/// </summary>
		/// <param name="bQuiet">When true, logging is suppressed.</param>
		/// <param name="SearchPattern">Pattern</param>
		/// <param name="Recursive">Recursive search</param>
		/// <param name="PathToSearch">Paths to search</param>
		/// <returns>An array of files found in the specified paths</returns>
		public static string[] FindDirectories_NoExceptions(bool bQuiet, string SearchPattern, bool Recursive, string PathToSearch)
		{
			List<string> FoundDirs = new List<string>();

			var NormalizedPath = ConvertSeparators(PathSeparator.Default, PathToSearch);
			if (DirectoryExists(NormalizedPath))
			{
				var FoundInPath = InternalUtils.SafeFindDirectories(NormalizedPath, SearchPattern, Recursive, bQuiet);
				if (FoundInPath != null)
				{
					FoundDirs.AddRange(FoundInPath);
				}
			}

			return FoundDirs.ToArray();
		}
		/// <summary>
		/// Deletes a file(s). 
		/// If the file does not exist, silently succeeds.
		/// If the deletion of the file fails, this function throws an Exception.
		/// </summary>
		/// <param name="FileName">Filename</param>
		public static void DeleteFile(string FileName)
		{
			var NormalizedFilename = ConvertSeparators(PathSeparator.Default, FileName);
			if (!InternalUtils.SafeDeleteFile(NormalizedFilename))
			{
				throw new AutomationException(String.Format("Failed to delete file '{0}'", NormalizedFilename));
			}
		}

		/// <summary>
		/// Deletes a file(s). 
		/// If the file does not exist, silently succeeds.
		/// If the deletion of the file fails, this function throws an Exception.
		/// </summary>
		/// <param name="FileName">Filename</param>
		public static void DeleteFile(FileReference FileName)
		{
			DeleteFile(FileName.FullName);
		}

		/// <summary>
		/// Deletes a file(s). 
		/// If the file does not exist, silently succeeds.
		/// If the deletion of the file fails, this function throws an Exception.
		/// </summary>
		/// <param name="bQuiet">When true, logging is suppressed.</param>
		/// <param name="FileName">Filename</param>
		public static void DeleteFile(bool bQuiet, string FileName)
		{
			var NormalizedFilename = ConvertSeparators(PathSeparator.Default, FileName);
			if (!InternalUtils.SafeDeleteFile(NormalizedFilename, bQuiet))
			{
				throw new AutomationException(String.Format("Failed to delete file '{0}'", NormalizedFilename));
			}
		}

		/// <summary>
		/// Deletes a file(s). 
		/// If the deletion of the file fails, prints a warning.
		/// </summary>
		/// <param name="FileName">Filename</param>
		public static bool DeleteFile_NoExceptions(string FileName)
		{
			bool Result = true;

			var NormalizedFilename = ConvertSeparators(PathSeparator.Default, FileName);
			if (!InternalUtils.SafeDeleteFile(NormalizedFilename))
			{
				Logger.LogWarning("Failed to delete file '{NormalizedFilename}'", NormalizedFilename);
				Result = false;
			}
			return Result;
		}
		/// <summary>
		/// Deletes a file(s). 
		/// If the deletion of the file fails, prints a warning.
		/// </summary>
		/// <param name="Filename">Filename</param>
		/// <param name="bQuiet">if true, then don't retry and don't print much.</param>
		public static bool DeleteFile_NoExceptions(string Filename, bool bQuiet = false)
		{
			bool Result = true;
			var NormalizedFilename = ConvertSeparators(PathSeparator.Default, Filename);
			if (!InternalUtils.SafeDeleteFile(NormalizedFilename, bQuiet))
			{
				LogWithVerbosity(bQuiet ? LogEventType.Log : LogEventType.Warning, "Failed to delete file '{0}'", NormalizedFilename);
				Result = false;
			}
			return Result;
		}

		/// <summary>
		/// Deletes a directory(or directories) including its contents (recursively, will delete read-only files).
		/// If the deletion of the directory fails, this function throws an Exception.
		/// </summary>
		/// <param name="bQuiet">Suppresses log output if true</param>
		/// <param name="Directory">Directories</param>
		public static void DeleteDirectory(bool bQuiet, string Directory)
		{
			var NormalizedDirectory = ConvertSeparators(PathSeparator.Default, Directory);
			if (!InternalUtils.SafeDeleteDirectory(NormalizedDirectory, bQuiet))
			{
				throw new AutomationException(String.Format("Failed to delete directory '{0}'", NormalizedDirectory));
			}
		}

		/// <summary>
		/// Deletes a directory(or directories) including its contents (recursively, will delete read-only files).
		/// If the deletion of the directory fails, this function throws an Exception.
		/// </summary>
		/// <param name="Directory">Directories</param>
		public static void DeleteDirectory(string Directory)
		{
			DeleteDirectory(false, Directory);
		}

		/// <summary>
		/// Deletes a directory(or directories) including its contents (recursively, will delete read-only files).
		/// If the deletion of the directory fails, this function throws an Exception.
		/// </summary>
		/// <param name="Directory">Directories</param>
		public static void DeleteDirectory(DirectoryReference Directory)
		{
			DeleteDirectory(Directory.FullName);
		}

		/// <summary>
		/// Deletes a directory(or directories) including its contents (recursively, will delete read-only files).
		/// If the deletion of the directory fails, prints a warning.
		/// </summary>
		/// <param name="bQuiet">Suppresses log output if true</param>
		/// <param name="Directory">Directories</param>
		public static bool DeleteDirectory_NoExceptions(bool bQuiet, string Directory)
		{
			bool Result = true;

			var NormalizedDirectory = ConvertSeparators(PathSeparator.Default, Directory);
			try
			{
				if (!InternalUtils.SafeDeleteDirectory(NormalizedDirectory, bQuiet))
				{
					Logger.LogWarning("Failed to delete directory '{NormalizedDirectory}'", NormalizedDirectory);
					Result = false;
				}
			}
			catch (Exception Ex)
			{
				if (!bQuiet)
				{
					Logger.LogWarning("Failed to delete directory, exception '{NormalizedDirectory}'", NormalizedDirectory);
					Logger.LogWarning(Ex, "{Text}", Ex.Message);
				}
				Result = false;
			}

			return Result;
		}

		/// <summary>
		/// Deletes a directory(or directories) including its contents (recursively, will delete read-only files).
		/// If the deletion of the directory fails, prints a warning.
		/// </summary>
		/// <param name="DirectoryName">Directories</param>
		public static bool DeleteDirectory_NoExceptions(string DirectoryName)
		{
			return DeleteDirectory_NoExceptions(false, DirectoryName);
		}


		/// <summary>
		/// Attempts to delete a directory, if that fails deletes all files and folder from the specified directory.
		/// This works around the issue when the user has a file open in a notepad from that directory. Somehow deleting the file works but
		/// deleting the directory with the file that's open, doesn't.
		/// </summary>
		/// <param name="DirectoryName"></param>
		public static void DeleteDirectoryContents(string DirectoryName)
		{
			Logger.LogDebug("DeleteDirectoryContents({DirectoryName})", DirectoryName);
			const bool bQuiet = true;
			var Files = CommandUtils.FindFiles_NoExceptions(bQuiet, "*", false, DirectoryName);
			foreach (var Filename in Files)
			{
				CommandUtils.DeleteFile_NoExceptions(Filename);
			}
			var Directories = CommandUtils.FindDirectories_NoExceptions(bQuiet, "*", false, DirectoryName);
			foreach (var SubDirectoryName in Directories)
			{
				CommandUtils.DeleteDirectory_NoExceptions(bQuiet, SubDirectoryName);
			}
		}

		/// <summary>
		/// Attempts to delete a directory, if that fails deletes all files and folder from the specified directory.
		/// This works around the issue when the user has a file open in a notepad from that directory. Somehow deleting the file works but
		/// deleting the directory with the file that's open, doesn't.
		/// </summary>
		/// <param name="DirectoryName"></param>
		public static void DeleteDirectoryContents(DirectoryReference DirectoryName)
		{
			DeleteDirectoryContents(DirectoryName.FullName);
		}

		/// <summary>
		/// Checks if a directory(or directories) exists.
		/// </summary>
		/// <param name="DirectoryName">Directories</param>
		/// <returns>True if the directory exists, false otherwise.</returns>
		public static bool DirectoryExists(string DirectoryName)
		{
			var NormalizedDirectory = ConvertSeparators(PathSeparator.Default, DirectoryName);
			return System.IO.Directory.Exists(NormalizedDirectory);
		}

		/// <summary>
		/// Checks if a directory(or directories) exists.
		/// </summary>
		/// <param name="DirectoryName">Directories</param>
		/// <returns>True if the directory exists, false otherwise.</returns>
		public static bool DirectoryExists_NoExceptions(string DirectoryName)
		{
			var NormalizedDirectory = ConvertSeparators(PathSeparator.Default, DirectoryName);
			try
			{
				return System.IO.Directory.Exists(NormalizedDirectory);
			}
			catch (Exception Ex)
			{
				Logger.LogWarning("Unable to check if directory exists: {NormalizedDirectory}", NormalizedDirectory);
				Logger.LogWarning(Ex, "{Text}", Ex.Message);
				return false;
			}
		}

		/// <summary>
		/// Renames/moves a directory.
		/// If the rename of the directory fails, this function throws an Exception.
		/// </summary>
		/// <param name="OldName">Old name</param>
		/// <param name="NewName">new name</param>
		/// <param name="bQuiet"></param>
		/// <param name="bRetry"></param>
		public static void RenameDirectory(string OldName, string NewName, bool bQuiet = false, bool bRetry = true)
		{
			var OldNormalized = ConvertSeparators(PathSeparator.Default, OldName);
			var NewNormalized = ConvertSeparators(PathSeparator.Default, NewName);
			InternalUtils.SafeRenameDirectory(OldNormalized, NewNormalized, bQuiet, bRetry, /*bThrow*/true);
		}

		/// <summary>
		/// Renames/moves a directory.
		/// If the rename of the directory fails, this function prints a warning.
		/// </summary>
		/// <param name="OldName">Old name</param>
		/// <param name="NewName">new name</param>
		/// <param name="bQuiet"></param>
		/// <param name="bRetry"></param>
		public static bool RenameDirectory_NoExceptions(string OldName, string NewName, bool bQuiet = false, bool bRetry = true)
		{
			var OldNormalized = ConvertSeparators(PathSeparator.Default, OldName);
			var NewNormalized = ConvertSeparators(PathSeparator.Default, NewName);
			return InternalUtils.SafeRenameDirectory(OldNormalized, NewNormalized, bQuiet, bRetry, /*bThrow*/false);
		}

		/// <summary>
		/// Creates a directory. Throws an exception on failure.
		/// </summary>
		/// <param name="DirectoryName">Name of the directory to create</param>
		public static void CreateDirectory(string DirectoryName)
		{
			string NormalizedDirectory = ConvertSeparators(PathSeparator.Default, DirectoryName);
			try
			{
				Directory.CreateDirectory(DirectoryName);
			}
			catch (Exception Ex)
			{
				throw new AutomationException(Ex, "Failed to create directory '{0}'", NormalizedDirectory);
			}
		}

		/// <summary>
		/// Creates a directory. Throws an exception on failure.
		/// </summary>
		/// <param name="Location">Name of the directory to create</param>
		public static void CreateDirectory(DirectoryReference Location)
		{
			CreateDirectory(Location.FullName);
		}

		/// <summary>
		/// Renames/moves a file.
		/// If the rename of the file fails, this function throws an Exception.
		/// </summary>
		/// <param name="bQuiet">When true, logging is suppressed.</param>
		/// <param name="OldName">Old name</param>
		/// <param name="NewName">new name</param>
		public static void RenameFile(string OldName, string NewName, bool bQuiet = false)
		{
			var OldNormalized = ConvertSeparators(PathSeparator.Default, OldName);
			var NewNormalized = ConvertSeparators(PathSeparator.Default, NewName);
			if (!InternalUtils.SafeRenameFile(OldNormalized, NewNormalized, bQuiet))
			{
				throw new AutomationException(String.Format("Failed to rename/move file '{0}' to '{1}'", OldNormalized, NewNormalized));
			}
		}

		/// <summary>
		/// Renames/moves a file.
		/// If the rename of the file fails, this function prints a warning.
		/// </summary>
		/// <param name="OldName">Old name</param>
		/// <param name="NewName">new name</param>
		public static bool RenameFile_NoExceptions(string OldName, string NewName)
		{
			var OldNormalized = ConvertSeparators(PathSeparator.Default, OldName);
			var NewNormalized = ConvertSeparators(PathSeparator.Default, NewName);
			var Result = InternalUtils.SafeRenameFile(OldNormalized, NewNormalized);
			if (!Result)
			{
				Logger.LogWarning("Failed to rename/move file '{OldName}' to '{NewName}'", OldName, NewName);
			}
			return Result;
		}

		/// <summary>
		/// Checks if a file(s) exists.
		/// </summary>
		/// <param name="FileName">Filename.</param>
		/// <returns>True if the file exists, false otherwise.</returns>
		public static bool FileExists(string FileName)
		{
			// need to remove the quotes before checking to see if it exists
			var NormalizedFilename = ConvertSeparators(PathSeparator.Default, FileName).Replace("\"", "");
			return InternalUtils.SafeFileExists(NormalizedFilename, true);
		}

		/// <summary>
		/// Checks if a file(s) exists.
		/// </summary>
		/// <param name="FileName">Filename.</param>
		/// <returns>True if the file exists, false otherwise.</returns>
		public static bool FileExists_NoExceptions(string FileName)
		{
			// Standard version doesn't throw, but keep this function for consistency.
			return FileExists(FileName);
		}

		/// <summary>
		/// Checks if a file(s) exists.
		/// </summary>
		/// <param name="bQuiet">When true, logging is suppressed.</param>
		/// <param name="FileName">Filename.</param>
		/// <returns>True if the file exists, false otherwise.</returns>
		public static bool FileExists(bool bQuiet, string FileName)
		{
			// need to remove the quotes before checking to see if it exists
			var NormalizedFilename = ConvertSeparators(PathSeparator.Default, FileName).Replace("\"", "");
			return InternalUtils.SafeFileExists(NormalizedFilename, bQuiet);
		}

		/// <summary>
		/// Checks if a file(s) exists.
		/// </summary>
		/// <param name="bQuiet">When true, logging is suppressed.</param>
		/// <param name="FileName">Filename.</param>
		/// <returns>True if the file exists, false otherwise.</returns>
		public static bool FileExists_NoExceptions(bool bQuiet, string FileName)
		{
			// Standard version doesn't throw, but keep this function for consistency.
			return FileExists(bQuiet, FileName);
		}

		static Stack<string> WorkingDirectoryStack = new Stack<string>();

		/// <summary>
		/// Pushes the current working directory onto a stack and sets CWD to a new value.
		/// </summary>
		/// <param name="WorkingDirectory">New working direcotry.</param>
		public static void PushDir(string WorkingDirectory)
		{
			string OrigCurrentDirectory = Environment.CurrentDirectory;
			WorkingDirectory = ConvertSeparators(PathSeparator.Default, WorkingDirectory);
			try
			{
				Environment.CurrentDirectory = WorkingDirectory;
			}
			catch (Exception Ex)
			{
				throw new AutomationException(Ex, "Unable to change current directory to {0}", WorkingDirectory);
			}

			WorkingDirectoryStack.Push(OrigCurrentDirectory);
		}

		/// <summary>
		/// Pushes the current working directory onto a stack and sets CWD to a new value.
		/// </summary>
		/// <param name="WorkingDirectory">New working direcotry.</param>
		public static bool PushDir_NoExceptions(string WorkingDirectory)
		{
			bool Result = true;
			string OrigCurrentDirectory = Environment.CurrentDirectory;
			WorkingDirectory = ConvertSeparators(PathSeparator.Default, WorkingDirectory);
			try
			{
				Environment.CurrentDirectory = WorkingDirectory;
				WorkingDirectoryStack.Push(OrigCurrentDirectory);
			}
			catch
			{
				Logger.LogWarning("Unable to change current directory to {WorkingDirectory}", WorkingDirectory);
				Result = false;
			}
			return Result;
		}

		/// <summary>
		/// Pops the last working directory from a stack and sets it as the current working directory.
		/// </summary>
		public static void PopDir()
		{
			if (WorkingDirectoryStack.Count > 0)
			{
				Environment.CurrentDirectory = WorkingDirectoryStack.Pop();
			}
			else
			{
				throw new AutomationException("Unable to PopDir. WorkingDirectoryStack is empty.");
			}
		}

		/// <summary>
		/// Pops the last working directory from a stack and sets it as the current working directory.
		/// </summary>
		public static bool PopDir_NoExceptions()
		{
			bool Result = true;
			if (WorkingDirectoryStack.Count > 0)
			{
				Environment.CurrentDirectory = WorkingDirectoryStack.Pop();
			}
			else
			{
				Logger.LogWarning("Unable to PopDir. WorkingDirectoryStack is empty.");
				Result = false;
			}
			return Result;
		}

		/// <summary>
		/// Clears the directory stack
		/// </summary>
		public static void ClearDirStack()
		{
			while (WorkingDirectoryStack.Count > 0)
			{
				PopDir();
			}
		}

		/// <summary>
		/// Changes the current working directory.
		/// </summary>
		/// <param name="WorkingDirectory">New working directory.</param>
		public static void ChDir(string WorkingDirectory)
		{
			WorkingDirectory = ConvertSeparators(PathSeparator.Default, WorkingDirectory);
			try
			{
				Environment.CurrentDirectory = WorkingDirectory;
			}
			catch (Exception Ex)
			{
				throw new ArgumentException(String.Format("Unable to change current directory to {0}", WorkingDirectory), Ex);
			}
		}

		/// <summary>
		/// Changes the current working directory.
		/// </summary>
		/// <param name="WorkingDirectory">New working directory.</param>
		public static bool ChDir_NoExceptions(string WorkingDirectory)
		{
			bool Result = true;
			WorkingDirectory = ConvertSeparators(PathSeparator.Default, WorkingDirectory);
			try
			{
				Environment.CurrentDirectory = WorkingDirectory;
			}
			catch
			{
				Logger.LogWarning("Unable to change current directory to {WorkingDirectory}", WorkingDirectory);
				Result = false;
			}
			return Result;
		}

		/// <summary>
		/// Updates a file with the specified modified and access date, creating the file if it does not already exist.
		/// An exception will be thrown if the directory does not already exist.
		/// </summary>
		/// <param name="Filename">The filename to touch, will be created if it does not exist.</param>
		/// <param name="UtcDate">The accessed and modified date to set.  If not specified, defaults to the current date and time.</param>
		public static void TouchFile(string Filename, DateTime? UtcDate = null)
		{
			DateTime Date = UtcDate ?? DateTime.UtcNow;
			Filename = ConvertSeparators(PathSeparator.Slash, Filename);
			if (!File.Exists(Filename))
			{
				var Dir = GetDirectoryName(Filename);
				if (!DirectoryExists_NoExceptions(Dir))
				{
					throw new AutomationException(new DirectoryNotFoundException("Directory not found: " + Dir), "Unable to create file {0} as directory does not exist.", Filename);
				}
				File.Create(Filename).Dispose();
			}
			File.SetLastAccessTimeUtc(Filename, Date);
			File.SetLastWriteTimeUtc(Filename, Date);
		}

		/// <summary>
		/// Determines whether the given file is read-only
		/// </summary>
		/// <param name="Filename">Filename</param>
		/// <returns>True if the file is read-only</returns>
		public static bool IsReadOnly(string Filename)
		{
			Filename = ConvertSeparators(PathSeparator.Default, Filename);
			if (!File.Exists(Filename))
			{
				throw new AutomationException(new FileNotFoundException("File not found.", Filename), "Unable to set attributes for a non-existing file.");
			}

			FileAttributes Attributes = File.GetAttributes(Filename);
			return (Attributes & FileAttributes.ReadOnly) != 0;
		}

		/// <summary>
		/// Sets file attributes. Will not change attributes that have not been specified.
		/// </summary>
		/// <param name="Filename">Filename</param>
		/// <param name="ReadOnly">Read-only attribute</param>
		/// <param name="Hidden">Hidden attribute.</param>
		/// <param name="Archive">Archive attribute.</param>
		public static void SetFileAttributes(string Filename, bool? ReadOnly = null, bool? Hidden = null, bool? Archive = null)
		{
			Filename = ConvertSeparators(PathSeparator.Default, Filename);
			if (!File.Exists(Filename))
			{
				throw new AutomationException(new FileNotFoundException("File not found.", Filename), "Unable to set attributes for a non-existing file.");
			}

			FileAttributes Attributes = File.GetAttributes(Filename);
			Attributes = InternalSetAttributes(ReadOnly, Hidden, Archive, Attributes);
			File.SetAttributes(Filename, Attributes);
		}

		/// <summary>
		/// Sets file attributes. Will not change attributes that have not been specified.
		/// </summary>
		/// <param name="Filename">Filename</param>
		/// <param name="ReadOnly">Read-only attribute</param>
		/// <param name="Hidden">Hidden attribute.</param>
		/// <param name="Archive">Archive attribute.</param>
		public static bool SetFileAttributes_NoExceptions(string Filename, bool? ReadOnly = null, bool? Hidden = null, bool? Archive = null)
		{
			Filename = ConvertSeparators(PathSeparator.Default, Filename);
			if (!File.Exists(Filename))
			{
				Logger.LogWarning("Unable to set attributes for a non-exisiting file ({Filename})", Filename);
				return false;
			}

			bool Result = true;
			try
			{
				FileAttributes Attributes = File.GetAttributes(Filename);
				Attributes = InternalSetAttributes(ReadOnly, Hidden, Archive, Attributes);
				File.SetAttributes(Filename, Attributes);
			}
			catch (Exception Ex)
			{
				Logger.LogWarning("Error trying to set file attributes for: {Filename}", Filename);
				Logger.LogWarning(Ex, "{Text}", Ex.Message);
				Result = false;
			}
			return Result;
		}

		private static FileAttributes InternalSetAttributes(bool? ReadOnly, bool? Hidden, bool? Archive, FileAttributes Attributes)
		{
			if (ReadOnly != null)
			{
				if ((bool)ReadOnly)
				{
					Attributes |= FileAttributes.ReadOnly;
				}
				else
				{
					Attributes &= ~FileAttributes.ReadOnly;
				}
			}
			if (Hidden != null)
			{
				if ((bool)Hidden)
				{
					Attributes |= FileAttributes.Hidden;
				}
				else
				{
					Attributes &= ~FileAttributes.Hidden;
				}
			}
			if (Archive != null)
			{
				if ((bool)Archive)
				{
					Attributes |= FileAttributes.Archive;
				}
				else
				{
					Attributes &= ~FileAttributes.Archive;
				}
			}
			return Attributes;
		}

		/// <summary>
		/// Writes a line of formatted string to a file. Creates the file if it does not exists.
		/// If the file does exists, appends a new line.
		/// </summary>
		/// <param name="Filename">Filename</param>
		/// <param name="Text">Text to write</param>
		public static void WriteToFile(string Filename, string Text)
		{
			Filename = ConvertSeparators(PathSeparator.Default, Filename);
			try
			{
				File.AppendAllText(Filename, Text + Environment.NewLine);
			}
			catch (Exception Ex)
			{
				throw new AutomationException(Ex, "Failed to Write to file {0}", Filename);
			}
		}

		/// <summary>
		/// Reads all text lines from a file.
		/// </summary>
		/// <param name="Filename">Filename</param>
		/// <returns>Array of lines of text read from the file. null if the file did not exist or could not be read.</returns>
		public static string[] ReadAllLines(string Filename)
		{
			Filename = ConvertSeparators(PathSeparator.Default, Filename);
			return InternalUtils.SafeReadAllLines(Filename);
		}

		/// <summary>
		/// Reads all text from a file.
		/// </summary>
		/// <param name="Filename">Filename</param>
		/// <returns>All text read from the file. null if the file did not exist or could not be read.</returns>
		public static string ReadAllText(string Filename)
		{
			Filename = ConvertSeparators(PathSeparator.Default, Filename);
			return InternalUtils.SafeReadAllText(Filename);
		}

		/// <summary>
		/// Writes lines to a file.
		/// </summary>
		/// <param name="Filename">Filename</param>
		/// <param name="Lines">Text</param>
		public static void WriteAllLines(string Filename, string[] Lines)
		{
			Filename = ConvertSeparators(PathSeparator.Default, Filename);
			if (!InternalUtils.SafeWriteAllLines(Filename, Lines))
			{
				throw new AutomationException("Unable to write to file: {0}", Filename);
			}
		}

		/// <summary>
		/// Writes lines to a file.
		/// </summary>
		/// <param name="Filename">Filename</param>
		/// <param name="Lines">Text</param>
		public static bool WriteAllLines_NoExceptions(string Filename, string[] Lines)
		{
			Filename = ConvertSeparators(PathSeparator.Default, Filename);
			return InternalUtils.SafeWriteAllLines(Filename, Lines);
		}

		/// <summary>
		/// Writes text to a file.
		/// </summary>
		/// <param name="Filename">Filename</param>
		/// <param name="Text">Text</param>
		public static void WriteAllText(string Filename, string Text)
		{
			Filename = ConvertSeparators(PathSeparator.Default, Filename);
			if (!InternalUtils.SafeWriteAllText(Filename, Text))
			{
				throw new AutomationException("Unable to write to file: {0}", Filename);
			}
		}

		/// <summary>
		/// Writes text to a file.
		/// </summary>
		/// <param name="Filename">Filename</param>
		/// <param name="Text">Text</param>
		public static bool WriteAllText_NoExceptions(string Filename, string Text)
		{
			Filename = ConvertSeparators(PathSeparator.Default, Filename);
			return InternalUtils.SafeWriteAllText(Filename, Text);
		}


		/// <summary>
		/// Writes byte array to a file.
		/// </summary>
		/// <param name="Filename">Filename</param>
		/// <param name="Bytes">Byte array</param>
		public static void WriteAllBytes(string Filename, byte[] Bytes)
		{
			Filename = ConvertSeparators(PathSeparator.Default, Filename);
			if (!InternalUtils.SafeWriteAllBytes(Filename, Bytes))
			{
				throw new AutomationException("Unable to write to file: {0}", Filename);
			}
		}

		/// <summary>
		/// Writes byte array to a file.
		/// </summary>
		/// <param name="Filename">Filename</param>
		/// <param name="Bytes">Byte array</param>
		public static bool WriteAllBytes_NoExceptions(string Filename, byte[] Bytes)
		{
			Filename = ConvertSeparators(PathSeparator.Default, Filename);
			return InternalUtils.SafeWriteAllBytes(Filename, Bytes);
		}

		/// <summary>
		/// Gets a character representing the specified separator type.
		/// </summary>
		/// <param name="SeparatorType">Separator type.</param>
		/// <returns>Separator character</returns>
		public static char GetPathSeparatorChar(PathSeparator SeparatorType)
		{
			char Separator;
			switch (SeparatorType)
			{
				case PathSeparator.Slash:
				case PathSeparator.Depot:
					Separator = '/';
					break;
				case PathSeparator.Backslash:
					Separator = '\\';
					break;
				default:
					Separator = Path.DirectorySeparatorChar;
					break;
			}
			return Separator;
		}

		/// <summary>
		/// Checks if the character is one of the two sperator types ('\' or '/')
		/// </summary>
		/// <param name="Character">Character to check.</param>
		/// <returns>True if the character is a separator, false otherwise.</returns>
		public static bool IsPathSeparator(char Character)
		{
			return (Character == '/' || Character == '\\');
		}

		/// <summary>
		/// Combines paths and replaces all path separators with the system default separator.
		/// </summary>
		/// <param name="Paths"></param>
		/// <returns>Combined Path</returns>
		public static string CombinePaths(params string[] Paths)
		{
			return CombinePaths(PathSeparator.Default, Paths);
		}

		/// <summary>
		/// Combines paths and replaces all path separators wth the system specified separator.
		/// </summary>
		/// <param name="SeparatorType">Type of separartor to use when combining paths.</param>
		/// <param name="Paths"></param>
		/// <returns>Combined Path</returns>
		public static string CombinePaths(PathSeparator SeparatorType, params string[] Paths)
		{
			// Pick a separator to use.
			var SeparatorToUse = GetPathSeparatorChar(SeparatorType);
			var SeparatorToReplace = SeparatorToUse == '/' ? '\\' : '/';

			// Allocate string builder
			int CombinePathMaxLength = 0;
			foreach (var PathPart in Paths)
			{
				CombinePathMaxLength += (PathPart != null) ? PathPart.Length : 0;
			}
			CombinePathMaxLength += Paths.Length;
			StringBuilder CombinedPath = new StringBuilder(CombinePathMaxLength);

			// Combine all paths
			CombinedPath.Append(Paths[0].Replace("\"", ""));
			for (int PathIndex = 1; PathIndex < Paths.Length; ++PathIndex)
			{
				if (String.IsNullOrEmpty(Paths[PathIndex]) == false)
				{
					var NextPath = Paths[PathIndex].Replace("\"", "");
					int NextPathStartIndex = 0;
					if (CombinedPath.Length != 0)
					{
						var LastChar = CombinedPath[CombinedPath.Length - 1];
						var NextChar = NextPath[0];
						var IsLastCharPathSeparator = IsPathSeparator(LastChar);
						var IsNextCharPathSeparator = IsPathSeparator(NextChar);
						// Check if a separator between paths is required
						if (!IsLastCharPathSeparator && !IsNextCharPathSeparator)
						{
							CombinedPath.Append(SeparatorToUse);
						}
						// Check if one of the saprators needs to be skipped.
						else if (IsLastCharPathSeparator && IsNextCharPathSeparator)
						{
							NextPathStartIndex = 1;
						}
					}
					CombinedPath.Append(NextPath, NextPathStartIndex, NextPath.Length - NextPathStartIndex);
				}
			}
			// Make sure there's only one separator type used.
			CombinedPath.Replace(SeparatorToReplace, SeparatorToUse);
			return CombinedPath.ToString();
		}

		/// <summary>
		/// Converts all separators in path to the specified separator type.
		/// </summary>
		/// <param name="ToSperatorType">Desired separator type.</param>
		/// <param name="PathToConvert">Path</param>
		/// <returns>Path where all separators have been converted to the specified type.</returns>
		public static string ConvertSeparators(PathSeparator ToSperatorType, string PathToConvert)
		{
			return CombinePaths(ToSperatorType, PathToConvert);
		}

		/// <summary>
		/// Copies a file, throwing an exception on failure.
		/// </summary>
		/// <param name="Source"></param>
		/// <param name="Dest"></param>
		/// <param name="bQuiet">When true, logging is suppressed.</param>
		/// <param name="bRetry">When true, the copy will be attempted up to 5 times before failing.</param>
		public static void CopyFile(string Source, string Dest, bool bQuiet = false, bool bRetry = false)
		{
			Source = ConvertSeparators(PathSeparator.Default, Source);
			Dest = ConvertSeparators(PathSeparator.Default, Dest);

			String DestDirName = "";
			try
			{
				DestDirName = Path.GetDirectoryName(Dest);
			}
			catch (Exception Ex)
			{
				throw new AutomationException(String.Format("Failed to get directory name for dest: {0}, {1}", Dest, Ex.Message));
			}

			if (!InternalUtils.SafeDirectoryExists(DestDirName, true))
			{
				if (!InternalUtils.SafeCreateDirectory(DestDirName, bQuiet))
				{
					throw new AutomationException("Failed to create directory {0} for copy", DestDirName);
				}
			}

			for (int AttemptsRemaining = 5; AttemptsRemaining >= 0; --AttemptsRemaining)
			{
				if (InternalUtils.SafeFileExists(Dest, true))
				{
					InternalUtils.SafeDeleteFile(Dest, bQuiet);
				}
				if (InternalUtils.SafeFileExists(Dest, true))
				{
					if (bRetry && AttemptsRemaining > 0)
					{
						Logger.LogDebug("Failed to delete {Dest} for copy, retrying..", Dest);
						Thread.Sleep(1000);
						continue;
					}
					throw new AutomationException("Failed to delete {0} for copy", Dest);
				}
				if (!InternalUtils.SafeCopyFile(Source, Dest, bQuiet))
				{
					if (bRetry && AttemptsRemaining > 0)
					{
						Logger.LogDebug("Failed to copy {Source} to {Dest}, retrying..", Source, Dest);
						Thread.Sleep(1000);
						continue;
					}
					throw new AutomationException("Failed to copy {0} to {1}", Source, Dest);
				}
				break;
			}
		}

		/// <summary>
		/// Copies a file. Does not throw exceptions.
		/// </summary>
		/// <param name="Source"></param>
		/// <param name="Dest"></param>
		/// <param name="bQuiet">When true, logging is suppressed.</param>
		/// <returns>True if the operation was successful, false otherwise.</returns>
		public static bool CopyFile_NoExceptions(string Source, string Dest, bool bQuiet = false)
		{
			Source = ConvertSeparators(PathSeparator.Default, Source);
			Dest = ConvertSeparators(PathSeparator.Default, Dest);
			if (InternalUtils.SafeFileExists(Dest, true))
			{
				InternalUtils.SafeDeleteFile(Dest, bQuiet);
			}
			else if (!InternalUtils.SafeDirectoryExists(Path.GetDirectoryName(Dest), true))
			{
				if (!InternalUtils.SafeCreateDirectory(Path.GetDirectoryName(Dest)))
				{
					return false;
				}
			}
			if (InternalUtils.SafeFileExists(Dest, true))
			{
				return false;
			}
			return InternalUtils.SafeCopyFile(Source, Dest, bQuiet);
		}

		/// <summary>
		/// Checks if the given target file is up to date with the source file (ie. it exists, and the timestamp matches)
		/// </summary>
		/// <param name="Target">The target file to check</param>
		/// <param name="Source">The source file to check</param>
		/// <returns>True if the files have the same timestamp, false otherwise.</returns>
		public static bool IsTargetFileUpToDate(FileReference Target, FileReference Source)
		{
			if (FileReference.Exists(Target))
			{
				TimeSpan Diff = FileReference.GetLastWriteTimeUtc(Target) - FileReference.GetLastWriteTimeUtc(Source);
				if (Math.Abs(Diff.TotalSeconds) < 1)
				{
					return true;
				}
			}
			return false;
		}

		/// <summary>
		/// Copies a file if the dest doesn't exist or (optionally) if the dest timestamp is different; after a copy, copies the timestamp
		/// </summary>
		/// <param name="Source">The full path to the source file</param>
		/// <param name="Dest">The full path to the destination file</param>
		/// <param name="OverrideCopyHandler"></param>
		/// <param name="bAllowDifferingTimestamps">If true, will always skip a file if the destination exists, even if timestamp differs; defaults to false</param>
		/// <param name="IniKeyDenyList"></param>
		/// <param name="IniSectionDenyList"></param>
		/// <returns>True if the operation was successful, false otherwise.</returns>
		public static void CopyFileIncremental(FileReference Source, FileReference Dest, OverrideCopyDelegate OverrideCopyHandler = null, bool bAllowDifferingTimestamps = false, List<string> IniKeyDenyList = null, List<string> IniSectionDenyList = null)
		{
			if (InternalUtils.SafeFileExists(Dest.FullName, true))
			{
				if (bAllowDifferingTimestamps == true)
				{
					Logger.LogDebug("CopyFileIncremental Skipping {Dest}, already exists", Dest);
					return;
				}
				TimeSpan Diff = File.GetLastWriteTimeUtc(Dest.FullName) - File.GetLastWriteTimeUtc(Source.FullName);
				if (Diff.TotalSeconds > -1 && Diff.TotalSeconds < 1)
				{
					Logger.LogDebug("CopyFileIncremental Skipping {Dest}, up to date.", Dest);
					return;
				}
				InternalUtils.SafeDeleteFile(Dest.FullName);
			}
			else if (!InternalUtils.SafeDirectoryExists(Path.GetDirectoryName(Dest.FullName), true))
			{
				if (!InternalUtils.SafeCreateDirectory(Path.GetDirectoryName(Dest.FullName)))
				{
					throw new AutomationException("Failed to create directory {0} for copy", Path.GetDirectoryName(Dest.FullName));
				}
			}
			if (InternalUtils.SafeFileExists(Dest.FullName, true))
			{
				throw new AutomationException("Failed to delete {0} for copy", Dest);
			}
			if (!InternalUtils.SafeCopyFile(Source.FullName, Dest.FullName, OverrideCopyHandler: OverrideCopyHandler, IniKeyDenyList: IniKeyDenyList, IniSectionDenyList: IniSectionDenyList))
			{
				throw new AutomationException("Failed to copy {0} to {1}", Source, Dest);
			}
			FileAttributes Attributes = File.GetAttributes(Dest.FullName);
			if ((Attributes & FileAttributes.ReadOnly) != 0)
			{
				File.SetAttributes(Dest.FullName, Attributes & ~FileAttributes.ReadOnly);
			}
			File.SetLastWriteTimeUtc(Dest.FullName, File.GetLastWriteTimeUtc(Source.FullName));
		}

		[Flags]
		public enum CopyDirectoryOptions
		{
			None = 0,
			SuppressLogging = 1,
			KeepExistingDirectories = 2,
			KeepExistingFiles = 4,

			Default = None,
			Merge = KeepExistingFiles | KeepExistingDirectories
		};

		/// <summary>
		/// Copies a directory and all of it's contents recursively. Does not throw exceptions.
		/// </summary>
		/// <param name="Source"></param>
		/// <param name="Dest"></param>
		/// <param name="bQuiet">When true, logging is suppressed.</param>
		/// <returns>True if the operation was successful, false otherwise.</returns>
		public static bool CopyDirectory_NoExceptions(DirectoryReference Source, DirectoryReference Dest, bool bQuiet = false)
		{
			return CopyDirectory_NoExceptions(Source.FullName, Dest.FullName, bQuiet);
		}

		/// <summary>
		/// Copies a directory and all of it's contents recursively. Does not throw exceptions.
		/// </summary>
		/// <param name="Source"></param>
		/// <param name="Dest"></param>
		/// <param name="bQuiet">When true, logging is suppressed.</param>
		/// <returns>True if the operation was successful, false otherwise.</returns>
		public static bool CopyDirectory_NoExceptions(string Source, string Dest, bool bQuiet = false)
		{
			return CopyDirectory_NoExceptions(Source, Dest, CopyDirectoryOptions.Default | (bQuiet ? CopyDirectoryOptions.SuppressLogging : CopyDirectoryOptions.None));
		}

		/// <summary>
		/// Copies a directory and all of it's contents recursively. Merges with the destination directory. Does not throw exceptions.
		/// </summary>
		/// <param name="Source"></param>
		/// <param name="Dest"></param>
		/// <param name="bQuiet">When true, logging is suppressed.</param>
		/// <returns>True if the operation was successful, false otherwise.</returns>
		public static bool MergeDirectory_NoExceptions(DirectoryReference Source, DirectoryReference Dest, bool bQuiet = false)
		{
			return CopyDirectory_NoExceptions(Source.FullName, Dest.FullName, CopyDirectoryOptions.KeepExistingDirectories | (bQuiet ? CopyDirectoryOptions.SuppressLogging : CopyDirectoryOptions.None));
		}

		/// <summary>
		/// Copies a directory and all of it's contents recursively. Merges with the destination directory. Does not throw exceptions.
		/// </summary>
		/// <param name="Source"></param>
		/// <param name="Dest"></param>
		/// <param name="bQuiet">When true, logging is suppressed.</param>
		/// <returns>True if the operation was successful, false otherwise.</returns>
		public static bool MergeDirectory_NoExceptions(string Source, string Dest, bool bQuiet = false)
		{
			return CopyDirectory_NoExceptions(Source, Dest, CopyDirectoryOptions.KeepExistingDirectories | (bQuiet ? CopyDirectoryOptions.SuppressLogging : CopyDirectoryOptions.None));
		}

		/// <summary>
		/// Copies a directory and all of it's contents recursively. Does not throw exceptions.
		/// </summary>
		/// <param name="Source"></param>
		/// <param name="Dest"></param>
		/// <param name="Options">Options to control logging and overwriting</param>
		/// <returns>True if the operation was successful, false otherwise.</returns>
		public static bool CopyDirectory_NoExceptions(string Source, string Dest, CopyDirectoryOptions Options)
		{
			bool bQuiet = !Options.HasFlag(CopyDirectoryOptions.SuppressLogging);

			Source = ConvertSeparators(PathSeparator.Default, Source);
			Dest = ConvertSeparators(PathSeparator.Default, Dest);
			Dest = Dest.TrimEnd(new char[] { Path.DirectorySeparatorChar, Path.AltDirectorySeparatorChar });

			// if we want to overwrite directories, and it already exists, delete it first
			if (!Options.HasFlag(CopyDirectoryOptions.KeepExistingDirectories) && InternalUtils.SafeDirectoryExists(Dest))
			{
				InternalUtils.SafeDeleteDirectory(Dest, bQuiet);
				if (InternalUtils.SafeDirectoryExists(Dest, true))
				{
					return false;
				}
			}

			if (!InternalUtils.SafeCreateDirectory(Dest, bQuiet))
			{
				return false;
			}

			if (InternalUtils.SafeDirectoryExists(Source))
			{
				foreach (var SourceSubDirectory in Directory.GetDirectories(Source))
				{
					string DestPath = Dest + GetPathSeparatorChar(PathSeparator.Default) + GetLastDirectoryName(SourceSubDirectory + GetPathSeparatorChar(PathSeparator.Default));
					if (!CopyDirectory_NoExceptions(SourceSubDirectory, DestPath, Options))
					{
						return false;
					}
				}
				foreach (var SourceFile in Directory.GetFiles(Source))
				{
					int FilenameStart = SourceFile.LastIndexOf(GetPathSeparatorChar(PathSeparator.Default));
					string DestPath = Dest + SourceFile.Substring(FilenameStart);
					// if we don't want to overwrite files, then if it already exists, skip it
					if (Options.HasFlag(CopyDirectoryOptions.KeepExistingFiles) && InternalUtils.SafeFileExists(DestPath, true))
					{
						continue;
					}
					if (!CopyFile_NoExceptions(SourceFile, DestPath, bQuiet))
					{
						return false;
					}
				}
			}

			return true;
		}

		/// <summary>
		/// Returns directory name without filename. 
		/// The difference between this and Path.GetDirectoryName is that this
		/// function will not throw away the last name if it doesn't have an extension, for example:
		/// D:\Project\Data\Asset -> D:\Project\Data\Asset
		/// D:\Project\Data\Asset.ussset -> D:\Project\Data
		/// </summary>
		/// <param name="FilePath"></param>
		/// <returns></returns>
		public static string GetDirectoryName(string FilePath)
		{
			var LastSeparatorIndex = Math.Max(FilePath.LastIndexOf('/'), FilePath.LastIndexOf('\\'));
			var ExtensionIndex = FilePath.LastIndexOf('.');
			if (ExtensionIndex > LastSeparatorIndex || LastSeparatorIndex == (FilePath.Length - 1))
			{
				return FilePath.Substring(0, LastSeparatorIndex);
			}
			else
			{
				return FilePath;
			}
		}

		/// <summary>
		/// Returns the last directory name in the path string.
		/// For example: D:\Temp\Project\File.txt -> Project, Data\Samples -> Samples
		/// </summary>
		/// <param name="FilePath"></param>
		/// <returns></returns>
		public static string GetLastDirectoryName(string FilePath)
		{
			var LastDir = GetDirectoryName(FilePath);
			var LastSeparatorIndex = Math.Max(LastDir.LastIndexOf('/'), LastDir.LastIndexOf('\\'));
			if (LastSeparatorIndex >= 0)
			{
				LastDir = LastDir.Substring(LastSeparatorIndex + 1);
			}
			return LastDir;
		}

		/// <summary>
		/// Removes multi-dot extensions from a filename (i.e. *.automation.csproj)
		/// </summary>
		/// <param name="Filename">Filename to remove the extensions from</param>
		/// <returns>Clean filename.</returns>
		public static string GetFilenameWithoutAnyExtensions(string Filename)
		{
			do
			{
				Filename = Path.GetFileNameWithoutExtension(Filename);
			}
			while (Filename.IndexOf('.') >= 0);
			return Filename;
		}

		/// <summary>
		/// Reads a file manifest and returns it
		/// </summary>
		/// <param name="ManifestFile">ManifestName</param>
		/// <returns></returns>
		public static BuildManifest ReadManifest(FileReference ManifestFile)
		{
			// Create a new default instance of the type
			BuildManifest Instance = new UnrealBuildTool.BuildManifest();
			XmlReader XmlStream = null;
			try
			{
				// Use the default reader settings if none are passed in
				XmlReaderSettings ReaderSettings = new XmlReaderSettings();
				ReaderSettings.CloseInput = true;
				ReaderSettings.IgnoreComments = true;

				// Get the xml data stream to read from
				XmlStream = XmlReader.Create(ManifestFile.FullName, ReaderSettings);

				// Create an object from the xml data
				Instance = (BuildManifest)BuildManifestSerializer.Deserialize(XmlStream);
			}
			catch (Exception Ex)
			{
				Debug.WriteLine(Ex.Message);
			}
			finally
			{
				if (XmlStream != null)
				{
					// Done with the file so close it
					XmlStream.Close();
				}
			}

			return Instance;
		}

		// statically create this to avoid exceptions while not leaking to much memory
		private static readonly XmlSerializer BuildManifestSerializer = XmlSerializer.FromTypes(new[] { typeof(UnrealBuildTool.BuildManifest) })[0];

		private static void CloneDirectoryRecursiveWorker(string SourcePathBase, string TargetPathBase, List<string> ClonedFiles, bool bIncremental = false)
		{
			bool bDirectoryCreated = InternalUtils.SafeCreateDirectory(TargetPathBase);
			if (!bIncremental && !bDirectoryCreated)
			{
				throw new AutomationException("Failed to create directory {0} for copy", TargetPathBase);
			}
			else if (bIncremental && !CommandUtils.DirectoryExists_NoExceptions(TargetPathBase))
			{
				throw new AutomationException("Target directory {0} does not exist", TargetPathBase);
			}

			DirectoryInfo SourceDirectory = new DirectoryInfo(SourcePathBase);
			DirectoryInfo[] SourceSubdirectories = SourceDirectory.GetDirectories();

			// Copy the files
			FileInfo[] SourceFiles = SourceDirectory.GetFiles();
			foreach (FileInfo SourceFI in SourceFiles)
			{
				string TargetFilename = CommandUtils.CombinePaths(TargetPathBase, SourceFI.Name);

				if (!bIncremental || !CommandUtils.FileExists(TargetFilename))
				{
					SourceFI.CopyTo(TargetFilename);

					if (ClonedFiles != null)
					{
						ClonedFiles.Add(TargetFilename);
					}
				}
			}

			// Recurse into subfolders
			foreach (DirectoryInfo SourceSubdir in SourceSubdirectories)
			{
				string NewSourcePath = CommandUtils.CombinePaths(SourcePathBase, SourceSubdir.Name);
				string NewTargetPath = CommandUtils.CombinePaths(TargetPathBase, SourceSubdir.Name);
				CloneDirectoryRecursiveWorker(NewSourcePath, NewTargetPath, ClonedFiles, bIncremental);
			}
		}

		/// <summary>
		/// Clones a directory.
		/// Warning: Will delete all of the existing files in TargetPath
		/// This is recursive, copying subfolders too.
		/// </summary>
		/// <param name="SourcePath">Source directory.</param>
		/// <param name="TargetPath">Target directory.</param>
		/// <param name="ClonedFiles">List of cloned files.</param>
		public static void CloneDirectory(string SourcePath, string TargetPath, List<string> ClonedFiles = null)
		{
			DeleteDirectory_NoExceptions(TargetPath);
			CloneDirectoryRecursiveWorker(SourcePath, TargetPath, ClonedFiles);
		}

		/// <summary>
		/// Clones a directory, skipping any files which already exist in the destination.
		/// This is recursive, copying subfolders too.
		/// </summary>
		/// <param name="SourcePath">Source directory.</param>
		/// <param name="TargetPath">Target directory.</param>
		/// <param name="ClonedFiles">List of cloned files.</param>
		public static void CloneDirectoryIncremental(string SourcePath, string TargetPath, List<string> ClonedFiles = null)
		{
			CloneDirectoryRecursiveWorker(SourcePath, TargetPath, ClonedFiles, bIncremental: true);
		}

		/// <summary>
		/// Copies files using multiple threads
		/// </summary>
		/// <param name="SourceDirectory"></param>
		/// <param name="DestDirectory"></param>
		/// <param name="MaxThreads"></param>
		public static void ThreadedCopyFiles(string SourceDirectory, string DestDirectory, int MaxThreads = 64)
		{
			CreateDirectory(DestDirectory);
			List<string> SourceFiles = Directory.EnumerateFiles(SourceDirectory, "*", SearchOption.AllDirectories).ToList();
			List<string> DestFiles = SourceFiles.Select(SourceFile => CommandUtils.MakeRerootedFilePath(SourceFile, SourceDirectory, DestDirectory)).ToList();
			ThreadedCopyFiles(SourceFiles, DestFiles, MaxThreads);
		}

		public static void ThreadedCopyFiles(DirectoryReference SourceDirectory, DirectoryReference DestDirectory, int MaxThreads = 64)
		{
			ThreadedCopyFiles(SourceDirectory.FullName, DestDirectory.FullName, MaxThreads);
		}

		/// <summary>
		/// Copies files using multiple threads
		/// </summary>
		/// <param name="SourceFiles">List of source files to copy</param>
		/// <param name="TargetFiles">List of target files</param>
		public static void ThreadedCopyFiles(List<FileReference> SourceFiles, List<FileReference> TargetFiles)
		{
			ThreadedCopyFiles(SourceFiles.ConvertAll(x => x.FullName), TargetFiles.ConvertAll(x => x.FullName));
		}

		/// <summary>
		/// Copies files using multiple threads
		/// </summary>
		/// <param name="SourceFiles">List of source files to copy</param>
		/// <param name="SourceDir">The source directory</param>
		/// <param name="TargetDir">The target directory</param>
		public static void ThreadedCopyFiles(List<FileReference> SourceFiles, DirectoryReference SourceDir, DirectoryReference TargetDir)
		{
			List<FileReference> TargetFiles = new List<FileReference>();
			foreach (FileReference SourceFile in SourceFiles)
			{
				if (!SourceFile.IsUnderDirectory(SourceDir))
				{
					throw new AutomationException($"Source file '{SourceFile}' is not under source directory '{SourceDir}'");
				}
				else
				{
					TargetFiles.Add(FileReference.Combine(TargetDir, SourceFile.MakeRelativeTo(SourceDir)));
				}
			}
			ThreadedCopyFiles(SourceFiles, TargetFiles);
		}

		/// <summary>
		/// Copies files using multiple threads
		/// </summary>
		/// <param name="Source"></param>
		/// <param name="Dest"></param>
		/// <param name="MaxThreads"></param>
		/// <param name="bQuiet"></param>
		/// <param name="bRetry"></param>
		public static void ThreadedCopyFiles(List<string> Source, List<string> Dest, int MaxThreads = 64, bool bQuiet = false, bool bRetry = false)
		{
			if (!bQuiet)
			{
				Logger.LogInformation("Copying {NumFiles} file(s) using max {MaxThreads} thread(s)", Source.Count, MaxThreads);
			}

			if (Source.Count != Dest.Count)
			{
				throw new AutomationException("Source count ({0}) does not match Dest count ({1})", Source.Count, Dest.Count);
			}
			Parallel.ForEach(Source.Zip(Dest, (Src, Dst) => new { SourceFile = Src, DestFile = Dst }), new ParallelOptions { MaxDegreeOfParallelism = MaxThreads }, (Pair) =>
			{
				CommandUtils.CopyFile(Pair.SourceFile, Pair.DestFile, true, bRetry);
			});
		}

		/// <summary>
		/// Copies a set of files from one folder to another
		/// </summary>
		/// <param name="SourceDir">Source directory</param>
		/// <param name="TargetDir">Target directory</param>
		/// <param name="RelativePaths">Paths relative to the source directory to copy</param>
		/// <param name="MaxThreads">Maximum number of threads to create</param>
		/// <param name="bRetry"></param>
		/// <returns>List of filenames copied to the target directory</returns>
		public static List<string> ThreadedCopyFiles(string SourceDir, string TargetDir, List<string> RelativePaths, int MaxThreads = 64, bool bRetry = false)
		{
			List<string> SourceFileNames = RelativePaths.Select(RelativePath => CommandUtils.CombinePaths(SourceDir, RelativePath)).ToList();
			List<string> TargetFileNames = RelativePaths.Select(RelativePath => CommandUtils.CombinePaths(TargetDir, RelativePath)).ToList();
			CommandUtils.ThreadedCopyFiles(SourceFileNames, TargetFileNames, MaxThreads, bRetry: bRetry);
			return TargetFileNames;
		}

		/// <summary>
		/// Copies a set of files from one folder to another
		/// </summary>
		/// <param name="SourceDir">Source directory</param>
		/// <param name="TargetDir">Target directory</param>
		/// <param name="Filter">Filter which selects files from the source directory to copy</param>
		/// <param name="bIgnoreSymlinks">Whether to ignore symlinks during the copy</param>
		/// <param name="MaxThreads">Maximum number of threads to create</param>
		/// <param name="bRetry"></param>
		/// <returns>List of filenames copied to the target directory</returns>
		public static List<string> ThreadedCopyFiles(string SourceDir, string TargetDir, FileFilter Filter, bool bIgnoreSymlinks, int MaxThreads = 64, bool bRetry = false)
		{
			// Filter all the relative paths
			Logger.LogInformation("Applying filter to {SourceDir}...", SourceDir);
			DirectoryReference SourceDirRef = new DirectoryReference(SourceDir);
			List<string> RelativePaths = Filter.ApplyToDirectory(SourceDirRef, bIgnoreSymlinks).Select(x => x.MakeRelativeTo(SourceDirRef)).ToList();
			return ThreadedCopyFiles(SourceDir, TargetDir, RelativePaths, MaxThreads, bRetry: bRetry);
		}

		/// <summary>
		/// Moves files in parallel
		/// </summary>
		/// <param name="SourceAndTargetPairs">Pairs of source and target files</param>
		public static void ParallelMoveFiles(IEnumerable<KeyValuePair<FileReference, FileReference>> SourceAndTargetPairs)
		{
			ParallelMoveFiles(SourceAndTargetPairs, false);
		}

		/// <summary>
		/// Moves files in parallel
		/// </summary>
		/// <param name="SourceAndTargetPairs">Pairs of source and target files</param>
		/// <param name="Overwrite">Whether or not to overwrite target files if they already exist</param>
		public static void ParallelMoveFiles(IEnumerable<KeyValuePair<FileReference, FileReference>> SourceAndTargetPairs, bool Overwrite)
		{
			try
			{
				Parallel.ForEach(SourceAndTargetPairs, x => MoveFile(x.Key, x.Value, Overwrite));
			}
			catch (AggregateException Ex)
			{
				throw new AutomationException(Ex, "Failed to thread-copy files.");
			}
		}

		/// <summary>
		/// Move a file from one place to another 
		/// </summary>
		/// <param name="SourceFile"></param>
		/// <param name="TargetFile"></param>
		public static void MoveFile(FileReference SourceFile, FileReference TargetFile)
		{
			MoveFile(SourceFile, TargetFile, false);
		}

		/// <summary>
		/// Move a file from one place to another 
		/// </summary>
		/// <param name="SourceFile"></param>
		/// <param name="TargetFile"></param>
		/// <param name="Overwrite"></param>
		public static void MoveFile(FileReference SourceFile, FileReference TargetFile, bool Overwrite)
		{
			// Create the directory for the target file
			try
			{
				Directory.CreateDirectory(TargetFile.Directory.FullName);
			}
			catch (Exception Ex)
			{
				throw new AutomationException(Ex, "Unable to create directory {0} while moving {1} to {2}", TargetFile.Directory, SourceFile, TargetFile);
			}

			// Move the file
			try
			{
				if (Overwrite && FileReference.Exists(TargetFile))
				{
					FileReference.Delete(TargetFile);
				}
				File.Move(SourceFile.FullName, TargetFile.FullName);
			}
			catch (Exception Ex)
			{
				throw new AutomationException(Ex, "Unable to move {0} to {1}", SourceFile, TargetFile);
			}
		}

		public enum SymlinkMode
		{
			Ignore,
			Follow,
			Retain,
		}

		// detemine how the OS wants to handle Symlinks when archiving, by default
		public static SymlinkMode DefaultSymlinkMode
		{
			get
			{
				return OperatingSystem.IsMacOS() ? SymlinkMode.Retain :
				(OperatingSystem.IsLinux() ? SymlinkMode.Ignore :
				SymlinkMode.Follow);
			}
		}

		/// <summary>
		/// Assumes directory has already been checked to exist, and that it is not a symlink itself
		/// The SymlinkMode here refers to directory links, as the file symlinks will be dealt with during the copy
		/// </summary>
		public static void FindFilesAndSymlinks(string RootPath, string Wildcard, bool bRecursive, SymlinkMode DirSymlinkMode, List<string> FoundFiles)
		{
			// default behaviour can just do standard findfiles
			if (DirSymlinkMode == SymlinkMode.Follow)
			{
				FoundFiles.AddRange(Directory.GetFiles(RootPath, Wildcard, bRecursive ? SearchOption.AllDirectories : SearchOption.TopDirectoryOnly));
			}
			else if (DirSymlinkMode == SymlinkMode.Retain || DirSymlinkMode == SymlinkMode.Ignore)
			{
				DirectoryInfo DirInfo = new DirectoryInfo(RootPath);

				// get all files in this dir
				FoundFiles.AddRange(Directory.GetFiles(RootPath, Wildcard, SearchOption.TopDirectoryOnly));

				// now walk over the directories, and recurse into true directories, and process the symlinks
				foreach (DirectoryInfo Dir in DirInfo.EnumerateDirectories(Wildcard, SearchOption.TopDirectoryOnly))
				{
					if (Dir.Attributes.HasFlag(FileAttributes.ReparsePoint))
					{
						// if mode is Ingore, then do nothing with it
						if (DirSymlinkMode == SymlinkMode.Retain)
						{
							// add the direcetory as a file since we are going to treat it as a file later
							FoundFiles.Add(Dir.FullName);
						}
					}
					else
					{
						FindFilesAndSymlinks(Dir.FullName, Wildcard, bRecursive, DirSymlinkMode, FoundFiles);
					}
				}
			}
		}

		public static void FindFilesAndSymlinks(string RootPath, string Wildcard, bool bRecursive, List<string> FoundFiles)
		{
			FindFilesAndSymlinks(RootPath, Wildcard, bRecursive, DefaultSymlinkMode, FoundFiles);
		}

		public static void CopyFileOrSymlink(FileReference Source, FileReference Dest, SymlinkMode SymlinkMode)
		{
			if (SymlinkMode == SymlinkMode.Retain || SymlinkMode == SymlinkMode.Ignore)
			{
				if (File.GetAttributes(Source.FullName).HasFlag(FileAttributes.ReparsePoint))
				{
					if (SymlinkMode == SymlinkMode.Retain)
					{
						if (OperatingSystem.IsWindows())
						{
							throw new AutomationException("Windows ReparsePoint copying is not supported at this time");
						}
						else
						{
							Logger.LogInformation("Retaining symlink {0} as {1}", Source, Dest);
							DirectoryReference.CreateDirectory(Dest.Directory);
							Utils.RunLocalProcessAndReturnStdOut("/usr/bin/env", $"cp -a \"{Source}\" \"{Dest}\"", null);
						}
					}
					else
					{
						Logger.LogInformation("Ignoring symlink {0}", Source);
					}
					return;
				}
			}

			// if we didn't handle a symlink above, just copy it normally
			InternalUtils.SafeCopyFile(Source.FullName, Dest.FullName, bSafeCreateDirectory: true);
		}

		public static void CopyFileOrSymlink(FileReference Source, FileReference Dest)
		{
			CopyFileOrSymlink(Source, Dest, DefaultSymlinkMode);
		}


		/// <summary>
		/// Gets environment variable value.
		/// </summary>
		/// <param name="Name">Name of the environment variable</param>
		/// <returns>Environment variable value as string.</returns>
		public static string GetEnvVar(string Name)
		{
			return InternalUtils.GetEnvironmentVariable(Name, "");
		}

		/// <summary>
		/// Gets environment variable value.
		/// </summary>
		/// <param name="Name">Name of the environment variable</param>
		/// <param name="DefaultValue">Default value of the environment variable if the variable is not set.</param>
		/// <returns>Environment variable value as string.</returns>
		public static string GetEnvVar(string Name, string DefaultValue)
		{
			return InternalUtils.GetEnvironmentVariable(Name, DefaultValue);
		}

		/// <summary>
		/// Sets environment variable.
		/// </summary>
		/// <param name="Name">Variable name.</param>
		/// <param name="Value">Variable value.</param>
		/// <returns>True if the value has been set, false otherwise.</returns>
		public static void SetEnvVar(string Name, object Value)
		{
			try
			{
				Logger.LogDebug("SetEnvVar {Name}={Value}", Name, Value);
				Environment.SetEnvironmentVariable(Name, Value.ToString());
			}
			catch (Exception Ex)
			{
				throw new AutomationException(Ex, "Failed to set environment variable {0} to {1}", Name, Value);
			}
		}

		/// <summary>
		/// Sets the environment variable if it hasn't been already.
		/// </summary>
		/// <param name="VarName">Environment variable name</param>
		/// <param name="Value">New value</param>
		public static void ConditionallySetEnvVar(string VarName, string Value)
		{
			if (String.IsNullOrEmpty(CommandUtils.GetEnvVar(VarName)))
			{
				Environment.SetEnvironmentVariable(VarName, Value);
			}
		}

		/// <summary>
		/// Parses the argument list for a parameter and returns whether it is defined or not.
		/// </summary>
		/// <param name="ArgList">Argument list.</param>
		/// <param name="Param">Param to check for.</param>
		/// <returns>True if param was found, false otherwise.</returns>
		public static bool ParseParam(string[] ArgList, string Param)
		{
			string ValueParam = Param;
			if (!ValueParam.EndsWith("="))
			{
				ValueParam += "=";
			}

			foreach (string ArgStr in ArgList)
			{
				if (ArgStr.Equals(Param, StringComparison.InvariantCultureIgnoreCase) || ArgStr.StartsWith(ValueParam, StringComparison.InvariantCultureIgnoreCase))
				{
					return true;
				}
			}
			return false;
		}

		/// <summary>
		/// Parses the argument list for a parameter and reads its value. 
		/// Ex. ParseParamValue(Args, "map=")
		/// </summary>
		/// <param name="ArgList">Argument list.</param>
		/// <param name="Param">Param to read its value.</param>
		/// <param name="Default"></param>
		/// <returns>Returns the value or Default if the parameter was not found.</returns>
		public static string ParseParamValue(object[] ArgList, string Param, string Default = null)
		{
			if (!Param.EndsWith("="))
			{
				Param += "=";
			}

			foreach (object Arg in ArgList)
			{
				string ArgStr = Arg.ToString();
				if (ArgStr.StartsWith(Param, StringComparison.InvariantCultureIgnoreCase))
				{
					return ArgStr.Substring(Param.Length);
				}
			}
			return Default;
		}

		/// <summary>
		/// Parses the argument list for any number of parameters.
		/// </summary>
		/// <param name="ArgList">Argument list.</param>
		/// <param name="Param">Param to read its value.</param>
		/// <returns>Returns an array of values for this parameter (or an empty array if one was not found.</returns>
		public static string[] ParseParamValues(object[] ArgList, string Param)
		{
			string ParamEquals = Param;
			if (!ParamEquals.EndsWith("="))
			{
				ParamEquals += "=";
			}

			List<string> Values = new List<string>();
			foreach (object Arg in ArgList)
			{
				string ArgStr = Arg.ToString();
				if (ArgStr.StartsWith(ParamEquals, StringComparison.InvariantCultureIgnoreCase))
				{
					Values.Add(ArgStr.Substring(ParamEquals.Length));
				}
			}
			return Values.ToArray();
		}

		/// <summary>
		/// Makes sure path can be used as a command line param (adds quotes if it contains spaces)
		/// </summary>
		/// <param name="InPath">Path to convert</param>
		/// <returns></returns>
		public static string MakePathSafeToUseWithCommandLine(string InPath)
		{
			return UnrealBuildTool.Utils.MakePathSafeToUseWithCommandLine(InPath);
		}

		public static string EscapePath(string InPath)
		{
			return InPath.Replace(":", "").Replace("/", "+").Replace("\\", "+").Replace(" ", "+");
		}

		/// <summary>
		/// Checks if collection is either null or empty.
		/// </summary>
		/// <param name="Collection">Collection to check.</param>
		/// <returns>True if the collection is either nur or empty.</returns>
		public static bool IsNullOrEmpty(ICollection Collection)
		{
			return Collection == null || Collection.Count == 0;
		}

		/// <summary>
		/// Checks if this command is running on a build machine.
		/// </summary>
		public static bool IsBuildMachine
		{
			get { return Automation.IsBuildMachine; }
		}

		/// <summary>
		/// Return the main engine directory and any platform extension engine directories
		/// </summary>
		public static DirectoryReference[] GetAllEngineDirectories()
		{
			List<DirectoryReference> EngineDirectories = new List<DirectoryReference>() { Unreal.EngineDirectory };
			DirectoryReference EnginePlatformsDirectory = DirectoryReference.Combine(Unreal.EngineDirectory, "Platforms");
			if (DirectoryReference.Exists(EnginePlatformsDirectory))
			{
				EngineDirectories.AddRange(DirectoryReference.EnumerateDirectories(EnginePlatformsDirectory).ToList());
			}

			return EngineDirectories.ToArray();
		}

		/// <summary>
		/// Return the main project directory and any platform extension project directories
		/// </summary>
		public static DirectoryReference[] GetAllProjectDirectories(FileReference ProjectFile)
		{
			List<DirectoryReference> ProjectDirectories = new List<DirectoryReference>() { ProjectFile.Directory };
			DirectoryReference ProjectPlatformsDirectory = DirectoryReference.Combine(ProjectFile.Directory, "Platforms");
			if (DirectoryReference.Exists(ProjectPlatformsDirectory))
			{
				ProjectDirectories.AddRange(DirectoryReference.EnumerateDirectories(ProjectPlatformsDirectory).ToList());
			}

			return ProjectDirectories.ToArray();
		}

		/// <summary>
		/// Telemetry data for the current run. Add -WriteTelemetry=[Path] to the command line to export to disk.
		/// </summary>
		public static TelemetryData Telemetry = new TelemetryData();

		/// <summary>
		/// Cached location of the build root storage because the logic to compute it is a little non-trivial.
		/// </summary>
		private static string CachedRootBuildStorageDirectory;

		/// <summary>
		/// "P:\Builds" or "/Volumes/Builds". Root Folder for all build storage.
		/// </summary>
		/// <returns>"P:\Builds" or "/Volumes/Builds" or "/mnt/Builds" unless overridden by -UseLocalBuildStorage from the commandline, where is uses Engine\Saved\LocalBuilds\.</returns>
		public static string RootBuildStorageDirectory()
		{
			if (string.IsNullOrEmpty(CachedRootBuildStorageDirectory))
			{
				if (GlobalCommandLine.UseLocalBuildStorage)
				{
					CachedRootBuildStorageDirectory = CombinePaths(CmdEnv.LocalRoot, "Engine", "Saved", "LocalBuilds");
					// Must create the directory because much of the system assumes it is already there.
					CreateDirectory(CombinePaths(CachedRootBuildStorageDirectory, "UE4"));
				}
				else
				{
					if (OperatingSystem.IsMacOS())
					{
						CachedRootBuildStorageDirectory = CombinePaths("/Volumes", "Builds");
					}
					else if (OperatingSystem.IsLinux())
					{
						CachedRootBuildStorageDirectory = CombinePaths("/mnt", "Builds");
					}
					else
					{
						CachedRootBuildStorageDirectory = CombinePaths("P:", "Builds");
					}
				}
			}
			return CachedRootBuildStorageDirectory;
		}

		public static bool DirectoryExistsAndIsWritable_NoExceptions(string Dir)
		{
			if (!DirectoryExists_NoExceptions(Dir))
			{
				Logger.LogDebug("Directory {Dir} does not exist", Dir);
				return false;
			}

			try
			{
				string Filename = CombinePaths(Dir, Guid.NewGuid().ToString() + ".Temp.txt");
				string NativeFilename = ConvertSeparators(PathSeparator.Default, Filename);
				using (StreamWriter Writer = new StreamWriter(NativeFilename))
				{
					Writer.Write("Test");
				}
				if (File.Exists(NativeFilename))
				{
					DeleteFile_NoExceptions(Filename, true);
					Logger.LogDebug("Directory {Dir} is writable", Dir);
					return true;
				}
			}
			catch (Exception)
			{
			}

			Logger.LogDebug("Directory {Dir} is not writable", Dir);
			return false;
		}

		public static void CleanFormalBuilds(string ParentDir, string SearchPattern, double MaximumDaysToKeepTempStorage = 4, int maxParallelDeletes = 4)
		{
			if (!IsBuildMachine || !ParentDir.StartsWith(RootBuildStorageDirectory()))
			{
				return;
			}
			DirectoryInfo DirInfo = new DirectoryInfo(ParentDir);
			Logger.LogInformation("Looking for directories to delete in {ParentDir}", ParentDir);
			List<DirectoryInfo> BuildsToClean = new List<DirectoryInfo>();
			foreach (DirectoryInfo ThisDirInfo in DirInfo.EnumerateDirectories(SearchPattern))
			{
				double AgeDays = (DateTime.UtcNow - ThisDirInfo.CreationTimeUtc).TotalDays;
				if (AgeDays > MaximumDaysToKeepTempStorage)
				{
					Logger.LogInformation("Marking formal build for delete directory {Directory}, because it is {Age} days old (maximum {MaximumDaysToKeepTempStorage}).", ThisDirInfo.FullName, AgeDays, MaximumDaysToKeepTempStorage);
					BuildsToClean.Add(new DirectoryInfo(ThisDirInfo.FullName));
				}
				else
				{
					Logger.LogDebug("Not deleting formal build directory {Directory}, because it is {Age} days old (maximum {MaximumDaysToKeepTempStorage}).", ThisDirInfo.FullName, AgeDays, MaximumDaysToKeepTempStorage);
				}
			}

			Parallel.ForEach(BuildsToClean, new ParallelOptions() { MaxDegreeOfParallelism = maxParallelDeletes }, (buildName, CancelToken) =>
			{
				try
				{
					Logger.LogInformation("Deleting {Path}", buildName.FullName);
					Stopwatch Timer = Stopwatch.StartNew();

					DeleteDirectory_NoExceptions(false, buildName.FullName);

					Timer.Stop();
					Logger.LogInformation("{Time} time spent deleting {Path}", Timer.Elapsed, buildName.FullName);
				}
				catch (System.Exception Ex)
				{
					Logger.LogWarning("Failed to delete {Path} due to error: {Exception}", buildName.FullName, LogUtils.FormatException(Ex));
					Logger.LogWarning(Ex, " Exception was {Exception}", LogUtils.FormatException(Ex));
				}
			});
		}


		/// <summary>
		/// Returns the generic name for a given platform (eg. "Windows" for Win32/Win64)
		/// </summary>
		/// <param name="Platform">Specific platform</param>
		public static string GetGenericPlatformName(UnrealBuildTool.UnrealTargetPlatform Platform)
		{
			if (Platform == UnrealTargetPlatform.Win64)
			{
				return "Windows";
			}
			else if (Platform == UnrealTargetPlatform.Linux || Platform == UnrealTargetPlatform.LinuxArm64)
			{
				return "Linux";
			}
			else
			{
				return Platform.ToString();
			}
		}

		/// <summary>
		/// Creates a zip file containing the given input files
		/// </summary>
		/// <param name="ZipFileName">Filename for the zip</param>
		/// <param name="Filter">Filter which selects files to be included in the zip</param>
		/// <param name="BaseDirectory">Base directory to store relative paths in the zip file to</param>
		public static void ZipFiles(FileReference ZipFileName, DirectoryReference BaseDirectory, FileFilter Filter)
		{
			List<FileReference> Files = Filter.ApplyToDirectory(BaseDirectory, true);
			ZipFiles(ZipFileName, BaseDirectory, Files);
		}

		/// <summary>
		/// Creates a zip file containing the given input files
		/// </summary>
		/// <param name="OutputFile">Filename for the zip</param>
		/// <param name="BaseDirectory">Base directory to store relative paths in the zip file to</param>
		/// <param name="Files">Files to include in the archive</param>
		/// <param name="ExecutableFiles">Files to flag with the executable attribute</param>
		public static void ZipFiles(FileReference OutputFile, DirectoryReference BaseDirectory, IEnumerable<FileReference> Files, HashSet<FileReference> ExecutableFiles = null)
		{
			if (!DirectoryReference.Exists(OutputFile.Directory))
			{
				DirectoryReference.CreateDirectory(OutputFile.Directory);
			}

			if (FileReference.Exists(OutputFile))
			{
				FileUtils.ForceDeleteFile(OutputFile);
			}

			using (ZipArchive ZipArchive = ZipFile.Open(OutputFile.FullName, ZipArchiveMode.Create))
			{
				foreach (FileReference File in Files)
				{
					string Name = File.MakeRelativeTo(BaseDirectory);
					if (Path.DirectorySeparatorChar != '/')
					{
						Name = Name.Replace(Path.DirectorySeparatorChar, '/');
					}

					ZipArchiveEntry entry = ZipArchive.CreateEntryFromFile_CrossPlatform(File.FullName, Name, CompressionLevel.Fastest);
					if (ExecutableFiles != null)
					{
						if (ExecutableFiles.Contains(File))
						{
							entry.ExternalAttributes |= 0b_111_111_101 << 16; // rwx rwx r-x
						}
						else
						{
							entry.ExternalAttributes |= 0b_110_110_100 << 16; // rw- rw- r--
						}
					}
				}
			}
		}

		/// <summary>
		/// Extracts the contents of a zip file
		/// </summary>
		/// <param name="ZipFileName">Name of the zip file</param>
		/// <param name="BaseDirectory">Output directory</param>
		/// <returns>List of files written</returns>
		public static IEnumerable<string> UnzipFiles(string ZipFileName, string BaseDirectory)
		{
			return UnzipFiles(new FileReference(ZipFileName), new DirectoryReference(BaseDirectory)).Select(x => x.FullName);
		}

		/// <summary>
		/// Extracts the contents of a zip file
		/// </summary>
		/// <param name="ZipFileName">Name of the zip file</param>
		/// <param name="BaseDirectory">Output directory</param>
		/// <param name="OverwriteFiles">Whether or not to overwrite files during unzip.</param>
		/// <returns>List of files written</returns>
		public static IEnumerable<FileReference> UnzipFiles(FileReference ZipFileName, DirectoryReference BaseDirectory, bool OverwriteFiles = true)
		{
			List<FileReference> OutputFiles = new List<FileReference>();

			if (!FileReference.Exists(ZipFileName))
			{
				throw new AutomationException("Cannot unzip {0}. File not found", ZipFileName);
			}

			using (ZipArchive ZipArchive = ZipFile.Open(ZipFileName.FullName, ZipArchiveMode.Read))
			{
				foreach (ZipArchiveEntry Entry in ZipArchive.Entries)
				{
					if (Entry.FullName.EndsWith("/"))
					{
						// ignore directories
						continue;
					}
					FileReference OutputFile = FileReference.Combine(BaseDirectory, Entry.FullName);
					if (OverwriteFiles || File.Exists(OutputFile.FullName) == false)
					{
						DirectoryReference.CreateDirectory(OutputFile.Directory);
						Entry.ExtractToFile_CrossPlatform(OutputFile.FullName, OverwriteFiles);
					}
					OutputFiles.Add(OutputFile);
				}
			}
			return OutputFiles;
		}

		/// <summary>
		/// Extracts the contents of a zip file
		/// </summary>
		/// <param name="ZipFileName">Name of the zip file</param>
		/// <param name="BaseDirectory">Output directory</param>
		/// <param name="OverwriteFiles">Whether or not to overwrite files during unzip.</param>
		/// <returns>List of files written</returns>
		public static IEnumerable<string> LegacyUnzipFiles(string ZipFileName, string BaseDirectory, bool OverwriteFiles = true)
		{
			List<string> OutputFileNames = new List<string>();
			if (!OperatingSystem.IsWindows())
			{
				CommandUtils.CreateDirectory(BaseDirectory);

				// -u  update files, create if necessary
				// -o  overwrite files WITHOUT prompting
				// -n  never overwrite existing files
				string OverwriteFileUnzipArg = (OverwriteFiles) ? "-u -o" : "-n";
				// Use system unzip tool as there have been instances of Ionic not being able to open zips created with Mac zip tool
				string Output = CommandUtils.RunAndLog("unzip", OverwriteFileUnzipArg + " \"" + ZipFileName + "\" -d \"" + BaseDirectory + "\"", Options: ERunOptions.Default | ERunOptions.SpewIsVerbose);

				// Split log output into lines
				string[] Lines = Output.Split(new char[] { '\n', '\r' });

				foreach (string LogLine in Lines)
				{
					Logger.LogInformation("{Text}", LogLine);

					// Split each line into two by whitespace
					string[] SplitLine = LogLine.Split(new char[] { ' ', '\t' }, 2, StringSplitOptions.RemoveEmptyEntries);
					if (SplitLine.Length == 2)
					{
						// Second part of line should be a path
						string FilePath = SplitLine[1].Trim();
						Logger.LogInformation("{Text}", FilePath);
						if (File.Exists(FilePath) && !OutputFileNames.Contains(FilePath) && FilePath != ZipFileName)
						{
							if (CommandUtils.IsProbablyAMacOrIOSExe(FilePath) || CommandUtils.IsProbablyALinuxExe(FilePath))
							{
								FixUnixFilePermissions(FilePath);
							}
							OutputFileNames.Add(FilePath);
						}
					}
				}
				if (OutputFileNames.Count == 0)
				{
					Logger.LogWarning("Unable to parse unzipped files from {ZipFileName}", ZipFileName);
				}
			}
			else
			{
				// Manually extract the files. Leaving this code as is in order to preserve behaviour around file permissions.
				using ZipArchive Zip = ZipFile.OpenRead(ZipFileName);
				FileMode OutputFileMode = (OverwriteFiles) ? FileMode.Create : FileMode.CreateNew;
				foreach (ZipArchiveEntry Entry in Zip.Entries.Where(e => !string.IsNullOrEmpty(e.Name)))
				{
					string OutputFileName = Path.Combine(BaseDirectory, Entry.FullName);
					if (OverwriteFiles || File.Exists(OutputFileName) == false)
					{
						Directory.CreateDirectory(Path.GetDirectoryName(OutputFileName));
						using FileStream OutputStream = new(OutputFileName, OutputFileMode, FileAccess.Write);
						using Stream EntryStream = Entry.Open();
						EntryStream.CopyTo(OutputStream);
					}
					OutputFileNames.Add(OutputFileName);
				}
			}
			return OutputFileNames;
		}

		/// <summary>
		/// Resolve an arbitrary file specification against a directory. May contain any number of p4 wildcard operators (?, *, ...).
		/// </summary>
		/// <param name="DefaultDir">Base directory for relative paths</param>
		/// <param name="Pattern">Pattern to match</param>
		/// <param name="ExcludePatterns">List of patterns to be excluded. May be null.</param>
		/// <returns>Sequence of file references matching the given pattern</returns>
		public static IEnumerable<FileReference> ResolveFilespec(DirectoryReference DefaultDir, string Pattern, IEnumerable<string> ExcludePatterns)
		{
			List<FileReference> Files = new List<FileReference>();

			// Check if it contains any wildcards. If not, we can just add the pattern directly without searching.
			int WildcardIdx = FileFilter.FindWildcardIndex(Pattern);
			if (WildcardIdx == -1)
			{
				// Construct a filter which removes all the excluded filetypes
				FileFilter Filter = new FileFilter(FileFilterType.Include);
				if (ExcludePatterns != null)
				{
					Filter.AddRules(ExcludePatterns, FileFilterType.Exclude);
				}

				// Match it against the given file
				FileReference File = FileReference.Combine(DefaultDir, Pattern);
				if (Filter.Matches(File.FullName))
				{
					Files.Add(File);
				}
			}
			else
			{
				// Find the base directory for the search. We construct this in a very deliberate way including the directory separator itself, so matches
				// against the OS root directory will resolve correctly both on Mac (where / is the filesystem root) and Windows (where / refers to the current drive).
				int LastDirectoryIdx = Pattern.LastIndexOfAny(new char[] { Path.DirectorySeparatorChar, Path.AltDirectorySeparatorChar }, WildcardIdx);
				DirectoryReference BaseDir = DirectoryReference.Combine(DefaultDir, Pattern.Substring(0, LastDirectoryIdx + 1));

				// Construct the absolute include pattern to match against, re-inserting the resolved base directory to construct a canonical path.
				string IncludePattern = BaseDir.FullName.TrimEnd(new char[] { Path.DirectorySeparatorChar, Path.AltDirectorySeparatorChar }) + "/" + Pattern.Substring(LastDirectoryIdx + 1);

				// Construct a filter and apply it to the directory
				if (DirectoryReference.Exists(BaseDir))
				{
					FileFilter Filter = new FileFilter();
					Filter.AddRule(IncludePattern, FileFilterType.Include);
					if (ExcludePatterns != null)
					{
						Filter.AddRules(ExcludePatterns, FileFilterType.Exclude);
					}
					Files.AddRange(Filter.ApplyToDirectory(BaseDir, BaseDir.FullName, true));
				}
			}
			return Files;
		}

		public static string FormatSizeString(long Size)
		{
			string[] Units = { "bytes", "KB", "MB", "GB", "TB", "PB", "EB", "ZB", "YB" };

			int UnitIndex = 0;
			double AbsSize = Math.Abs(Size);
			while (AbsSize >= 1024)
			{
				AbsSize /= 1024;
				++UnitIndex;
			}

			return String.Format("{0} {1}", AbsSize.ToString("N2"), Units[UnitIndex]);
		}

		public static void TakeLock(DirectoryReference LockDirectory, TimeSpan Timeout, System.Action Callback)
		{
			string LockFilePath = Path.Combine(LockDirectory.FullName, ".lock");

			FileStream Stream = null;
			DateTime StartTime = DateTime.Now;
			DateTime Deadline = StartTime.Add(Timeout);

			try
			{
				DirectoryReference.CreateDirectory(LockDirectory);

				for (int Iterations = 0; ; ++Iterations)
				{
					// Attempt to create the lock file. Ignore any IO exceptions. Stream will be null if this fails.
					try { Stream = new FileStream(LockFilePath, FileMode.Create, FileAccess.Write, FileShare.Read, 4096, FileOptions.DeleteOnClose); }
					catch (IOException) { }

					if (Stream != null)
					{
						// If we have a stream, we've taken the lock.
						try
						{
							// Write the machine name to the file.
							Stream.Write(Encoding.UTF8.GetBytes(Unreal.MachineName));
							Stream.Flush();
							break;
						}
						catch
						{
							throw new AutomationException("Failed to write to the lock file '{0}'.", LockFilePath);
						}
					}

					// We've failed to take the lock. Throw an exception if the timeout has elapsed.
					// Otherwise print a log message and retry.
					DateTime CurrentTime = DateTime.Now;
					if (CurrentTime >= Deadline)
					{
						throw new AutomationException("Couldn't create lock file '{0}' after {1} seconds.", LockFilePath, CurrentTime.Subtract(StartTime).TotalSeconds);
					}

					if (Iterations == 0)
					{
						Logger.LogInformation("Waiting for lock file '{LockFilePath}' to be removed...", LockFilePath);
					}
					else if ((Iterations % 30) == 0)
					{
						Logger.LogInformation("Still waiting for lock file '{LockFilePath}' after {Seconds} seconds.", LockFilePath, CurrentTime.Subtract(StartTime).TotalSeconds);
					}

					// Wait for a while before retrying.
					Thread.Sleep(1000);
				}

				// Invoke the user callback now that we own the lock.
				Callback();
			}
			finally
			{
				// Always dispose the lock file stream if we took the lock.
				// The file will delete on close.
				if (Stream != null)
				{
					Stream.Dispose();
					Stream = null;
				}
			}
		}
		public static void OptionallyTakeLock(bool Condition, DirectoryReference LockDirectory, TimeSpan Timeout, System.Action Callback)
		{
			if (Condition)
			{
				TakeLock(LockDirectory, Timeout, Callback);
			}
			else
			{
				// No lock required, invoke the callback directly.
				Callback();
			}
		}

		public static FileReference FindToolInPath(string ToolName)
		{
			string PathVariable = Environment.GetEnvironmentVariable("PATH");
			foreach (string PathEntry in PathVariable.Split(Path.PathSeparator))
			{
				try
				{
					DirectoryReference PathDir = new DirectoryReference(PathEntry);
					if (HostPlatform.Current.HostEditorPlatform == UnrealTargetPlatform.Win64)
					{
						FileReference ToolFile = FileReference.Combine(PathDir, $"{ToolName}.exe");
						if (FileReference.Exists(ToolFile))
						{
							return ToolFile;
						}
					}
					else
					{
						FileReference ToolFile = FileReference.Combine(PathDir, ToolName);
						if (FileReference.Exists(ToolFile))
						{
							return ToolFile;
						}
					}
				}
				catch
				{
				}
			}
			return null;
		}
	}

	/// <summary>
	/// Valid units for telemetry data
	/// </summary>
	public enum TelemetryUnits
	{
		Count,
		Milliseconds,
		Seconds,
		Minutes,
		Bytes,
		Megabytes,
		Percentage,
	}

	/// <summary>
	/// Sample for telemetry data
	/// </summary>
	public class TelemetrySample
	{
		/// <summary>
		/// Name of this sample
		/// </summary>
		public string Name;

		/// <summary>
		/// The value for this sample
		/// </summary>
		public double Value;

		/// <summary>
		/// Units for this sample
		/// </summary>
		public TelemetryUnits Units;
	}

	/// <summary>
	/// Stores a set of key/value telemetry samples which can be read and written to a JSON file
	/// </summary>
	public class TelemetryData
	{
		/// <summary>
		/// The current file version
		/// </summary>
		const int CurrentVersion = 1;

		/// <summary>
		/// Maps from a sample name to its value
		/// </summary>
		public List<TelemetrySample> Samples = new List<TelemetrySample>();

		/// <summary>
		/// Adds a telemetry sample
		/// </summary>
		/// <param name="Name">Name of the sample</param>
		/// <param name="Value">Value of the sample</param>
		/// <param name="Units">Units for the sample value</param>
		public void Add(string Name, double Value, TelemetryUnits Units)
		{
			Samples.RemoveAll(x => x.Name == Name);
			Samples.Add(new TelemetrySample() { Name = Name, Value = Value, Units = Units });
			Logger.LogDebug("Added telemetry value: {Name} = {Value} ({Units})", Name, Value, Units);
		}

		/// <summary>
		/// Add samples from another telemetry block into this one
		/// </summary>
		/// <param name="Prefix">Prefix for the telemetry data</param>
		/// <param name="Other">The other telemetry data</param>
		public void Merge(string Prefix, TelemetryData Other)
		{
			foreach (TelemetrySample Sample in Other.Samples)
			{
				Add(Sample.Name, Sample.Value, Sample.Units);
			}
		}

		/// <summary>
		/// Tries to read the telemetry data from the given file
		/// </summary>
		/// <param name="FileName">The file to read from</param>
		/// <param name="Telemetry">On success, the read telemetry data</param>
		/// <returns>True if a telemetry object was read</returns>
		public static bool TryRead(FileReference FileName, out TelemetryData Telemetry)
		{
			// Try to read the raw json object
			JsonObject RawObject;
			if (!JsonObject.TryRead(FileName, out RawObject))
			{
				Telemetry = null;
				return false;
			}

			// Check the version is valid
			int Version;
			if (!RawObject.TryGetIntegerField("Version", out Version) || Version != CurrentVersion)
			{
				Telemetry = null;
				return false;
			}

			// Read all the samples
			JsonObject[] RawSamples;
			if (!RawObject.TryGetObjectArrayField("Samples", out RawSamples))
			{
				Telemetry = null;
				return false;
			}

			// Parse out all the samples
			Telemetry = new TelemetryData();
			foreach (JsonObject RawSample in RawSamples)
			{
				Telemetry.Add(RawSample.GetStringField("Name"), RawSample.GetDoubleField("Value"), RawSample.GetEnumField<TelemetryUnits>("Units"));
			}
			return true;
		}

		/// <summary>
		/// Writes out the telemetry data to a file
		/// </summary>
		/// <param name="FileName"></param>
		public void Write(string FileName)
		{
			Logger.LogDebug("Writing telemetry to {FileName}...", FileName);
			using (JsonWriter Writer = new JsonWriter(FileName))
			{
				Writer.WriteObjectStart();
				Writer.WriteValue("Version", CurrentVersion);
				Writer.WriteArrayStart("Samples");
				foreach (TelemetrySample Sample in Samples)
				{
					Writer.WriteObjectStart();
					Writer.WriteValue("Name", Sample.Name);
					Writer.WriteValue("Value", Sample.Value);
					Writer.WriteValue("Units", Sample.Units.ToString());
					Writer.WriteObjectEnd();
				}
				Writer.WriteArrayEnd();
				Writer.WriteObjectEnd();
			}
		}
	}

	/// <summary>
	/// Timer class used for telemetry reporting.
	/// </summary>
	public class TelemetryStopwatch : IDisposable
	{
		string Name;
		DateTime StartTime;
		bool bFinished;

		[StringFormatMethod("Format")]
		public TelemetryStopwatch(string Format, params object[] Args)
		{
			Name = String.Format(Format, Args);
			StartTime = DateTime.Now;
		}

		public void Cancel()
		{
			bFinished = true;
		}

		/// <summary>
		/// Flushes the time to <see cref="CommandEnvironment.CSVFile"/> if we are the build machine and that environment variable is specified.
		/// Call Finish manually with an alternate name to use that one instead. Useful for dynamically generated names that you can't specify at construction.
		/// </summary>
		/// <param name="AlternateName">Used in place of the Name specified during construction.</param>
		public void Finish(string AlternateName = null)
		{
			if (!bFinished)
			{
				if (!String.IsNullOrEmpty(AlternateName))
				{
					Name = AlternateName;
				}

				var OutputStr = $"UAT,{Name},{StartTime},{DateTime.Now}" + Environment.NewLine;
				Logger.LogDebug("{CSVHeader}", OutputStr);
				if (CommandUtils.IsBuildMachine && !String.IsNullOrEmpty(CommandUtils.CmdEnv.CSVFile) && CommandUtils.CmdEnv.CSVFile != "nul")
				{
					try
					{
						File.AppendAllText(CommandUtils.CmdEnv.CSVFile, OutputStr);
					}
					catch (Exception Ex)
					{
						Logger.LogWarning("Could not append to csv file ({File}) : {Exception}", CommandUtils.CmdEnv.CSVFile, Ex.ToString());
					}
				}
			}
			bFinished = true;
		}

		public void Dispose()
		{
			Finish();
		}
	}

	/// <summary>
	/// Stopwatch that uses DateTime.UtcNow for timing. Not hi-res, but also not subject to short time limitations of System.Diagnostics.Stopwatch.
	/// </summary>
	public class DateTimeStopwatch
	{
		public static DateTimeStopwatch Start()
		{
			return new DateTimeStopwatch();
		}

		/// <summary>
		/// Hide public ctor.
		/// </summary>
		private DateTimeStopwatch() { }

		readonly DateTime StartTime = DateTime.UtcNow;

		public TimeSpan ElapsedTime { get { return DateTime.UtcNow - StartTime; } }
	}

	/// <summary>
	/// Use with "using" syntax to push and pop directories in a convenient, exception-safe way
	/// </summary>
	public class PushedDirectory : IDisposable
	{
		public PushedDirectory(string DirectoryName)
		{
			CommandUtils.PushDir(DirectoryName);
		}

		public void Dispose()
		{
			CommandUtils.PopDir();
			GC.SuppressFinalize(this);
		}
	}

	/// <summary>
	/// Use with "using" syntax to temporarily set and environment variable  in a convenient, exception-safe way
	/// </summary>
	public class ScopedEnvVar : IDisposable
	{
		private string StoredEnvVar = null;

		public ScopedEnvVar(string EnvVar, string Value)
		{
			StoredEnvVar = EnvVar;
			CommandUtils.SetEnvVar(StoredEnvVar, Value);
		}

		public void Dispose()
		{
			CommandUtils.SetEnvVar(StoredEnvVar, "");
			GC.SuppressFinalize(this);
		}
	}

	/// <summary>
	/// Helper class to associate a file and its contents
	/// </summary>
	public class EMSFileInfo
	{
		public string FileName { get; set; }
		public byte[] Bytes { get; set; }
	}

	/// <summary>
	/// Wrapper class for the enumerate files JSON response from MCP
	/// </summary>
	[DataContract]
	public sealed class EnumerationResponse
	{
		[DataMember(Name = "doNotCache", IsRequired = true)]
		public Boolean DoNotCache { get; set; }

		[DataMember(Name = "uniqueFilename", IsRequired = true)]
		public string UniqueFilename { get; set; }

		[DataMember(Name = "filename", IsRequired = true)]
		public string Filename { get; set; }

		[DataMember(Name = "hash", IsRequired = true)]
		public string Hash { get; set; }

		[DataMember(Name = "length", IsRequired = true)]
		public long Length { get; set; }

		[DataMember(Name = "uploaded", IsRequired = true)]
		public string Uploaded { get; set; }
	}

	static class CodeSignWindows
	{
		/// <summary>
		/// The signing identity to find a certificate for
		/// </summary>
		const string SigningIdentity = "Epic Games";

		/// <summary>
		/// Whether to look in the machine store for signing certificates, rather than the user store.
		/// </summary>
		static bool bUseMachineStoreForCertificates = true;

		public enum SignatureType
		{
			SHA1,
			SHA256
		};

		// List of known / valid timestamp servers: https://gist.github.com/Manouchehri/fd754e402d98430243455713efada710
		static readonly string[] TimestampServersSHA1 =
		{
			"http://timestamp.digicert.com",
			"http://ts.ssl.com",
			"http://kstamp.keynectis.com/KSign"
		};

		static readonly string[] TimestampServersSHA256 =
		{
			"http://timestamp.digicert.com",
			"http://ts.ssl.com",
			"http://rfc3161timestamp.globalsign.com/advanced"
		};

		static readonly int LongestSHA1 = TimestampServersSHA1.Select(x => x.Length).Max();
		static readonly int LongestSHA256 = TimestampServersSHA1.Select(x => x.Length).Max();

		[SupportedOSPlatform("windows")]
		public static void Sign(FileReference File, SignatureType SignatureType, bool AllowMultipleSignatures = true, string Description = null, bool RunInParallel = false)
		{
			List<FileReference> Files = new List<FileReference> { File };
			Sign(Files, SignatureType, AllowMultipleSignatures, Description, RunInParallel);
		}

		[SupportedOSPlatform("windows")]
		public static void Sign(List<FileReference> Files, SignatureType SignatureType, bool AllowMultipleSignatures = true, string Description = null, bool RunInParallel = false)
		{
			const int MaxAttempts = 10;
			string SignToolPath = GetSignToolPath();
			string DescriptionArg = String.IsNullOrEmpty(Description) ? "" : $"/d \"{Description}\"";
			string SpecificStoreArg = bUseMachineStoreForCertificates ? " /sm" : "";
			string MultipleSignatureArg = AllowMultipleSignatures ? " /as" : "";
			string SHA1TimestampArg = " /tr";
			string SHA256TimestampArg = " /tr";

			ICodeSignHelper SigningHelper = CodeSignHelper.FindHelper(SignToolPath);

			SigningHelper?.Validate(Logger);

			List<Task> SignTaskList = new List<Task>();

			for (int FileIdx = 0; FileIdx < Files.Count;)
			{
				//@TODO: Verbosity choosing
				//  /v will spew lots of info
				//  /q does nothing on success and minimal output on failure
				StringBuilder CommandLine = new StringBuilder();
				StringBuilder FileList = new StringBuilder();
				if (SignatureType == SignatureType.SHA1)
				{
					CommandLine.Append($"sign{SpecificStoreArg} /a /fd sha1 /td sha256 /n \"{SigningIdentity}\" /v {MultipleSignatureArg} {DescriptionArg}");
				}
				else if (SignatureType == SignatureType.SHA256)
				{
					CommandLine.Append($"sign{SpecificStoreArg} /a /fd sha256 /td sha256 /as /n \"{SigningIdentity}\" {DescriptionArg}");
				}
				else
				{
					throw new ArgumentException($"Invalid signature type type ({SignatureType})");
				}

				// Append the files for this batch
				// per: https://docs.microsoft.com/en-us/dotnet/api/system.diagnostics.process.start?view=net-6.0#system-diagnostics-process-start(system-diagnostics-processstartinfo)
				// The length of the application + command line arguments cannot exceed 2080 characters, otherwise a Win32Exception will be thrown.
				int NextFileIdx = FileIdx;
				int FileCount = 0;
				int TimeStampLength = SignatureType == SignatureType.SHA1 ? SHA1TimestampArg.Length + LongestSHA1 : SHA256TimestampArg.Length + LongestSHA256;
				while (NextFileIdx < Files.Count && (SignToolPath.Length + TimeStampLength + CommandLine.Length + FileList.Length + Files[NextFileIdx].FullName.Length < 2080))
				{
					FileList.Append($" \"{Files[NextFileIdx]}\"");
					NextFileIdx++;
					FileCount++;
				}

				Action SignToolAction = new Action(() =>
				{
					double TaskTimerLimit = 180.0;
					Stopwatch TaskTimer = Stopwatch.StartNew();
					for (int Attempts = 0; ; ++Attempts)
					{
						StringBuilder FinalCommandline = new StringBuilder();
						FinalCommandline.Append(CommandLine);

						if (SignatureType == SignatureType.SHA1)
						{
							FinalCommandline.Append($"{SHA1TimestampArg} {TimestampServersSHA1[Attempts % TimestampServersSHA1.Length]} ");
						}
						else
						{
							FinalCommandline.Append($"{SHA256TimestampArg} {TimestampServersSHA256[Attempts % TimestampServersSHA256.Length]} ");
						}

						FinalCommandline.Append(FileList);


						IProcessResult Result = CommandUtils.Run(SignToolPath, FinalCommandline.ToString(), null, CommandUtils.ERunOptions.AllowSpew);

						if (Result.ExitCode != 1)
						{
							if (Result.ExitCode == 2)
							{
								Logger.LogWarning(KnownLogEvents.AutomationTool_CodeSign, "Signtool returned a warning.");
							}
							// Success!
							break;
						}

						if (SigningHelper != null && !SigningHelper.IsHealthy(Result, Logger))
						{
							Stopwatch FixTimer = Stopwatch.StartNew();

							SigningHelper.Fix(Attempts, Logger);

							//  Push out the max time.
							TaskTimerLimit += FixTimer.Elapsed.TotalSeconds;
						}

						if (TaskTimer.Elapsed.TotalSeconds > TaskTimerLimit && Attempts > MaxAttempts)
						{
							throw new AutomationException($"Failed to sign files {Attempts} times over a period of {TaskTimer.Elapsed}");
						}
					}
				});

				FileIdx = NextFileIdx;

				SignTaskList.Add(new Task(SignToolAction));
			}

			// If running in parallel, Limit to between 4 and 16 concurrent. Otherwise 1 instance.
			int MaxParallelism = RunInParallel ? Math.Max(16, Math.Min(Environment.ProcessorCount, 4)) : 1;
			Logger.LogInformation(KnownLogEvents.AutomationTool_CodeSign, "Running {Count}, {Concurrent} max concurrent, signtool instances", SignTaskList.Count, MaxParallelism);
			Parallel.ForEach(SignTaskList, new ParallelOptions() { MaxDegreeOfParallelism = MaxParallelism }, x => { x.Start(); x.Wait(); });
		}

		/// <summary>
		/// Finds the path to SignTool.exe, or throws an exception.
		/// </summary>
		/// <returns>Path to signtool.exe</returns>
		[SupportedOSPlatform("windows")]
		static string GetSignToolPath()
		{
			List<KeyValuePair<string, DirectoryReference>> WindowsSdkDirs = WindowsExports.GetWindowsSdkDirs();
			foreach (KeyValuePair<string, DirectoryReference> Pair in WindowsSdkDirs)
			{
				FileReference SignToolFile = FileReference.Combine(Pair.Value, "bin", Pair.Key, "x64", "SignTool.exe");
				if (FileReference.Exists(SignToolFile))
				{
					return SignToolFile.FullName;
				}
			}

			string[] PossibleSignToolNames =
			{
				"C:/Program Files (x86)/Windows Kits/8.1/bin/x86/SignTool.exe",
				"C:/Program Files (x86)/Windows Kits/10/bin/x86/SignTool.exe"
			};

			foreach (string PossibleSignToolName in PossibleSignToolNames)
			{
				if (File.Exists(PossibleSignToolName))
				{
					return PossibleSignToolName;
				}
			}

			throw new AutomationException("SignTool not found at '{0}' (are you missing the Windows SDK?)", String.Join("' or '", PossibleSignToolNames));
		}
	}

	public class CodeSignHelper
	{
		private static ICodeSignHelper HelperClass = null;

		public static ICodeSignHelper FindHelper(string SignToolPath)
		{
			if (HelperClass == null)
			{
				// Find and load the first helper class
				Assembly[] LoadedAssemblies = AppDomain.CurrentDomain.GetAssemblies();
				foreach (Assembly Dll in LoadedAssemblies)
				{
					Type[] AllTypes = Dll.SafeGetLoadedTypes();
					foreach (Type PotentialConfigType in AllTypes.Where(x => !x.IsAbstract && x != typeof(ICodeSignHelper) && typeof(ICodeSignHelper).IsAssignableFrom(x)))
					{
						try
						{
							ICodeSignHelper NewHelper = Activator.CreateInstance(PotentialConfigType) as ICodeSignHelper;
							if (NewHelper != null)
							{
								NewHelper.Initialize(SignToolPath);
								HelperClass = NewHelper;
								break;
							}
						}
						catch (Exception Ex)
						{
							Exception Inner = Ex.InnerException;
							while (null != Inner)
							{
								Logger.LogWarning("Exception encountered creating HelperClass [{Name}] with error: {Message}", PotentialConfigType.Name, Inner.Message);
								Inner = Inner.InnerException;
							}
							Logger.LogWarning("Unable to create HelperClass [{Name}] with error: {Message} \n {Trace}", PotentialConfigType.Name, Ex.Message, Ex.StackTrace);
						}

						if (HelperClass != null)
						{
							break;
						}
					}
				}
			}

			return HelperClass;
		}
	}

	/// <summary>
	/// Interface helper class handle signing as a service environments
	/// 
	/// Implement a custom class for any provides
	/// </summary>
	public interface ICodeSignHelper
	{
		public void Initialize(string SignToolPath);
		public bool Validate(ILogger Logger);
		public bool IsHealthy(IProcessResult Result, ILogger Logger);
		public bool IsFixing();
		public bool Fix(int SignAttempt, ILogger Logger);
	}

	/// <summary>
	/// Code signing
	/// </summary>
	[Help("NoSign", "Skips signing of code/content files.")]
	public class CodeSign
	{
		/// <summary>
		/// Code signs the specified file
		/// </summary>
		public static void SignSingleExecutableIfEXEOrDLL(string Filename, bool bIgnoreExtension = false, string Description = null)
		{
			if (!OperatingSystem.IsWindows())
			{
				Logger.LogDebug(KnownLogEvents.AutomationTool_CodeSign, "Can't sign '{File}' on non-Windows platform.", Filename);
				return;
			}
			if (!CommandUtils.FileExists(Filename))
			{
				throw new AutomationException("Can't sign '{0}', file does not exist.", Filename);
			}
			// Make sure the file isn't read-only
			FileInfo TargetFileInfo = new FileInfo(Filename);

			// Executable extensions
			List<string> Extensions = new List<string>
			{
				".dll",
				".exe"
			};

			bool IsExecutable = bIgnoreExtension;

			foreach (var Ext in Extensions)
			{
				if (TargetFileInfo.FullName.EndsWith(Ext, StringComparison.InvariantCultureIgnoreCase))
				{
					IsExecutable = true;
					break;
				}
			}
			if (!IsExecutable)
			{
				Logger.LogDebug(KnownLogEvents.AutomationTool_CodeSign, "Won't sign '{File}', not an executable.", TargetFileInfo.FullName);
				return;
			}

			TargetFileInfo.IsReadOnly = false;

			// MSI files can only have one signature; prefer SHA1 for compatibility, so don't run SHA256 on msi files.
			if (!TargetFileInfo.FullName.EndsWith(".msi", StringComparison.InvariantCultureIgnoreCase))
			{
				CodeSignWindows.Sign(new FileReference(TargetFileInfo), CodeSignWindows.SignatureType.SHA256, Description: Description);
			}
			else
			{
				CodeSignWindows.Sign(new FileReference(TargetFileInfo), CodeSignWindows.SignatureType.SHA1, AllowMultipleSignatures: false, Description: Description);
			}
		}

		/// <summary>
		/// Code signs the specified file or folder
		/// </summary>
		public static void SignMacFileOrFolder(string InPath, bool bIgnoreExtension = false, string NotarizationEntitlements = null)
		{
			TimeSpan CodeSignTimeOut = new TimeSpan(0, 3, 0); // Keep trying to sign one file for up to 3 minutes
			bool bIsDirectory = CommandUtils.DirectoryExists(InPath);
			bool bExists = CommandUtils.FileExists(InPath) || bIsDirectory;
			if (!bExists)
			{
				throw new AutomationException("Can't sign '{0}', file or folder does not exist.", InPath);
			}

			if (string.IsNullOrEmpty(NotarizationEntitlements))
			{
				string EntitlementsPath = Path.Combine(Path.GetDirectoryName(InPath), "Notarization.entitlements");
				if (File.Exists(EntitlementsPath))
				{
					NotarizationEntitlements = EntitlementsPath;
				}
			}

			if (bIsDirectory && !InPath.EndsWith(".app/Contents/_CodeSignature"))
			{
				DirectoryReference PathRef = new DirectoryReference(InPath);
				foreach (DirectoryReference DirRef in DirectoryReference.EnumerateDirectories(PathRef, "*", SearchOption.TopDirectoryOnly))
				{
					SignMacFileOrFolder(DirRef.FullName, bIgnoreExtension, NotarizationEntitlements);
				}

				foreach (FileReference FileRef in DirectoryReference.EnumerateFiles(PathRef, "*", SearchOption.TopDirectoryOnly))
				{
					if (FileRef.GetExtension() == ".dylib" || FileRef.GetExtension() == ".so" || FileRef.GetExtension() == ".bundle")
					{
						SignMacFileOrFolder(FileRef.FullName, bIgnoreExtension, NotarizationEntitlements);
					}
				}
			}

			// Executable extensions
			List<string> Extensions = new List<string>
			{
				".dylib",
				".so",
				".app",
				".framework",
				".bundle"
			};

			bool bIsExecutable = bIgnoreExtension || (!bIsDirectory && Path.GetExtension(InPath) == "" && !InPath.EndsWith("PkgInfo"));

			foreach (var Ext in Extensions)
			{
				if (InPath.EndsWith(Ext, StringComparison.InvariantCultureIgnoreCase))
				{
					bIsExecutable = true;
					break;
				}
			}
			if (!bIsExecutable)
			{
				if (!bIsDirectory)
				{
					Logger.LogDebug(KnownLogEvents.AutomationTool_CodeSign, "Won't sign '{File}', not an executable.", InPath);
				}
				return;
			}

			// Use the old codesigning tool after the upgrade due to segmentation fault on Sierra
			string SignToolName = "/usr/local/bin/codesign_old";

			// unless it doesn't exist or we're codesigning for notarization, then use the latest one.
			if (!File.Exists(SignToolName) || !string.IsNullOrEmpty(NotarizationEntitlements))
			{
				SignToolName = "/usr/bin/codesign";
			}


			string CodeSignArgs;
			if (string.IsNullOrEmpty(NotarizationEntitlements))
			{
				CodeSignArgs = String.Format("-f --deep -s \"{0}\" -v \"{1}\" --no-strict", "Developer ID Application", InPath);
			}
			else
			{
				CodeSignArgs = String.Format("-f --deep -s \"{0}\" -o runtime --entitlements \"{1}\" -v \"{2}\" --timestamp --no-strict", "Developer ID Application", NotarizationEntitlements, InPath);
			}

			DateTime StartTime = DateTime.Now;

			int NumTrials = 0;
			for (; ; )
			{
				IProcessResult Result = CommandUtils.Run(SignToolName, CodeSignArgs, null, CommandUtils.ERunOptions.AllowSpew);
				int ExitCode = Result.ExitCode;
				++NumTrials;

				if (ExitCode == 0)
				{
					// Success!
					break;
				}
				else
				{
					// Keep retrying until we run out of time
					TimeSpan RunTime = DateTime.Now - StartTime;
					if (RunTime > CodeSignTimeOut)
					{
						throw new AutomationException("Failed to sign '{0}' {1} times over a period of {2}", InPath, NumTrials, RunTime);
					}
				}
			}
		}

		/// <summary>
		/// Codesigns multiple files, but skips anything that's not an EXE or DLL file
		/// Will automatically skip signing if -NoSign is specified in the command line.
		/// </summary>
		/// <param name="Command"></param>
		/// <param name="Files">List of files to sign</param>
		/// <param name="Description"></param>
		/// <param name="bRunInParallel">Run the sign operation in paralle, Windows only</param>
		public static void SignMultipleIfEXEOrDLL(BuildCommand Command, IEnumerable<string> Files, string Description = null, bool bRunInParallel = false)
		{
			if (!Command.ParseParam("NoSign"))
			{
				Logger.LogInformation(KnownLogEvents.AutomationTool_CodeSign, "Signing up to {FileCount} files...", Files.Count());
				UnrealBuildTool.UnrealTargetPlatform TargetPlatform = UnrealBuildTool.BuildHostPlatform.Current.Platform;
				if (TargetPlatform == UnrealBuildTool.UnrealTargetPlatform.Mac)
				{
					foreach (var File in Files)
					{
						SignMacFileOrFolder(File);
					}
				}
				else
				{
					List<FileReference> FilesToSign = new List<FileReference>();
					foreach (string File in Files)
					{
						FilesToSign.Add(new FileReference(File));
					}
					SignMultipleFilesIfEXEOrDLL(FilesToSign, Description: Description, bRunInParallel: bRunInParallel);
				}
			}
			else
			{
				Logger.LogDebug(KnownLogEvents.AutomationTool_CodeSign, "Skipping signing {FileCount} files due to -nosign.", Files.Count());
			}
		}

		public static void SignMultipleFilesIfEXEOrDLL(List<FileReference> Files, bool bIgnoreExtension = false, string Description = null, bool bRunInParallel = false)
		{
			if (!OperatingSystem.IsWindows())
			{
				Logger.LogDebug(KnownLogEvents.AutomationTool_CodeSign, "Can't sign on non-Windows platform.");
				return;
			}
			List<FileReference> FinalFiles = new List<FileReference>();
			foreach (string Filename in Files.Select(x => x.FullName))
			{
				FileInfo TargetFileInfo = new FileInfo(Filename);

				// Executable extensions
				List<string> Extensions = new List<string>
				{
					".dll",
					".exe",
					".msi",
					".dle"
				};

				bool IsExecutable = bIgnoreExtension;

				foreach (var Ext in Extensions)
				{
					if (TargetFileInfo.FullName.EndsWith(Ext, StringComparison.InvariantCultureIgnoreCase))
					{
						// force file writable
						TargetFileInfo.IsReadOnly = false;
						IsExecutable = true;
						break;
					}
				}
				if (IsExecutable && CommandUtils.FileExists(Filename))
				{
					FinalFiles.Add(new FileReference(TargetFileInfo));
				}
			}

			if (FinalFiles.Any(x => x.HasExtension(".msi")))
			{
				// If an MSI is in the list fallback to SHA1
				CodeSignWindows.Sign(FinalFiles, CodeSignWindows.SignatureType.SHA1, AllowMultipleSignatures: false, Description: Description, RunInParallel: bRunInParallel);
			}
			else
			{
				CodeSignWindows.Sign(FinalFiles, CodeSignWindows.SignatureType.SHA256, Description: Description, RunInParallel: bRunInParallel);
			}

		}
	}

}
