// Copyright Epic Games, Inc. All Rights Reserved.
using System;
using System.Collections.Generic;
using System.Linq;
using System.Text;
using System.IO;
using System.Threading;
using UnrealBuildTool;
using EpicGames.Core;
using UnrealBuildBase;
using Microsoft.Extensions.Logging;

using static AutomationTool.CommandUtils;

namespace AutomationTool
{
	/// <summary>
	/// Exception thrown when the execution of a commandlet fails
	/// </summary>
	public class CommandletException : AutomationException
	{
		/// <summary>
		/// The log file output by this commandlet
		/// </summary>
		public string LogFileName;

		/// <summary>
		/// The exit code
		/// </summary>
		public int ErrorNumber;

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="LogFilename">File containing the error log</param>
		/// <param name="ErrorNumber">The exit code from the commandlet</param>
		/// <param name="Format">Formatting string for the message</param>
		/// <param name="Args">Arguments for the formatting string</param>
		public CommandletException(string LogFilename, int ErrorNumber, string Format, params object[] Args)
			: base(Format,Args)
		{
			this.LogFileName = LogFilename;
			this.ErrorNumber = ErrorNumber;
		}
	}

	public partial class CommandUtils
	{
		/// <summary>
		/// Runs Cook commandlet.
		/// </summary>
		/// <param name="ProjectFile">Project file.</param>
		/// <param name="UnrealExe">The name of the Unreal Editor executable to use.</param>
		/// <param name="Maps">List of maps to cook, can be null in which case -MapIniSection=AllMaps is used.</param>
		/// <param name="Dirs">List of directories to cook, can be null</param>
        /// <param name="InternationalizationPreset">The name of a prebuilt set of internationalization data to be included.</param>
        /// <param name="CulturesToCook">List of culture names whose localized assets should be cooked, can be null (implying defaults should be used).</param>
		/// <param name="TargetPlatform">Target platform.</param>
		/// <param name="Parameters">List of additional parameters.</param>
		public static void CookCommandlet(FileReference ProjectFile, string UnrealExe = null, string[] Maps = null, string[] Dirs = null, string InternationalizationPreset = "", string[] CulturesToCook = null, string TargetPlatform = "Windows", string Parameters = "-Unversioned")
		{
            string CommandletArguments = "";

			if (IsNullOrEmpty(Maps))
			{
				// MapsToCook = "-MapIniSection=AllMaps";
			}
			else
			{
				string MapsToCookArg = "-Map=" + CombineCommandletParams(Maps).Trim();
                CommandletArguments += (CommandletArguments.Length > 0 ? " " : "") + MapsToCookArg;
			}

			if (!IsNullOrEmpty(Dirs))
			{
				foreach(string Dir in Dirs)
				{
					CommandletArguments += (CommandletArguments.Length > 0 ? " " : "") + String.Format("-CookDir={0}", CommandUtils.MakePathSafeToUseWithCommandLine(Dir));
				}
            }

            if (!String.IsNullOrEmpty(InternationalizationPreset))
            {
                CommandletArguments += (CommandletArguments.Length > 0 ? " " : "") + InternationalizationPreset;
            }

            if (!IsNullOrEmpty(CulturesToCook))
            {
                string CulturesToCookArg = "-CookCultures=" + CombineCommandletParams(CulturesToCook).Trim();
                CommandletArguments += (CommandletArguments.Length > 0 ? " " : "") + CulturesToCookArg;
            }

            RunCommandlet(ProjectFile, UnrealExe, "Cook", String.Format("{0} -TargetPlatform={1} {2}",  CommandletArguments, TargetPlatform, Parameters));
		}

		/// <summary>
		/// Runs DDC commandlet.
		/// </summary>
		/// <param name="ProjectFile">Project file.</param>
		/// <param name="UnrealExe">The name of the Unreal Editor executable to use.</param>
		/// <param name="Maps">List of maps to cook, can be null in which case -MapIniSection=AllMaps is used.</param>
		/// <param name="TargetPlatform">Target platform.</param>
		/// <param name="Parameters">List of additional parameters.</param>
		public static void DDCCommandlet(FileReference ProjectFile, string UnrealExe = null, string[] Maps = null, string TargetPlatform = "Windows", string Parameters = "")
        {
            string MapsToCook = "";
            if (!IsNullOrEmpty(Maps))
            {
                MapsToCook = "-Map=" + CombineCommandletParams(Maps).Trim();
            }

			RunCommandlet(ProjectFile, UnrealExe, "DerivedDataCache", String.Format("{0} -TargetPlatform={1} {2}", MapsToCook, TargetPlatform, Parameters));
        }

		/// <summary>
		/// Runs RebuildLightMaps commandlet.
		/// </summary>
		/// <param name="ProjectFile">Project file.</param>
		/// <param name="UnrealExe">The name of the Unreal Editor executable to use.</param>
		/// <param name="Maps">List of maps to rebuild light maps for. Can be null in which case -MapIniSection=AllMaps is used.</param>
		/// <param name="Parameters">List of additional parameters.</param>
		public static void RebuildLightMapsCommandlet(FileReference ProjectFile, string UnrealExe = null, string[] Maps = null, string Parameters = "")
		{
			string MapsToRebuildLighting = "";
			if (!IsNullOrEmpty(Maps))
			{
				MapsToRebuildLighting = "-Map=" + CombineCommandletParams(Maps).Trim();
			}

			RunCommandlet(ProjectFile, UnrealExe, "ResavePackages", String.Format("-buildtexturestreaming -buildlighting -MapsOnly -ProjectOnly -AllowCommandletRendering -SkipSkinVerify {0} {1}", MapsToRebuildLighting, Parameters));
		}

		public static void RebuildHLODCommandlet(FileReference ProjectName, string UnrealExe = null, string[] Maps = null, string Parameters = "")
		{
			RebuildHLODCommandlet(ProjectName, out string LogFile, UnrealExe, Maps, Parameters);
		}

		public static void RebuildHLODCommandlet(FileReference ProjectFile, out string DestLogFile, string UnrealExe = null, string[] Maps = null, string Parameters = "")
        {
            string MapsToRebuildHLODs = "";
            if (!IsNullOrEmpty(Maps))
            {
                MapsToRebuildHLODs = "-Map=" + CombineCommandletParams(Maps).Trim();
            }

			RunCommandlet(ProjectFile, UnrealExe, "ResavePackages", String.Format("-BuildHLOD -ProjectOnly -AllowCommandletRendering -SkipSkinVerify {0} {1}", MapsToRebuildHLODs, Parameters), out DestLogFile);
        }

		/// <summary>
		/// Runs RebuildLightMaps commandlet.
		/// </summary>
		/// <param name="ProjectFile">Project file.</param>
		/// <param name="UnrealExe">The name of the Unreal Editor executable to use.</param>
		/// <param name="Maps">List of maps to rebuild light maps for. Can be null in which case -MapIniSection=AllMaps is used.</param>
		/// <param name="Parameters">List of additional parameters.</param>
		public static void ResavePackagesCommandlet(FileReference ProjectFile, string UnrealExe = null, string[] Maps = null, string Parameters = "")
        {
            string MapsToRebuildLighting = "";
            if (!IsNullOrEmpty(Maps))
            {
                MapsToRebuildLighting = "-Map=" + CombineCommandletParams(Maps).Trim();
            }

            RunCommandlet(ProjectFile, UnrealExe, "ResavePackages", String.Format((!String.IsNullOrEmpty(MapsToRebuildLighting) ? "-MapsOnly" : "") + "-ProjectOnly {0} {1}", MapsToRebuildLighting, Parameters));
        }

		/// <summary>
		/// Runs GenerateDistillFileSets commandlet.
		/// </summary>
		/// <param name="ProjectFile">Project file.</param>
		/// <param name="ManifestFile"></param>
		/// <param name="UnrealExe">The name of the Unreal Editor executable to use.</param>
		/// <param name="Maps">List of maps to cook, can be null in which case -MapIniSection=AllMaps is used.</param>
		/// <param name="Parameters">List of additional parameters.</param>
		public static List<FileReference> GenerateDistillFileSetsCommandlet(FileReference ProjectFile, string ManifestFile, string UnrealExe = null, string[] Maps = null, string Parameters = "")
        {
            string MapsToCook = "";
            if (!IsNullOrEmpty(Maps))
            {
                MapsToCook = CombineCommandletParams(Maps, " ").Trim();
            }
            var Dir = Path.GetDirectoryName(ManifestFile);
            var Filename = Path.GetFileName(ManifestFile);
            if (String.IsNullOrEmpty(Dir) || String.IsNullOrEmpty(Filename))
            {
                throw new AutomationException("GenerateDistillFileSets should have a full path and file for {0}.", ManifestFile);
            }
            CreateDirectory(Dir);
            if (FileExists_NoExceptions(ManifestFile))
            {
                DeleteFile(ManifestFile);
            }

            RunCommandlet(ProjectFile, UnrealExe, "GenerateDistillFileSets", String.Format("{0} -OutputFolder={1} -Output={2} {3}", MapsToCook, CommandUtils.MakePathSafeToUseWithCommandLine(Dir), Filename, Parameters));

            if (!FileExists_NoExceptions(ManifestFile))
            {
                throw new AutomationException("GenerateDistillFileSets did not produce a manifest for {0}.", ProjectFile);
            }
			List<string> Lines = new List<string>(ReadAllLines(ManifestFile));
            if (Lines.Count < 1)
            {
                throw new AutomationException("GenerateDistillFileSets for {0} did not produce any files.", ProjectFile);
            }
			List<FileReference> Result = new List<FileReference>();
            foreach (var ThisFile in Lines)
            {
                var TestFile = CombinePaths(ThisFile);
                if (!FileExists_NoExceptions(TestFile))
                {
                    throw new AutomationException("GenerateDistillFileSets produced {0}, but {1} doesn't exist.", ThisFile, TestFile);
                }
				// we correct the case here
				FileInfo TestFileInfo = new FileInfo(TestFile);
				FileReference FinalFile = new FileReference(TestFileInfo);
                if (!FileReference.Exists(FinalFile))
                {
                    throw new AutomationException("GenerateDistillFileSets produced {0}, but {1} doesn't exist.", ThisFile, FinalFile);
                }
                Result.Add(FinalFile);
            }
            return Result;
        }

		/// <summary>
		/// Runs UpdateGameProject commandlet.
		/// </summary>
		/// <param name="ProjectFile">Project file.</param>
		/// <param name="UnrealExe">The name of the Unreal Editor executable to use.</param>
		/// <param name="Parameters">List of additional parameters.</param>
		public static void UpdateGameProjectCommandlet(FileReference ProjectFile, string UnrealExe = null, string Parameters = "")
        {
            RunCommandlet(ProjectFile, UnrealExe, "UpdateGameProject", Parameters);
        }

		/// <summary>
		/// Runs a commandlet using Engine/Binaries/Win64/UnrealEditor-Cmd.exe.
		/// </summary>
		/// <param name="ProjectFile">Project file.</param>
		/// <param name="UnrealExe">The name of the Unreal Editor executable to use.</param>
		/// <param name="Commandlet">Commandlet name.</param>
		/// <param name="Parameters">Command line parameters (without -run=)</param>
		/// <param name="ErrorLevel">The minimum exit code, which is treated as an error.</param>
		public static void RunCommandlet(FileReference ProjectFile, string UnrealExe, string Commandlet, string Parameters = null, uint ErrorLevel = 1)
		{
			string LogFile;
			RunCommandlet(ProjectFile, UnrealExe, Commandlet, Parameters, out LogFile, ErrorLevel);
		}

		/// <summary>
		/// Runs a commandlet using Engine/Binaries/Win64/UnrealEditor-Cmd.exe.
		/// </summary>
		/// <param name="ProjectFile">Project file.</param>
		/// <param name="UnrealExe">The name of the Unreal Editor executable to use.</param>
		/// <param name="Commandlet">Commandlet name.</param>
		/// <param name="Parameters">Command line parameters (without -run=)</param>
		/// <param name="ErrorLevel">The minimum exit code, which is treated as an error.</param>
		public static void RunCommandlet(FileReference ProjectFile, string UnrealExe, string Commandlet, string Parameters, int ErrorLevel)
		{
			string LogFile;
			RunCommandlet(ProjectFile, UnrealExe, Commandlet, Parameters, out LogFile, (uint)ErrorLevel);
		}

		/// <summary>
		/// Runs a commandlet using Engine/Binaries/Win64/UnrealEditor-Cmd.exe.
		/// </summary>
		/// <param name="ProjectFile">Project file.</param>
		/// <param name="UnrealExe">The name of the Unreal Editor executable to use.</param>
		/// <param name="Commandlet">Commandlet name.</param>
		/// <param name="Parameters">Command line parameters (without -run=)</param>
		/// <param name="DestLogFile">Log file after completion</param>
		/// <param name="ErrorLevel">The minimum exit code, which is treated as an error.</param>
		public static void RunCommandlet(FileReference ProjectFile, string UnrealExe, string Commandlet, string Parameters, out string DestLogFile, uint ErrorLevel = 1)
		{
			string LocalLogFile;
			IProcessResult RunResult;

			DateTime StartTime = DateTime.UtcNow;

			StartRunCommandlet(ProjectFile, UnrealExe, Commandlet, Parameters, ERunOptions.Default, out LocalLogFile, out RunResult);
			FinishRunCommandlet(ProjectFile, Commandlet, StartTime, RunResult, LocalLogFile, out DestLogFile, ErrorLevel);
		}

		/// <summary>
		/// Runs a commandlet using Engine/Binaries/Win64/UnrealEditor-Cmd.exe.
		/// </summary>
		/// <param name="ProjectFile">Project file</param>
		/// <param name="UnrealExe">The name of the Unreal Editor executable to use.</param>
		/// <param name="Commandlet">Commandlet name.</param>
		/// <param name="Parameters">Command line parameters (without -run=)</param>
		/// <param name="DestLogFile">Log file after completion</param>
		/// <param name="ErrorLevel">The minimum exit code, which is treated as an error.</param>
		public static void RunCommandlet(FileReference ProjectFile, string UnrealExe, string Commandlet, string Parameters, out string DestLogFile, int ErrorLevel)
		{
			RunCommandlet(ProjectFile, UnrealExe, Commandlet, Parameters, out DestLogFile, (uint)ErrorLevel);
		}

		public static void StartRunCommandlet(FileReference ProjectFile, string UnrealExe, string Commandlet, string Parameters, ERunOptions RunOptions, out string LocalLogFile, out IProcessResult RunResult, ProcessResult.SpewFilterCallbackType SpewFilterCallback=null)
		{
			Logger.LogInformation("Running UnrealEditor {Commandlet} for project {ProjectName}", Commandlet, ProjectFile);

			var CWD = Path.GetDirectoryName(UnrealExe);

			string EditorExe = UnrealExe;

			if (EditorExe == null)
			{
				EditorExe = ProjectUtils.GetEditorForProject(ProjectFile).FullName;
			}

			if (String.IsNullOrEmpty(CWD))
			{
				EditorExe = HostPlatform.Current.GetUnrealExePath(EditorExe);
				CWD = CombinePaths(CmdEnv.LocalRoot, HostPlatform.Current.RelativeBinariesFolder);
			}

			PushDir(CWD);

			LocalLogFile = LogUtils.GetUniqueLogName(CombinePaths(CmdEnv.EngineSavedFolder, Commandlet));
			Logger.LogInformation("Commandlet log file is {LocalLogFile}", LocalLogFile);
			string Args = String.Format(
				"{0} -run={1} {2} -abslog={3} -stdout -CrashForUAT -unattended -NoLogTimes {5}{4}",
				(ProjectFile == null) ? "" : CommandUtils.MakePathSafeToUseWithCommandLine(ProjectFile.FullName),
				Commandlet,
				String.IsNullOrEmpty(Parameters) ? "" : Parameters,
				CommandUtils.MakePathSafeToUseWithCommandLine(LocalLogFile),
				IsBuildMachine ? "-buildmachine" : "",
				(GlobalCommandLine.Verbose || GlobalCommandLine.AllowStdOutLogVerbosity) ? "-AllowStdOutLogVerbosity " : ""
			);
			if (GlobalCommandLine.UTF8Output)
			{
				RunOptions |= ERunOptions.UTF8Output;
			}
			RunResult = Run(EditorExe, Args, Options: RunOptions, Identifier: Commandlet, SpewFilterCallback: SpewFilterCallback);
			PopDir();
		}

		public static void FinishRunCommandlet(FileReference ProjectFile, string Commandlet, DateTime StartTime, IProcessResult RunResult, string LocalLogFile, out string DestLogFile, uint ErrorLevel = 1)
		{
			// If we're running on a Windows build machine, copy any crash dumps into the log folder
			if(HostPlatform.Current.HostEditorPlatform == UnrealTargetPlatform.Win64 && IsBuildMachine)
			{
				DirectoryInfo CrashesDir = new DirectoryInfo(GetCrashesDirectory(ProjectFile).FullName);
				if(CrashesDir.Exists)
				{
					foreach(DirectoryInfo CrashDir in CrashesDir.EnumerateDirectories())
					{
						if(CrashDir.LastWriteTimeUtc > StartTime)
						{
							DirectoryInfo OutputCrashesDir = new DirectoryInfo(Path.Combine(CmdEnv.LogFolder, "Crashes", CrashDir.Name));
							try
							{
								Logger.LogInformation("Copying crash data to {Arg0}...", OutputCrashesDir.FullName);
								OutputCrashesDir.Create();

								foreach(FileInfo CrashFile in CrashDir.EnumerateFiles())
								{
									CrashFile.CopyTo(Path.Combine(OutputCrashesDir.FullName, CrashFile.Name));
								}
							}
							catch(Exception Ex)
							{
								Logger.LogWarning("Unable to copy crash data; skipping. See log for exception details.");
								Logger.LogDebug("{Text}", EpicGames.Core.ExceptionUtils.FormatExceptionDetails(Ex));
							}
						}
					}
				}
			}

			// If we're running on a Mac, dump all the *.crash files that were generated while the editor was running.
			if(HostPlatform.Current.HostEditorPlatform == UnrealTargetPlatform.Mac)
			{
				// If the exit code indicates the main process crashed, introduce a small delay because the crash report is written asynchronously.
				// If we exited normally, still check without waiting in case SCW or some other child process crashed.
				if(RunResult.ExitCode > 128)
				{
					Logger.LogInformation("Pausing before checking for crash logs...");
					Thread.Sleep(10 * 1000);
				}

				// Create a list of directories containing crash logs, and add the system log folder
				List<string> CrashDirs = new List<string>();
				CrashDirs.Add("/Library/Logs/DiagnosticReports");

				// Add the user's log directory too
				string HomeDir = Environment.GetEnvironmentVariable("HOME");
				if(!String.IsNullOrEmpty(HomeDir))
				{
					CrashDirs.Add(Path.Combine(HomeDir, "Library/Logs/DiagnosticReports"));
				}

				// Check each directory for crash logs
				List<FileInfo> CrashFileInfos = new List<FileInfo>();
				foreach(string CrashDir in CrashDirs)
				{
					try
					{
						DirectoryInfo CrashDirInfo = new DirectoryInfo(CrashDir);
						if (CrashDirInfo.Exists)
						{
							CrashFileInfos.AddRange(CrashDirInfo.EnumerateFiles("*.crash", SearchOption.TopDirectoryOnly).Where(x => x.LastWriteTimeUtc >= StartTime));
						}
					}
					catch (UnauthorizedAccessException)
					{
						// Not all account types can access /Library/Logs/DiagnosticReports
					}
				}

				// Dump them all to the log
				foreach(FileInfo CrashFileInfo in CrashFileInfos)
				{
					// snmpd seems to often crash (suspect due to it being starved of CPU cycles during cooks)
					// also ignore spotlight crash with the excel plugin
					if(!CrashFileInfo.Name.StartsWith("snmpd_") && !CrashFileInfo.Name.StartsWith("mdworker32_") && !CrashFileInfo.Name.StartsWith("Dock_"))
					{
						Logger.LogInformation("Found crash log - {Arg0}", CrashFileInfo.FullName);
						try
						{
							string[] Lines = File.ReadAllLines(CrashFileInfo.FullName);
							foreach(string Line in Lines)
							{
								Logger.LogInformation("Crash: {Line}", Line);
							}
						}
						catch(Exception Ex)
						{
							Logger.LogWarning("Failed to read file ({Arg0})", Ex.Message);
						}
					}
				}
			}

			// Copy the local commandlet log to the destination folder.
			DestLogFile = LogUtils.GetUniqueLogName(CombinePaths(CmdEnv.LogFolder, Commandlet));
			if (!CommandUtils.CopyFile_NoExceptions(LocalLogFile, DestLogFile))
			{
				Logger.LogWarning("Commandlet {Commandlet} failed to copy the local log file from {LocalLogFile} to {DestLogFile}. The log file will be lost.", Commandlet, LocalLogFile, DestLogFile);
			}
            string ProjectStatsDirectory = CombinePaths((ProjectFile == null)? CombinePaths(CmdEnv.LocalRoot, "Engine") : Path.GetDirectoryName(ProjectFile.FullName), "Saved", "Stats");
            if (Directory.Exists(ProjectStatsDirectory))
            {
                string DestCookerStats = CmdEnv.LogFolder;
                foreach (var StatsFile in Directory.EnumerateFiles(ProjectStatsDirectory, "*.csv"))
                {
                    if (!CommandUtils.CopyFile_NoExceptions(StatsFile, CombinePaths(DestCookerStats, Path.GetFileName(StatsFile))))
                    {
						Logger.LogWarning("Commandlet {Commandlet} failed to copy the local log file from {StatsFile} to {Arg2}. The log file will be lost.", Commandlet, StatsFile, CombinePaths(DestCookerStats, Path.GetFileName(StatsFile)));
                    }
                }
            }
//			else
//			{
//				CommandUtils.LogWarning("Failed to find directory {0} will not save stats", ProjectStatsDirectory);
//			}

			// Whether it was copied correctly or not, delete the local log as it was only a temporary file.
			CommandUtils.DeleteFile_NoExceptions(LocalLogFile);

			// Throw an exception if the execution failed. Draw attention to signal exit codes on Posix systems, rather than just printing the exit code
			if (RunResult.ExitCode != 0 && (uint)RunResult.ExitCode >= ErrorLevel)
			{
				string ExitCodeDesc = "";
				if(RunResult.ExitCode > 128 && RunResult.ExitCode < 128 + 32)
				{
					if(RunResult.ExitCode == 139)
					{
						ExitCodeDesc = " (segmentation fault)";
					}
					else
					{
						ExitCodeDesc = String.Format(" (signal {0})", RunResult.ExitCode - 128);
					}
				}
				throw new CommandletException(DestLogFile, RunResult.ExitCode, "Editor terminated with exit code {0}{1} while running {2}{3}; see log {4}", RunResult.ExitCode, ExitCodeDesc, Commandlet, (ProjectFile == null)? "" : String.Format(" for {0}", ProjectFile), DestLogFile) { OutputFormat = AutomationExceptionOutputFormat.Minimal };
			}
		}
		
		public static void FinishRunCommandlet(FileReference ProjectFile, string Commandlet, DateTime StartTime, IProcessResult RunResult, string LocalLogFile, out string DestLogFile, int ErrorLevel)
		{
			FinishRunCommandlet(ProjectFile, Commandlet, StartTime, RunResult, LocalLogFile, out DestLogFile, (uint)ErrorLevel);
		}

		/// <summary>
		/// Returns the default path of the editor executable to use for running commandlets.
		/// </summary>
		/// <param name="BuildRoot">Root directory for the build</param>
		/// <param name="HostPlatform">Platform to get the executable for</param>
		/// <returns>Path to the editor executable</returns>
		public static string GetEditorCommandletExe(string BuildRoot, UnrealBuildTool.UnrealTargetPlatform HostPlatform)
		{
			if (HostPlatform == UnrealBuildTool.UnrealTargetPlatform.Mac)
			{
				return CommandUtils.CombinePaths(BuildRoot, "Engine/Binaries/Mac/UnrealEditor.app/Contents/MacOS/UnrealEditor");
			}
			if (HostPlatform == UnrealBuildTool.UnrealTargetPlatform.Win64)
			{
				return CommandUtils.CombinePaths(BuildRoot, "Engine/Binaries/Win64/UnrealEditor-Cmd.exe");
			}
			if (HostPlatform == UnrealBuildTool.UnrealTargetPlatform.Linux)
			{
				return CommandUtils.CombinePaths(BuildRoot, "Engine/Binaries/Linux/UnrealEditor");
			}
			throw new AutomationException("EditorCommandlet is not supported for platform {0}", HostPlatform);
		}

		/// <summary>
		/// Combines a list of parameters into a single commandline param separated with '+':
		/// For example: Map1+Map2+Map3
		/// </summary>
		/// <param name="ParamValues">List of parameters (must not be empty)</param>
		/// <param name="Separator"></param>
		/// <returns>Combined param</returns>
		public static string CombineCommandletParams(IEnumerable<string> ParamValues, string Separator = "+")
		{
			string CombinedParams = String.Empty;
			if (ParamValues != null)
			{
				IEnumerator<string> Iter = ParamValues.GetEnumerator();
				while (Iter.MoveNext())
				{
					if (!String.IsNullOrEmpty(Iter.Current))
					{
						CombinedParams += Iter.Current;
						break;
					}
				}

				while (Iter.MoveNext())
				{
					if (!String.IsNullOrEmpty(Iter.Current))
					{
						CombinedParams += Separator + Iter.Current;
					}
				}
			}

			return CombinedParams;
		}

		/// <summary>
		/// Converts project name to FileReference used by other commandlet functions
		/// </summary>
		/// <param name="ProjectName">Project name.</param>
		/// <returns>FileReference to project location</returns>
		public static FileReference GetCommandletProjectFile(string ProjectName)
		{
			FileReference ProjectFullPath = null;
			var OriginalProjectName = ProjectName;
			ProjectName = ProjectName.Trim(new char[] { '\"' });
			if (ProjectName.IndexOfAny(new char[] { '\\', '/' }) < 0)
			{
				ProjectName = CombinePaths(CmdEnv.LocalRoot, ProjectName, ProjectName + ".uproject");
			}
			else if (!FileExists_NoExceptions(ProjectName))
			{
				ProjectName = CombinePaths(CmdEnv.LocalRoot, ProjectName);
			}
			if (FileExists_NoExceptions(ProjectName))
			{
				ProjectFullPath = new FileReference(ProjectName);
			}
			else
			{
				BranchInfo Branch = new BranchInfo();
				BranchInfo.BranchUProject GameProj = Branch.FindGame(OriginalProjectName);
				if (GameProj != null)
				{
					ProjectFullPath = GameProj.FilePath;
				}
				if (!FileExists_NoExceptions(ProjectFullPath.FullName))
				{
					throw new AutomationException("Could not find a project file {0}.", ProjectName);
				}
			}
			return ProjectFullPath;
		}

		/// <summary>
		/// Get the crashes directory for the given project file
		/// </summary>
		/// <param name="ProjectFullPath">Path to a project file</param>
		/// <returns>DirectoryReference for the directory where crashes are stored for this project</returns>
		public static DirectoryReference GetCrashesDirectory(FileReference ProjectFullPath)
		{
			DirectoryReference CrashesDir = DirectoryReference.Combine(DirectoryReference.FromFile(ProjectFullPath) ?? Unreal.EngineDirectory, "Saved", "Crashes");
			return CrashesDir;
		}
	}
}
