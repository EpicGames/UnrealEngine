// Copyright Epic Games, Inc. All Rights Reserved.

#nullable enable

using System;
using System.Collections.Generic;
using System.Linq;

namespace Microsoft.IdentityModel.Tokens
{
	/// <summary>
	/// Compatibility shim restoring <c>IsNullOrEmpty</c> extension methods that were made internal
	/// in <c>Microsoft.IdentityModel.Tokens</c> v8.0.0 (see issue #2651).
	/// </summary>
	public static class CollectionUtilitiesShim
	{
		/// <summary>
		/// Returns <see langword="true"/> if <paramref name="value"/> is <see langword="null"/> or empty.
		/// </summary>
		/// <param name="value">The string to check.</param>
		public static bool IsNullOrEmpty(this string? value)
		{
			return String.IsNullOrEmpty(value);
		}

		/// <summary>
		/// Returns <see langword="true"/> if <paramref name="enumerable"/> is <see langword="null"/> or contains no elements.
		/// </summary>
		/// <typeparam name="T">The element type.</typeparam>
		/// <param name="enumerable">The sequence to check.</param>
		public static bool IsNullOrEmpty<T>(this IEnumerable<T>? enumerable)
		{
			return enumerable == null || !enumerable.Any();
		}
	}
}
