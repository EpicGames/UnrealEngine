// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Linq;
using System.Text;
using System.IO;
using AutomationTool;
using UnrealBuildTool;
using EpicGames.Core;
using UnrealBuildBase;
using Microsoft.Extensions.Logging;

/// <summary>
/// Copies all UAT and UBT build products to a directory
/// </summary>
public class CopyUAT : BuildCommand
{
	public override void ExecuteBuild()
	{
		// Get the output directory
		string TargetDirParam = ParseParamValue("TargetDir");
		if(TargetDirParam == null)
		{
			throw new AutomationException("Missing -TargetDir=... argument to CopyUAT");
		}

		// Construct a dummy UnrealBuild object to get a list of the UAT and UBT build products
		UnrealBuild Build = new UnrealBuild(this);
		Build.AddUATFilesToBuildProducts();
		Build.AddUBTFilesToBuildProducts();

		// Get a list of all the input files
		HashSet<FileReference> SourceFilesSet = new HashSet<FileReference>();
		foreach(string BuildProductFile in Build.BuildProductFiles)
		{
			FileReference SourceFile = new FileReference(BuildProductFile);
			SourceFilesSet.Add(SourceFile);

			FileReference SourceSymbolFile = SourceFile.ChangeExtension(".pdb");
			if(FileReference.Exists(SourceSymbolFile))
			{
				SourceFilesSet.Add(SourceSymbolFile);
			}

			FileReference DocumentationFile = SourceFile.ChangeExtension(".xml");
			if(FileReference.Exists(DocumentationFile))
			{
				SourceFilesSet.Add(DocumentationFile);
			}
		}

		// Copy all the files over
		DirectoryReference TargetDir = new DirectoryReference(TargetDirParam);
		List<FileReference> SourceFiles = SourceFilesSet.OrderBy(x => x.FullName).ToList();
		CommandUtils.ThreadedCopyFiles(SourceFiles, Unreal.RootDirectory, TargetDir);

		Logger.LogInformation("Copied {NumFiles} files to {TargetDir}", SourceFiles.Count, TargetDir);
		File.WriteAllLines(Path.Combine(TargetDirParam, "CopiedFiles.txt"), SourceFiles.Select(F => F.FullName));
	}
}
