// Copyright Epic Games, Inc. All Rights Reserved.
using System;
using System.Collections.Generic;
using System.Reflection;
using AutomationTool;
using UnrealBuildTool;
using System.Linq;

/// <summary>
/// Allows a target to change how it is packaged, staged and deployed - for example, when packaging for a specific game store
/// </summary>
public abstract class CustomDeploymentHandler
{
	protected readonly Platform Platform;
	protected CustomDeploymentHandler( Platform AutomationPlatform, ProjectParams Params, DeploymentContext SC )
		: this(AutomationPlatform)
	{
	}
	protected CustomDeploymentHandler( Platform AutomationPlatform )
	{
		this.Platform = AutomationPlatform;
	}
	public abstract bool SupportsPlatform( UnrealTargetPlatform Platform );

	// will return false if the caller should do the default platform processing
	public virtual bool GetPlatformPakCommandLine(ProjectParams Params, DeploymentContext SC, ref string CmdLine) => false;
	public virtual bool GetPlatformIoStoreCommandLine(ProjectParams Params, DeploymentContext SC, ref string CmdLine) => false;
	public virtual bool PrePackage(ProjectParams Params, DeploymentContext SC, int WorkingCL) => false;
	public virtual bool Deploy(ProjectParams Params, DeploymentContext SC) => false;
	public virtual bool PreGetFilesToDeployOrStage(ProjectParams Params, DeploymentContext SC) => false;
	public virtual bool GetFilesToStageForDLC(ProjectParams Params, DeploymentContext SC) => false;
	public virtual bool GetFilesToArchive(ProjectParams Params, DeploymentContext SC) => false;
	public virtual bool PostStagingFileCopy(ProjectParams Params, DeploymentContext SC) => false;

	// always called
	public virtual void PostPackage(ProjectParams Params, DeploymentContext SC, int WorkingCL) { }
	public virtual void PostGetFilesToDeployOrStage(ProjectParams Params, DeploymentContext SC) { }

	// returns null if the caller should do the default platform processing
	public virtual bool? GetPlatformPatchesWithDiffPak(ProjectParams Params, DeploymentContext SC) => null;
	public virtual IProcessResult RunClient(CommandUtils.ERunOptions ClientRunFlags, string ClientApp, string ClientCmdLine, ProjectParams Params, DeploymentContext SC) => null;


	#region Private/boilerplate

	/// <summary>
	/// Validate and instantiate the given custom deployment handler
	/// </summary>
	public static CustomDeploymentHandler Create(string Name, Platform AutomationPlatform, ProjectParams Params, DeploymentContext SC)
	{
		// one-time static init. NB. Can't use the partial struct convention because automation classes are all in different assemblies
		if (HandlerRegistry == null)
		{
			HandlerRegistry = new();

			foreach (Assembly LoadedAssembly in ScriptManager.AllScriptAssemblies)
			{
				IEnumerable<Type> HandlerTypes = LoadedAssembly.GetTypes().Where(X => !X.IsAbstract && X.IsSubclassOf(typeof(CustomDeploymentHandler)) && X.GetCustomAttribute(typeof(CustomDeploymentHandlerAttribute)) != null);
				foreach (Type HandlerType in HandlerTypes)
				{
					string HandlerName = HandlerType.GetCustomAttribute<CustomDeploymentHandlerAttribute>().Name;
					HandlerRegistry.Add(HandlerName, HandlerType);
				}
			}
		}

		Type SelectedType;
		if (!HandlerRegistry.TryGetValue(Name, out SelectedType) || SelectedType == null)
		{
			throw new BuildException($"Unknown custom deployment handler {Name}");
		}

		CustomDeploymentHandler Handler = (CustomDeploymentHandler)Activator.CreateInstance(SelectedType, AutomationPlatform, Params, SC);
		if (Handler == null)
		{
			throw new BuildException($"Could not instantiate the custom deployment handler {Name}");
		}

		if (!Handler.SupportsPlatform(AutomationPlatform.PlatformType))
		{
			throw new BuildException($"Custom deployment handler {Name} does not support platform {AutomationPlatform.PlatformType}");
		}

		return Handler;

	}

	private static Dictionary<string, Type> HandlerRegistry = null;

	#endregion
}


/// <summary>
/// CustomDeploymentHandler class declarations must be tagged with this, used to find the handler by name
/// </summary>
[System.AttributeUsage(System.AttributeTargets.Class)]
public class CustomDeploymentHandlerAttribute : System.Attribute
{
	public string Name;
	public CustomDeploymentHandlerAttribute( string Name)
	{
		this.Name = Name;
	}
}