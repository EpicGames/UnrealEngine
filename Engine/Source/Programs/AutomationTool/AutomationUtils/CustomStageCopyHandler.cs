// Copyright Epic Games, Inc. All Rights Reserved.

using AutomationTool;
using EpicGames.Core;
using System;
using System.Collections.Generic;
using System.Linq;
using System.Reflection;
using UnrealBuildTool;
using Microsoft.Extensions.Logging;

public abstract class CustomStageCopyHandler
{
	/// <summary>
	/// Called when copying files to the staging directory.
	/// </summary>
	/// <returns>true if the file copy can be handled.</returns>
	public virtual bool CanCopyFile(string SourceName)
	{
		return false;
	}

	/// <summary>
	/// Called when copying files to the staging directory.
	/// </summary>
	/// <returns>true if the file was handled. false to fallback on the default stage copy implementation</returns>
	public virtual bool StageFile(ILogger Logger, string SourceName, string TargetName)
	{ 
		return false;
	}

	/// <summary>
	/// Called when writing plugin descriptor files to the plugin manifest file.
	/// </summary>
	/// <returns>The PluginDescriptor to write into the manifest</returns>
	public virtual PluginDescriptor CreateDescriptorForPluginManifest(ILogger Logger, FileReference PluginFile)
	{ 
		return PluginDescriptor.FromFile(PluginFile);
	}

	public static CustomStageCopyHandler Create(string Name)
	{
		// one-time static init. NB. Can't use the partial struct convention because automation classes are all in different assemblies
		if (HandlerRegistry == null)
		{
			HandlerRegistry = new();

			foreach (Assembly LoadedAssembly in ScriptManager.AllScriptAssemblies)
			{
				IEnumerable<Type> HandlerTypes = LoadedAssembly.GetTypes().Where(X => !X.IsAbstract && X.IsSubclassOf(typeof(CustomStageCopyHandler)) && X.GetCustomAttribute(typeof(CustomStageCopyHandlerAttribute)) != null);
				foreach (Type HandlerType in HandlerTypes)
				{
					string HandlerName = HandlerType.GetCustomAttribute<CustomStageCopyHandlerAttribute>().Name;
					HandlerRegistry.Add(HandlerName, HandlerType);
				}
			}
		}

		Type SelectedType;
		if (!HandlerRegistry.TryGetValue(Name, out SelectedType) || SelectedType == null)
		{
			throw new BuildException($"Unknown custom stage copy handler {Name}");
		}

		CustomStageCopyHandler Handler = (CustomStageCopyHandler)Activator.CreateInstance(SelectedType);
		if (Handler == null)
		{
			throw new BuildException($"Could not instantiate the custom stage copy handler {Name}");
		}

		return Handler;
	}

	public virtual void Init(ConfigHierarchy InPlatformEngineConfig)
	{
		PlatformEngineConfig = InPlatformEngineConfig;
	}

	private static Dictionary<string, Type> HandlerRegistry = null;

	/// <summary>
	/// Platform engine config hierarchy for the platform being staged.
	/// </summary>
	protected ConfigHierarchy PlatformEngineConfig = null;
}

/// <summary>
/// CustomStageCopyHandler class declarations must be tagged with this, used to find the handler by name
/// </summary>
[System.AttributeUsage(System.AttributeTargets.Class)]
public class CustomStageCopyHandlerAttribute : System.Attribute
{
	public string Name;
	public CustomStageCopyHandlerAttribute(string Name)
	{
		this.Name = Name;
	}
}
