// Copyright Epic Games, Inc. All Rights Reserved.
using AutomationTool;
using System;
using System.Collections.Generic;
using System.Linq;
using System.Reflection;
using static AutomationTool.CommandUtils;

/// <summary>
/// Support for generating custom packages, for example when targeting a specific game store
/// </summary>
public abstract class CustomStagingHandler
{
	/// <summary>
	/// Determines whether this custom packaging handler should be used in the given deployment context
	/// </summary>
	protected abstract bool TryInitialize(ProjectParams Params, DeploymentContext SC);

	/// <summary>
	/// Hook for adding additional files for staging.
	/// </summary>
	public virtual void GetFilesToStage(ProjectParams Params, DeploymentContext SC) { }

	/// <summary>
	/// Hook for adding additional files for staging when packaging a DLC plugin
	/// </summary>
	public virtual void GetFilesToStageForDLC(ProjectParams Params, DeploymentContext SC) { }

	/// <summary>
	/// Hook for when the build has been staged and the list of staged files is available
	/// </summary>
	public virtual void PostStagingFileCopy(ProjectParams Params, DeploymentContext SC) { }

	/// <summary>
	/// Hook for package generation
	/// </summary>
	public virtual void Package(ProjectParams Params, DeploymentContext SC, int WorkingCL) { }

	/// <summary>
	/// Hook for adding additional files for archiving.
	/// </summary>
	public virtual void GetFilesToArchive(ProjectParams Params, DeploymentContext SC ) { }


	/// <summary>
	/// Hook for deploying a package we have created
	/// </summary>
	public virtual bool TryDeploy(ProjectParams Params, DeploymentContext SC) => false;

	/// <summary>
	/// Hook for launching a package we have created
	/// </summary>
	public virtual IProcessResult TryRunClient(ERunOptions ClientRunFlags, string ClientApp, string ClientCmdLine, ProjectParams Params, DeploymentContext SC) => null;



	#region Private/boilerplate

	/// <summary>
	/// Validate and instantiate the given custom staging handlers for the current project. (Note this is returning a List not an IEnumerable to allow the caller to .ForEach() on the handlers)
	/// </summary>
	public static List<CustomStagingHandler> GetCustomStagingHandlers( ProjectParams Params, DeploymentContext SC)
	{
		// one-time static init. NB. Can't use the partial struct convention because automation classes are all in different assemblies
		if (HandlerRegistry == null)
		{
			HandlerRegistry = new();

			foreach (Assembly LoadedAssembly in ScriptManager.AllScriptAssemblies)
			{
				try
				{
					IEnumerable<Type> HandlerTypes = LoadedAssembly.GetTypes().Where(X => !X.IsAbstract && X.IsSubclassOf(typeof(CustomStagingHandler)));
					foreach (Type HandlerType in HandlerTypes)
					{
						HandlerRegistry.Add(HandlerType);
					}
				}
				catch(ReflectionTypeLoadException)
				{
				}
			}
		}

		// collect all handlers that are valid for the current project
		List<CustomStagingHandler> AvailableHandlers = [];
		foreach (Type HandlerType in HandlerRegistry)
		{
			CustomStagingHandler Handler;
			try
			{
				Handler = (CustomStagingHandler)Activator.CreateInstance(HandlerType);
			}
			catch (Exception)
			{
				continue;
			}

			if (Handler != null && Handler.TryInitialize(Params, SC))
			{
				AvailableHandlers.Add(Handler);
			}
		}

		return AvailableHandlers;
	}

	private static List<Type> HandlerRegistry = null;

	#endregion
}


