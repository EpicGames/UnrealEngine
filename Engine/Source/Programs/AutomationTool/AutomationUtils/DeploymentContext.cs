// Copyright Epic Games, Inc. All Rights Reserved.
using System;
using System.Collections.Generic;
using System.IO;
using System.Threading;
using System.Reflection;
using AutomationTool;
using UnrealBuildTool;
using System.Linq;
using EpicGames.Core;
using EpicGames.ProjectStore;

public struct StageTarget
{
	public TargetReceipt Receipt;
	public bool RequireFilesExist;
}

/// <summary>
/// Controls which directories are searched when staging files
/// </summary>
public enum StageFilesSearch
{
	/// <summary>
	/// Only search the top directory
	/// </summary>
	TopDirectoryOnly,

	/// <summary>
	/// Search the entire directory tree
	/// </summary>
	AllDirectories,
}

/// <summary>
/// Contains the set of files to be staged
/// </summary>
public class FilesToStage
{
	/// <summary>
	/// After staging, this is a map from staged file to source file. These file are content, and can go into a pak file.
	/// </summary>
	public Dictionary<StagedFileReference, FileReference> UFSFiles = new Dictionary<StagedFileReference, FileReference>();

	/// <summary>
	/// After staging, this is a map from staged file to source file. These file are binaries, etc and can't go into a pak file.
	/// </summary>
	public Dictionary<StagedFileReference, FileReference> NonUFSFiles = new Dictionary<StagedFileReference, FileReference>();

	/// <summary>
	/// After staging, this is a map from staged file to source file. These file are for debugging, and should not go into a pak file.
	/// </summary>
	public Dictionary<StagedFileReference, FileReference> NonUFSDebugFiles = new Dictionary<StagedFileReference, FileReference>();

	/// <summary>
	/// After staging, this is a map from staged file to source file. These files are system files, and should not be renamed or remapped.
	/// </summary>
	public Dictionary<StagedFileReference, FileReference> NonUFSSystemFiles = new Dictionary<StagedFileReference, FileReference>();

	/// <summary>
	/// Adds a file to be staged as the given type
	/// </summary>
	/// <param name="FileType">The type of file to be staged</param>
	/// <param name="StagedFile">The staged file location</param>
	/// <param name="InputFile">The input file</param>
	public void Add(StagedFileType FileType, StagedFileReference StagedFile, FileReference InputFile)
	{
		if (FileType == StagedFileType.UFS)
		{
			AddToDictionary(UFSFiles, StagedFile, InputFile);
		}
		else if (FileType == StagedFileType.NonUFS)
		{
			AddToDictionary(NonUFSFiles, StagedFile, InputFile);
		}
		else if (FileType == StagedFileType.DebugNonUFS)
		{
			AddToDictionary(NonUFSDebugFiles, StagedFile, InputFile);
		}
		else if(FileType == StagedFileType.SystemNonUFS)
		{
			AddToDictionary(NonUFSSystemFiles, StagedFile, InputFile);
		}
	}

	/// <summary>
	/// Adds a file to be staged to the given dictionary
	/// </summary>
	/// <param name="FilesToStage">Dictionary of files to be staged</param>
	/// <param name="StagedFile">The staged file location</param>
	/// <param name="InputFile">The input file</param>
	private void AddToDictionary(Dictionary<StagedFileReference, FileReference> FilesToStage, StagedFileReference StagedFile, FileReference InputFile)
	{
		FilesToStage[StagedFile] = InputFile;
	}
}

public class ZenCookedFile
{
	public string Filename;
	public IoHash ChunkId;
}

public class PackageStoreData
{
	public ZenServerStoreData ZenServerStore { get; set; }
	public string ManifestFullPath { get; set; }
	public string MarkerFullPath { get; set; }
	public IList<ZenCookedFile> ZenCookedFiles { get; set; }

	/// <summary>
	/// Cached parsed ChunkAssignment op from the cook oplog, populated by
	/// ReadZenCookedFilesFromZenServer so downstream consumers (e.g. CreatePaksUsingChunkManifests)
	/// can read pakchunk assignment without re-fetching the oplog. Null if the op
	/// was absent or the oplog wasn't read with cooked-files loading enabled.
	/// Stored as <c>object</c> so DeploymentContext doesn't have to take a dependency
	/// on AutomationScripts.Oplog — cast to <c>ChunkAssignmentOp</c> at the use site.
	/// </summary>
	public object ChunkAssignment { get; set; }
}

public class DeploymentContext //: ProjectParams
{
	/// <summary>
	/// Full path to the .uproject file
	/// </summary>
	public FileReference RawProjectPath;

	/// <summary>
	///  true if we should stage crash reporter
	/// </summary>
	public bool bStageCrashReporter;

	/// <summary>
	///  CookPlatform, where to get the cooked data from and use for sandboxes
	/// </summary>
	public string CookPlatform;

	/// <summary>
	///  FinalCookPlatform, directory to stage and archive the final result to
	/// </summary>
	public string FinalCookPlatform;

	/// <summary>
	///  Source platform to get the cooked data from
	/// </summary>
	public Platform CookSourcePlatform;

	/// <summary>
	///  Target platform used for sandboxes and stage directory names
	/// </summary>
	public Platform StageTargetPlatform;

	/// <summary>
	///  Configurations to stage. Used to determine which ThirdParty configurations to copy.
	/// </summary>
	public List<UnrealTargetConfiguration> StageTargetConfigurations;

	/// <summary>
	/// Receipts for the build targets that should be staged.
	/// </summary>
	public List<StageTarget> StageTargets;

	/// <summary>
	/// Extra subdirectory to load config files out of, for making multiple types of builds with the same platform
	/// </summary>
	public string CustomConfig;

	/// <summary>
	/// Allows a platform to change how pak files are generated and how it is staged, packaged and deployed - for example, when packaging for a specific game store
	/// </summary>
	public CustomDeploymentHandler CustomDeployment = null;

	/// <summary>
	/// Support for generating custom packages, for example when targeting a specific game store
	/// </summary>
	public List<CustomStagingHandler> CustomStagingHandlers = [];

	/// <summary>
	/// This is the root directory that contains the engine: d:\a\UE\
	/// </summary>
	public DirectoryReference LocalRoot;

	/// <summary>
	/// This is the directory that contains the engine.
	/// </summary>
	public DirectoryReference EngineRoot;

	/// <summary>
	/// The directory that contains the project: d:\a\UE\ShooterGame
	/// </summary>
	public DirectoryReference ProjectRoot;

	/// <summary>
	/// The directory that contains the DLC being processed (or null for non-DLC)
	/// </summary>
	public DirectoryReference DLCRoot;

	/// <summary>
	/// The list of AdditionalPluginDirectories from the project.uproject. Files in plugins in these
	/// directories are staged into $(StageRoot)/RemappedPlugins/PluginName.
	/// </summary>
	public List<DirectoryReference> AdditionalPluginDirectories;

	/// <summary>
	///  raw name used for platform subdirectories Win32
	/// </summary>
	public string PlatformDir;

	/// <summary>
	/// Directory to put all of the files in: d:\stagedir\Windows
	/// </summary>
	public DirectoryReference StageDirectory;

	/// <summary>
	/// Directory to put all of the debug files in: d:\stagedir\Windows
	/// </summary>
	public DirectoryReference DebugStageDirectory;

	/// <summary>
	/// Directory to put all of the optional (ie editor only) files in: d:\stagedir\Windows
	/// </summary>
	public DirectoryReference OptionalFileStageDirectory = null;

	/// <summary>
	/// Directory to read all of the optional (ie editor only) files from (written earlier with OptionalFileStageDirectory)
	/// </summary>
	public DirectoryReference OptionalFileInputDirectory = null;

	/// <summary>
	/// If this is specified, any files written into the receipt with the CookerSupportFiles tag will be copied into here during staging
	/// </summary>
	public string CookerSupportFilesSubdirectory = null;

	/// <summary>
	/// Directory name for staged projects
	/// </summary>
	public StagedDirectoryReference RelativeProjectRootForStage;

	/// <summary>
	/// This is what you use to test the engine which uproject you want. Many cases.
	/// </summary>
	public string ProjectArgForCommandLines;

	/// <summary>
	/// The directory containing the cooked data to be staged. This may be different to the target platform, eg. when creating cooked data for dedicated servers.
	/// </summary>
	public DirectoryReference CookSourceRuntimeRootDir;

	/// <summary>
	/// This is the root that we are going to run from. This will be the stage directory if we're staging, or the input directory if not.
	/// </summary>
	public DirectoryReference RuntimeRootDir;

	/// <summary>
	/// This is the project root that we are going to run from. Many cases.
	/// </summary>
	public DirectoryReference RuntimeProjectRootDir;

	/// <summary>
	/// The directory containing the metadata from the cooker
	/// </summary>
	public DirectoryReference MetadataDir;

	/// <summary>
	/// The directory containing the cooker generated data for this platform
	/// </summary>
	public DirectoryReference PlatformCookDir;

	/// <summary>
	/// List of executables we are going to stage
	/// </summary>
	public List<string> StageExecutables;

	/// <summary>
	/// Probably going away, used to construct ProjectArgForCommandLines in the case that we are running staged
	/// </summary>
	public const string UProjectCommandLineArgInternalRoot = "../../../";

	/// <summary>
	/// Probably going away, used to construct the pak file list
	/// </summary>
	public string PakFileInternalRoot = "../../../";

	/// <summary>
	/// Cooked package store data if available
	/// </summary>
	public PackageStoreData PackageStoreData;

	/// <summary>
	/// List of files to be staged
	/// </summary>
	public FilesToStage FilesToStage = new FilesToStage();

	/// <summary>
	/// Map of staged crash reporter file to source location 
	/// </summary>
	public Dictionary<StagedFileReference, FileReference> CrashReporterUFSFiles = new Dictionary<StagedFileReference, FileReference>();

	/// <summary>
	/// List of files to be archived
	/// </summary>
	public Dictionary<string, string> ArchivedFiles = new Dictionary<string, string>();

	/// <summary>
	/// List of restricted folder names which are not permitted in staged build
	/// </summary>
	public HashSet<string> RestrictedFolderNames = new HashSet<string>(StringComparer.OrdinalIgnoreCase);

	/// <summary>
	/// List of directories to remap during staging, allowing moving files to different final paths
	/// This list is read from the +RemapDirectories=(From=, To=) array in the [Staging] section of *Game.ini files
	/// </summary>
	public List<Tuple<StagedDirectoryReference, StagedDirectoryReference>> RemapDirectories = new List<Tuple<StagedDirectoryReference, StagedDirectoryReference>>();

	/// <summary>
	/// List of directories to allow staging, even if they contain restricted folder names
	/// This list is read from the +AllowedDirectories=... array in the [Staging] section of *Game.ini files
	/// </summary>
	public List<StagedDirectoryReference> DirectoriesAllowList = new List<StagedDirectoryReference>();

	/// <summary>
	/// Set of config files which are allow listed to be staged. By default, we warn for config files which are not well known to prevent internal data (eg. editor/server settings)
	/// leaking in packaged builds. This list is read from the +AllowedConfigFiles=... array in the [Staging] section of *Game.ini files.
	/// </summary>
	public HashSet<StagedFileReference> ConfigFilesAllowList = new HashSet<StagedFileReference>();

	/// <summary>
	/// Set of config files which are denied from staging. By default, we warn for config files which are not well known to prevent internal data (eg. editor/server settings)
	/// leaking in packaged builds. This list is read from the +DisallowedConfigFiles=... array in the [Staging] section of *Game.ini files.
	/// </summary>
	public HashSet<StagedFileReference> ConfigFilesDenyList = new HashSet<StagedFileReference>();

	/// <summary>
	/// Set files which are allow listed to be staged that would otherwise be excluded by RestrictedFolderNames.
	/// This list is read from the +ExtraAllowedFiles=... array in the [Staging] section of *Game.ini files.
	/// </summary>
	public HashSet<StagedFileReference> ExtraFilesAllowList = new HashSet<StagedFileReference>();

	/// <summary>
	/// Optional stage handler that during CopyOrWriteManifestFilesToStageDir will handle the copy operation of files and creation
	/// of the plugin manifest file.
	/// </summary>
	public CustomStageCopyHandler CustomStageCopyHandler = null;

	/// <summary>
	/// List of ini keys to strip when staging
	/// </summary>
	public List<string> IniKeyDenyList = null;

	/// <summary>
	/// List of ini sections to strip when staging
	/// </summary>
	public List<string> IniSectionDenyList = null;

	/// <summary>
	/// List of ini suffixes to always stage
	/// </summary>
	public List<string> IniSuffixAllowList = null;

	/// <summary>
	/// List of ini suffixes to never stage
	/// </summary>
	public List<string> IniSuffixDenyList = null;

	/// <summary>
	/// List of localization targets that are not included in staged build. By default, all project Content/Localization targets are automatically staged.
	/// This list is read from the +DisallowedLocalizationTargets=... array in the [Staging] section of *Game.ini files.
	/// </summary>
	public List<string> LocalizationTargetsDenyList = new List<string>();

	/// <summary>
	///  Directory to archive all of the files in: d:\archivedir\Windows
	/// </summary>
	public DirectoryReference ArchiveDirectory;

	/// <summary>
	///  Directory to project binaries
	/// </summary>
	public DirectoryReference ProjectBinariesFolder;

	/// <summary>
	/// The client connects to dedicated server to get data
	/// </summary>
	public bool DedicatedServer;

	/// <summary>
	/// True if this build is staged
	/// </summary>
	public bool Stage;

	/// <summary>
	/// True if this build is archived
	/// </summary>
	public bool Archive;

	/// <summary>
	/// True if this project has code
	/// </summary>	
	public bool IsCodeBasedProject;

	/// <summary>
	/// Project name (name of the uproject file without extension or directory name where the project is localed)
	/// </summary>
	public string ShortProjectName;

	/// <summary>
	/// If true, multiple platforms are being merged together - some behavior needs to change (but not much)
	/// </summary>
	public bool bIsCombiningMultiplePlatforms = false;

	/// <summary>
	/// If true if this platform is using streaming install chunk manifests
	/// </summary>
	public bool PlatformUsesChunkManifests = false;

	/// <summary>
	/// Temporary setting to exclude non cooked packages from I/O store container file(s)
	/// </summary>	
	public bool OnlyAllowPackagesFromStdCookPathInIoStore = false;

	/// <summary>
	/// Allows a target to ignore PakFileRules.ini when the project rules are still needed for most cases
	/// </summary>
	public bool UsePakFileRulesIni = true;

	public DeploymentContext(
		FileReference RawProjectPathOrName,
		DirectoryReference InLocalRoot,
		DirectoryReference BaseStageDirectory,
		DirectoryReference OptionalFileStageDirectory,
		DirectoryReference OptionalFileInputDirectory,
		DirectoryReference BaseArchiveDirectory,
		string CookerSupportFilesSubdirectory,
		Platform InSourcePlatform,
		Platform InTargetPlatform,
		List<UnrealTargetConfiguration> InTargetConfigurations,
		IEnumerable<StageTarget> InStageTargets,
		List<String> InStageExecutables,
		bool InServer,
		bool InCooked,
		bool InStageCrashReporter,
		bool InStage,
		bool InCookOnTheFly,
		bool InArchive,
		bool InProgram,
		bool IsClientInsteadOfNoEditor,
		bool InForceChunkManifests,
		bool InSeparateDebugStageDirectory,
		DirectoryReference InDLCRoot,
		List<DirectoryReference> InAdditionalPluginDirectories
		)
	{
		bStageCrashReporter = InStageCrashReporter;
		RawProjectPath = RawProjectPathOrName;
		DedicatedServer = InServer;
		LocalRoot = InLocalRoot;
		CookSourcePlatform = InSourcePlatform;
		StageTargetPlatform = InTargetPlatform;
		StageTargetConfigurations = new List<UnrealTargetConfiguration>(InTargetConfigurations);
		StageTargets = new List<StageTarget>(InStageTargets);
		StageExecutables = InStageExecutables;
		IsCodeBasedProject = ProjectUtils.IsCodeBasedUProjectFile(RawProjectPath, StageTargetPlatform.PlatformType, StageTargetConfigurations);
		ShortProjectName = ProjectUtils.GetShortProjectName(RawProjectPath);
		Stage = InStage;
		Archive = InArchive;
		DLCRoot = InDLCRoot;
		AdditionalPluginDirectories = InAdditionalPluginDirectories;

		if (CookSourcePlatform != null && InCooked)
		{
			CookPlatform = CookSourcePlatform.GetCookPlatform(DedicatedServer, IsClientInsteadOfNoEditor);
		}
		else if (CookSourcePlatform != null && InProgram)
		{
			CookPlatform = CookSourcePlatform.GetCookPlatform(false, false);
		}
		else
		{
			CookPlatform = "";
		}

		if (StageTargetPlatform != null && InCooked)
		{
			FinalCookPlatform = StageTargetPlatform.GetCookPlatform(DedicatedServer, IsClientInsteadOfNoEditor);
		}
		else if (StageTargetPlatform != null && InProgram)
		{
			FinalCookPlatform = StageTargetPlatform.GetCookPlatform(false, false);
		}
		else
		{
			FinalCookPlatform = "";
		}

		PlatformDir = StageTargetPlatform.PlatformType.ToString();

		if (BaseStageDirectory != null)
		{
			StageDirectory = DirectoryReference.Combine(BaseStageDirectory, FinalCookPlatform);
			DebugStageDirectory = InSeparateDebugStageDirectory ? DirectoryReference.Combine(BaseStageDirectory, FinalCookPlatform + "Debug") : StageDirectory;
		}
		this.OptionalFileStageDirectory = OptionalFileStageDirectory;
		this.OptionalFileInputDirectory = OptionalFileInputDirectory;
		this.CookerSupportFilesSubdirectory = CookerSupportFilesSubdirectory;

		if (BaseArchiveDirectory != null)
		{
			// If the user specifies a path that contains the platform or cooked platform names, don't append to it.
			string PlatformName = StageTargetPlatform.GetStagePlatforms().FirstOrDefault().ToString();
			IEnumerable<string> PathComponents = new DirectoryInfo(BaseArchiveDirectory.FullName).FullName.Split(Path.DirectorySeparatorChar, StringSplitOptions.RemoveEmptyEntries);

			if (!PathComponents.Any(C => C.StartsWith(FinalCookPlatform, StringComparison.OrdinalIgnoreCase) || C.StartsWith(PlatformName, StringComparison.OrdinalIgnoreCase)))
			{
				ArchiveDirectory = DirectoryReference.Combine(BaseArchiveDirectory, FinalCookPlatform);
			}
			else
			{
				ArchiveDirectory = BaseArchiveDirectory;
			}
		}

		if (!FileReference.Exists(RawProjectPath))
		{
			throw new AutomationException("Can't find uproject file {0}.", RawProjectPathOrName);
		}

		EngineRoot = DirectoryReference.Combine(LocalRoot, "Engine");
		ProjectRoot = RawProjectPath.Directory;

		RelativeProjectRootForStage = new StagedDirectoryReference(ShortProjectName);

		ProjectArgForCommandLines = string.Format("-project={0}", CommandUtils.MakePathSafeToUseWithCommandLine(RawProjectPath.FullName));
		CookSourceRuntimeRootDir = RuntimeRootDir = LocalRoot;
		RuntimeProjectRootDir = ProjectRoot;

		// Parse the custom config dir out of the receipts
		foreach (StageTarget Target in StageTargets)
		{
			IEnumerable<ReceiptProperty> Results = Target.Receipt.AdditionalProperties.Where(x => x.Name == "CustomConfig");
			foreach (ReceiptProperty Property in Results)
			{
				string FoundCustomConfig = Property.Value;
				if (String.IsNullOrEmpty(FoundCustomConfig))
				{
					continue;
				}
				else if (String.IsNullOrEmpty(CustomConfig))
				{
					CustomConfig = FoundCustomConfig;
				}
				else if (CustomConfig != FoundCustomConfig)
				{
					throw new AutomationException("Cannot deploy targts with conflicting CustomConfig values! {0} does not match {1}", FoundCustomConfig, CustomConfig);
				}
			}
		}

		if (Stage)
		{
			CommandUtils.CreateDirectory(StageDirectory.FullName);

			RuntimeRootDir = StageDirectory;
			CookSourceRuntimeRootDir = DirectoryReference.Combine(BaseStageDirectory, CookPlatform);
			RuntimeProjectRootDir = DirectoryReference.Combine(StageDirectory, RelativeProjectRootForStage.Name);
			ProjectArgForCommandLines = string.Format("-project={0}", CommandUtils.MakePathSafeToUseWithCommandLine(UProjectCommandLineArgInternalRoot + RelativeProjectRootForStage.Name + "/" + ShortProjectName + ".uproject"));
		}
		if (Archive)
		{
			CommandUtils.CreateDirectory(ArchiveDirectory.FullName);
		}
		ProjectArgForCommandLines = ProjectArgForCommandLines.Replace("\\", "/");
		ProjectBinariesFolder = DirectoryReference.Combine(ProjectUtils.GetClientProjectBinariesRootPath(RawProjectPath, TargetType.Game, IsCodeBasedProject), PlatformDir);

		Dictionary<string,string> SubstitutionVariables = new()
		{
			{ "ProjectName", ShortProjectName }
		};

		// Build a list of restricted folder names. This will comprise all other restricted platforms, plus standard restricted folder names such as NoRedist, NotForLicensees, etc...
		RestrictedFolderNames.UnionWith(PlatformExports.GetPlatformFolderNames());
		RestrictedFolderNames.UnionWith(RestrictedFolder.GetNames());
		foreach (UnrealTargetPlatform StagePlatform in StageTargetPlatform.GetStagePlatforms())
		{
			RestrictedFolderNames.ExceptWith(PlatformExports.GetIncludedFolderNames(StagePlatform));
		}
		RestrictedFolderNames.Remove(StageTargetPlatform.IniPlatformType.ToString());

		// Read the game config files
		ConfigHierarchy GameConfig = ConfigCache.ReadHierarchy(ConfigHierarchyType.Game, ProjectRoot, InTargetPlatform.PlatformType, CustomConfig);

		// Read the list of directories to remap when staging
		List<string> RemapDirectoriesList;
		if (GameConfig.GetArray("Staging", "RemapDirectories", out RemapDirectoriesList))
		{
			foreach (string RemapDirectory in RemapDirectoriesList)
			{
				Dictionary<string, string> Properties;
				if (!ConfigHierarchy.TryParse(RemapDirectory, out Properties))
				{
					throw new AutomationException("Unable to parse '{0}'", RemapDirectory);
				}

				string FromDir;
				if (!Properties.TryGetValue("From", out FromDir))
				{
					throw new AutomationException("Missing 'From' property in '{0}'", RemapDirectory);
				}
				FromDir = Utils.ExpandVariables(FromDir, SubstitutionVariables, bUseAdditionalVariablesOnly:true);

				string ToDir;
				if (!Properties.TryGetValue("To", out ToDir))
				{
					throw new AutomationException("Missing 'To' property in '{0}'", RemapDirectory);
				}
				ToDir = Utils.ExpandVariables(ToDir, SubstitutionVariables, bUseAdditionalVariablesOnly:true);

				RemapDirectories.Add(Tuple.Create(new StagedDirectoryReference(FromDir), new StagedDirectoryReference(ToDir)));
			}
		}

		// Read the list of directories to allow (prevent from warning about from restricted folders)
		List<string> DirectoriesAllowListStrings;
		if (GameConfig.GetArray("Staging", "AllowedDirectories", out DirectoriesAllowListStrings))
		{
			foreach (string AllowedDir in DirectoriesAllowListStrings)
			{
				string AllowedDirAdjusted = Utils.ExpandVariables(AllowedDir, SubstitutionVariables, bUseAdditionalVariablesOnly:true);

				DirectoriesAllowList.Add(new StagedDirectoryReference(AllowedDirAdjusted));
			}
		}

		List<string> LocTargetsDenyListStrings;
		if (GameConfig.GetArray("Staging", "DisallowedLocalizationTargets", out LocTargetsDenyListStrings))
		{
			foreach (string DeniedLocTarget in LocTargetsDenyListStrings)
			{
				LocalizationTargetsDenyList.Add(DeniedLocTarget);
			}
		}

		// Read the list of files which are allow listed to be staged
		ReadAllowDenyFileList(GameConfig, "Staging", "AllowedConfigFiles", ConfigFilesAllowList);
		ReadAllowDenyFileList(GameConfig, "Staging", "DisallowedConfigFiles", ConfigFilesDenyList);
		ReadAllowDenyFileList(GameConfig, "Staging", "ExtraAllowedFiles", ExtraFilesAllowList);

		// Grab the game ini data
		String PackagingIniPath = "/Script/UnrealEd.ProjectPackagingSettings";

		// Read the config deny lists
		GameConfig.GetArray(PackagingIniPath, "IniKeyDenylist", out IniKeyDenyList);
		GameConfig.GetArray(PackagingIniPath, "IniSectionDenylist", out IniSectionDenyList);

		// TODO: Drive these lists from a config file
		IniSuffixAllowList = new List<string>
		{
			".ini",
			"compat.ini",
			"deviceprofiles.ini",
			"engine.ini",
			"enginechunkoverrides.ini",
			"game.ini",
			"gameplaytags.ini",
			"gameusersettings.ini",
			"hardware.ini",
			"input.ini",
			"materialexpressions.ini",
			"scalability.ini",
			"runtimeoptions.ini",
			"installbundle.ini"
		};

		IniSuffixDenyList = new List<string>
		{
			"crypto.ini",
			"editor.ini",
			"editorgameagnostic.ini",
			"editorkeybindings.ini",
			"editorlayout.ini",
			"editorperprojectusersettings.ini",
			"editorsettings.ini",
			"editorusersettings.ini",
			"lightmass.ini",
			"pakfilerules.ini",
			"sourcecontrolsettings.ini"
		};

		// Only stage the Insights base settings with the program.
		if (InProgram)
		{
			IniSuffixAllowList.Add("unrealinsightssettings.ini");
		}
		else
		{
			IniSuffixDenyList.Add("unrealinsightssettings.ini");
		}

		// If we were configured to use manifests across the whole project, then this platform should use manifests.
		// Otherwise, read whether we are generating chunks from the ProjectPackagingSettings ini.
		if (InForceChunkManifests)
		{
			PlatformUsesChunkManifests = true;
		}
		else if (DLCRoot != null)
		{
			PlatformUsesChunkManifests = false;
		}
		else
		{
			bool bSetting = false;
			if (GameConfig.GetBool(PackagingIniPath, "bGenerateChunks", out bSetting))
			{
				PlatformUsesChunkManifests = bSetting;
			}
		}
	}

	/// <summary>
	/// Read a list of allowed or denied files names from a config file
	/// </summary>
	/// <param name="Config">The config hierarchy to read from</param>
	/// <param name="SectionName">The section name</param>
	/// <param name="KeyName">The key name to read from</param>
	/// <param name="FilesRef">Receives a list of file paths</param>
	private static void ReadAllowDenyFileList(ConfigHierarchy Config, string SectionName, string KeyName, HashSet<StagedFileReference> FilesRef)
	{
		List<string> FileNames;
		if (Config.GetArray(SectionName, KeyName, out FileNames))
		{
			foreach (string FileName in FileNames)
			{
				FilesRef.Add(new StagedFileReference(FileName));
			}
		}
	}

	/// <summary>
	/// Finds files to stage under a given base directory.
	/// </summary>
	/// <param name="BaseDir">The directory to search under</param>
	/// <param name="Option">Options for the search</param>
	/// <returns>List of files to be staged</returns>
	public List<FileReference> FindFilesToStage(DirectoryReference BaseDir, StageFilesSearch Option)
	{
		return FindFilesToStage(BaseDir, "*", Option);
	}

	/// <summary>
	/// Finds files to stage under a given base directory.
	/// </summary>
	/// <param name="BaseDir">The directory to search under</param>
	/// <param name="Pattern">Pattern for files to match</param>
	/// <param name="Option">Options for the search</param>
	/// <returns>List of files to be staged</returns>
	public List<FileReference> FindFilesToStage(DirectoryReference BaseDir, string Pattern, StageFilesSearch Option)
	{
		List<FileReference> Files = new List<FileReference>();
		FindFilesToStageInternal(BaseDir, Pattern, Option, Files);
		return Files;
	}

	/// <summary>
	/// Finds files to stage under a given base directory.
	/// </summary>
	/// <param name="BaseDir">The directory to search under</param>
	/// <param name="Pattern">Pattern for files to match</param>
	/// <param name="Option">Options for the search</param>
	/// <param name="Files">List to receive the enumerated files</param>
	private void FindFilesToStageInternal(DirectoryReference BaseDir, string Pattern, StageFilesSearch Option, List<FileReference> Files)
	{
		// if the directory doesn't exist, this will crash in EnumerateFiles
		if (!DirectoryReference.Exists(BaseDir))
		{
			return;
		}

		// Enumerate all the files in this directory
		Files.AddRange(DirectoryReference.EnumerateFiles(BaseDir, Pattern));

		// Recurse through subdirectories if necessary
		if (Option == StageFilesSearch.AllDirectories)
		{
			foreach (DirectoryReference SubDir in DirectoryReference.EnumerateDirectories(BaseDir))
			{
				string Name = SubDir.GetDirectoryName();
				if (!RestrictedFolderNames.Contains(Name))
				{
					FindFilesToStageInternal(SubDir, Pattern, Option, Files);
				}
			}
		}
	}

	/// <summary>
	/// Gets the default location to stage an input file
	/// </summary>
	/// <param name="InputFile">Location of the file in the file system</param>
	/// <returns>Staged file location</returns>
	public StagedFileReference GetStagedFileLocation(FileReference InputFile)
	{
		StagedFileReference OutputFile;
		foreach (DirectoryReference AdditionalPluginDir in AdditionalPluginDirectories)
		{
			if (InputFile.IsUnderDirectory(AdditionalPluginDir))
			{
				// This is a plugin that lives outside of the Engine/Plugins or Game/Plugins directory so needs to be remapped for staging/packaging
				// We need to remap C:\SomePath\PluginName\RelativePath to RemappedPlugins\PluginName\RelativePath
				OutputFile = new StagedFileReference(
					String.Format("RemappedPlugins/{0}", InputFile.MakeRelativeTo(AdditionalPluginDir)));
				return OutputFile;
			}
		}

		if (InputFile.IsUnderDirectory(ProjectRoot))
		{
			OutputFile = StagedFileReference.Combine(RelativeProjectRootForStage, InputFile.MakeRelativeTo(ProjectRoot));
		}
		else if (InputFile.IsUnderDirectory(LocalRoot))
		{
			OutputFile = new StagedFileReference(InputFile.MakeRelativeTo(LocalRoot));
		}
		else if (DLCRoot != null && InputFile.IsUnderDirectory(DLCRoot))
		{
			OutputFile = new StagedFileReference(InputFile.MakeRelativeTo(DLCRoot));
		}
		else
		{
			throw new AutomationException("Can't deploy {0} because it doesn't start with {1} or {2}", InputFile, ProjectRoot, LocalRoot);
		}
		return OutputFile;
	}

	/// <summary>
	/// Stage a single file to its default location
	/// </summary>
	/// <param name="FileType">The type of file being staged</param>
	/// <param name="InputFile">Path to the file</param>
	public void StageFile(StagedFileType FileType, FileReference InputFile)
	{
		StagedFileReference OutputFile = GetStagedFileLocation(InputFile);
		StageFile(FileType, InputFile, OutputFile);
	}

	/// <summary>
	/// Stage a single file
	/// </summary>
	/// <param name="FileType">The type for the staged file</param>
	/// <param name="InputFile">The source file</param>
	/// <param name="OutputFile">The staged file location</param>
	public void StageFile(StagedFileType FileType, FileReference InputFile, StagedFileReference OutputFile)
	{
		FilesToStage.Add(FileType, OutputFile, InputFile);
	}

	/// <summary>
	/// Stage multiple files
	/// </summary>
	/// <param name="FileType">The type for the staged files</param>
	/// <param name="Files">The files to stage</param>
	public void StageFiles(StagedFileType FileType, IEnumerable<FileReference> Files)
	{
		foreach (FileReference File in Files)
		{
			StageFile(FileType, File);
		}
	}

	/// <summary>
	/// Stage multiple files
	/// </summary>
	/// <param name="FileType">The type for the staged files</param>
	/// <param name="InputDir"></param>
	/// <param name="Files">The files to stage</param>
	/// <param name="OutputDir"></param>
	public void StageFiles(StagedFileType FileType, DirectoryReference InputDir, IEnumerable<FileReference> Files, StagedDirectoryReference OutputDir)
	{
		foreach (FileReference File in Files)
		{
			StagedFileReference OutputFile = StagedFileReference.Combine(OutputDir, File.MakeRelativeTo(InputDir));
			StageFile(FileType, File, OutputFile);
		}
	}

	/// <summary>
	/// Stage multiple files
	/// </summary>
	/// <param name="FileType">The type for the staged files</param>
	/// <param name="InputDir">Input directory</param>
	/// <param name="Option">Whether to stage all subdirectories or just the top-level directory</param>
	public void StageFiles(StagedFileType FileType, DirectoryReference InputDir, StageFilesSearch Option)
	{
		StageFiles(FileType, InputDir, "*", Option);
	}

	/// <summary>
	/// Stage multiple files
	/// </summary>
	/// <param name="FileType">The type for the staged files</param>
	/// <param name="InputDir">Input directory</param>
	/// <param name="Option">Whether to stage all subdirectories or just the top-level directory</param>
	/// <param name="OutputDir">Base directory for output files</param>
	public void StageFiles(StagedFileType FileType, DirectoryReference InputDir, StageFilesSearch Option, StagedDirectoryReference OutputDir)
	{
		StageFiles(FileType, InputDir, "*", Option, OutputDir);
	}

	/// <summary>
	/// Stage multiple files
	/// </summary>
	/// <param name="FileType">The type for the staged files</param>
	/// <param name="InputDir">Input directory</param>
	/// <param name="Pattern"></param>
	/// <param name="Option"></param>
	public void StageFiles(StagedFileType FileType, DirectoryReference InputDir, string Pattern, StageFilesSearch Option)
	{
		List<FileReference> InputFiles = FindFilesToStage(InputDir, Pattern, Option);
		foreach (FileReference InputFile in InputFiles)
		{
			StageFile(FileType, InputFile);
		}
	}

	/// <summary>
	/// Stage multiple files
	/// </summary>
	/// <param name="FileType">The type for the staged files</param>
	/// <param name="InputDir">Input directory</param>
	/// <param name="Pattern"></param>
	/// <param name="Option"></param>
	/// <param name="OutputDir">Output directory</param>
	public void StageFiles(StagedFileType FileType, DirectoryReference InputDir, string Pattern, StageFilesSearch Option, StagedDirectoryReference OutputDir)
	{
		List<FileReference> InputFiles = FindFilesToStage(InputDir, Pattern, Option);
		foreach (FileReference InputFile in InputFiles)
		{
			StagedFileReference OutputFile = StagedFileReference.Combine(OutputDir, InputFile.MakeRelativeTo(InputDir));
			StageFile(FileType, InputFile, OutputFile);
		}
	}

	/// <summary>
	/// Stages a file for use by crash reporter.
	/// </summary>
	/// <param name="FileType">The type of the staged file</param>
	/// <param name="InputFile">Location of the input file</param>
	/// <param name="StagedFile">Location of the file in the staging directory</param>
	public void StageCrashReporterFile(StagedFileType FileType, FileReference InputFile, StagedFileReference StagedFile)
	{
		if (FileType == StagedFileType.UFS)
		{
			CrashReporterUFSFiles[StagedFile] = InputFile;
		}
		else
		{
			StageFile(FileType, InputFile, StagedFile);
		}
	}

	/// <summary>
	/// Stage multiple files for use by crash reporter
	/// </summary>
	/// <param name="FileType">The type of the staged file</param>
	/// <param name="InputDir">Location of the input directory</param>
	/// <param name="Option">Whether to stage all subdirectories or just the top-level directory</param>
	public void StageCrashReporterFiles(StagedFileType FileType, DirectoryReference InputDir, StageFilesSearch Option)
	{
		StageCrashReporterFiles(FileType, InputDir, Option, new StagedDirectoryReference(InputDir.MakeRelativeTo(LocalRoot)));
	}

	/// <summary>
	/// Stage multiple files for use by crash reporter
	/// </summary>
	/// <param name="FileType">The type of the staged file</param>
	/// <param name="InputDir">Location of the input directory</param>
	/// <param name="Option">Whether to stage all subdirectories or just the top-level directory</param>
	/// <param name="OutputDir">Location of the output directory within the staging folder</param>
	public void StageCrashReporterFiles(StagedFileType FileType, DirectoryReference InputDir, StageFilesSearch Option, StagedDirectoryReference OutputDir)
	{
		List<FileReference> InputFiles = FindFilesToStage(InputDir, Option);
		foreach (FileReference InputFile in InputFiles)
		{
			StagedFileReference StagedFile = StagedFileReference.Combine(OutputDir, InputFile.MakeRelativeTo(InputDir));
			StageCrashReporterFile(FileType, InputFile, StagedFile);
		}
	}

	public void StageVulkanValidationLayerFiles(ProjectParams Params, StagedFileType FileType, DirectoryReference InputDir, StageFilesSearch Option)
	{
		StageVulkanValidationLayerFiles(Params, FileType, InputDir, Option, new StagedDirectoryReference(InputDir.MakeRelativeTo(LocalRoot)));
	}

	public void StageVulkanValidationLayerFiles(ProjectParams Params, StagedFileType FileType, DirectoryReference InputDir, StageFilesSearch Option, StagedDirectoryReference OutputDir)
	{
		// This needs to match the c++ define VULKAN_HAS_DEBUGGING_ENABLED to avoid mismatched functionality/files
		bool bShouldStageVulkanLayers = !Params.IsProgramTarget && (StageTargetConfigurations.Contains(UnrealTargetConfiguration.Debug) || StageTargetConfigurations.Contains(UnrealTargetConfiguration.Development));
		if (bShouldStageVulkanLayers)
		{
			List<FileReference> InputFiles = FindFilesToStage(InputDir, Option);
			foreach (FileReference InputFile in InputFiles)
			{
				StagedFileReference StagedFile = StagedFileReference.Combine(OutputDir, InputFile.MakeRelativeTo(InputDir));
				StageFile(FileType, InputFile, StagedFile);
			}
		}
	}

	public void StageBuildProductsFromReceipt(TargetReceipt Receipt, bool RequireDependenciesToExist, bool TreatNonShippingBinariesAsDebugFiles)
	{
		// Stage all the build products needed at runtime
		foreach (BuildProduct BuildProduct in Receipt.BuildProducts)
		{
			// allow missing files if needed
			if (RequireDependenciesToExist == false && FileReference.Exists(BuildProduct.Path) == false)
			{
				continue;
			}

			if (BuildProduct.Type == BuildProductType.Executable || BuildProduct.Type == BuildProductType.DynamicLibrary || BuildProduct.Type == BuildProductType.RequiredResource)
			{
				StagedFileType FileTypeToUse = StagedFileType.NonUFS;
				if (TreatNonShippingBinariesAsDebugFiles && Receipt.Configuration != UnrealTargetConfiguration.Shipping)
				{
					FileTypeToUse = StagedFileType.DebugNonUFS;
				}

				StageFile(FileTypeToUse, BuildProduct.Path);
			}
			else if (BuildProduct.Type == BuildProductType.SymbolFile || BuildProduct.Type == BuildProductType.MapFile)
			{
				// Symbol files aren't true dependencies so we can skip if they don't exist
				if (FileReference.Exists(BuildProduct.Path))
				{
					StageFile(StagedFileType.DebugNonUFS, BuildProduct.Path);
				}
			}
		}
	}

	public void StageRuntimeDependenciesFromReceipt(TargetReceipt Receipt, bool RequireDependenciesToExist, bool bUsingPakFile)
	{
		// Patterns to exclude from wildcard searches. Any maps and assets must be cooked. 
		List<string> ExcludePatterns = new List<string>();
		ExcludePatterns.Add(".../*.umap");
		ExcludePatterns.Add(".../*.uasset");

		// Also stage any additional runtime dependencies, like ThirdParty DLLs
		foreach (RuntimeDependency RuntimeDependency in Receipt.RuntimeDependencies)
		{
			// allow missing files if needed
			if ((RequireDependenciesToExist && RuntimeDependency.Type != StagedFileType.DebugNonUFS) || FileReference.Exists(RuntimeDependency.Path))
			{
				StageFile(RuntimeDependency.Type, RuntimeDependency.Path);
			}
		}
	}

	public int ArchiveFiles(string InPath, string Wildcard = "*", bool bRecursive = true, string[] ExcludeWildcard = null, string NewPath = null, UnrealTargetPlatform[] AdditionalPlatforms = null)
	{
		int FilesAdded = 0;

		if (CommandUtils.DirectoryExists(InPath))
		{
			List<string> All = new();
			CommandUtils.FindFilesAndSymlinks(InPath, Wildcard, bRecursive, All);

			HashSet<string> Exclude = new HashSet<string>();
			if (ExcludeWildcard != null)
			{
				foreach (var Excl in ExcludeWildcard)
				{
					var Remove = CommandUtils.FindFiles(Excl, bRecursive, InPath);
					foreach (var File in Remove)
					{
						Exclude.Add(CommandUtils.CombinePaths(File));
					}
				}
			}
			foreach (var AllFile in All)
			{
				var FileToCopy = CommandUtils.CombinePaths(AllFile);
				if (Exclude.Contains(FileToCopy))
				{
					continue;
				}

				if (!bIsCombiningMultiplePlatforms)
				{
					FileReference InputFile = new FileReference(FileToCopy);

					bool OtherPlatform = false;
					foreach (UnrealTargetPlatform Plat in UnrealTargetPlatform.GetValidPlatforms())
					{
						if (Plat != StageTargetPlatform.PlatformType)
						{
							if (AdditionalPlatforms != null && AdditionalPlatforms.Contains(Plat))
							{
								break;
							}

							var Search = FileToCopy;
							if (InputFile.IsUnderDirectory(LocalRoot))
							{
								Search = InputFile.MakeRelativeTo(LocalRoot);
							}
							else if (InputFile.IsUnderDirectory(ProjectRoot))
							{
								Search = InputFile.MakeRelativeTo(ProjectRoot);
							}
							if (Search.IndexOf(CommandUtils.CombinePaths("/" + Plat.ToString() + "/"), 0, StringComparison.InvariantCultureIgnoreCase) >= 0)
							{
								OtherPlatform = true;
								break;
							}
						}
					}
					if (OtherPlatform)
					{
						continue;
					}
				}

				string Dest;
				if (!FileToCopy.StartsWith(InPath))
				{
					throw new AutomationException("Can't archive {0}; it was supposed to start with {1}", FileToCopy, InPath);
				}

				// If the specified a new directory, first we deal with that, then apply the other things
				// this is used to collapse the sandbox, among other things
				if (NewPath != null)
				{
					Dest = FileToCopy.Substring(InPath.Length);
					if (Dest.StartsWith("/") || Dest.StartsWith("\\"))
					{
						Dest = Dest.Substring(1);
					}
					Dest = CommandUtils.CombinePaths(NewPath, Dest);
				}
				else
				{
					Dest = FileToCopy.Substring(InPath.Length);
				}

				if (Dest.StartsWith("/") || Dest.StartsWith("\\"))
				{
					Dest = Dest.Substring(1);
				}

				if (ArchivedFiles.ContainsKey(FileToCopy))
				{
					if (ArchivedFiles[FileToCopy] != Dest)
					{
						throw new AutomationException("Can't archive {0}: it was already in the files to archive with a different destination '{1}'", FileToCopy, Dest);
					}
				}
				else
				{
					ArchivedFiles.Add(FileToCopy, Dest);
				}

				FilesAdded++;
			}
		}

		return FilesAdded;
	}

	private static string GetSanitizedDeviceNameSuffix(string DeviceName)
	{
		if (string.IsNullOrWhiteSpace(DeviceName))
			return string.Empty;

		return "_" + DeviceName
			.Replace(":", "")
			.Replace("/", "")
			.Replace("\\", "")
			.Replace("-", "")
			.Replace(".exe", "");
	}

	public string GetUFSDeploymentDeltaPath(string DeviceName)
	{
		return Path.Combine(StageDirectory.FullName, string.Format("Manifest_DeltaUFSFiles{0}.txt", GetSanitizedDeviceNameSuffix(DeviceName)));
	}

	public string GetNonUFSDeploymentDeltaPath(string DeviceName)
	{
		return Path.Combine(StageDirectory.FullName, string.Format("Manifest_DeltaNonUFSFiles{0}.txt", GetSanitizedDeviceNameSuffix(DeviceName)));
	}

	public string GetUFSDeploymentObsoletePath(string DeviceName)
	{
		return Path.Combine(StageDirectory.FullName, string.Format("Manifest_ObsoleteUFSFiles{0}.txt", GetSanitizedDeviceNameSuffix(DeviceName)));
	}

	public string GetNonUFSDeploymentObsoletePath(string DeviceName)
	{
		return Path.Combine(StageDirectory.FullName, string.Format("Manifest_ObsoleteNonUFSFiles{0}.txt", GetSanitizedDeviceNameSuffix(DeviceName)));
	}

	public string GetNonUFSDeployedManifestFileName(string DeviceName)
	{
		return string.Format("Manifest_NonUFSFiles_{0}{1}.txt", StageTargetPlatform.PlatformType, GetSanitizedDeviceNameSuffix(DeviceName));
	}

	public string GetUFSDeployedManifestFileName(string DeviceName)
	{
		return string.Format("Manifest_UFSFiles_{0}{1}.txt", StageTargetPlatform.PlatformType, GetSanitizedDeviceNameSuffix(DeviceName));
	}

	public static StagedFileReference ApplyDirectoryRemap(DeploymentContext SC, StagedFileReference InputFile)
	{
		StagedFileReference CurrentFile = InputFile;
		foreach (Tuple<StagedDirectoryReference, StagedDirectoryReference> RemapDirectory in SC.RemapDirectories)
		{
			StagedFileReference NewFile;
			if (StagedFileReference.TryRemap(CurrentFile, RemapDirectory.Item1, RemapDirectory.Item2, out NewFile))
			{
				CurrentFile = NewFile;
			}
		}
		return CurrentFile;
	}

	public static StagedFileReference MakeRelativeStagedReference(DeploymentContext SC, FileSystemReference Ref)
	{
		return MakeRelativeStagedReference(SC, Ref, out _);
	}

	public static StagedFileReference MakeRelativeStagedReference(DeploymentContext SC, FileSystemReference Ref, out DirectoryReference RootDir)
	{
		foreach (DirectoryReference AdditionalPluginDir in SC.AdditionalPluginDirectories)
		{
			if (Ref.IsUnderDirectory(AdditionalPluginDir))
			{
				// This is a plugin that lives outside of the Engine/Plugins or Game/Plugins directory so needs to be remapped for staging/packaging
				// We need to remap C:\SomePath\PluginName\RelativePath to RemappedPlugins\PluginName\RelativePath
				string RemainingPath = Ref.MakeRelativeTo(AdditionalPluginDir).Replace('\\', '/');
				int PluginEndIndex = RemainingPath.IndexOf("/");
				if (PluginEndIndex >= 0 && PluginEndIndex < RemainingPath.Length - 1)
				{
					string PluginName = RemainingPath.Substring(0, PluginEndIndex);
					RemainingPath = RemainingPath.Substring(PluginEndIndex + 1);
					RootDir = DirectoryReference.Combine(AdditionalPluginDir, PluginName);
					StagedFileReference StagedFile = new StagedFileReference(String.Format("RemappedPlugins/{0}/{1}", PluginName, RemainingPath));
					return ApplyDirectoryRemap(SC, StagedFile);
				}
			}
		}

		if (Ref.IsUnderDirectory(SC.ProjectRoot))
		{
			RootDir = SC.ProjectRoot;
			return ApplyDirectoryRemap(SC, new StagedFileReference(SC.ShortProjectName + "/" + Ref.MakeRelativeTo(SC.ProjectRoot).Replace('\\', '/')));
		}

		if (Ref.IsUnderDirectory(SC.EngineRoot))
		{
			RootDir = SC.EngineRoot;
			return ApplyDirectoryRemap(SC, new StagedFileReference("Engine/" + Ref.MakeRelativeTo(SC.EngineRoot).Replace('\\', '/')));
		}

		throw new Exception();
	}
	public static FileReference UnmakeRelativeStagedReference(DeploymentContext SC, StagedFileReference Ref)
	{
		// paths will be in the form "Engine/Foo" or "{ProjectName}/Foo" or "RemappedPlugins/{PluginName}/Foo
		// Anything else we don't handle.
		// So, replace the Engine/ with {EngineDir} and {ProjectName}/ with {ProjectDir}, or change PluginDir to RemappedPlugins/{PluginName}
		// with the plugin path from AdditionalPluginDirectories, and then append Foo

		string RemappedPluginsStr = "RemappedPlugins/";
		if (Ref.Name.StartsWith(RemappedPluginsStr, StringComparison.CurrentCultureIgnoreCase))
		{
			int PluginEndIndex = Ref.Name.IndexOf("/", RemappedPluginsStr.Length);
			if (PluginEndIndex >= 0 && PluginEndIndex < Ref.Name.Length - 1)
			{
				string PluginName = Ref.Name.Substring(RemappedPluginsStr.Length, PluginEndIndex - RemappedPluginsStr.Length);
				foreach (DirectoryReference AdditionalPluginDir in SC.AdditionalPluginDirectories)
				{
					DirectoryReference PossiblePluginDir = DirectoryReference.Combine(AdditionalPluginDir, PluginName);
					if (System.IO.Directory.Exists(PossiblePluginDir.FullName))
					{
						return FileReference.Combine(PossiblePluginDir, Ref.Name.Substring(PluginEndIndex + 1));
					}
				}
			}
		}

		if (Ref.Name.StartsWith("Engine/", StringComparison.CurrentCultureIgnoreCase))
		{
			// skip over "Engine/" which is 7 chars long
			return FileReference.Combine(SC.EngineRoot, Ref.Name.Substring(7));
		}

		if (Ref.Name.StartsWith(SC.ShortProjectName + "/", StringComparison.CurrentCultureIgnoreCase))
		{
			return FileReference.Combine(SC.ProjectRoot, Ref.Name.Substring(SC.ShortProjectName.Length + 1));
		}

		throw new Exception($"Don't know how to convert staged file {Ref.Name} to its original editor path, because it is not in a recognized root directory.");
	}
}
