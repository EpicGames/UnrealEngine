// Copyright Epic Games, Inc. All Rights Reserved.
using System;
using System.Collections.Generic;
using System.IO;
using System.Linq;
using System.Threading;
using System.Net;
using System.Text;
using System.Text.Json;
using System.Text.Json.Serialization;
using UnrealBuildBase;
using Microsoft.Extensions.Logging;
using EpicGames.Core;

#pragma warning disable SYSLIB0014

namespace AutomationTool.DeviceReservation
{
	/// <summary>
	/// Co-operatively reserves remote devices for build automation.
	///
	/// The constructor blocks until the specified type and number of devices are available.
	/// The reservation is automatically renewed for the lifetime of this object, and released
	/// when the object is disposed or garbage collected.
	/// </summary>
	public sealed class DeviceReservationAutoRenew : IDisposable
	{
		private Uri ReservationBaseUri;

		private static readonly TimeSpan ReserveTime = TimeSpan.FromMinutes(10);
		private static readonly TimeSpan RenewTime = TimeSpan.FromMinutes(5);

		// Max times to attempt reservation renewal, in case device starvation, reservation service being restarted, etc
		private static readonly int RenewRetryMax = 14;
		private static readonly TimeSpan RenewRetryTime = TimeSpan.FromMinutes(1);

		private Thread RenewThread;
		private AutoResetEvent WaitEvent = new AutoResetEvent(false);

		private Reservation ActiveReservation;
		private List<Device> ReservedDevices;

		/// <summary>
		/// When true, the ActiveReservation will be force deleted upon termination of the auto renewal thread
		/// Force deleted reservations will clear any associated reservation blocks, regardless of the active step
		/// </summary>
		public bool ForceDelete { get; set; } = false;

		/// <summary>
		/// Whether the reservation requires an installation
		/// </summary>
		public bool? InstallRequired => ActiveReservation?.InstallRequired;

		public IReadOnlyList<Device> Devices
		{
			get
			{
				// Return a copy so our list can't be modified.
				return ReservedDevices.Select(d => d.Clone()).ToList();
			}
		}

		/// <summary>
		/// Creates a device reservation for the specified type and number of devices.
		/// Blocks until the devices are available.
		/// </summary>
		/// <param name="InReservationBaseUri">Working directory which contains the devices.xml and reservations.xml files. Usually a network share</param>
		/// <param name="RetryMax"></param>
		/// <param name="PoolID"></param>
		/// <param name="InDeviceTypes">An array of device types to reserve, one for each device requested. These must match the device types listed in devices.xml.</param>
		public DeviceReservationAutoRenew(string InReservationBaseUri, int RetryMax, string PoolID, params string[] InDeviceTypes)
		{
			ReservationBaseUri = new Uri(InReservationBaseUri);

			// Make a device reservation for all the required device types.
			// This blocks until the reservation is successful.
			ActiveReservation = Reservation.Create(ReservationBaseUri, InDeviceTypes, ReserveTime, RetryMax, PoolID);

			StartRenewThread();
		}

		/// <summary>
		/// Creates a device reservation for the specified type and number of devices.
		/// Blocks until the devices are available.
		/// </summary>
		/// <param name="InReservationBaseUri">Working directory which contains the devices.xml and reservations.xml files. Usually a network share</param>
		/// <param name="Reservation">Reservation to continually renew</param>
		public DeviceReservationAutoRenew(string InReservationBaseUri, Reservation Reservation)
		{
			ReservationBaseUri = new Uri(InReservationBaseUri);
			ActiveReservation = Reservation;
			StartRenewThread();
		}

		private void StartRenewThread()
		{
			// Resolve the device IPs
			ReservedDevices = new List<Device>();
			foreach (string DeviceName in ActiveReservation.DeviceNames)
			{
				ReservedDevices.Add(Device.Get(ReservationBaseUri, DeviceName));
			}

			RenewThread = new Thread(DoAutoRenew);
			RenewThread.Start();
		}

		private void DoAutoRenew()
		{
			int RetryCurrent = 0;
			TimeSpan RenewTimeCurrent = RenewTime;

			while (!WaitEvent.WaitOne(RenewTimeCurrent))
			{
				try
				{
					// Renew the reservation on the backend
					if (RetryCurrent <= RenewRetryMax)
					{
						ActiveReservation.Renew(ReservationBaseUri, ReserveTime);
						RetryCurrent = 0;
						RenewTimeCurrent = RenewTime;
					}
				}
				catch (Exception Ex)
				{
					// There was an exception, warn if we've exceeded retry
					if (RetryCurrent == RenewRetryMax)
					{
						Utils.Logger.LogWarning(KnownLogEvents.Horde_BuildHealth_Ignore, "Reservation renewal returned bad status: {Exception}", Ex.Message);
					}

					// try again
					RetryCurrent++;
					RenewTimeCurrent = RenewRetryTime;
				}
			}

			// Delete reservation on server, if the web request fails backend has logic to cleanup reservations
			try
			{
				ActiveReservation.Delete(ReservationBaseUri, ForceDelete);
			}
			catch (Exception Ex)
			{
				Utils.Logger.LogWarning(KnownLogEvents.Horde_BuildHealth_Ignore, "Reservation delete returned bad status: {Exception}", Ex.Message);
			}

		}

		private void StopAutoRenew()
		{
			if (RenewThread != null)
			{
				WaitEvent.Set();
				RenewThread.Join();
				RenewThread = null;
			}
		}

		~DeviceReservationAutoRenew()
		{
			StopAutoRenew();
		}

		public void Dispose()
		{
			StopAutoRenew();
			GC.SuppressFinalize(this);
		}
	}

	public static class Utils
	{

		static public ILogger Logger = EpicGames.Core.Log.Logger;

		public static Uri AppendPath(this Uri BaseUri, string NewPath)
		{
			UriBuilder Builder = new UriBuilder(BaseUri);
			Builder.Path += NewPath;
			return Builder.Uri;
		}

		public static Uri AppendQuery(this Uri BaseUri, string NewQuery)
		{
			UriBuilder Builder = new UriBuilder(BaseUri);
			Builder.Query += NewQuery;
			return Builder.Uri;
		}

		public static T InvokeAPI<T>(Uri UriToRequest, string Method, object ObjectToSerialize)
		{
			JsonSerializerOptions SerializeOptions = new JsonSerializerOptions() { PropertyNameCaseInsensitive = true };
			SerializeOptions.Converters.Add(new TimeSpanJsonConverter());

			HttpWebRequest Request = (HttpWebRequest)WebRequest.Create(UriToRequest);
			Request.UseDefaultCredentials = true;
			Request.Method = Method;

			if (ObjectToSerialize != null)
			{
				Request.ContentType = "application/json";

				using (Stream RequestStream = Request.GetRequestStream())
				{
					var JsonString = JsonSerializer.Serialize(ObjectToSerialize, SerializeOptions);
					StreamWriter Writer = new StreamWriter(RequestStream);
					Writer.Write(JsonString);
					Writer.Flush();
					RequestStream.Flush();
				}
			}

			using (HttpWebResponse Response = (HttpWebResponse)Request.GetResponse())
			using (Stream ResponseStream = Response.GetResponseStream())
			{
				MemoryStream MemoryStream = new MemoryStream();
				ResponseStream.CopyTo(MemoryStream);
				string JsonString = Encoding.UTF8.GetString(MemoryStream.ToArray());
				return JsonSerializer.Deserialize<T>(JsonString, SerializeOptions );
			}
		}

		public static void InvokeAPI(Uri UriToRequest, string Method, object ObjectToSerialize = null)
		{
			HttpWebRequest Request = (HttpWebRequest)WebRequest.Create(UriToRequest);
			Request.UseDefaultCredentials = true;
			Request.Method = Method;

			if (ObjectToSerialize != null)
			{
				Request.ContentType = "application/json";

				using (Stream RequestStream = Request.GetRequestStream())
				{
					var JsonString = JsonSerializer.Serialize(ObjectToSerialize, new JsonSerializerOptions());
					StreamWriter Writer = new StreamWriter(RequestStream);
					Writer.Write(JsonString);
					Writer.Flush();
					RequestStream.Flush();
				}
			}
			else if (Method == "PUT")
			{
				Request.ContentLength = 0;
			}

			using (HttpWebResponse Response = (HttpWebResponse)Request.GetResponse())
			using (Stream ResponseStream = Response.GetResponseStream())
			{
				// Nothing to do here. Error codes throw exceptions.
			}
		}
	}

	public sealed class Reservation
	{
		public string[] DeviceNames { get; set; } = new string[] { };
		public string HostName { get; set; }
		public DateTime StartDateTime { get; set; }
		public TimeSpan Duration { get; set; }
		public Guid Guid { get; set; }
		public static string ReservationDetails = "";
		public bool? InstallRequired { get; set; } = null;

		static readonly Utf8String DeviceIdAnnotation = new Utf8String("DeviceId");

		private sealed class CreateReservationData
		{
			public string[] DeviceTypes { get; set; }
			public string Hostname { get; set; }
			public TimeSpan Duration { get; set; }
			public string ReservationDetails { get; set; }
			public string PoolId { get; set; }
			public string JobId { get; set; }
			public string StepId { get; set; }
			public string DeviceName { get; set; }
			public Tags Tags { get; set; }
		}

		public class Tags : ICloneable
		{
			public enum Type
			{
				Required,
				Preferred,
				Undesired,
				Blocked
			}

			/// <summary>
			/// Adds a tag to the request
			/// </summary>
			/// <param name="Tag">The name of the requested tag</param>
			/// <param name="TagType">How the tag request behaves</param>
			/// <exception cref="NotImplementedException"></exception>
			public void AddTag(string Tag, Type TagType)
			{
				switch (TagType)
				{
					case Type.Required:
						Add(Tag, Required);
						break;
					case Type.Preferred:
						Add(Tag, Preferred);
						break;
					case Type.Undesired:
						Add(Tag, Undesired);
						break;
					case Type.Blocked:
						Add(Tag, Blocked);
						break;
					default:
						throw new NotImplementedException($"Missing AddTag case for {TagType.ToString()}");
				}
			}

			/// <summary>
			/// Adds multiple tags to the request
			/// </summary>
			/// <param name="Tags">The names of the requested tags</param>
			/// <param name="TagType">How the tag request behaves</param>
			/// <exception cref="NotImplementedException"></exception>
			public void AddTags(IEnumerable<string> Tags, Type TagType)
			{
				foreach(string Tag in Tags)
				{
					AddTag(Tag, TagType);
				}
			}

			/// <summary>
			/// Required tags
			/// </summary>
			public IReadOnlyList<string> RequiredTags => Required;

			/// <summary>
			/// Preferred Tags
			/// </summary>
			public IReadOnlyList<string> PreferredTags => Preferred;

			/// <summary>
			/// Undesired Tags
			/// </summary>
			public IReadOnlyList<string> UndesiredTags => Undesired;

			/// <summary>
			/// Blocked Tags
			/// </summary>
			public IReadOnlyList<string> BlockedTags => Blocked;

			/// <summary>
			/// Devices that do not have this tag will not be reserved
			/// </summary>
			private List<string> Required { get; set; } = new List<string>();

			/// <summary>
			/// Devices that have this tag will be selected over devices that do not, provided at least one is available
			/// </summary>
			private List<string> Preferred{ get; set; } = new List<string>();

			/// <summary>
			/// Devices that have this tag will be de-prioritized over devices that do not
			/// </summary>
			private List<string> Undesired { get; set; } = new List<string>();

			/// <summary>
			/// Devices that have this tag will not be reserved
			/// </summary>
			private List<string> Blocked { get; set; } = new List<string>();

			public override string ToString()
			{
				return $"\n\tRequired: {(RequiredTags == null || !RequiredTags.Any() ? string.Empty : string.Join(", ", RequiredTags))}" +
				       $"\n\tPreferred: {(PreferredTags == null || !PreferredTags.Any() ? string.Empty : string.Join(", ", PreferredTags))}" +
				       $"\n\tUndesired: {(UndesiredTags == null || !UndesiredTags.Any() ? string.Empty : string.Join(", ", UndesiredTags))}" +
				       $"\n\tBlocked: {(BlockedTags == null || !BlockedTags.Any() ? string.Empty : string.Join(", ", BlockedTags))}";
			}

			public override bool Equals(object OtherObj)
			{
				if (ReferenceEquals(OtherObj, null))
				{
					return false;
				}

				if (ReferenceEquals(this, OtherObj))
				{
					return true;
				}

				return OtherObj is Tags Other && Other.ToString().Equals(ToString(), StringComparison.OrdinalIgnoreCase);
			}

			public override int GetHashCode()
			{
				return ToString().GetHashCode();
			}

			object ICloneable.Clone() => Clone();
			public Tags Clone()
			{
				Tags Other = (Tags)MemberwiseClone();
				Other.Required = new List<string>(Required);
				Other.Preferred = new List<string>(Preferred);
				Other.Undesired = new List<string>(Undesired);
				Other.Blocked = new List<string>(Blocked);
				return Other;
			}

			/// <summary>
			/// Does this Tag request have any tags associated with it
			/// </summary>
			/// <returns></returns>
			public bool Any()
			{
				return RequiredTags?.Count > 0 || PreferredTags?.Count > 0 || Undesired?.Count > 0 || BlockedTags?.Count > 0;
			}

			private void Add(string Tag, List<string> TagList)
			{
				if (!string.IsNullOrWhiteSpace(Tag))
				{
					TagList.Add(FormatTag(Tag));
				}
			}

			private string FormatTag(string Tag)
			{
				// Trim whitespaces and capitilaze the first letter of each word. E.g. foo bar => Foo Bar
				string[] TagWords = Tag.Trim().Split(' ').Select(Word => string.Concat(Word[0].ToString().ToUpper(), Word.AsSpan(1))).ToArray();
				return string.Join(' ', TagWords);
			}
		}

		public sealed class DeviceErrorRequest
		{
			public string Message { get; set; }
		}

		public static Reservation Create(Uri BaseUri, string[] DeviceTypes, TimeSpan Duration, int RetryMax = 5, string PoolID = "", string DeviceName = "", Tags Tags = null)
		{
			bool bFirst = true;
			TimeSpan RetryTime = TimeSpan.FromMinutes(1);
			int RetryCount = 0;

			while (true)
			{
				if (!bFirst)
				{
					Thread.Sleep(RetryTime);
				}

				bFirst = false;

				Utils.Logger.LogInformation("Requesting device reservation...");

				Exception UnknownException;

				try
				{
					return Utils.InvokeAPI<Reservation>(BaseUri.AppendPath("api/v1/reservations"), "POST", new CreateReservationData()
					{
						DeviceTypes = DeviceTypes,
						Hostname = Unreal.MachineName,
						Duration = Duration,
						ReservationDetails = ReservationDetails,
						PoolId = PoolID,
						JobId = Environment.GetEnvironmentVariable("UE_HORDE_JOBID"),
						StepId = Environment.GetEnvironmentVariable("UE_HORDE_STEPID"),
						DeviceName = DeviceName,
						Tags = Tags
					});
				}
				catch (WebException WebEx)
				{

					if (RetryCount == RetryMax)
					{
						Utils.Logger.LogWarning(KnownLogEvents.Horde_BuildHealth_Ignore, "Device reservation unsuccessful");
						string PoolName = string.IsNullOrEmpty(PoolID) ? "default" : PoolID;
						string FinalMessage = $"Device reservation for {string.Join(", ", DeviceTypes)} from {PoolName} pool unsuccessful";
						string MaxRetryWebExMessage = GetReservationErrorMessage(WebEx);
						if (!string.IsNullOrEmpty(MaxRetryWebExMessage))
						{
							FinalMessage = string.Format("{0} - Caused by: {1}", FinalMessage, MaxRetryWebExMessage);
						}
						else
						{
							FinalMessage = string.Format("{0} - Request returned status {1}", FinalMessage, WebEx.Status.ToString());
						}
						throw new AutomationException(WebEx, FinalMessage);
					}

					string RetryMessage = String.Format("retry {0} of {1} in {2} minutes", RetryCount + 1, RetryMax, RetryTime.Minutes);
					string Message = String.Format("Unknown device server error, {0}", RetryMessage);

					string WebExMessage = GetReservationErrorMessage(WebEx, RetryMessage);

					if (!string.IsNullOrEmpty(WebExMessage))
					{
						Message = WebExMessage;
					}

					Utils.Logger.LogWarning(KnownLogEvents.Horde_BuildHealth_Ignore, Message);
					RetryCount++;
					UnknownException = WebEx;
				}
				catch (Exception Ex)
				{
					UnknownException = Ex;
					Utils.Logger.LogWarning(KnownLogEvents.Horde_BuildHealth_Ignore, "Device reservation unsuccessful: {Exception}", UnknownException.Message);
				}
			}
		}

		private static string GetReservationErrorMessage(WebException WebEx, string RetryMessage = "")
		{
			string Message = string.Empty;
			string RetryMessageEx = string.IsNullOrEmpty(RetryMessage) ? string.Empty : (", " + RetryMessage);

			if (WebEx.Response == null)
			{
				Message = String.Format("Devices service currently not available{0}", RetryMessageEx);
			}
			else if ((WebEx.Response as HttpWebResponse).StatusCode == HttpStatusCode.Conflict)
			{
				Message = String.Format("No devices currently available{0}", RetryMessageEx);
			}
			else
			{
				using (HttpWebResponse Response = (HttpWebResponse)WebEx.Response)
				{
					using (StreamReader Reader = new StreamReader(Response.GetResponseStream()))
					{
						Message = String.Format("WebException on reservation request: {0} : {1} : {2}", WebEx.Message, WebEx.Status, Reader.ReadToEnd());
					}
				}
			}
			return Message;
		}

		public void Renew(Uri BaseUri, TimeSpan NewDuration)
		{
			Utils.InvokeAPI(BaseUri.AppendPath("api/v1/reservations/" + Guid.ToString()), "PUT", NewDuration);
		}

		public void Delete(Uri BaseUri, bool Force = false)
		{
			string Path = $"api/v1/reservations/{Guid}";
			Uri DeleteUri = BaseUri.AppendPath(Path);
			if (Force)
			{
				DeleteUri = DeleteUri.AppendQuery("force=true");
			}
			Utils.InvokeAPI(DeleteUri, "DELETE");
			Utils.Logger.LogInformation("Successfully deleted device reservation \"{GUID}\".", Guid);
		}

		static public void ReportDeviceError(string InBaseUri, string DeviceName, string Error)
		{
			if (String.IsNullOrEmpty(InBaseUri) || String.IsNullOrEmpty(DeviceName))
			{
				return;
			}

			LogValue DeviceId = new LogValue(DeviceIdAnnotation, DeviceName);

			try
			{
				Uri BaseUri = new Uri(InBaseUri);
				Utils.InvokeAPI(BaseUri.AppendPath("api/v1/deviceerror/" + DeviceName), "PUT", new DeviceErrorRequest() { Message = Error });
				Utils.Logger.LogInformation("Reported device problem {DeviceName}: {Error}", DeviceId, Error);
			}
			catch (Exception Ex)
			{
				Utils.Logger.LogWarning(KnownLogEvents.Horde_BuildHealth_Ignore, "Failed to report device {DeviceName}: {Exception}", DeviceId, Ex.Message);
			}
		}

	}

	public sealed class Device
	{
		public string Name { get; set; }
		public string Type { get; set; }
		public string IPOrHostName { get; set; }
		public string PerfSpec { get; set; }
		public string Model { get; set; }
		public TimeSpan AvailableStartTime { get; set; }
		public TimeSpan AvailableEndTime { get; set; }
		public bool Enabled { get; set; }
		public string DeviceData { get; set; }
		public List<string> Tags { get; set; }

		public Device Clone() { return (Device)MemberwiseClone(); }

		public static Device Get(Uri BaseUri, string DeviceName)
		{
			return Utils.InvokeAPI<Device>(BaseUri.AppendPath("api/v1/devices/" + DeviceName), "GET", null);
		}
	}

	public class TimeSpanJsonConverter : JsonConverter<TimeSpan>
	{
		public override bool CanConvert(Type ObjectType)
		{
			return ObjectType == typeof(TimeSpan);
		}

		public override TimeSpan Read(ref Utf8JsonReader Reader, Type TypeToConvert, JsonSerializerOptions Options)
		{
			return TimeSpan.Parse(Reader.GetString());
		}

		public override void Write(Utf8JsonWriter Writer, TimeSpan Value, JsonSerializerOptions Options)
		{
			var StrValue = Value.ToString();
			Writer.WriteStringValue(Options.PropertyNamingPolicy?.ConvertName(StrValue) ?? StrValue);
		}
	}
}
