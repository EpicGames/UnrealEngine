// Copyright Epic Games, Inc. All Rights Reserved.
using System;
using System.Collections;
using System.Collections.Generic;
using System.Diagnostics;
using System.IO;
using UnrealBuildTool;

namespace AutomationTool
{
    public partial class CommandUtils
    {
		/// <summary>
		/// Given a path to a file, strips off the base directory part of the path
		/// </summary>
		/// <param name="InFilePath"></param>
		/// <param name="InBaseDirectory">The base directory, which must be the first part of the path</param>
		/// <returns>The part of the path after the base directory</returns>
		public static string StripBaseDirectory(string InFilePath, string InBaseDirectory)
        {
            var FilePath = CombinePaths(InFilePath);
            var BaseDirectory = CombinePaths(InBaseDirectory);
            if (!FilePath.StartsWith(BaseDirectory, StringComparison.InvariantCultureIgnoreCase))
            {
                throw new AutomationException("Cannot strip the base directory {0} from {1} because it doesn't start with the base directory.", BaseDirectory, FilePath);
            }
            if (BaseDirectory.EndsWith("/") || BaseDirectory.EndsWith("\\"))
            {
                return FilePath.Substring(BaseDirectory.Length);
            }
            return FilePath.Substring(BaseDirectory.Length + 1);
        }


        /// <summary>
        /// Given a path to a "source" file, re-roots the file path to be located under the "destination" folder.  The part of the source file's path after the root folder is unchanged.
        /// </summary>
        /// <param name="FilePath"></param>
        /// <param name="BaseDirectory"></param>
        /// <param name="NewBaseDirectory"></param>
        /// <returns></returns>
        public static string MakeRerootedFilePath(string FilePath, string BaseDirectory, string NewBaseDirectory)
        {
            var RelativeFile = StripBaseDirectory(FilePath, BaseDirectory);
            var DestFile = CombinePaths(NewBaseDirectory, RelativeFile);
            return DestFile;
        }
    }
}
