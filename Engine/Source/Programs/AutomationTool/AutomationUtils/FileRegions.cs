// Copyright Epic Games, Inc. All Rights Reserved.

using System.Collections.Generic;
using System.Linq;
using System.IO;
using System;
using Microsoft.Extensions.Logging;

using static AutomationTool.CommandUtils;

namespace AutomationTool
{
	/// <summary>
	/// Provides an identifier for the type of data within a given file region.
	/// NOTE: Enum values here must match those in Serialization/FileRegions.h
	/// </summary>
	public enum FileRegionType : byte
	{
		None = 0,
		BC1 = 2,
		BC2 = 5,
		BC3 = 6,
		BC4 = 3,
		BC5 = 7
	}

	public struct FileRegion
	{
		public const string RegionsFileExtension = ".uregs";

		public ulong Offset;
		public ulong Length;
		public FileRegionType Type;

		public static List<FileRegion> ReadRegionsFromFile(string Filename)
		{
			// This serialization function must match FFileRegion::SerializeFileRegions in Serialization/FileRegions.cpp
			try
			{
				using (BinaryReader Reader = new BinaryReader(File.OpenRead(Filename)))
				{
					int NumRegions = Reader.ReadInt32();
					List<FileRegion> Results = new List<FileRegion>(NumRegions);
					for (int Index = 0; Index < NumRegions; ++Index)
					{
						FileRegion Region;
						Region.Offset = Reader.ReadUInt64();
						Region.Length = Reader.ReadUInt64();
						Region.Type = (FileRegionType)Reader.ReadByte();
						Results.Add(Region);
					}

					return Results;
				}
			}
			catch (Exception Ex)
			{
				throw new AutomationException(Ex, "Failed to read regions from file \"{0}\".", Filename);
			}
		}

		public static void PrintSummaryTable(IEnumerable<FileRegion> Regions)
        {
			Dictionary<FileRegionType, List<FileRegion>> RegionsByType = (from Region in Regions group Region by Region.Type).ToDictionary(g => g.Key, g => g.ToList());

			Logger.LogInformation(" +-------------------+----------------------------------+---------------+");
			Logger.LogInformation(" |        Type       |           Total Length           |  Num Regions  |");
			Logger.LogInformation(" +-------------------+----------------------------------+---------------+");
			foreach (KeyValuePair<FileRegionType, List<FileRegion>> Pair in RegionsByType)
			{
				// Find the total number of bytes covered by regions with this pattern
				long TotalSizeInBytes = Pair.Value.Sum(r => (long)r.Length);

				var Suffixes = new[] { "  B", "KiB", "MiB", "GiB", "TiB" };
				int Suffix = 0;

				double TotalSizeInUnits = TotalSizeInBytes;
				while (TotalSizeInUnits > 1024 && (Suffix + 1) < Suffixes.Length)
				{
					TotalSizeInUnits = (double)TotalSizeInBytes / (1024L << (Suffix * 10));
					Suffix++;
				}

				Logger.LogInformation("{Message}", String.Format(" | {0,-17} | {1,18:#,##} ({2,7:0.00} {3}) | {4,13:#,##} |",
					Pair.Key.ToString().ToUpperInvariant(),
					TotalSizeInBytes,
					TotalSizeInUnits,
					Suffixes[Suffix],
					Pair.Value.Count));
			}
			Logger.LogInformation(" +-------------------+----------------------------------+---------------+");
		}
	}
}
