// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.IO;
using System.Linq;
using System.Text;
using System.Threading;
using AutomationTool;

using Google.Apis.Auth.OAuth2;
using Google.Apis.Drive.v3;
using Google.Apis.Drive.v3.Data;
using Google.Apis.Services;
using Google.Apis.Util.Store;
using Google.Apis.Upload;
using Microsoft.Extensions.Logging;

using static AutomationTool.CommandUtils;

namespace DriveHelper
{

	/// <summary>
	/// File formats that we support. These are used with a ToString() to get the actual MIME Type string used in our API calls.
	/// Note: The MIME Types for some of these support a conversion to Google Docs and may not be the "pure" types. For example: CSV will only recognize the first sheet of a given spreadsheet.
	/// In many cases, it may be safer to use either PlainText or BinaryDefault rather than your specific format to ensure that the file is uploaded correctly.
	/// <para>If you wish to add a new type, also add a return case to the ToString() function below.</para>
	/// </summary>
	public enum MIMETypes
	{
		Audio,
		Photo,
		Video,
		Unknown,
		GoogleAppScripts,
		GoogleDocs,
		GoogleDrawing,
		GoogleForms,
		GoogleFusionTables,
		GoogleMyMaps,
		GoogleSites,
		GoogleSlides,
		GoogleSheets,
		GoogleDriveFile,
		GoogleDriveFolder,
		ThirdPartyShortcut,
		MSExcel_XLS,
		MSExcel_XLSX,
		MSPowerPoint,
		MSWord_DOC,
		MSWord_DOCX,
		OpenOfficeDoc,
		OpenOfficePresentation,
		OpenOfficeSheet,
		PlainText,
		RichText,
		HTML,
		ZippedHTML,
		XML,
		JS,
		PHP,
		PDF,
		EPUB,
		CSV,
		TSV,
		JPEG,
		PNG,
		SVG,
		GIF,
		BMP,
		ZIP,
		RAR,
		TAR,
		ARJ,
		CAB,
		MP3,
		BinaryDefault
	}

	public static class MIMETypesExtensions
	{
		/// <summary>
		/// Returns the properly formatted MIME Type string as opposed to a string that matches the enum value's name.
		/// For example: An argument of 'MIMETypes.PlainText' returns "text/plain".
		/// </summary>
		public static string ToMimeString(this MIMETypes Type)
		{
			switch (Type)
			{
				case MIMETypes.Audio:
					return "application/vnd.google-apps.audio";
				case MIMETypes.Photo:
					return "application/vnd.google-apps.photo";
				case MIMETypes.Video:
					return "application/vnd.google-apps.video";
				case MIMETypes.Unknown:
					return "application/vnd.google-apps.unknown";
				case MIMETypes.GoogleAppScripts:
					return "application/vnd.google-apps.script";
				case MIMETypes.GoogleDocs:
					return "application/vnd.google-apps.document";
				case MIMETypes.GoogleDrawing:
					return "application/vnd.google-apps.drawing";
				case MIMETypes.GoogleForms:
					return "application/vnd.google-apps.form";
				case MIMETypes.GoogleFusionTables:
					return "application/vnd.google-apps.fusiontable";
				case MIMETypes.GoogleMyMaps:
					return "application/vnd.google-apps.map";
				case MIMETypes.GoogleSites:
					return "application/vnd.google-apps.site";
				case MIMETypes.GoogleSlides:
					return "application/vnd.google-apps.presentation";
				case MIMETypes.GoogleSheets:
					return "application/vnd.google-apps.spreadsheet";
				case MIMETypes.GoogleDriveFile:
					return "application/vnd.google-apps.file";
				case MIMETypes.GoogleDriveFolder:
					return "application/vnd.google-apps.folder";
				case MIMETypes.ThirdPartyShortcut:
					return "application/vnd.google-apps.drive-sdk";
				case MIMETypes.MSExcel_XLS:
					return "application/vnd.ms-excel";
				case MIMETypes.MSExcel_XLSX:
					return "application/vnd.openxmlformats-officedocument.spreadsheetml.sheet";
				case MIMETypes.MSPowerPoint:
					return "application/vnd.openxmlformats-officedocument.presentationml.presentation";
				case MIMETypes.MSWord_DOC:
					return "application/msword";
				case MIMETypes.MSWord_DOCX:
					return "application/vnd.openxmlformats-officedocument.wordprocessingml.document";
				case MIMETypes.OpenOfficeDoc:
					return "application/vnd.oasis.opendocument.text";
				case MIMETypes.OpenOfficePresentation:
					return "application/vnd.oasis.opendocument.presentation";
				case MIMETypes.OpenOfficeSheet:
					return "application/x-vnd.oasis.opendocument.spreadsheet";
				case MIMETypes.PlainText:
					return "text/plain";
				case MIMETypes.RichText:
					return "application/rtf";
				case MIMETypes.HTML:
					return "text/html";
				case MIMETypes.ZippedHTML:
					return "application/zip";
				case MIMETypes.XML:
					return "text/xml";
				case MIMETypes.JS:
					return "text/js";
				case MIMETypes.PHP:
					return "application/x-httpd-php";
				case MIMETypes.PDF:
					return "application/pdf";
				case MIMETypes.EPUB:
					return "application/epub+zip";
				case MIMETypes.CSV:
					return "text/csv";
				case MIMETypes.TSV:
					return "text/tab-separated-values";
				case MIMETypes.JPEG:
					return "image/jpeg";
				case MIMETypes.PNG:
					return "image/png";
				case MIMETypes.SVG:
					return "image/svg+xml";
				case MIMETypes.GIF:
					return "image/gif";
				case MIMETypes.BMP:
					return "image/bmp";
				case MIMETypes.ZIP:
					return "application/zip";
				case MIMETypes.RAR:
					return "application/rar";
				case MIMETypes.TAR:
					return "application/tar";
				case MIMETypes.ARJ:
					return "application/arj";
				case MIMETypes.CAB:
					return "application/cab";
				case MIMETypes.MP3:
					return "audio/mpeg";
				case MIMETypes.BinaryDefault:
					return "application/octet-stream";
				default:
					Logger.LogError("Called ToString() on an unsupported MIME type! Did you forget to add a return case when adding a new type?");
					return "";
			}
		}
	}

	public class DriveServiceHelper
	{

		protected static string[] Scopes = { DriveService.Scope.Drive };

		public DriveService Service
		{
			get; protected set;
		}

		/// <summary>
		/// A helper class that wraps a DriveService with the given auth credentials and provides common functions such as UploadFile()
		/// </summary>
		public DriveServiceHelper(string AppName, string SecretKeyPath, string CredentialPath)
		{
			try
			{
				UserCredential Credential;

				using (FileStream stream = new FileStream(SecretKeyPath, FileMode.Open, FileAccess.Read))
				{
					Credential = GoogleWebAuthorizationBroker.AuthorizeAsync(
						GoogleClientSecrets.Load(stream).Secrets,
						Scopes,
						"user",
						CancellationToken.None,
						new FileDataStore(CredentialPath, true)).Result;
				}

				Service = new DriveService(new BaseClientService.Initializer()
				{
					HttpClientInitializer = Credential,
					ApplicationName = AppName,
				});
			}
			catch (System.Exception ex)
			{
				Console.WriteLine("Error: Failed to create DriveServiceHelper.\r\n\r\nException={0}\r\n\r\nInnerException={1}", ex, ex.InnerException);
				throw ex.InnerException;
			}
		}



		/// <summary>
		/// Creates a new folder and returns the new folder's ID. Creates the folder within a parent folder if ParentFolderId is specified.
		/// </summary>
		public string CreateFolder(string FolderName, string ParentFolderId = "")
		{
			Google.Apis.Drive.v3.Data.File FileMetaData = new Google.Apis.Drive.v3.Data.File();
			FileMetaData.Name = FolderName;
			FileMetaData.MimeType = MIMETypes.GoogleDriveFolder.ToMimeString();
			if (ParentFolderId != "")
			{
				FileMetaData.Parents = new List<string> { ParentFolderId };
			}

			FilesResource.CreateRequest NewFolderRequest = Service.Files.Create(FileMetaData);
			NewFolderRequest.Fields = "id";
			if (ParentFolderId != "")
			{
				NewFolderRequest.Fields += ", parents";
			}

			Google.Apis.Drive.v3.Data.File NewFolder;
			try
			{
				NewFolder = NewFolderRequest.Execute();
			}
			catch (System.Exception ex)
			{
				Console.WriteLine("Error: Failed to execute new folder request.\r\n\r\nException={0}\r\n\r\nInnerException={1}", ex, ex.InnerException);
				throw ex.InnerException;
			}

			return NewFolder.Id;
		}

		/// <summary>
		/// Returns true if a file exists with the given ID. This attempts to get the file then does a null check, so if your intention is to get the file if it exists, you may want to use GetFile() and null check the result instead for fewer API requests.
		/// </summary>
		public bool DoesFileExist(string FileID)
		{
			// TODO GetFile() throws an exception if no file is found with the given ID. Change that handling to give us a better way to accomplish this check.
			Google.Apis.Drive.v3.Data.File NewFile = GetFile(FileID, new string[] { "id" });
			return NewFile != null;
		}

		/// <summary>
		/// Downloads a file to memory then creates a new file for it at the given file path
		/// </summary>
		public void DownloadFile(string FileID, string DestinationFilePathAndName)
		{
			if (!DoesFileExist(FileID))
			{
				Logger.LogWarning("Attempted to download file that does not exist in Google Drive to {Arg0} with FileID {Arg1}.", DestinationFilePathAndName, FileID);
				return;
			}

			FilesResource.GetRequest Request = Service.Files.Get(FileID);
			Request.SupportsTeamDrives = true;
			Request.Fields = "size";
			Google.Apis.Drive.v3.Data.File Response = Request.Execute();
			long FileSize = (Response.Size != null) ? (long)Response.Size : 0;
			using (FileStream Stream = System.IO.File.Create(DestinationFilePathAndName + ".tmp"))
			{
				DateTime TimeStamp = DateTime.Now;
				long Percent = 0;
				Request.MediaDownloader.ProgressChanged += (Google.Apis.Download.IDownloadProgress progress) => HandleDownloadProgressAndCreateFileOnComplete(ref TimeStamp, ref Percent, Stream, FileSize, progress, DestinationFilePathAndName);
				Console.WriteLine("Beginning download...");
				Request.Download(Stream);
			}
		}

		/// <summary>
		/// Downloads a file to memory then creates a new file for it at the given path with a file type to match the ExportFormat.
		/// Note: This function expects the downloaded file to be one of the Google Docs types such as a Document or Sheet and the ExportFormat must be supported for that type.
		/// </summary>
		public void DownloadGoogleDoc(string FileID, string DestinationFilePathAndName, MIMETypes ExportFormat)
		{
			if (!DoesFileExist(FileID))
			{
				Logger.LogWarning("Attempted to download doc that does not exist in Google Drive to {Arg0} with FileID {Arg1}.", DestinationFilePathAndName, FileID);
				return;
			}

			FilesResource.GetRequest Request = Service.Files.Get(FileID);
			Request.SupportsTeamDrives = true;
			Request.Fields = "size";
			Google.Apis.Drive.v3.Data.File Response = Request.Execute();
			long FileSize = (Response.Size != null) ? (long)Response.Size : 0;
			using (FileStream Stream = System.IO.File.Create(DestinationFilePathAndName + ".tmp"))
			{
				DateTime TimeStamp = DateTime.Now;
				long Percent = 0;
				Request.MediaDownloader.ProgressChanged += (Google.Apis.Download.IDownloadProgress Progress) => HandleDownloadProgressAndCreateFileOnComplete(ref TimeStamp, ref Percent, Stream, FileSize, Progress, DestinationFilePathAndName);
				Console.WriteLine("Beginning download...");
				Request.Download(Stream);
			}
		}

		// Handles a non-resumable download by copying to memory and then creating a new file at the destination
		protected void HandleDownloadProgressAndCreateFileOnComplete(ref DateTime TimeStamp, ref long Percent, FileStream FileStream, long FileSize, Google.Apis.Download.IDownloadProgress Progress, string DestinationFilePathAndName)
		{
			switch (Progress.Status)
			{
				case Google.Apis.Download.DownloadStatus.Downloading:
					{
						DateTime Now = DateTime.Now;
						if ((Now - TimeStamp).TotalSeconds >= 5)
						{
							if (FileSize != 0)
							{
								long NewPercent = 100 * Progress.BytesDownloaded / FileSize;
								if (NewPercent != Percent)
								{
									Percent = NewPercent;
									Console.WriteLine($"{Percent}% Downloaded");
								}
							}
							else
							{
								Console.WriteLine($"{Progress.BytesDownloaded / 1024}KB Downloaded");
							}
							TimeStamp = Now;
						}
						break;
					}
				case Google.Apis.Download.DownloadStatus.Failed:
					{
						Logger.LogWarning("Download failed! New file will not be created at {DestinationFilePathAndName}. {Arg1}", DestinationFilePathAndName, Progress.Exception.Message);
						break;
					}
				case Google.Apis.Download.DownloadStatus.Completed:
					{
						Logger.LogInformation("Download completed. Attempting to create file at {DestinationFilePathAndName}.", DestinationFilePathAndName);
						try
						{
							FileStream.Close();
							System.IO.File.Move(FileStream.Name, DestinationFilePathAndName);
						}
						catch (System.Exception ex)
						{
							Console.WriteLine("Error: Failed to rename new file from {0} to {1} !\r\n\r\nException={2}\r\n\r\nInnerException={3}", FileStream.Name, DestinationFilePathAndName, ex, ex.InnerException);
							throw ex.InnerException;
						}
						break;
					}
			}
		}

		/// <summary>
		/// Gets a metadata File object (with specified fields) for the file with the given FileID
		/// </summary>
		public Google.Apis.Drive.v3.Data.File GetFile(string FileID, IList<string> Fields)
		{
			Google.Apis.Drive.v3.Data.File ReturnFile = null;

			try
			{
				FilesResource.GetRequest Request = Service.Files.Get(FileID);
				Request.SupportsTeamDrives = true;
				if (Fields.Count > 0)
				{
					foreach (string Field in Fields)
					{
						Request.Fields += Field + ", ";
					}
					Request.Fields = Request.Fields.TrimEnd(',', ' ');
				}
				ReturnFile = Request.Execute();
			}
			catch (System.Exception ex)
			{
				if (!ex.Message.Contains("File not found:"))
				{
					throw;
				}
			}
			return ReturnFile;
		}

		/// <summary>
		/// Searches for a file by name and outputs its ID to OutFileID.
		/// Returns true if only exactly one file with the matching name was found. Otherwise, does not set the OutFileID.
		/// </summary>
		public bool TryGetFileID(string FileName, out string OutFileID)
		{
			List<Google.Apis.Drive.v3.Data.File> MatchingFiles = SearchFilesByName(FileName);
			if (MatchingFiles.Count != 1)
			{
				OutFileID = string.Empty;
				Logger.LogWarning("Failed to get a file ID when searching for {Arg0}. Found {Arg1} results instead of 1.", FileName, MatchingFiles.Count.ToString());
				return false;
			}
			OutFileID = MatchingFiles[0].Id;
			return true;
		}

		/// <summary>
		/// Creates a new wrapper class for a folder with the given ID.
		/// Will not fail if a folder with the ID does not exist, so check NewFolder.Exists and call CreateFolder() if necessary.
		/// </summary>
		public DriveFolderWrapper GetNewFolderWrapper(string FolderID)
		{
			return new DriveFolderWrapper(this, FolderID);
		}

		/// <summary>
		/// Changes a file's parent folder to the desired folder by ID 
		/// </summary>
		public void MoveFile(string FileID, string DestinationFolderID)
		{
			if (!DoesFileExist(FileID))
			{
				Logger.LogWarning("Could not move file because no file exists with ID {FileID}.", FileID);
			}
			if (!DoesFileExist(DestinationFolderID))
			{
				Logger.LogWarning("Could not move file because destination folder does not exist with ID {DestinationFolderID}.", DestinationFolderID);
			}
			FilesResource.GetRequest Request = Service.Files.Get(FileID);
			Request.Fields = "parents";
			Google.Apis.Drive.v3.Data.File MovingFile = Request.Execute();
			string PreviousParents = String.Join(",", MovingFile.Parents);
			FilesResource.UpdateRequest MoveRequest = Service.Files.Update(new Google.Apis.Drive.v3.Data.File(), FileID);
			MoveRequest.Fields = "id, parents";
			MoveRequest.AddParents = DestinationFolderID;
			MoveRequest.RemoveParents = PreviousParents;
			MovingFile = MoveRequest.Execute();
		}

		/// <summary>
		/// Calls a ListRequest with the given SearchQuery string.
		/// For more info on Google Drive's search queries, refer to: https://developers.google.com/drive/v3/web/search-parameters
		/// </summary>
		public List<Google.Apis.Drive.v3.Data.File> SearchFiles(string SearchQuery)
		{
			List<Google.Apis.Drive.v3.Data.File> FoundFiles = new List<Google.Apis.Drive.v3.Data.File>();
			string PageToken = null;
			do
			{
				FilesResource.ListRequest ListRequest = Service.Files.List();
				ListRequest.Q = SearchQuery;
				ListRequest.Spaces = "drive";
				ListRequest.Fields = "nextPageToken, files(id, name, parents, mimeType, version)";
				ListRequest.PageToken = PageToken;
				ListRequest.SupportsTeamDrives = true;
				ListRequest.IncludeTeamDriveItems = true;
				FileList PageResult = ListRequest.Execute();
				foreach (Google.Apis.Drive.v3.Data.File File in PageResult.Files)
				{
					FoundFiles.Add(File);
				}
				PageToken = PageResult.NextPageToken;
			} while (PageToken != null);

			return FoundFiles;
		}

		/// <summary>
		/// Searches for files with names that match the given FileName. May return no results if matching files are not found.
		/// </summary>
		public List<Google.Apis.Drive.v3.Data.File> SearchFilesByName(string FileName)
		{
			return SearchFiles("name = '" + FileName + "'");
		}

		/// <summary>
		/// Calls a ListRequest with the given SearchQuery string.
		/// For more info on Google Drive's search queries, refer to: https://developers.google.com/drive/v3/web/search-parameters
		/// </summary>
		public List<Google.Apis.Drive.v3.Data.TeamDrive> SearchDrives(string SearchQuery)
		{
			List<Google.Apis.Drive.v3.Data.TeamDrive> FoundDrives = new List<Google.Apis.Drive.v3.Data.TeamDrive>();
			string PageToken = null;
			do
			{
				TeamdrivesResource.ListRequest ListRequest = Service.Teamdrives.List();
				ListRequest.Q = SearchQuery;
				ListRequest.PageToken = PageToken;
				TeamDriveList PageResult = ListRequest.Execute();
				FoundDrives.AddRange(PageResult.TeamDrives);
				PageToken = PageResult.NextPageToken;
			} while (PageToken != null);

			return FoundDrives;
		}

		/// <summary>
		/// Searches a drives with the name that match the given FileName. May return no results if matching drives are not found.
		/// </summary>
		public Google.Apis.Drive.v3.Data.TeamDrive SearchForDriveByName(string DriveName)
		{
			// to query by name requires UseDomainAdminAccess to be true, but it's likely that the user won't have admin access in a corporate environment
			// so the safest solution is to get all the drives the user has access to and manually look up by name (
			List<Google.Apis.Drive.v3.Data.TeamDrive> AllDrives = SearchDrives("");
			return AllDrives.Find(x => x.Name == DriveName);
		}

		/// <summary>
		/// Uploads the given file to the destination with the given new name and returns the new ID. Returns an empty string and logs a warning if the upload fails.
		/// </summary>
		public string UploadFile(FileInfo FileToUpload, string UploadedFileName, string DestinationFolderID, MIMETypes FileType)
		{
			if (FileToUpload == null || !FileToUpload.Exists)
			{
				Logger.LogWarning("Local file could not be uploaded to Google Drive as {UploadedFileName} because file does not exist!", UploadedFileName);
				return string.Empty;
			}

			DriveFolderWrapper OutputDriveFolder = GetNewFolderWrapper(DestinationFolderID);
			if (!OutputDriveFolder.Exists)
			{
				Logger.LogWarning("Local file could not be uploaded to Google Drive because destination folder does not exist! Intended Drive folder ID: {DestinationFolderID}", DestinationFolderID);
				return string.Empty;
			}

			Google.Apis.Drive.v3.Data.File FileMetaData = new Google.Apis.Drive.v3.Data.File();
			FileMetaData.Name = UploadedFileName;
			FileMetaData.Parents = new List<string> { DestinationFolderID };

			FilesResource.CreateMediaUpload Request;
			using (FileStream Stream = new FileStream(FileToUpload.FullName, FileMode.Open))
			{
				Request = Service.Files.Create(FileMetaData, Stream, FileType.ToMimeString());
				Request.Fields = "id";
				Request.Upload();
			}

			if (Request.ResponseBody == null)
			{
				Logger.LogWarning("Upload failed for {Arg0}!", FileToUpload.Name);
				return string.Empty;
			}
			return Request.ResponseBody.Id;
		}
	}

	public class DriveFolderWrapper
	{

		protected DriveServiceHelper ServiceHelper;
		/// <summary>
		/// Whether a file with the matching FolderID exists
		/// </summary>
		public bool Exists
		{
			get
			{
				return ServiceHelper.DoesFileExist(FolderID);
			}
		}

		/// <summary>
		/// The file ID for our wrapped folder. Not guaranteed to represent an existing folder in the Drive.
		/// </summary>
		public string FolderID
		{
			get; protected set;
		}

		/// <summary>
		/// A convenient wrapper that represents an existing folder in Google Drive and provides common functions one may need in the context of working within that folder.
		/// </summary>
		public DriveFolderWrapper(DriveServiceHelper InServiceHelper, string ExistingFolderID)
		{
			ServiceHelper = InServiceHelper;
			FolderID = ExistingFolderID;
			// TODO check/handle if folder doesnt exist
		}

		/// <summary>
		/// A convenient wrapper that creates a new folder in Google Drive and provides common functions one may need in the context of working within that folder.
		/// </summary>
		public DriveFolderWrapper(DriveServiceHelper InServiceHelper, string NewFolderName, string ParentFolderID)
		{
			ServiceHelper = InServiceHelper;
			FolderID = ServiceHelper.CreateFolder(NewFolderName, ParentFolderID);
			// TODO check/handle if folder doesn't exist
		}

		/// <summary>
		/// Creates a new folder within this folder and returns the new folder's ID.
		/// </summary>
		public string CreateSubFolder(string FolderName)
		{
			return ServiceHelper.CreateFolder(FolderName, FolderID);
		}

		/// <summary>
		/// Gets a file metadata object that represents the wrapped folder (which is not guaranteed to exist)
		/// </summary>
		public Google.Apis.Drive.v3.Data.File Get()
		{
			return ServiceHelper.GetFile(FolderID, new string[] { "id", "name", "parents" });
		}

		/// <summary>
		/// Returns true if any file within this folder has the exact FileName
		/// </summary>
		public bool ContainsFileNamed(string FileName)
		{
			List<Google.Apis.Drive.v3.Data.File> ContainedFiles = ListFiles();
			foreach (Google.Apis.Drive.v3.Data.File LocalFile in ContainedFiles)
			{
				if (LocalFile.Name == FileName)
				{
					return true;
				}
			}
			return false;
		}

		/// <summary>
		/// Returns metadata objects for all files whose parents include this folder
		/// </summary>
		public List<Google.Apis.Drive.v3.Data.File> ListFiles()
		{
			return ServiceHelper.SearchFiles("'" + FolderID + "' in parents");
		}

		/// <summary>
		/// Uploads a file to the wrapped Google Drive folder
		/// </summary>
		public string UploadFile(FileInfo FileToUpload, MIMETypes FileType)
		{
			return UploadFile(FileToUpload, FileToUpload.Name, FileType);
		}

		/// <summary>
		/// Uploads a file as the given UploadedFileName to the wrapped Google Drive folder
		/// </summary>
		public string UploadFile(FileInfo FileToUpload, string UploadedFileName, MIMETypes FileType)
		{
			return ServiceHelper.UploadFile(FileToUpload, UploadedFileName, FolderID, FileType);
		}

	}

}
 
