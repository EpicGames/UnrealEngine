// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.IO;
using System.Linq;

using Google.Apis.Auth.OAuth2;
using Google.Apis.Sheets.v4;
using Google.Apis.Sheets.v4.Data;
using System.Threading;
using Google.Apis.Util.Store;
using Google.Apis.Services;
using System.Threading.Tasks;


namespace SheetsHelper
{
	public enum ESheetsDimension
	{
		ROWS,
		COLUMNS
	}

	public enum ESheetsValueInputOption
	{
		RAW,
		USER_ENTERED
	}

	public enum EAuthTokenType 
	{ 
		UserToken,
		ServiceAccountToken
	}

	public class SheetsUtils
	{
		private static DateTime QuotaTimeStart = new DateTime(0);
		private static int QuotaRequestCount = 0;
		private static int MAX_QUERY_COUNT = 100;
		private static int MAX_QUERY_SECONDS = 100;

		// Synchronous method that should be called before Any Google Sheets Query
		// to ensure that a given test instance doesn't push over the limit by itself
		public static void TryCheckQuotaSync()
		{
			DateTime RequestStart = DateTime.Now;
			TimeSpan DiffTime = RequestStart - QuotaTimeStart;

			QuotaRequestCount++;
			if (DiffTime.TotalSeconds > MAX_QUERY_SECONDS)
			{
				// it's been long enough, so we just restart our time-counter
				QuotaTimeStart = RequestStart;
				QuotaRequestCount = 1;
			}
			else
			{
				if (QuotaRequestCount > MAX_QUERY_COUNT)
				{
					// Sleep the remaining time plus one second for safety
					int RemainingMilliseconds = (101 * 1000) - (int)DiffTime.TotalMilliseconds;
					Console.Write(string.Format("Waiting on Google Sheets quota for {0} milliseconds...", RemainingMilliseconds));
					Thread.Sleep(RemainingMilliseconds);
					Console.WriteLine("Complete!");
					QuotaRequestCount -= MAX_QUERY_COUNT;
				}
			}

			Console.WriteLine(string.Format("RequestCount: {0},  QuotaTime: {1}", QuotaRequestCount, (DateTime.Now - QuotaTimeStart).TotalSeconds));
		}

		/// <summary>
		/// Converts a zero-based index into a string in A1 notation. Example: ColumnIndex = 27 should return "AB".
		/// </summary>
		public static string ColumnIndexToA1(int ColumnIndex)
		{
			string A1String = "";
			int Base = 26;
			int RemainingValue = ColumnIndex;
			while (RemainingValue >= 0)
			{
				int RemainderDigit = RemainingValue % Base;
				A1String = (char)(RemainderDigit + 65) + A1String; // (char)65 = 'A'
				RemainingValue = (RemainingValue / Base) - 1;
			}

			return A1String;
		}
	}

	public class SpreadSheetWrapper
	{
		protected Spreadsheet SourceSpreadsheet;
		protected SheetsService Service;
		protected string SpreadsheetID;
        protected BatchUpdateSpreadsheetRequest UpdateSpreadsheetBatchRequest;
        protected BatchUpdateValuesRequest UpdateValuesBatchRequest;
		
        public SpreadSheetWrapper(string AppTitle, string SecretKeyPath, string CredentialPath, string InSpreadsheetID, EAuthTokenType TokenType = EAuthTokenType.UserToken)
		{
			string[] Scopes = { SheetsService.Scope.Spreadsheets };

			if (!File.Exists(SecretKeyPath))
			{
				throw new FileNotFoundException($"Google auth credential file not found: {SecretKeyPath}");
			}

			// Catch any exceptions to report both outer and inner as the Google stuff has a dependency on old Newtonsoft
			// JSON that has been addressed via a rebinding, but has a habit of coming back and is tricky to solve.
			try
			{
				Console.Write("Using Google Auth Type: '{0}'\n", TokenType);

				switch (TokenType)
				{
					case EAuthTokenType.ServiceAccountToken:
						GoogleCredential ServiceAccount = GoogleCredential.FromFile(SecretKeyPath).CreateScoped(SheetsService.Scope.SpreadsheetsReadonly);
						// Create Google Sheets API service, using a Service Account token
						Service = new SheetsService(new BaseClientService.Initializer()
						{
							HttpClientInitializer = ServiceAccount,
							ApplicationName = AppTitle,
						});

						break;
					case EAuthTokenType.UserToken:
						UserCredential UserAccount = null;
						using (var stream = new FileStream(SecretKeyPath, FileMode.Open, FileAccess.Read))
						{
							Task<UserCredential> Tsk = GoogleWebAuthorizationBroker.AuthorizeAsync(
								GoogleClientSecrets.Load(stream).Secrets,
								Scopes,
								"user",
								CancellationToken.None,
								new FileDataStore(CredentialPath, true));

							if (!Tsk.Wait(TimeSpan.FromMinutes(1)))
							{
								throw new TimeoutException("Google Sheets authorization timed out after 1 minute");
							}

							UserAccount = Tsk.Result;

							// Create Google Sheets API service, using a User token
							Service = new SheetsService(new BaseClientService.Initializer()
							{
								HttpClientInitializer = UserAccount,
								ApplicationName = AppTitle,
							});
						}
						break;
					default:
						throw new NotImplementedException($"Google Sheets authorization method {TokenType.ToString()} does not have an implementation.");
				}
			}
			catch (System.Exception ex)
			{
				Console.Write("Error: Failed to create Sheet.\r\n\r\nException={0}\r\n\r\nInnerException={1}", ex, ex.InnerException);
				throw ex.InnerException;
			}

			// Define request parameters.
			SpreadsheetID = InSpreadsheetID;

			Refresh();

			UpdateSpreadsheetBatchRequest = new BatchUpdateSpreadsheetRequest();
			UpdateValuesBatchRequest = new BatchUpdateValuesRequest();
		}

		public void Refresh()
		{
			SheetsUtils.TryCheckQuotaSync();
			SourceSpreadsheet = Service.Spreadsheets.Get(SpreadsheetID).Execute();
		}

        // Gets an existing Sheet by its name or adds a new Sheet with the given name
        public SheetWrapper GetSheet(string Title)
        {
            SheetWrapper ResultSheet = GetSheetByName(Title);

            if (ResultSheet == null)
            {
                ResultSheet = AddSheet(Title);
            }

            return ResultSheet;
        }

        // Gets an existing Sheet with the given name or returns null
        public SheetWrapper GetSheetByName(string Title)
		{
			Refresh();

			Sheet Sheet = SourceSpreadsheet.Sheets.FirstOrDefault(S => S.Properties.Title == Title);

			if (Sheet == null)
			{
				return null;
			}

			return new SheetWrapper(Service, SpreadsheetID, Sheet);
		}

		// Checks if the sheet with the given name exists
		public bool SheetExists(string Title)
		{
			Refresh();

			Sheet Sheet = SourceSpreadsheet.Sheets.FirstOrDefault(S => S.Properties.Title == Title);
			return Sheet != null;
		}

		// Deletes the sheet with the given name
		public void DeleteSheet(string Title)
		{
			Refresh();

			Sheet Sheet = SourceSpreadsheet.Sheets.FirstOrDefault(S => S.Properties.Title == Title);

			if (Sheet != null)
			{
				Request Request = new Request();
				Request.DeleteSheet = new DeleteSheetRequest();
				Request.DeleteSheet.SheetId = Sheet.Properties.SheetId;
				BatchUpdateSpreadsheetRequest BatchRequest = new BatchUpdateSpreadsheetRequest();

				BatchRequest.Requests = new List<Request> { Request };

				SheetsUtils.TryCheckQuotaSync();
				Service.Spreadsheets.BatchUpdate(BatchRequest, SpreadsheetID).Execute();
			}
		}


		public SheetWrapper AddSheet(string Title, int Index = 0)
		{
			Request Request = new Request();
			Request.AddSheet = new AddSheetRequest();
			Request.AddSheet.Properties = new SheetProperties();
			Request.AddSheet.Properties.Title = Title;
			Request.AddSheet.Properties.Index = Index;

			BatchUpdateSpreadsheetRequest BatchRequest = new BatchUpdateSpreadsheetRequest();

			BatchRequest.Requests = new List<Request> { Request };

			SheetsUtils.TryCheckQuotaSync();
			Service.Spreadsheets.BatchUpdate(BatchRequest, SpreadsheetID).Execute();

			return GetSheetByName(Title);
		}

		public SheetWrapper AddSheetAtNextIndexAlphabetically(string Title, int StartIndex = 0)
		{
			Refresh();
			List<Sheet> Sheets = SourceSpreadsheet.Sheets.OrderBy(sheet => sheet.Properties.Index).ToList();

			int SheetIndex = StartIndex;
			while (SheetIndex < Sheets.Count)
			{
				if (string.Compare(Title, Sheets[SheetIndex].Properties.Title) < 0)
				{
					break;
				}
				SheetIndex++;
			}

			return AddSheet(Title, SheetIndex);
		}

		/// <summary>
		/// Queues a new request to add a new sheet to the spreadsheet. Call FlushSpreadsheetBatchRequest() to execute all queued spreadsheet requests.
		/// </summary>
		public void BatchAddSheet(string Title, int index = 0)
        {
			Request Request = new Request();
            Request.AddSheet = new AddSheetRequest();
            Request.AddSheet.Properties = new SheetProperties();
            Request.AddSheet.Properties.Title = Title;
            Request.AddSheet.Properties.Index = index;

            if (UpdateSpreadsheetBatchRequest.Requests == null)
            {
                UpdateSpreadsheetBatchRequest.Requests = new List<Request>();
            }
            UpdateSpreadsheetBatchRequest.Requests.Add(Request);
        }
		
		/// <summary>
		/// Queues a new request to append new columns or rows. Call FlushSpreadsheetBatchRequest() to execute all queued spreadsheet requests.
		/// </summary>
		public void BatchAppendDimension(SheetWrapper Sheet, int RowsOrColumnsToAppend = 1, ESheetsDimension Dimension = ESheetsDimension.ROWS)
        {
			Request Request = new Request();
            Request.AppendDimension = new AppendDimensionRequest();
            Request.AppendDimension.SheetId = Sheet.SheetID;
            Request.AppendDimension.Length = RowsOrColumnsToAppend;
            Request.AppendDimension.Dimension = Enum.GetName(typeof(ESheetsDimension), Dimension);

			if (UpdateSpreadsheetBatchRequest.Requests == null)
            {
                UpdateSpreadsheetBatchRequest.Requests = new List<Request>();
            }
            UpdateSpreadsheetBatchRequest.Requests.Add(Request);

            if (Dimension == ESheetsDimension.ROWS)
            {
                Sheet.Properties.GridProperties.RowCount += RowsOrColumnsToAppend;
            }
            else if (Dimension == ESheetsDimension.COLUMNS)
            {
                Sheet.Properties.GridProperties.ColumnCount += RowsOrColumnsToAppend;
            }
        }
		
		/// <summary>
		/// Queues a new request to insert new columns or rows. Call FlushSpreadsheetBatchRequest() to execute all queued spreadsheet requests.
		/// </summary>
		public void BatchInsertDimension(SheetWrapper Sheet, int RowsOrColumnsToAppend = 1, int StartIndex = 0, ESheetsDimension Dimension = ESheetsDimension.ROWS, bool? bInheritFromBefore = null)
        {
			DimensionRange Range = new DimensionRange();
            Range.SheetId = Sheet.SheetID;
            Range.StartIndex = StartIndex;
            Range.EndIndex = StartIndex + RowsOrColumnsToAppend;
            Range.Dimension = Enum.GetName(typeof(ESheetsDimension), Dimension);

			// create a request to insert the dimension
			Request Request = new Request();
            Request.InsertDimension = new InsertDimensionRequest();
            Request.InsertDimension.Range = Range;
            Request.InsertDimension.InheritFromBefore = bInheritFromBefore;

            if (UpdateSpreadsheetBatchRequest.Requests == null)
            {
                UpdateSpreadsheetBatchRequest.Requests = new List<Request>();
            }
            UpdateSpreadsheetBatchRequest.Requests.Add(Request);

            //update sheet properties
            if(Dimension == ESheetsDimension.ROWS)
            {
                Sheet.Properties.GridProperties.RowCount += RowsOrColumnsToAppend;
            }
            else if(Dimension == ESheetsDimension.COLUMNS)
            {
                Sheet.Properties.GridProperties.ColumnCount += RowsOrColumnsToAppend;
            }
        }

		/// <summary>
		/// Queues a new request to set the values of a range of cells. Call FlushValuesBatchRequest() to execute all queued value requests.
		/// </summary>
		public void BatchUpdateValueRange(SheetWrapper Sheet, string A1Range, IList<IList<object>> Values, ESheetsDimension MajorDimensions = ESheetsDimension.ROWS)
        {
            string RefRange = string.Format("\'{0}\'!{1}", Sheet.Title, A1Range);

            ValueRange EntryRange = new ValueRange();
            EntryRange.Range = RefRange;
            EntryRange.MajorDimension = Enum.GetName(typeof(ESheetsDimension), MajorDimensions);
            EntryRange.Values = Values;

            if(UpdateValuesBatchRequest.Data == null)
            {
                UpdateValuesBatchRequest.Data = new List<ValueRange>();
            }
            UpdateValuesBatchRequest.Data.Add(EntryRange);
        }

		/// <summary>
		/// Queues a new request to set the value of a specific cell. Call FlushValuesBatchRequest() to execute all queued value requests.
		/// </summary>
		public void BatchUpdateCellValue(SheetWrapper Sheet, string A1Cell, object Value)
        {
            BatchUpdateValueRange(Sheet, A1Cell, new List<IList<object>> { new List<object> { Value } });
        }

		/// <summary>
		/// Convenience method to flush batched spreadsheet and then values requests without returning a response.
		/// Avoid using this when you have both inserts and updates batched as the inserts will happen first and the updates may apply to unintended cells.
		/// </summary>
		public void FlushBatchRequests()
		{
			FlushSpreadsheetBatchRequest();
			FlushValuesBatchRequest();
		}

		/// <summary>
		/// Executes all queued spreadsheet requests in a single batched call
		/// </summary>
		public BatchUpdateSpreadsheetResponse FlushSpreadsheetBatchRequest()
        {
            if(UpdateSpreadsheetBatchRequest.Requests == null || UpdateSpreadsheetBatchRequest.Requests.Count() < 1)
            {
                return null;
            }

			UpdateSpreadsheetBatchRequest.IncludeSpreadsheetInResponse = true;
			SheetsUtils.TryCheckQuotaSync();
            BatchUpdateSpreadsheetResponse Response = Service.Spreadsheets.BatchUpdate(UpdateSpreadsheetBatchRequest, SpreadsheetID).Execute();
			SourceSpreadsheet = Response.UpdatedSpreadsheet;

            UpdateSpreadsheetBatchRequest = new BatchUpdateSpreadsheetRequest();
            return Response;
        }
		
		/// <summary>
		/// Executes all queued values requests in a single batched call
		/// </summary>
		public BatchUpdateValuesResponse FlushValuesBatchRequest(ESheetsValueInputOption ValueInputOption = ESheetsValueInputOption.USER_ENTERED)
        {
            if(UpdateValuesBatchRequest.Data == null || UpdateValuesBatchRequest.Data.Count() < 1)
            {
                return null;
            }
			UpdateValuesBatchRequest.ValueInputOption = Enum.GetName(typeof(ESheetsValueInputOption), ValueInputOption);
			SheetsUtils.TryCheckQuotaSync();
            BatchUpdateValuesResponse Response = Service.Spreadsheets.Values.BatchUpdate(UpdateValuesBatchRequest, SpreadsheetID).Execute();

            UpdateValuesBatchRequest = new BatchUpdateValuesRequest();
            return Response;
        }
	}

	public class SheetWrapper
	{
		public SheetsService Service { get; protected set; }

		public string SpreadsheetID { get; protected set; }

		public Sheet SourceSheet { get; protected set; }

		public SheetProperties Properties
		{
			get { return SourceSheet.Properties; }
		}

		public int ColumnCount
		{
			get { return Properties.GridProperties.ColumnCount.Value; }
		}

		public int RowCount
		{
			get { return Properties.GridProperties.RowCount.Value; }
		}

		public int SheetID
		{
			get { return SourceSheet.Properties.SheetId.Value; }
		}

		public string Title
		{
			get { return SourceSheet.Properties.Title; }
		}

		public SheetWrapper(SheetsService InService, string InSpreadsheetID, Sheet InSheet)
		{
			Service = InService;
			SpreadsheetID = InSpreadsheetID;
			SourceSheet = InSheet;
		}

		public void Refresh()
		{
			SheetsUtils.TryCheckQuotaSync();
			Spreadsheet Response = Service.Spreadsheets.Get(SpreadsheetID).Execute();
			SourceSheet = Response.Sheets.FirstOrDefault(S => S.Properties.SheetId == SheetID);
		}

        // Returns a 2D container of the values of cells within the given range. The elements of the outer container are determined by the MajorDimension. For example: this returns a list of lists of cells in each row by default.
        public IList<IList<object>> GetCellValues(string A1Range, SpreadsheetsResource.ValuesResource.GetRequest.MajorDimensionEnum MajorDimension = SpreadsheetsResource.ValuesResource.GetRequest.MajorDimensionEnum.ROWS)
        {
            string Range = A1Range;
            if(!Range.Contains("!"))
            {
                Range = string.Format("'{0}'!{1}", Title, Range);
            }
            SpreadsheetsResource.ValuesResource.GetRequest Request = Service.Spreadsheets.Values.Get(SpreadsheetID, Range);
            Request.MajorDimension = MajorDimension;
			SheetsUtils.TryCheckQuotaSync();
			ValueRange Values = Request.Execute();
			return Values.Values != null ? Values.Values : new List<IList<object>>();
        }

		public void InsertRows(int RowIndex, int RowCount)
		{
			/*
			 * https://developers.google.com/sheets/samples/rowcolumn
			 */

			// define the range
			DimensionRange Range = new DimensionRange();
			Range.SheetId = SheetID;
			Range.StartIndex = RowIndex;
			Range.EndIndex = RowIndex + RowCount;
			Range.Dimension = "ROWS";

			// create a request to insert the dimension
			Request Request = new Request();
			Request.InsertDimension = new InsertDimensionRequest();
			Request.InsertDimension.Range = Range;
			Request.InsertDimension.InheritFromBefore = false;

			// Request to update the spreadsheet
			BatchUpdateSpreadsheetRequest BatchRequest = new BatchUpdateSpreadsheetRequest();
			BatchRequest.Requests = new List<Request> { Request };

			SheetsUtils.TryCheckQuotaSync();
            Service.Spreadsheets.BatchUpdate(BatchRequest, SpreadsheetID).Execute(); // Replace Spreadsheet.SpreadsheetId with your recently created spreadsheet ID

			Properties.GridProperties.RowCount += RowCount;
		}

        public void AppendColumns(int ColumnCount)
        {
			/*
			 * https://developers.google.com/sheets/samples/rowcolumn
			 */

			// create a request to insert the dimension
			Request Request = new Request();
            Request.AppendDimension = new AppendDimensionRequest();
            Request.AppendDimension.Length = ColumnCount;
            Request.AppendDimension.Dimension = "Columns";

			// Request to update the spreadsheet
			BatchUpdateSpreadsheetRequest BatchRequest = new BatchUpdateSpreadsheetRequest();
            BatchRequest.Requests = new List<Request> { Request };

			SheetsUtils.TryCheckQuotaSync();
            Service.Spreadsheets.BatchUpdate(BatchRequest, SpreadsheetID).Execute();

            Properties.GridProperties.ColumnCount += ColumnCount;
        }

        public void AppendRows(int RowCount)
		{
			/*
			 * https://developers.google.com/sheets/samples/rowcolumn
			 */

			// create a request to insert the dimension
			Request Request = new Request();
			Request.AppendDimension = new AppendDimensionRequest();
			Request.AppendDimension.Length = RowCount;
			Request.AppendDimension.Dimension = "Rows";

			// Request to update the spreadsheet
			BatchUpdateSpreadsheetRequest BatchRequest = new BatchUpdateSpreadsheetRequest();
			BatchRequest.Requests = new List<Request> { Request };

			SheetsUtils.TryCheckQuotaSync();
            Service.Spreadsheets.BatchUpdate(BatchRequest, SpreadsheetID).Execute();

			Properties.GridProperties.RowCount += RowCount;
		}

		public void SetRow(object[] Fields, string StartingCell = "A1")
		{
			String RefRange = string.Format("\'{0}\'!{1}", this.Title, StartingCell);

			ValueRange EntryRange = new ValueRange();
			EntryRange.MajorDimension = "ROWS";
			EntryRange.Values = new List<IList<object>> { Fields };
			EntryRange.Range = RefRange;

			BatchUpdateValuesRequest batchRequest = new BatchUpdateValuesRequest();
			batchRequest.ValueInputOption = "RAW";
			batchRequest.Data = new List<ValueRange> { EntryRange };

			SheetsUtils.TryCheckQuotaSync();
            Service.Spreadsheets.Values.BatchUpdate(batchRequest, SpreadsheetID).Execute();
		}

		public void SetRow(List<IList<object>> Fields)
		{
			ValueRange EntryRange = new ValueRange();
			EntryRange.MajorDimension = "ROWS";
			EntryRange.Values = Fields;
			EntryRange.Range = "A1";

			BatchUpdateValuesRequest batchRequest = new BatchUpdateValuesRequest();
			batchRequest.ValueInputOption = "RAW";
			batchRequest.Data = new List<ValueRange> { EntryRange };

			SheetsUtils.TryCheckQuotaSync();
			Service.Spreadsheets.Values.BatchUpdate(batchRequest, SpreadsheetID).Execute();
		}

	}
}
