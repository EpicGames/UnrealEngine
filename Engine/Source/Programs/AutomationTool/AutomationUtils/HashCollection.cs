// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.IO;
using System.Text;
using System.Xml;
using System.Xml.Serialization;
using System.Linq;
using EpicGames.Core;
using Microsoft.Extensions.Logging;
using UnrealBuildBase;

namespace AutomationTool
{

	/// <summary>
	/// Class that holds a single hash entry. Only the hash is used for comparisons but the
	/// name and meta fields can hold related data for debugging etc.
	/// </summary>
	public class HashEntry : IEquatable<HashEntry>
	{
		public string Name { get; set; }
		public string MetaData { get; set; }

		[XmlIgnore]
		public ContentHash Hash { get; set; }
		
		[XmlElement(ElementName = "Hash", DataType = "hexBinary")]
		public byte[] HashBytes
		{
			get { return Hash.Bytes; }
			set { Hash = new ContentHash(value); }
		}

		public HashEntry(string InName, ContentHash InHash, string InMeta)
		{
			Name = InName;
			Hash = InHash;
			MetaData = InMeta;
		}

		// parameterless constructor for serialization
		public HashEntry()
		{
		}

		// Equality check using the underlying content hash
		public bool Equals(HashEntry RHS)
		{
			return Hash == RHS.Hash;
		}

		public override int GetHashCode()
		{
			return Hash.GetHashCode();
		}

		public override string ToString()
		{
			return Hash.ToString();
		}
	}

	/// <summary>
	/// Class that holds a collection of hashes with helpers for adding files, comparisions, and
	/// serialization to and from files
	/// </summary>
	public class HashCollection
	{
		public enum HashType
		{
			MetaData,
			Content
		}

		// underlying hashset that holds our data
		public HashSet<HashEntry> Hashes { get; protected set; }

		public HashCollection()
		{
			Hashes = new HashSet<HashEntry>();
		}

		/// <summary>
		/// Adds a file to our hash collection. If the specified path does not exist an exception
		/// will be thrown
		/// </summary>
		/// <param name="InPath"></param>
		/// <param name="HashType"></param>
		public bool AddFile(FileReference InPath, HashCollection.HashType HashType)
		{
			FileInfo Fi = new FileInfo(InPath.FullName);

			if (!Fi.Exists)
			{
				return false;
			}

			string MetaData = string.Format("File={0} Size={1} LastWrite={2}", InPath, Fi.Length, Fi.LastWriteTimeUtc.ToString());

			// Hash metadata or content
			ContentHash Hash = HashType == HashCollection.HashType.MetaData ? ContentHash.SHA1(MetaData) : ContentHash.SHA1(InPath);

			HashEntry Entry = new HashEntry(InPath.FullName, Hash, MetaData);

			Hashes.Add(Entry);

			return true;
		}

		/// <summary>
		/// Convenience function that adds a range of files
		/// </summary>
		/// <param name="Files"></param>
		/// <param name="HashType"></param>
		public bool AddFiles(IEnumerable<FileReference> Files, HashCollection.HashType HashType)
		{
			foreach (FileReference File in Files)
			{
				if (!AddFile(File, HashType))
				{
					return false;
				}
			}

			return true;
		}

		/// <summary>
		/// Compares two collections by comparing the underlying hashsets
		/// </summary>
		/// <param name="obj"></param>
		/// <returns></returns>
		public override bool Equals(object obj)
		{
			HashCollection RHS = obj as HashCollection;

			if (RHS == null)
			{
				return false;

			}
			return Hashes.SetEquals(RHS.Hashes);
		}

		/// <summary>
		/// Debugging helper that returns true if there's a hash with the specified name
		/// </summary>
		/// <param name="Name"></param>
		/// <returns></returns>
		public bool HasEntryForName(string Name)
		{
			return Hashes.Where(E => E.Name == Name).Any();
		}

		/// <summary>
		/// Debugging helper that returns the hash entry for the specified name
		/// </summary>
		/// <param name="Input"></param>
		/// <returns></returns>
		public HashEntry GetEntryForName(string Input)
		{
			return Hashes.Where(E => E.Name == Input).FirstOrDefault();
		}

		/// <summary>
		/// Logs differences between two collections for debugging
		/// </summary>
		/// <param name="RHS"></param>
		public void LogDifferences(HashCollection RHS)
		{
			ILogger Logger = Log.Logger;
			foreach (HashEntry Entry in Hashes)
			{
				if (!RHS.HasEntryForName(Entry.Name))
				{
					Logger.LogInformation("RHS does not have an entry for {Name}", Entry.Name);
				}
				else if (!RHS.Hashes.Contains(Entry))
				{
					HashEntry RHSEntry = RHS.GetEntryForName(Entry.Name);
					Logger.LogInformation("RHS hash mismatch for {Name}", Entry.Name);
					Logger.LogInformation("LHS:\n\t{Hash}\n\t{Name}\n\t{MetaData}", Entry.Hash, Entry.Name, Entry.MetaData);
					Logger.LogInformation("RHS:\n\t{Hash}\n\t{Name}\n\t{MetaData}", RHSEntry.Hash, RHSEntry.Name, RHSEntry.MetaData);
				}
			}
		}

		/// <summary>
		/// 
		/// </summary>
		/// <returns></returns>
		public override int GetHashCode()
		{
			return EqualityComparer<HashSet<HashEntry>>.Default.GetHashCode(Hashes);
		}

		/// <summary>
		/// Creates a hash collection from a previously serialized file
		/// </summary>
		/// <param name="InPath"></param>
		/// <returns></returns>
		public static HashCollection CreateFromFile(string InPath)
		{
			try
			{
				using (FileStream stream = System.IO.File.OpenRead(InPath))
				{
					XmlSerializer Serializer = new XmlSerializer(typeof(HashCollection));
					return Serializer.Deserialize(stream) as HashCollection;
				}
			}
			catch (Exception Ex)
			{
				Log.Logger.LogWarning(Ex, "Failed to load HashCollection from {InPath}. {Message}", InPath, Ex.Message);
			}

			return null;
		}

		/// <summary>
		/// Saves a hash collection to the specified file path as XML.
		/// </summary>
		/// <param name="OutPath"></param>
		public void SaveToFile(string OutPath)
		{
			XmlSerializer XS = new XmlSerializer(this.GetType());

			//first serialize the object to memory stream,
			//in case of exception, the original file is not corrupted
			using (MemoryStream MS = new MemoryStream())
			{
				XmlWriterSettings Settings = new XmlWriterSettings();
				Settings.Indent = true;
				Settings.IndentChars = ("\t");
				Settings.Encoding = Encoding.UTF8;

				using (XmlWriter Writer = XmlWriter.Create(MS, Settings))
				{
					XS.Serialize(Writer, this);
				}

				MS.Flush();

				string String = Encoding.UTF8.GetString(MS.ToArray());

				Directory.CreateDirectory(Path.GetDirectoryName(OutPath));

				File.WriteAllText(OutPath, String);
			}
		}
	}

	/// <summary>
	/// Extension methods for CsProject support
	/// </summary>
	public static class CsProjectInfoExtensionMethods
	{
		/// <summary>
		/// Adds aall input/output properties of a CSProject to a hash collection
		/// </summary>
		/// <param name="Hasher"></param>
		/// <param name="Project"></param>
		/// <param name="HashType"></param>
		/// <returns></returns>
		public static bool AddCsProjectInfo(this HashCollection Hasher, IDotnetProjectInfo Project, HashCollection.HashType HashType)
		{
			// Get the output assembly and pdb file
			FileReference OutputFile;
			if (!Project.TryGetOutputFile(out OutputFile))
			{
				throw new Exception(String.Format("Unable to get output file for {0}", Project.ProjectPath));
			}
			FileReference DebugFile = OutputFile.ChangeExtension("pdb");

			// build a list of all input and output files from this module
			List<FileReference> DependentFiles = new List<FileReference> { Project.ProjectPath, OutputFile, DebugFile };

			DependentFiles.AddRange(Project.CompileReferences);

			if (!Hasher.AddFiles(DependentFiles, HashType))
			{
				return false;
			}

			return true;
		}
	}
}
