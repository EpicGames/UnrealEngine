// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections;
using System.Collections.Generic;
using System.Diagnostics;
using System.IO;
using System.Net;
using System.Runtime.CompilerServices;
using System.Runtime.Serialization;
using System.Runtime.Serialization.Json;
using System.Text;
using System.Threading;
using System.ComponentModel;
using System.Reflection;
using EpicGames.Core;
using System.Linq;
using Microsoft.Extensions.Logging;

namespace AutomationTool
{
	/// <summary>
	/// Help Attribute.
	/// </summary>
	[AttributeUsage(AttributeTargets.All, AllowMultiple = true)]
	public class HelpAttribute : DescriptionAttribute
	{
		private bool IsParamHelp;

		/// <summary>
		/// Basic constructor for class descriptions.
		/// </summary>
		/// <param name="Description">Class description</param>
		public HelpAttribute(string Description)
			: base(Description)
		{
			IsParamHelp = false;
		}

		/// <summary>
		/// Constructor for parameter descriptions.
		/// </summary>
		/// <param name="ParamName">Parameter name</param>
		/// <param name="Description">Parameter description</param>
		public HelpAttribute(string ParamName, string Description)
			: base(FormatDescription(ParamName, Description))
		{
			IsParamHelp = true;
		}

		/// <summary>
		/// Additional type to display help for.
		/// </summary>
		/// <param name="AdditionalType"></param>
		public HelpAttribute(Type AdditionalType)
		{
			IsParamHelp = false;
			AdditionalHelp = AdditionalType;
		}

		private static string FormatDescription(string Name, string Description)
		{
			return String.Format("-{0} {1}", Name, Description);
		}

		public bool IsParam
		{
			get { return IsParamHelp; }
		}

		public Type AdditionalHelp
		{
			get;
			private set;
		}
	}

	[AttributeUsage(AttributeTargets.Class | AttributeTargets.Interface, AllowMultiple = true)]
	public class ParamHelpAttribute : HelpAttribute
	{
		public enum ParamAction
		{
			Store,
			Store_True,
			Store_False,
			Append,
			Override
		}

		private ParamHelpAttribute(string Description) : base(Description)
		{
		}

		public ParamHelpAttribute() : this("", "")
		{
			// Needed for de-serialization
		}

		public ParamHelpAttribute(string Name, string Description) : base(Name, Description)
		{
			ParamName = Name.TrimStart('-');
			int EqualsInParamNameIndex = ParamName.IndexOf("=");
			if (EqualsInParamNameIndex > 0)
			{
				ParamName = ParamName.Substring(0, EqualsInParamNameIndex);
			}
			_Action = ParamAction.Store;
			Required = false;
			ParamType = typeof(string);
			DefaultValue = null;
			_Choices = null;
			MultiSelectSeparator = null;
			Deprecated = false;
		}

		public string ParamName
		{
			get;
			set;
		}
		
		private string _ParamDescription = string.Empty;
		public string ParamDescription
		{
			get
			{
				if (!string.IsNullOrEmpty(_ParamDescription))
				{
					return _ParamDescription;
				}

				string RequiredStr = Required ? ". Required" : ". Optional";
				string ChoicesStr = "";
				IEnumerable<object> ValidChoices = Choices as IEnumerable<object>;
				if (ValidChoices != null)
				{
					ChoicesStr += ". Choices=[";
					foreach (var Value in ValidChoices)
					{
						if (Value != null)
						{
							ChoicesStr += Value.ToString() + (MultiSelectSeparator != null ? MultiSelectSeparator : "|");
						}
					}
					ChoicesStr = ChoicesStr.Remove(ChoicesStr.Length - 1);
					ChoicesStr += "]";
				}
				string DefaultStr = DefaultValue != null ? ". DefaultValue=" + DefaultValue.ToString() : "";
				return base.Description + string.Format("{0}{1}{2}",
						RequiredStr,
						DefaultStr,
						ChoicesStr
					);
			}
			set
			{
				_ParamDescription = value;
			}
		}

		private ParamAction _Action = ParamAction.Store;
		public ParamAction Action
		{
			get
			{
				return _Action;
			}
			set
			{
				if (value == ParamAction.Store_True || value == ParamAction.Store_False)
				{
					ParamType = typeof(bool);
				}
				_Action = value;
			}
		}

		public object DefaultValue
		{
			get;
			set;
		}

		public Type ParamType
		{
			get;
			set;
		}

		object _Choices;
		public object Choices
		{
			get
			{
				return _Choices;
			}
			set
			{
				IEnumerable Enumerable = value as System.Collections.IEnumerable;
				if (Enumerable != null)
				{
					_Choices = Enumerable;
				}
			}
		}

		public bool Required
		{
			get;
			set;
		}

		public bool IsArgument
		{
			get;
			set;
		}

		public string MultiSelectSeparator
		{
			get;
			set;
		}

		public bool Deprecated
		{
			get;
			set;
		}

		private string _Flag = null;
		public string Flag
		{
			get
			{
				if (_Flag == null)
				{
					return "-" + ParamName;
				}
				return _Flag;
			}
			set
			{
				if (!string.IsNullOrEmpty(value))
				{
					_Flag = value;
				}
			}
		}

		/// <summary>
		/// This string/character separates the flag from the value; ex. -foo=bar
		/// </summary>
		public string ParamKeyValueDelimiter
		{
			get;
			set;
		} = "=";
	}

/// <summary>
/// Base utility function for script commands.
/// </summary>
	public partial class CommandUtils
	{
		/// <summary>
		/// Displays help for the specified command.
		/// <param name="Command">Command type.</param>
		/// </summary>
		public static void Help(Type Command)
		{
			string Description;
			List<string> Params;
			GetTypeHelp(Command, out Description, out Params);
			LogHelp(Command, Description, Params);
		}

		/// <summary>
		/// Displays help for the specified Type.
		/// </summary>
		/// <param name="Command">Type to display help for.</param>
		public static void LogHelp(Type Command)
		{
			string Description;
			List<string> Params;
			GetTypeHelp(Command, out Description, out Params);
			LogHelp(Command, Description, Params);
		}

		/// <summary>
		/// Displays a formatted help for the specified Command type.
		/// </summary>
		/// <param name="Command">Command Type</param>
		/// <param name="Description">Command Desscription</param>
		/// <param name="Params">Command Parameters</param>
		public static void LogHelp(Type Command, string Description, List<string> Params)
		{
			Dictionary<string, string> ParamDict = new Dictionary<string, string>(StringComparer.InvariantCultureIgnoreCase);

			// Extract Params/Descriptions into Key/Value pairs
			foreach (var Param in Params)
			{
				// Find the first space (should be following the param name)
				if (!String.IsNullOrWhiteSpace(Param))
				{
					var ParamName = String.Empty;
					var ParamDesc = String.Empty;

					var SplitPoint = Param.IndexOf(' ');
					if (SplitPoint > 0)
					{
						// Extract the name and description separately
						ParamName = Param.Substring(0, SplitPoint);
						ParamDesc = Param.Substring(SplitPoint + 1, Param.Length - (SplitPoint + 1));
					}
					else
					{
						ParamName = Param;
					}

					// build dictionary using Name and Desc as Key and Value
					if (!ParamDict.ContainsKey(ParamName))
					{
						ParamDict.Add(ParamName, ParamDesc);
					}
					else
					{
						Logger.LogWarning("Duplicated help parameter \"{ParamName}\"", ParamName);
					}
				}
			}

			Logger.LogInformation("");
			HelpUtils.PrintHelp(String.Format("{0} Help:", Command.Name), Description, ParamDict.ToList());
		}

		/// <summary>
		/// Gets the description and a list of parameters for the specified type.
		/// </summary>
		/// <param name="ObjType">Type to get help for.</param>
		/// <param name="ObjectDescription">Description</param>
		/// <param name="Params">List of parameters</param>
		public static void GetTypeHelp(Type ObjType, out string ObjectDescription, out List<string> Params)
		{
			ObjectDescription = String.Empty;
			Params = new List<string>();

			var AllAttributes = ObjType.GetCustomAttributes(false);			
			foreach (var CustomAttribute in AllAttributes)
			{
				HelpAttribute HelpAttribute = CustomAttribute as HelpAttribute;
				if (HelpAttribute != null)
				{
					if (HelpAttribute.IsParam)
					{
						Params.Add(HelpAttribute.Description);
					}
					else if (HelpAttribute.AdditionalHelp != null)
					{
						string DummyDescription;
						List<string> AdditionalParams;
						GetTypeHelp(HelpAttribute.AdditionalHelp, out DummyDescription, out AdditionalParams);
						Params.AddRange(AdditionalParams);
					}
					else
					{
						if (!String.IsNullOrEmpty(ObjectDescription))
						{
							ObjectDescription += Environment.NewLine;
						}
						ObjectDescription += HelpAttribute.Description;
					}
				}
			}

			List<MemberInfo> AllMembers = GetAllMemebers(ObjType);
			foreach (MemberInfo Member in AllMembers)
			{
				var AllMemeberAtributes = Member.GetCustomAttributes(false);
				foreach (var CustomAttribute in AllMemeberAtributes)
				{
					HelpAttribute HelpAttribute = CustomAttribute as HelpAttribute;
					if (HelpAttribute != null && HelpAttribute.IsParam)
					{
						Params.Add(HelpAttribute.Description);
					}
				}
			}
		}

		private static List<MemberInfo> GetAllMemebers(Type ObjType)
		{
			List<MemberInfo> Members = new List<MemberInfo>();
			for (; ObjType != null; ObjType = ObjType.BaseType)
			{
				MemberInfo[] ObjMembers = ObjType.GetMembers(BindingFlags.Public | BindingFlags.NonPublic | BindingFlags.Instance | BindingFlags.Static);
				Members.AddRange(ObjMembers);
			}
			return Members;
		}
	}
}
