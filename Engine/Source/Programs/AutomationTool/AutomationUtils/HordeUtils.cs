// Copyright Epic Games, Inc. All Rights Reserved.

using EpicGames.Horde;
using EpicGames.Horde.Jobs;
using EpicGames.Horde.Secrets;
using Microsoft.Extensions.DependencyInjection;
using Microsoft.Extensions.Logging;
using System;
using System.Collections.Concurrent;
using System.Collections.Generic;
using System.Net.Http;
using System.Threading;
using System.Threading.Tasks;

#nullable enable

namespace AutomationTool
{
	/// <summary>
	/// Helper methods for Horde functionality
	/// </summary>
	public partial class CommandUtils
	{
		static readonly ConcurrentDictionary<SecretId, IReadOnlyDictionary<string, string>> _cachedSecrets = new ConcurrentDictionary<SecretId, IReadOnlyDictionary<string, string>>();

		/// <summary>
		/// Reads a secret from Horde
		/// </summary>
		/// <param name="secretId">Identifier of the secret to retrieve</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns>Values configured for the given secret</returns>
		public static async Task<IReadOnlyDictionary<string, string>> GetHordeSecretAsync(SecretId secretId, CancellationToken cancellationToken = default)
		{
			IReadOnlyDictionary<string, string>? secret;
			if (!_cachedSecrets.TryGetValue(secretId, out secret))
			{
				IHordeClient hordeClient = ServiceProvider.GetRequiredService<IHordeClient>();

				using HordeHttpClient hordeHttpClient = hordeClient.CreateHttpClient();

				GetSecretResponse response;
				try
				{
					response = await hordeHttpClient.GetSecretAsync(secretId);
				}
				catch (HttpRequestException ex) when (ex.StatusCode == System.Net.HttpStatusCode.NotFound)
				{
					throw new AutomationException(ex, $"Secret '{secretId}' was not found on {hordeClient.ServerUrl}");
				}
				catch (HttpRequestException ex) when (ex.StatusCode == System.Net.HttpStatusCode.Forbidden)
				{
					throw new AutomationException(ex, $"User does not have permissions to read '{secretId}' on {hordeClient.ServerUrl}");
				}

				secret = response.Data;
				_cachedSecrets.TryAdd(secretId, secret);
			}
			return secret;
		}

		/// <summary>
		/// Reads a secret from Horde
		/// </summary>
		/// <param name="secretId">Identifier of the secret to retrieve</param>
		/// <param name="propertyName">Name of the property within the secret</param>
		/// <param name="cancellationToken">Cancellation token for the operation</param>
		/// <returns>Values configured for the given secret</returns>
		public static async Task<string> GetHordeSecretAsync(SecretId secretId, string propertyName, CancellationToken cancellationToken = default)
		{
			IReadOnlyDictionary<string, string> secrets = await GetHordeSecretAsync(secretId, cancellationToken);
			if (!secrets.TryGetValue(propertyName, out string? propertyValue))
			{
				throw new AutomationException("Secret '{secretId}' does not contain a value for '{propertyName}'");
			}
			return propertyValue;
		}

		public static async Task AddHordeJobMetadata(JobId id, IEnumerable<string>? jobMetaData = null, Dictionary<string, List<string>>? stepMetaData = null, CancellationToken cancellationToken = default)
		{
			IHordeClient hordeClient = ServiceProvider.GetRequiredService<IHordeClient>();

			using HordeHttpClient hordeHttpClient = hordeClient.CreateHttpClient();

			bool bConnected = hordeHttpClient.CheckConnectionAsync(cancellationToken).Result;

			if (!bConnected)
			{
				Logger.LogWarning("No connection to the Horde Server when tagging job metadata");
				return;
			}

			await hordeHttpClient.PutJobMetadataAsync(id, jobMetaData, stepMetaData, cancellationToken);			
		}
	}
}
