// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.IO;
using System.Linq;
using System.Text;
using System.Diagnostics;
using EpicGames.Core;
using Microsoft.Extensions.Logging;
using System.Runtime.CompilerServices;

namespace AutomationTool
{
	/// <summary>
	/// Host platform abstraction
	/// </summary>
	public abstract class HostPlatform
	{
		protected static ILogger Logger => Log.Logger;

		/// <summary>
		/// Current running host platform.
		/// </summary>
		public static readonly HostPlatform Current = Initialize();

		/// <summary>
		/// Initializes the current platform.
		/// </summary>
		private static HostPlatform Initialize()
		{
			switch (RuntimePlatform.Current)
			{
				case RuntimePlatform.Type.Windows: return new WindowsHostPlatform();
				case RuntimePlatform.Type.Mac:     return new MacHostPlatform();
				case RuntimePlatform.Type.Linux:   return new LinuxHostPlatform();
			}
			throw new Exception ("Unhandled runtime platform " + Environment.OSVersion.Platform);
		}

		/// <summary>
		/// Gets the build executable filename for NET Framework projects e.g. msbuild
		/// </summary>
		/// <returns></returns>
		abstract public string GetFrameworkMsbuildExe();

		/// <summary>
		/// Folder under UE/ to the platform's binaries.
		/// </summary>
		abstract public string RelativeBinariesFolder { get; }

		/// <summary>
		/// Full path to the UnrealEditor executable for the current platform.
		/// </summary>
		/// <param name="UnrealExe"></param>
		/// <returns></returns>
		abstract public string GetUnrealExePath(string UnrealExe);

		/// <summary>
		/// Log folder for local builds.
		/// </summary>
		abstract public string LocalBuildsLogFolder { get; }

		/// <summary>
		/// Name of the p4 executable.
		/// </summary>
		abstract public string P4Exe { get; }

		/// <summary>
		/// Creates a process and sets it up for the current platform.
		/// </summary>
		/// <param name="AppName"></param>
		/// <returns></returns>
		abstract public Process CreateProcess(string AppName);

		/// <summary>
		/// Sets any additional options for running an executable.
		/// </summary>
		/// <param name="AppName"></param>
		/// <param name="Options"></param>
		/// <param name="CommandLine"></param>
		abstract public void SetupOptionsForRun(ref string AppName, ref CommandUtils.ERunOptions Options, ref string CommandLine);

		/// <summary>
		/// Sets the console control handler for the current platform.
		/// </summary>
		/// <param name="Handler"></param>
		abstract public void SetConsoleCtrlHandler(ProcessManager.CtrlHandlerDelegate Handler);

		/// <summary>
		/// Returns the type of the host editor platform.
		/// </summary>
		abstract public UnrealBuildTool.UnrealTargetPlatform HostEditorPlatform { get; }

		/// <summary>
		/// Returns the type of the current running host platform
		/// </summary>
		public static UnrealBuildTool.UnrealTargetPlatform Platform { get => Current.HostEditorPlatform; }

		/// <summary>
		/// Returns the pdb file extenstion for the host platform.
		/// </summary>
		abstract public string PdbExtension { get; }

		/// <summary>
		/// List of processes that can't not be killed
		/// </summary>
		abstract public string[] DontKillProcessList { get; }
	}
}
