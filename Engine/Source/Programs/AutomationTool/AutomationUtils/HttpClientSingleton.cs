// Copyright Epic Games, Inc. All Rights Reserved.
using Microsoft.Extensions.Http;
using Polly;
using Polly.Extensions.Http;
using Polly.Retry;
using System;
using System.Net.Http;

namespace AutomationUtils
{
	// This is used for instances where DI cannot be used for the HTTPClient
	// More info: https://learn.microsoft.com/en-us/dotnet/fundamentals/networking/http/httpclient-guidelines
	public class HttpClientSingleton<T>
	{
		private static readonly Lazy<HttpClient> internalInstance =
			new Lazy<HttpClient>(() =>
			{
				AsyncRetryPolicy<HttpResponseMessage> retryPolicy = HttpPolicyExtensions
				// 408, 5XX responses 
				.HandleTransientHttpError()
				// 404, 504 should have been covered by HandleTransientHttpError but adding here to be safe
				.OrResult(msg => msg.StatusCode == System.Net.HttpStatusCode.NotFound || 
								 msg.StatusCode == System.Net.HttpStatusCode.GatewayTimeout)
				.WaitAndRetryAsync(3, retryAttempt => TimeSpan.FromSeconds(Math.Pow(2, retryAttempt)));

				// per MS documentation, with a lifetime specified we shouldn't have socket exhaustion issues
				SocketsHttpHandler socketHandler = new SocketsHttpHandler { PooledConnectionLifetime = TimeSpan.FromMinutes(15) };
				PolicyHttpMessageHandler pollyHandler = new PolicyHttpMessageHandler(retryPolicy)
				{
					InnerHandler = socketHandler,
				};

				return new HttpClient(pollyHandler);
			});
		public static HttpClient Client { get => internalInstance.Value; }
	}
}
