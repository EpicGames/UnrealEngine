// Copyright Epic Games, Inc. All Rights Reserved.

using EpicGames.Core;
using ImageMagick;
using Microsoft.Extensions.Logging;
using System;

#nullable enable

namespace AutomationTool
{
	/// <summary>
	/// Performs a set of operations on an existing image
	/// </summary>
	public class ImagePipelineBuilder : IDisposable
	{
		private static ILogger Logger => Log.Logger;

		private MagickImage? image = null;

		public ImagePipelineBuilder(string ImagePath)
		{
			try
			{
				image = new MagickImage(ImagePath);
			}
			catch (MagickException ex)
			{
				Logger.LogError(ex, "Failed to create new MagickImage from {Path}", ImagePath);
			}
		}

		/// <inheritdoc/>
		public void Dispose()
		{
			image?.Dispose();
			image = null;
		}

		/// <summary>
		/// Rotates an image clockwise
		/// </summary>
		/// <param name="degrees">Degrees to rotate the image by</param>
		/// <returns>A reference to <see cref="ImagePipelineBuilder"/></returns>
		public ImagePipelineBuilder Rotate(double degrees)
		{
			image?.Rotate(degrees);
			return this;
		}

		/// <summary>
		/// Flips the image vertically. Can be chained with <seealso cref="FlipHorizontally"/>
		/// </summary>
		/// <returns>A reference to <see cref="ImagePipelineBuilder"/></returns>
		public ImagePipelineBuilder FlipVertically()
		{
			image?.Flip();
			return this;
		}

		/// <summary>
		/// Flips the image horizontally. Can be chained with <seealso cref="FlipVertically"/>
		/// </summary>
		/// <returns>A reference to <see cref="ImagePipelineBuilder"/></returns>
		public ImagePipelineBuilder FlipHorizontally()
		{
			image?.Flop();
			return this;
		}

		/// <summary>
		/// Scales the image's current width and height by the specified value
		/// </summary>
		/// <param name="Scale">Value to scale by</param>
		/// <returns>A reference to <see cref="ImagePipelineBuilder"/></returns>
		public ImagePipelineBuilder Scale(float Scale)
		{
			if (image is null)
			{
				return this;
			}

			try
			{
				image.Scale((uint)(image!.Width * Scale), (uint)(image!.Height * Scale));
			}
			catch (MagickException ex)
			{
				Logger.LogError(ex, "Exception encountered when trying to scale the image {Width}x{Height} by {Scale}", image.Width, image.Height, Scale);
			}
			return this;
		}

		/// <summary>
		/// Adjusts the images compression level for JPEG and PNG images
		/// </summary>
		/// <param name="Quality">Compression level quality</param>
		/// <returns>A reference to <see cref="ImagePipelineBuilder"/></returns>
		public ImagePipelineBuilder CompressionLevel(uint Quality)
		{
			if (image is not null)
			{
				image.Quality = Quality;
			}

			return this;
		}

		/// <summary>
		/// Saves the image as a JPEG
		/// </summary>
		/// <param name="OutputPath">Absolute path of the image output</param>
		/// <returns>True if the image was able to be successfully saved, false otherwise</returns>
		public bool SaveAsJpeg(string OutputPath)
		{
			if (image is null)
			{
				return false;
			}

			image.Format = MagickFormat.Jpeg;
			return Save(OutputPath);
		}

		/// <summary>
		/// Saves the image as a PNG
		/// </summary>
		/// <param name="OutputPath">Absolute path of the image output</param>
		/// <returns>True if the image was able to be successfully saved, false otherwise</returns>
		public bool SaveAsPng(string OutputPath)
		{
			if (image is null)
			{
				return false;
			}

			image.Format = MagickFormat.Png;
			return Save(OutputPath);
		}

		/// <summary>
		/// Saves the image using it's current format
		/// </summary>
		/// <param name="OutputPath">Absolute path of the image output</param>
		/// <returns>True if the image was able to be successfully saved, false otherwise</returns>
		public bool Save(string OutputPath)
		{
			if (image is null)
			{
				return false;
			}

			try
			{
				image.Write(OutputPath);
				return true;
			}
			catch (MagickException ex)
			{
				Logger.LogError(ex, "Failed to save image to {Path}", OutputPath);
			}

			return false;
		}
	}
}
