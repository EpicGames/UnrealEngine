// Copyright Epic Games, Inc. All Rights Reserved.

using ImageMagick;


namespace AutomationTool
{
	public static class ImageUtils
	{
		/// <summary>
		/// Resave an image file as a .jpg file, with the specified quality and scale.
		/// </summary>
		public static void ResaveImageAsJpgWithScaleAndQuality(string SrcImagePath, string DstImagePath, float Scale, int JpgQuality)
		{
			ResaveImageScaleAndQuality(SrcImagePath, DstImagePath, MagickFormat.Jpeg, Scale, (uint)JpgQuality);
		}

		/// <summary>
		/// Resave an image file as a .png file, with the specified quality and scale.
		/// </summary>
		public static void ResaveImageAsPngWithScaleAndQuality(string SrcImagePath, string DstImagePath, float Scale, int PngQuality)
		{
			ResaveImageScaleAndQuality(SrcImagePath, DstImagePath, MagickFormat.Png, Scale, (uint)PngQuality);
		}

		static void ResaveImageScaleAndQuality(string SrcImagePath, string DstImagePath, MagickFormat Format, float Scale, uint Quality)
		{
			using (MagickImage image = new MagickImage(SrcImagePath))
			{
				image.Format = Format;
				image.Quality = Quality;
				image.Scale((uint)(image.Width * Scale), (uint)(image.Height * Scale));
				image.Write(DstImagePath);
			}
		}
	}
}
