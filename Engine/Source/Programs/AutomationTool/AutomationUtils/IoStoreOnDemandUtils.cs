// Copyright Epic Games, Inc. All Rights Reserved.

using EpicGames.Core;
using System.Collections.Generic;

namespace AutomationTool
{

	public class IoStoreOnDemandSettings
	{
		public void SetIASGloballyEnabled()
		{
			IASSettings = new IASSpecificSettings();
			IASSettings.Mode = IASSpecificSettings.RuleMode.Default;
			IASSettings.ApplyToStagingManifest = true;
		}

		public void ReadFromCommandLine(BuildCommand Command)
		{
			if (Command.ParseParam("ApplyIoStoreOnDemand"))
			{
				IASSettings ??= new IASSpecificSettings();
				IASSettings.Mode = IASSpecificSettings.RuleMode.Default;
			}

			if (Command.ParseParam("CreateDefaultOnDemandPakRule"))
			{
				IASSettings ??= new IASSpecificSettings();
				IASSettings.Mode = IASSpecificSettings.RuleMode.Default;
			}

			if (Command.ParseParam("GenerateOnDemandPakForNonChunkedBuild"))
			{
				IASSettings ??= new IASSpecificSettings();
				IASSettings.ApplyToStagingManifest = true;
			}
		}

		public bool IsIASEnabled()
		{
			return IASSettings != null && IASSettings.Mode != IASSpecificSettings.RuleMode.None;
		}

		public bool ShouldCreateDefaultOnDemandPakRule()
		{
			return IASSettings != null && IASSettings.Mode == IASSpecificSettings.RuleMode.Default;
		}

		public bool ShouldCreateAssetTypeOnDemandPakRule()
		{
			return IASSettings != null && IASSettings.Mode == IASSpecificSettings.RuleMode.AssetType;
		}

		public Dictionary<string, List<string>> GetEnabledAssetList()
		{
			if (IASSettings != null)
			{
				return IASSettings.Assets;
			}
			else
			{
				return new Dictionary<string, List<string>>();
			}
		}

		/// <summary>
		/// Poll if pak rules should be applied when staging from a manifest rather than a chunked build.
		/// This allows IAS to force the creation of 'OnDemand' container files  even when chunked builds
		/// are not being used.
		/// </summary>
		/// <returns>True if the rules should be applied, otherwise false</returns>
		public bool ApplyRulesForStagingManifest()
		{
			return IASSettings != null && IASSettings.ApplyToStagingManifest;
		}

		public class IADSpecificSettings
		{

		}

		public class IASSpecificSettings
		{
			public enum RuleMode
			{
				/** No additional paking rules should be produced. */
				None,
				/** The default rule (move all bulkdata to 'OnDemand' containers) should be produced */
				Default,
				/** Produce rules based on asset types/ package lists to create 'OnDemand' containers */
				AssetType
			}

			public RuleMode Mode { get; set; } = RuleMode.None;

			public bool ApplyToStagingManifest = false;

			// AssetName->ListOfPackages
			public Dictionary<string, List<string>> Assets { get; set; } = new();
		}

		protected IADSpecificSettings IADSettings { get; }

		protected IASSpecificSettings IASSettings;
	}

}