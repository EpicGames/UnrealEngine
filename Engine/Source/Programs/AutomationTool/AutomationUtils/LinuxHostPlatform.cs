// Copyright Epic Games, Inc. All Rights Reserved.

using EpicGames.Core;
using Microsoft.Extensions.Logging;
using System;
using System.Diagnostics;
using System.IO;
using UnrealBuildTool;

namespace AutomationTool
{
	class LinuxHostPlatform : HostPlatform
	{
		static string CachedFrameworkMsbuildExe = string.Empty;

		public override string GetFrameworkMsbuildExe()
		{
			// Look for dotnet, we only support dotnet.
			if (string.IsNullOrEmpty(CachedFrameworkMsbuildExe))
			{
				FileReference dotnet = FileReference.FromString(CommandUtils.WhichApp("dotnet"));
				if (dotnet != null && FileReference.Exists(dotnet))
				{
					Logger.LogInformation("Using {DotNet}", dotnet.FullName);
					CachedFrameworkMsbuildExe = "dotnet msbuild";
				}
				else
				{
					throw new BuildException("Unable to find installation of dotnet.");
				}
			}

			return CachedFrameworkMsbuildExe;
		}

		public override string RelativeBinariesFolder
		{
			get { return @"Engine/Binaries/Linux/"; }
		}

		public override string GetUnrealExePath(string UnrealExe)
		{
			if(Path.IsPathRooted(UnrealExe))
			{
				return CommandUtils.CombinePaths(UnrealExe);
			}

			int CmdExeIndex = UnrealExe.IndexOf("-Cmd.exe");
			if (CmdExeIndex != -1)
			{
				UnrealExe = UnrealExe.Substring (0, CmdExeIndex);
			}
			else
			{
				CmdExeIndex = UnrealExe.IndexOf (".exe");
				if (CmdExeIndex != -1)
				{
					UnrealExe = UnrealExe.Substring (0, CmdExeIndex);
				}
			}
			return CommandUtils.CombinePaths(CommandUtils.CmdEnv.LocalRoot, RelativeBinariesFolder, UnrealExe);
		}

		public override string LocalBuildsLogFolder
		{
			// @FIXME: should use xdg-user-dir DOCUMENTS
			get { return Path.Combine(Environment.GetFolderPath(Environment.SpecialFolder.Personal), "Documents/Unreal Engine/LocalBuildLogs"); }
		}

		public override string P4Exe
		{
			get { return "p4"; }
		}

		public override Process CreateProcess(string AppName)
		{
			Process NewProcess = new Process();
			return NewProcess;
		}

		public override void SetupOptionsForRun(ref string AppName, ref CommandUtils.ERunOptions Options, ref string CommandLine)
		{
			if (AppName == "sh" || AppName == "codesign")
			{
				Options &= ~CommandUtils.ERunOptions.AppMustExist;
			}
			if (AppName.EndsWith(".exe") || ((AppName.Contains("/Binaries/Win64/") || AppName.Contains("/Binaries/Linux/")) && string.IsNullOrEmpty(Path.GetExtension(AppName))))
			{
				if (AppName.Contains("/Binaries/Win64/") || AppName.Contains("/Binaries/Linux/"))
				{
					AppName = AppName.Replace("/Binaries/Win64/", "/Binaries/Linux/");
					AppName = AppName.Replace("-cmd.exe", "");
					AppName = AppName.Replace("-Cmd.exe", "");
					AppName = AppName.Replace(".exe", "");
				}
				// some of our C# applications are converted to dotnet core, do not run those via dotnet
				else if (AppName.Contains("UnrealBuildTool") || AppName.Contains("AutomationTool"))
				{
					Options &= ~CommandUtils.ERunOptions.AppMustExist;
				}
				else
				{
					// It's a C# app, so run it with dotnet
					CommandLine = "\"" + AppName + "\" " + (String.IsNullOrEmpty(CommandLine) ? "" : CommandLine);
					AppName = "dotnet";
					Options &= ~CommandUtils.ERunOptions.AppMustExist;
				}
			}
		}

		public override void SetConsoleCtrlHandler(ProcessManager.CtrlHandlerDelegate Handler)
		{
			// @todo: add dotnet support
		}

		public override UnrealTargetPlatform HostEditorPlatform
		{
			get { return UnrealTargetPlatform.Linux; }
		}

		public override string PdbExtension
		{
			get { return ".exe.mdb"; }
		}
		static string[] SystemServices = new string[]
		{
			// TODO: Add any system process names here
		};
		public override string[] DontKillProcessList
		{
			get
			{
				return SystemServices;
			}
		}
	}
}
