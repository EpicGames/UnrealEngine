// Copyright Epic Games, Inc. All Rights Reserved.
using System;
using System.Collections.Generic;
using System.Linq;
using System.Text;
using System.IO;
using System.Runtime.CompilerServices;
using System.Diagnostics;
using UnrealBuildTool;
using EpicGames.Core;

namespace AutomationTool
{
	public static class LogUtils
	{
		/// <summary>
		/// Stores the log filename.
		/// </summary>
		public static FileReference LogFileName
		{
			get;
			private set;
		}

        /// <summary>
		/// Stores the final log filename. The build system uses this to display the network path that the log will be copied to once builds complete.
        /// </summary>
		public static FileReference FinalLogFileName
        {
			get;
			private set;
        }

		/// <summary>
		/// Creates the TraceListener used for file logging.
		/// We cannot simply use a TextWriterTraceListener because we need more flexibility when the file cannot be created.
		/// TextWriterTraceListener lazily creates the file, silently failing when it cannot.
		/// </summary>
		/// <returns>The newly created TraceListener, or null if it could not be created.</returns>
		public static void AddLogFileListener(DirectoryReference LogFolder, DirectoryReference FinalLogFolder)
		{
			int NumAttempts = 0;
			for (; ; )
			{
				try
				{
					string Name = (NumAttempts == 0) ? "Log.txt" : string.Format("Log_{0}.txt", NumAttempts + 1);

					FileReference LogFileName = FileReference.Combine(LogFolder, Name);
					Log.AddFileWriterWithoutBackup("AutomationFileLogListener", LogFileName);

					LogUtils.LogFileName = LogFileName;
					LogUtils.FinalLogFileName = FileReference.Combine(FinalLogFolder, Name);
					break;
				}
				catch (Exception Ex)
				{
					if (NumAttempts++ >= 10)
					{
						throw new AutomationException(Ex, "Unable to create log file after {0} attempts.", NumAttempts);
					}
				}
			}
		}

		/// <summary>
		/// Dumps exception info to log.
		/// @todo: Remove this function as it doesn't do a good job printing the exception information.
		/// </summary>
		/// <param name="Ex">Exception</param>
		public static string FormatException(Exception Ex)
		{
			var Message = String.Format("Exception in {0}: {1}{2}Stacktrace: {3}", Ex.Source, Ex.Message, Environment.NewLine, Ex.StackTrace);
			if (Ex.InnerException != null)
			{
				Message += String.Format("InnerException in {0}: {1}{2}Stacktrace: {3}", Ex.InnerException.Source, Ex.InnerException.Message, Environment.NewLine, Ex.InnerException.StackTrace);
			}
			return Message;
		}

		/// <summary>
		/// Returns a unique logfile name.
		/// </summary>
		/// <param name="Base">Base name for the logfile</param>
		/// <returns>Unique logfile name.</returns>
		public static string GetUniqueLogName(string Base)
		{
			const int MaxAttempts = 1000;

			DateTime Now     = DateTime.Now;
			int Attempt = 0;

			string LogFilename;
			for (;;)
			{
				var NowStr = Now.ToString("yyyy.MM.dd-HH.mm.ss");

				LogFilename = String.Format("{0}-{1}.txt", Base, NowStr);

				if (!File.Exists(LogFilename))
				{
					try
					{
						Directory.CreateDirectory(Path.GetDirectoryName(LogFilename));
						using (FileStream Stream = File.Open(LogFilename, FileMode.CreateNew, FileAccess.Write, FileShare.None))
						{
						}
						break;
					}
					catch (IOException)
					{
					}
				}

				++Attempt;
				if (Attempt == MaxAttempts)
				{
					throw new AutomationException(String.Format("Failed to create logfile {0}.", LogFilename));
				}

				Now = Now.AddSeconds(1);
			}

			return LogFilename;
		}

	}

}
