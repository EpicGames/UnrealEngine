// Copyright Epic Games, Inc. All Rights Reserved.

using EpicGames.Core;
using Microsoft.Extensions.Logging;
using System;
using System.Diagnostics;
using System.IO;
using UnrealBuildTool;

namespace AutomationTool
{
	class MacHostPlatform : HostPlatform
	{
		static string CachedFrameworkMsbuildExe = string.Empty;

		public override string GetFrameworkMsbuildExe()
		{
			// Look for dotnet, we only support dotnet.
			if (string.IsNullOrEmpty(CachedFrameworkMsbuildExe))
			{
				FileReference dotnet = FileReference.FromString(CommandUtils.WhichApp("dotnet"));
				if (dotnet != null && FileReference.Exists(dotnet))
				{
					Logger.LogInformation("Using {DotNet}", dotnet.FullName);
					CachedFrameworkMsbuildExe = "dotnet msbuild";
				}
				else
				{
					throw new BuildException("Unable to find installation of dotnet.");
				}
			}

			return CachedFrameworkMsbuildExe;
		}

		public override string RelativeBinariesFolder
		{
			get { return @"Engine/Binaries/Mac/"; }
		}

		public override string GetUnrealExePath(string UnrealExe)
		{
			if(Path.IsPathRooted(UnrealExe))
			{
				return CommandUtils.CombinePaths(UnrealExe);
			}

			int CmdExeIndex = UnrealExe.IndexOf("-Cmd.exe");
			if (CmdExeIndex != -1)
			{
				UnrealExe = UnrealExe.Substring(0, CmdExeIndex + 4);
			}
			else
			{
				CmdExeIndex = UnrealExe.IndexOf(".exe");
				if (CmdExeIndex != -1)
				{
					UnrealExe = UnrealExe.Substring(0, CmdExeIndex);
				}
			}

			if (UnrealExe.EndsWith("-Cmd", StringComparison.OrdinalIgnoreCase))
			{
				return CommandUtils.CombinePaths(CommandUtils.CmdEnv.LocalRoot, RelativeBinariesFolder, UnrealExe);
			}
			else
			{
				return CommandUtils.CombinePaths(CommandUtils.CmdEnv.LocalRoot, RelativeBinariesFolder, UnrealExe + ".app/Contents/MacOS", UnrealExe);
			}
		}

		public override string LocalBuildsLogFolder
		{
			get { return Path.Combine(Environment.GetFolderPath(Environment.SpecialFolder.Personal), "Library/Logs/Unreal Engine/LocalBuildLogs"); }
		}

		private string P4ExePath = null;

		public override string P4Exe
		{
			get
			{
				if (P4ExePath == null)
				{
					string[] p4Paths = { 
						"/usr/bin/p4", // Default path
						"/opt/homebrew/bin/p4", // Apple Silicon Homebrew Path
						"/usr/local/bin/p4" // Apple Intel Homebrew Path
					};
					
					foreach (string path in p4Paths)
					{
						if (File.Exists(path))
						{
							P4ExePath = path;
							break;
						}
					}
				}
				return P4ExePath;
			}
		}

		public override Process CreateProcess(string AppName)
		{
			Process NewProcess = new Process();
			return NewProcess;
		}

		public override void SetupOptionsForRun(ref string AppName, ref CommandUtils.ERunOptions Options, ref string CommandLine)
		{
			if (AppName == "sh" || AppName == "codesign")
			{
				Options &= ~CommandUtils.ERunOptions.AppMustExist;
			}
			if (AppName.EndsWith(".exe") || ((AppName.Contains("/Binaries/Win64/") || AppName.Contains("/Binaries/Mac/")) && string.IsNullOrEmpty(Path.GetExtension(AppName))))
			{
				if (AppName.Contains("/Binaries/Win64/") || AppName.Contains("/Binaries/Mac/"))
				{
					AppName = AppName.Replace("/Binaries/Win64/", "/Binaries/Mac/");
					AppName = AppName.Replace("-cmd.exe", "");
					AppName = AppName.Replace("-Cmd.exe", "");
					AppName = AppName.Replace(".exe", "");
					string AppFilename = Path.GetFileName(AppName);
					if (!CommandUtils.FileExists(AppName) && AppName.IndexOf("/Contents/MacOS/") == -1)
					{
						AppName = AppName + ".app/Contents/MacOS/" + AppFilename;
					}
				}
				// some of our C# applications are converted to dotnet core
				else if (AppName.Contains("UnrealBuildTool") || AppName.Contains("AutomationTool"))
				{
					Options &= ~CommandUtils.ERunOptions.AppMustExist;
				}
				else
				{
					// It's a C# app, so run it with dotnet
					CommandLine = "\"" + AppName + "\" " + (String.IsNullOrEmpty(CommandLine) ? "" : CommandLine);
					AppName = "dotnet";
					Options &= ~CommandUtils.ERunOptions.AppMustExist;
				}
			}
		}

		public override void SetConsoleCtrlHandler(ProcessManager.CtrlHandlerDelegate Handler)
		{
			// @todo: add dotnet support
		}

		public override UnrealTargetPlatform HostEditorPlatform
		{
			get { return UnrealTargetPlatform.Mac; }
		}

		public override string PdbExtension
		{
			get { return ".exe.mdb"; }
		}

		static string[] SystemServices = new string[]
		{
			// TODO: Add any system process names here
		};
		public override string[] DontKillProcessList
		{
			get
			{
				return SystemServices;
			}
		}
	}
}
