// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Linq;
using System.Text;
using UnrealBuildTool;
using EpicGames.Core;
using EpicGames.Perforce;
using System.Threading.Tasks;
using System.Diagnostics;
using Microsoft.Extensions.Logging;

using static AutomationTool.CommandUtils;
using UnrealBuildBase;

namespace AutomationTool
{
	/// <summary>
	/// Contains detected Perforce settings.
	/// </summary>
	public class P4Environment
	{
		/// <summary>
		/// The Perforce host and port number (eg. perforce:1666)
		/// </summary>
		public string ServerAndPort
		{
			get;
			private set;
		}

		/// <summary>
		/// The Perforce username
		/// </summary>
		public string User
		{
			get;
			private set;
		}

		/// <summary>
		/// Name of the Perforce client containing UAT.
		/// </summary>
		public string Client
		{
			get;
			private set;
		}

		/// <summary>
		/// The current branch containing UAT. This may be a depot path (e.g. //depot/UE4) or the name of a stream (e.g. //UE5/Main).
		/// </summary>
		public string Branch
		{
			get;
			private set;
		}

		/// <summary>
		/// Path to the root of the branch as a client path (//MyClient/UE).
		/// </summary>
		public string ClientRoot
		{
			get;
			private set;
		}

		/// <summary>
		/// The currently synced changelist.
		/// </summary>
		public int Changelist
		{
			get
			{
				if (ChangelistInternal <= 0)
				{
					throw new AutomationException("P4Environment.Changelist has not been initialized but is requested. Set uebp_CL env var or run UAT with -P4 to automatically detect changelist.");
				}
				return ChangelistInternal;
			}
			protected set
			{
				ChangelistInternal = value;
			}
		}

		/// <summary>
		/// Backing value for the current changelist property
		/// </summary>
		private int ChangelistInternal = -1;

		/// <summary>
		/// The currently synced code changelist.
		/// </summary>
		public int CodeChangelist
		{
			get
			{
				if (CodeChangelistInternal <= 0)
				{
					throw new AutomationException("P4Environment.CodeChangelist has not been initialized but is requested. Set uebp_CodeCL env var or run UAT with -P4 to automatically detect changelist.");
				}
				return CodeChangelistInternal;
			}
			protected set
			{
				CodeChangelistInternal = value;
			}
		}

		/// <summary>
		/// Backing value for the current code changelist
		/// </summary>
		private int CodeChangelistInternal = -1;

		static ILogger Logger => Log.Logger;

		/// <summary>
		/// Constructor. Derives the Perforce environment settings.
		/// </summary>
		internal P4Environment(CommandEnvironment CmdEnv)
		{
			// Get the Perforce port setting
			ServerAndPort = CommandUtils.ParseParamValue(Environment.GetCommandLineArgs(), "-p4port", null);  
			if(String.IsNullOrEmpty(ServerAndPort))
			{
				// check env var
				ServerAndPort = CommandUtils.GetEnvVar(EnvVarNames.P4Port);

				if (String.IsNullOrEmpty(ServerAndPort))
				{
					ServerAndPort = DetectP4Port();
				}
				CommandUtils.SetEnvVar(EnvVarNames.P4Port, ServerAndPort);
			}

			// Get the Perforce user setting
			User = CommandUtils.ParseParamValue(Environment.GetCommandLineArgs(), "-p4user", null);
			if (String.IsNullOrEmpty(User))
			{
				User = CommandUtils.GetEnvVar(EnvVarNames.User);
				if (String.IsNullOrEmpty(User))
				{
					P4Connection DefaultConnection = new P4Connection(User: null, Client: null, ServerAndPort: ServerAndPort);
					User = DetectUserName(DefaultConnection);
				}
				CommandUtils.SetEnvVar(EnvVarNames.User, User);
			}

			// Get the Perforce client setting
			Client = CommandUtils.GetEnvVar(EnvVarNames.Client);
			Branch = CommandUtils.GetEnvVar(EnvVarNames.BuildRootP4);
			ClientRoot = CommandUtils.GetEnvVar(EnvVarNames.ClientRoot);
			if(String.IsNullOrEmpty(Client) || String.IsNullOrEmpty(Branch) || String.IsNullOrEmpty(ClientRoot))
			{
				// Create a connection using the current setting
				P4Connection DefaultConnection = new P4Connection(User: User, Client: null, ServerAndPort: ServerAndPort);

				// Get the client info
				P4ClientInfo ThisClient;
				if(String.IsNullOrEmpty(Client))
				{
					string HostName = Unreal.MachineName;
					ThisClient = DetectClient(DefaultConnection, User, HostName, CmdEnv.AutomationToolDll);
					Logger.LogInformation("Using user {User} clientspec {ClientName} {ClientPath}", User, ThisClient.Name, ThisClient.RootPath);
					Client = ThisClient.Name;
					CommandUtils.SetEnvVar(EnvVarNames.Client, Client);
				}
				else
				{
					ThisClient = DefaultConnection.GetClientInfo(Client);
				}

				// Detect the root paths
				if(String.IsNullOrEmpty(Branch) || String.IsNullOrEmpty(ClientRoot))
				{
					P4Connection ClientConnection = new P4Connection(User: User, Client: ThisClient.Name, ServerAndPort: ServerAndPort);

					string BranchPath;
					string ClientRootPath;
					DetectRootPaths(ClientConnection, CmdEnv.LocalRoot, ThisClient, out BranchPath, out ClientRootPath);

					if(String.IsNullOrEmpty(Branch))
					{
						Branch = BranchPath;
						CommandUtils.SetEnvVar(EnvVarNames.BuildRootP4, Branch);
					}

					if(String.IsNullOrEmpty(ClientRoot))
					{
						ClientRoot = ClientRootPath;
						CommandUtils.SetEnvVar(EnvVarNames.ClientRoot, ClientRootPath);
					}
				}
			}

			// We expect the build root to not end with a path separator
			if (Branch.EndsWith("/"))
			{
				Branch = Branch.TrimEnd('/');
				CommandUtils.SetEnvVar(EnvVarNames.BuildRootP4, Branch);
			}

			// Set the current changelist
			string ChangelistString = CommandUtils.GetEnvVar(EnvVarNames.Changelist, null);
			if(String.IsNullOrEmpty(ChangelistString) && CommandUtils.P4CLRequired)
			{
				P4Connection Connection = new P4Connection(User, Client, ServerAndPort);
				ChangelistString = DetectCurrentCL(Connection, ClientRoot);
				CommandUtils.SetEnvVar(EnvVarNames.Changelist, ChangelistString);
			}
			if(!String.IsNullOrEmpty(ChangelistString))
			{
				Changelist = int.Parse(ChangelistString);
			}

			// Set the current code changelist
			string CodeChangelistString = CommandUtils.GetEnvVar(EnvVarNames.CodeChangelist);
			if(String.IsNullOrEmpty(CodeChangelistString) && CommandUtils.P4CLRequired)
			{
				Stopwatch Timer = Stopwatch.StartNew();
				using PerforceConnection Connection = new PerforceConnection(new PerforceSettings(ServerAndPort, User) { ClientName = Client }, Log.Logger);
				CodeChangelistString = DetectCurrentCodeCL(Connection, Changelist).Result.ToString();
				CommandUtils.SetEnvVar(EnvVarNames.CodeChangelist, CodeChangelistString);
				Logger.LogDebug("Took {ElapsedMs}ms to query last code change", Timer.ElapsedMilliseconds);
			}
			if(!String.IsNullOrEmpty(CodeChangelistString))
			{
				CodeChangelist = int.Parse(CodeChangelistString);
			}

			// Set the standard environment variables based on the values we've found
			CommandUtils.SetEnvVar("P4PORT", ServerAndPort);
			CommandUtils.SetEnvVar("P4USER", User);
			CommandUtils.SetEnvVar("P4CLIENT", Client);

			// Write a summary of the settings to the output window
			if (!CommandUtils.CmdEnv.IsChildInstance)
			{
				Logger.LogInformation("Detected Perforce Settings:");
				Logger.LogInformation("  Server: {ServerAndPort}", ServerAndPort);
				Logger.LogInformation("  User: {User}", User);
				Logger.LogInformation("  Client: {Client}", Client);
				Logger.LogInformation("  Branch: {Branch}", Branch);
				if (ChangelistInternal != -1)
				{
					Logger.LogInformation("  Last Change: {Changelist}", Changelist);
				}
				if (CodeChangelistInternal != -1)
				{
					Logger.LogInformation("  Last Code Change: {CodeChangelist}", CodeChangelist);
				}
			}

			// Write all the environment variables to the log
			Logger.LogDebug("Perforce Environment Variables:");
			Logger.LogDebug("  {Arg0}={Arg1}", EnvVarNames.P4Port, InternalUtils.GetEnvironmentVariable(EnvVarNames.P4Port, "", true));
			Logger.LogDebug("  {Arg0}={Arg1}", EnvVarNames.User, InternalUtils.GetEnvironmentVariable(EnvVarNames.User, "", true));
			Logger.LogDebug("  {Arg0}={Arg1}", EnvVarNames.Client, InternalUtils.GetEnvironmentVariable(EnvVarNames.Client, "", true));
			Logger.LogDebug("  {Arg0}={Arg1}", EnvVarNames.BuildRootP4, InternalUtils.GetEnvironmentVariable(EnvVarNames.BuildRootP4, "", true));
			Logger.LogDebug("  {Arg0}={Arg1}", EnvVarNames.BuildRootEscaped, InternalUtils.GetEnvironmentVariable(EnvVarNames.BuildRootEscaped, "", true));
			Logger.LogDebug("  {Arg0}={Arg1}", EnvVarNames.ClientRoot, InternalUtils.GetEnvironmentVariable(EnvVarNames.ClientRoot, "", true));
			Logger.LogDebug("  {Arg0}={Arg1}", EnvVarNames.Changelist, InternalUtils.GetEnvironmentVariable(EnvVarNames.Changelist, "", true));
			Logger.LogDebug("  {Arg0}={Arg1}", EnvVarNames.CodeChangelist, InternalUtils.GetEnvironmentVariable(EnvVarNames.CodeChangelist, "", true));
			Logger.LogDebug("  {Arg0}={Arg1}", "P4PORT", InternalUtils.GetEnvironmentVariable("P4PORT", "", true));
			Logger.LogDebug("  {Arg0}={Arg1}", "P4USER", InternalUtils.GetEnvironmentVariable("P4USER", "", true));
			Logger.LogDebug("  {Arg0}={Arg1}", "P4CLIENT", InternalUtils.GetEnvironmentVariable("P4CLIENT", "", true));
		}

		/// <summary>
		/// Attempts to detect source control server address from environment variables.
		/// </summary>
		/// <returns>Source control server address.</returns>
		private static string DetectP4Port()
		{
			string P4Port = null;			

			// If it's not set, spawn Perforce to get the current server port setting
			IProcessResult Result = CommandUtils.Run(HostPlatform.Current.P4Exe, "set P4PORT", null, CommandUtils.ERunOptions.NoLoggingOfRunCommand);
			if (Result.ExitCode == 0)
			{
				const string KeyName = "P4PORT=";
				if (Result.Output.StartsWith(KeyName))
				{
					int LastIdx = Result.Output.IndexOfAny(new char[] { ' ', '\n', '\r' });
					if (LastIdx == -1)
					{
						LastIdx = Result.Output.Length;
					}
					P4Port = Result.Output.Substring(KeyName.Length, LastIdx - KeyName.Length);
				}
			}

			// Otherwise fallback to the uebp variables, or the default
			if(String.IsNullOrEmpty(P4Port))
			{
				Logger.LogWarning("P4PORT is not set. Using perforce:1666");
				P4Port = "perforce:1666";
			}

			return P4Port;
		}

		/// <summary>
		/// Detects current user name.
		/// </summary>
		/// <returns></returns>
		public static string DetectUserName(P4Connection Connection)
		{
			var UserName = String.Empty;
			IProcessResult P4Result = Connection.P4("info", AllowSpew: false);
			if (P4Result.ExitCode != 0)
			{
				throw new AutomationException("Perforce command failed: {0}. Please make sure your P4PORT or {1} is set properly.", P4Result.Output, EnvVarNames.P4Port);
			}

			// Retrieve the P4 user name			
			Dictionary<string, string> Tags = Connection.ParseTaggedP4Output(P4Result.Output);
			if(!Tags.TryGetValue("User name", out UserName) || String.IsNullOrEmpty(UserName))
			{
				if (!String.IsNullOrEmpty(UserName))
				{
					Logger.LogWarning("Unable to retrieve perforce user name. Trying to fall back to {UserNameEnvVar} which is set to {UserName}.", EnvVarNames.User, UserName);
				}
				else
				{
					throw new AutomationException("Failed to retrieve user name.");
				}
			}

			return UserName;
		}

		/// <summary>
		/// Detects a workspace given the current user name, host name and depot path.
		/// </summary>
		/// <param name="Connection"></param>
		/// <param name="UserName">User name</param>
		/// <param name="HostName">Host</param>
		/// <param name="AutomationToolDll">Path to UAT dll, this will be checked agains the root path</param>
		/// <returns>Client to use.</returns>
		private static P4ClientInfo DetectClient(P4Connection Connection, string UserName, string HostName, string AutomationToolDll)
		{
			Logger.LogDebug("uebp_CLIENT not set, detecting current client...");

			// Check the default client. If it matches we can save any guess work.
			IProcessResult Result = CommandUtils.Run(HostPlatform.Current.P4Exe, "set -q P4CLIENT", null, CommandUtils.ERunOptions.NoLoggingOfRunCommand);
			if (Result.ExitCode == 0)
			{
				const string KeyName = "P4CLIENT=";
				if (Result.Output.StartsWith(KeyName))
				{
					string ClientName = Result.Output.Substring(KeyName.Length).Trim();
					P4ClientInfo ClientInfo = Connection.GetClientInfo(ClientName, true);
					if (ClientInfo != null && Connection.IsValidClientForFile(ClientInfo, AutomationToolDll))
					{
						return ClientInfo;
					}
				}
			}

			// Otherwise search for all clients that match
			List<P4ClientInfo> MatchingClients = new List<P4ClientInfo>();
			P4ClientInfo[] P4Clients = Connection.GetClientsForUser(UserName, AutomationToolDll);
			foreach (P4ClientInfo Client in P4Clients)
			{
				if (!String.IsNullOrEmpty(Client.Host) && String.Compare(Client.Host, HostName, true) != 0)
				{
					Logger.LogInformation("Rejecting client because of different Host {ClientName} \"{ClientHost}\" != \"{HostName}\"", Client.Name, Client.Host, HostName);
					continue;
				}
				
				MatchingClients.Add(Client);
			}

			P4ClientInfo ClientToUse = null;
			if (MatchingClients.Count == 0)
			{
				throw new AutomationException("No matching clientspecs found!");
			}
			else if (MatchingClients.Count == 1)
			{
				ClientToUse = MatchingClients[0];
			}
			else
			{
				// We may have empty host clients here, so pick the first non-empty one if possible
				foreach (P4ClientInfo Client in MatchingClients)
				{
					if (!String.IsNullOrEmpty(Client.Host) && String.Compare(Client.Host, HostName, true) == 0)
					{
						ClientToUse = Client;
						break;
					}
				}
				if (ClientToUse == null)
				{
					Logger.LogWarning("{NumClients} clients found that match the current host and root path. The most recently accessed client will be used.", MatchingClients.Count);
					ClientToUse = GetMostRecentClient(MatchingClients);
				}
			}

			return ClientToUse;
		}

		/// <summary>
		/// Given a list of clients with the same owner and root path, tries to find the most recently accessed one.
		/// </summary>
		/// <param name="Clients">List of clients with the same owner and path.</param>
		/// <returns>The most recent client from the list.</returns>
		private static P4ClientInfo GetMostRecentClient(List<P4ClientInfo> Clients)
		{
			Logger.LogDebug("Detecting the most recent client.");
			P4ClientInfo MostRecentClient = null;
			DateTime MostRecentAccessTime = DateTime.MinValue;
			foreach (P4ClientInfo ClientInfo in Clients)
			{
				if (ClientInfo.Access.Ticks > MostRecentAccessTime.Ticks)
				{
					MostRecentAccessTime = ClientInfo.Access;
					MostRecentClient = ClientInfo;
				}
			}
			if (MostRecentClient == null)
			{
				throw new AutomationException("Failed to determine the most recent client in {0}", Clients[0].RootPath);
			}
			return MostRecentClient;
		}

		/// <summary>
		/// Detects the current changelist the workspace is synced to.
		/// </summary>
		/// <param name="Connection"></param>
		/// <param name="ClientRootPath">Workspace path.</param>
		/// <returns>Changelist number as a string.</returns>
		private static string DetectCurrentCL(P4Connection Connection, string ClientRootPath)
		{
			Logger.LogDebug("uebp_CL not set, detecting 'have' CL...");

			// Retrieve the current changelist 
			IProcessResult P4Result = Connection.P4("changes -m 1 " + CommandUtils.CombinePaths(PathSeparator.Depot, ClientRootPath, "/...#have"), AllowSpew: false);

			string[] CLTokens = P4Result.Output.Split(new char[] { ' ' }, StringSplitOptions.RemoveEmptyEntries);
			if(CLTokens.Length == 0)
			{
				throw new AutomationException("Unable to find current changelist (no output from 'p4 changes' command)");
			}
			
			string CLString = CLTokens[1];

			int CL;
			if(!Int32.TryParse(CLString, out CL) || CLString != CL.ToString())
			{
				throw new AutomationException("Unable to parse current changelist from Perforce output:\n{0}", P4Result.Output);
			}

			return CLString;
		}

		/// <summary>
		/// Detects the current code changelist the workspace is synced to.
		/// </summary>
		/// <returns>Changelist number as a string.</returns>
		private static async Task<int> DetectCurrentCodeCL(PerforceConnection Connection, int Changelist)
		{
			Logger.LogDebug("uebp_CodeCL not set, detecting last code CL...");

			// Start by just testing whether the current change is a code change, so we can early out without any expensive p4 calls.
			int[] Changes = new[] { Changelist };
			while (Changes.Length > 0)
			{
				const int NumChangesPerDescribeCall = 10;
				foreach (IReadOnlyList<int> ChangeBatch in Changes.Batch(NumChangesPerDescribeCall))
				{
					const int InitialMaxFiles = 50;

					List<DescribeRecord> Descriptions = await Connection.DescribeAsync(DescribeOptions.None, InitialMaxFiles, ChangeBatch.ToArray());
					foreach (DescribeRecord Description in Descriptions)
					{
						DescribeRecord CurrentDescription = Description;

						// The initial p4 describe call only queries up to InitialMaxFiles in the response. Fetching all files for large merge changelists can be prohibitively slow,
						// so we query an increasing number of files with the goal of earlying out as soon as we hit a code change.
						for (int MaxFiles = InitialMaxFiles; ;)
						{
							if (CurrentDescription.Files.Any(x => x.DepotFile != null && PerforceUtils.IsCodeFile(x.DepotFile)))
							{
								return CurrentDescription.Number;
							}
							if (CurrentDescription.Files.Count < MaxFiles)
							{
								break;
							}

							MaxFiles *= 10;
							CurrentDescription = await Connection.DescribeAsync(DescribeOptions.None, MaxFiles, CurrentDescription.Number);
						}
					}
				}

				// Query the last NumChanges, then split it into batches for calling p4 describe
				const int NumChanges = 30;
				List<ChangesRecord> NextChanges = await Connection.GetChangesAsync(ChangesOptions.None, NumChanges, ChangeStatus.Submitted, $"//{Connection.ClientName}/...@<{Changes.Min()}");
				Changes = NextChanges.Select(x => x.Number).ToArray();
			}
			return 0;
		}

		/// <summary>
		/// Detects root paths for the specified client.
		/// </summary>
		/// <param name="Connection"></param>
		/// <param name="ThisClient">Client to detect the root paths for</param>
		/// <param name="BuildRootPath">Build root</param>
		/// <param name="LocalRootPath">Local root</param>
		/// <param name="ClientRootPath">Client root</param>
		private static void DetectRootPaths(P4Connection Connection, string LocalRootPath, P4ClientInfo ThisClient, out string BuildRootPath, out string ClientRootPath)
		{
			if(!String.IsNullOrEmpty(ThisClient.Stream))
			{
				BuildRootPath = ThisClient.Stream;
				ClientRootPath = String.Format("//{0}", ThisClient.Name);
			}
			else
			{
				// Figure out the build root
				string KnownFilePathFromRoot = CommandEnvironment.KnownFileRelativeToRoot;
				string KnownLocalPath = CommandUtils.MakePathSafeToUseWithCommandLine(CommandUtils.CombinePaths(PathSeparator.Slash, LocalRootPath, KnownFilePathFromRoot));
				IProcessResult P4Result = Connection.P4(String.Format("files -m 1 {0}", KnownLocalPath), AllowSpew: false);

				string KnownFileDepotMapping = P4Result.Output;

				// Get the build root
				Logger.LogDebug("Looking for {KnownFilePathFromRoot} in {KnownFileDepotMapping}", KnownFilePathFromRoot, KnownFileDepotMapping);
				int EndIdx = KnownFileDepotMapping.IndexOf(KnownFilePathFromRoot, StringComparison.CurrentCultureIgnoreCase);
				if (EndIdx < 0)
				{
					EndIdx = KnownFileDepotMapping.IndexOf(CommandUtils.ConvertSeparators(PathSeparator.Slash, KnownFilePathFromRoot), StringComparison.CurrentCultureIgnoreCase);
				}
				// Get the root path without the trailing path separator
				BuildRootPath = KnownFileDepotMapping.Substring(0, EndIdx - 1);

				// Get the client root
				if (LocalRootPath.StartsWith(CommandUtils.CombinePaths(PathSeparator.Slash, ThisClient.RootPath, "/"), StringComparison.InvariantCultureIgnoreCase) || LocalRootPath == CommandUtils.CombinePaths(PathSeparator.Slash, ThisClient.RootPath))
				{
					ClientRootPath = CommandUtils.CombinePaths(PathSeparator.Depot, String.Format("//{0}", ThisClient.Name), LocalRootPath.Substring(ThisClient.RootPath.Length));
				}
				else
				{
					throw new AutomationException("LocalRootPath ({0}) does not start with the client root path ({1})", LocalRootPath, ThisClient.RootPath);
				}
			}
		}
	}
}
