// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Text;
using System.IO;
using EpicGames.Core;
using System.Collections.Generic;
using System.Linq;
using UnrealBuildBase;
using Microsoft.Extensions.Logging;

namespace AutomationTool
{
	[Help("Auto-detects P4 settings based on the current path and creates a p4config file with the relevant settings.")]
	[ParamHelp("SetIgnore", "Adds a P4IGNORE to the default file (p4ignore in the root, or Engine/Extras/Perforce/p4ignore)", ParamType = typeof(bool), Flag = "-SetIgnore")]
	[ParamHelp("Path", "Write to a path other than the current directory")]
	[ParamHelp("ReplaceEnv", "When a filename passed in via 'Path' an error will be thrown if it does not match the existing value for P4CONFIG (if set), using this parameter will force the existing environment setting to be replaced", ParamType = typeof(bool), Flag = "-ReplaceEnv")]
	[ParamHelp("p4port=<server:port>", "Optional hint/override of the server to use during lookup")]
	[ParamHelp("p4user=<username>", "Optional hint/override of the username to use during lookup")]
	public class P4WriteConfig : BuildCommand
	{
		public override ExitCode Execute()
		{
			Logger.LogInformation("Setting up Perforce environment.");

			// User can specify these to help auto detection
			string Port = ParseParamValue("p4port", "");
			string User = ParseParamValue("p4user", "");

			bool SetIgnore = ParseParam("setignore");
			bool ListOnly = ParseParam("listonly");

			// apply any hints
			if (!string.IsNullOrEmpty(Port))
			{
				Environment.SetEnvironmentVariable(EnvVarNames.P4Port, Port);
			}

			if (!string.IsNullOrEmpty(User))
			{
				Environment.SetEnvironmentVariable(EnvVarNames.User, User);
			}

			// try to init P4
			try
			{
				CommandUtils.InitP4Environment();
				CommandUtils.InitDefaultP4Connection();
			}
			catch (Exception Ex)
			{
				Logger.LogError("Unable to find matching Perforce info. If the below does not help try P4WriteConfig -p4port=<server:port> and -p4user=<username> to supply more info");
				Logger.LogError(Ex, "{Message}", Ex.Message);
				return ExitCode.Error_Arguments;
			}

			// store all our settings
			StringBuilder P4Config = new StringBuilder();

			P4Config.AppendLine($"P4PORT={P4Env.ServerAndPort}");
			P4Config.AppendLine($"P4USER={P4Env.User}");
			P4Config.AppendLine($"P4CLIENT={P4Env.Client}");

			if (SetIgnore)
			{
				// If a p4 ignore file is in the root, prefer it
				string IgnorePath = Directory.EnumerateFiles(Unreal.RootDirectory.FullName, "*p4ignore*", SearchOption.TopDirectoryOnly).FirstOrDefault();
				if (string.IsNullOrEmpty(IgnorePath))
				{
					IgnorePath = Path.Combine(Unreal.EngineDirectory.ToString(), "Extras", "Perforce", "p4ignore");
				}
				else
				{
					// If a file was found in the root, use the filename so will apply to any same-named file in the tree
					IgnorePath = Path.GetFileName(IgnorePath);
				}
				P4Config.AppendLine($"P4IGNORE={IgnorePath}");
			}

			string P4Settings = P4Config.ToString();

			(string P4ConfigDirectory, string P4ConfigFilename) = GenerateP4ConfigPath();

			string FullP4ConfigPath = Path.Combine(P4ConfigDirectory, P4ConfigFilename);

			Logger.LogInformation("***\nWriting\n----------\n{Settings}----------\nto '{OutputPath}'\n***", P4Settings.Replace("\r\n", "\n"), FullP4ConfigPath);

			if (!ListOnly)
			{
				File.WriteAllText(FullP4ConfigPath, P4Settings);

				Logger.LogInformation("Wrote P4 settings to {OutputPath}", FullP4ConfigPath);

				P4.P4(string.Format("set P4CONFIG={0}", P4ConfigFilename));
				Logger.LogInformation("set P4CONFIG={Filename}", P4ConfigFilename);
			}
			else
			{
				Logger.LogInformation("Skipped write");
			}

			return ExitCode.Success;
		}

		/// <summary>
		/// Generate the path of the p4config file based on the users input on the commandline and the existing P4CONFIG settings in
		/// the current environment.
		/// </summary>
		/// <returns>The directory that we should write the p4config file to, and the filename to use for it</returns>
		/// <exception cref="AutomationException">Exceptions will be thrown if the users input is invalid and the script cannot continue</exception>
		private (string Directory, string Filename) GenerateP4ConfigPath()
		{
			bool ReplaceEnv = ParseParam("replaceenv");

			(string OutputDirectory, string Filename) = ParseCmdlinePath();
			
			if (string.IsNullOrEmpty(OutputDirectory))
			{
				// We will write the file to the current directory if none was supplied on the cmdline
				OutputDirectory = Environment.CurrentDirectory;
			}

			if (string.IsNullOrEmpty(Filename))
			{
				// If no filename was provided on the cmdline we will first try to use the existing value set in the users environment
				// falling back to our default name if nothing has been set there either.
				Filename = FindP4ConfigEnvValue();
				if (string.IsNullOrEmpty(Filename))
				{
					Filename = DefaultFilename;
				}
			}
			else
			{
				// As the caller provided a filename via the commandline we need to check if it will work with the current environment settings (if any)
				string EnvFilename = FindP4ConfigEnvValue();

				if (!string.IsNullOrEmpty(EnvFilename) && Filename != EnvFilename)
				{
					if (ReplaceEnv)
					{
						Logger.LogWarning("Changing P4CONFIG from '{CurrentName} to '{NewName}'", EnvFilename, Filename);
					}
					else
					{
						throw new AutomationException("The provided filename '{0}' will not work with the current environment setting 'P4CONFIG={1}'" +
							"\nEither update the -path parameter or use '-ReplaceEnv' to change the environment setting", Filename, EnvFilename);
					}
				}
			}

			return (OutputDirectory, Filename);
		}

		/// <summary>
		/// Parses the cmdline arg '-Path=' and returns it split into the directory and filename components.
		/// </summary>
		/// <returns>The directory and the filename that the user supplied on the commandline, either or both values can be empty strings</returns>
		/// <exception cref="AutomationException">Throws an exception if the user specifies a directory on the commandline that does not exist</exception>
		private (string Directory, string Filename) ParseCmdlinePath()
		{
			string CmdlinePath = ParseParamValue("path", "");

			string Directory = "";
			string Filename = "";

			if (!string.IsNullOrEmpty(CmdlinePath))
			{
				if (Path.HasExtension(CmdlinePath))
				{
					Directory = Path.GetDirectoryName(CmdlinePath);
					Filename = Path.GetFileName(CmdlinePath);
				}
				else
				{
					Directory = CmdlinePath;
				}

				if (!string.IsNullOrEmpty(Directory) && !DirectoryExists(Directory))
				{
					throw new AutomationException("Directory {0} does not exist, please create it before calling the script", Directory);
				}
			}

			return (Directory, Filename);
		}

		/// <summary>
		/// Utility to return the currently set p4config value from the users environment
		/// </summary>
		/// <returns>The p4config filename currently set in the environment or an empty string if none is set</returns>
		private string FindP4ConfigEnvValue()
		{
			string Filename = DefaultFilename;

			IProcessResult P4SetResult = P4.P4("set -q P4CONFIG");
			if (P4SetResult.ExitCode == 0)
			{
				string[] Tokens = P4SetResult.Output.Trim().Split("=");
				if (Tokens.Length == 2)
				{
					Filename = Tokens[1];
				}
			}

			return Filename;
		}

		/// <summary>
		/// The standard name we should use for the p4config file if there is nothing existing set in the users environment
		/// </summary>
		private const string DefaultFilename = "p4config.txt";
	}
}
