// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections;
using System.Collections.Generic;
using System.Linq;
using System.Text;
using System.IO;
using System.Threading;
using AutomationTool;
using EpicGames.MCP.Automation;
using UnrealBuildTool;
using System.Diagnostics;
using EpicGames.Core;
using System.Reflection;
using System.Threading.Tasks;
using Microsoft.Extensions.Logging;

using static AutomationTool.CommandUtils;

namespace AutomationTool
{
	public partial class PackageUtils
	{
		private static void UnpakBuild(FileReference ProjectFile, string SourceDirectory, List<string> PakFiles, string TargetDirectory, string CryptoFilename, string AdditionalArgs)
		{
			if (!CommandUtils.DirectoryExists(SourceDirectory))
			{
				Logger.LogError("Pak file directory {SourceDirectory} doesn't exist.", SourceDirectory);
				return;
			}

			string UnrealPakExe = CommandUtils.CombinePaths(CommandUtils.CmdEnv.LocalRoot, "Engine/Binaries/Win64/UnrealPak.exe");
			CommandUtils.CreateDirectory(TargetDirectory);

			string ProjectArg = ProjectFile != null ? ProjectFile.FullName : "";

			Parallel.ForEach(PakFiles, pakFile =>
			{
				string PathFileFullPath = CommandUtils.CombinePaths(SourceDirectory, pakFile);
				string UnrealPakCommandLine = string.Format("{0} {1} -Extract {2} -ExtractToMountPoint -cryptokeys=\"{3}\" {4}",
															ProjectArg, PathFileFullPath, TargetDirectory, CryptoFilename, AdditionalArgs);
				CommandUtils.RunAndLog(CommandUtils.CmdEnv, UnrealPakExe, UnrealPakCommandLine, Options: CommandUtils.ERunOptions.Default | CommandUtils.ERunOptions.UTF8Output | CommandUtils.ERunOptions.SpewIsVerbose);
				//CommandUtils.Log(UnrealPakCommandLine);
			});
		}

		private static List<FileInfo>[] SortFilesByPatchLayers(FileInfo[] FileInfoList)
		{
			int NumLevels = 0;
			bool FoundFilesInLevel = true;

			while (FoundFilesInLevel)
			{
				FoundFilesInLevel = false;
				string PatchPakSuffix = String.Format("_{0}_P", NumLevels);
				if (FileInfoList.Any(fileInfo => Path.GetFileNameWithoutExtension(fileInfo.Name).EndsWith(PatchPakSuffix)))
				{
					NumLevels++;
					FoundFilesInLevel = true;
				}
			}

			List<FileInfo>[] FileList = new List<FileInfo>[NumLevels + 1];

			FileList[0] = FileInfoList.Where(fileInfo => !Path.GetFileNameWithoutExtension(fileInfo.Name).EndsWith("_P")).ToList();
			
			for (int PatchLevel = 0; PatchLevel < NumLevels; PatchLevel++)
			{
				string PatchPakSuffix = String.Format("_{0}_P", NumLevels);
				FileList[PatchLevel + 1] = FileInfoList.Where(fileInfo => Path.GetFileNameWithoutExtension(fileInfo.Name).EndsWith(PatchPakSuffix)).ToList();
			}

			return FileList;
		}

		private static void GetFileListByPatchLayers(FileInfo[] PakFiles, out List<string>[] PatchLayers)
		{
			List<FileInfo>[] SortedLayers = SortFilesByPatchLayers(PakFiles);

			PatchLayers = SortedLayers.Select(patchLayer => patchLayer.Select(fileInfo => fileInfo.Name).ToList()).ToArray();
		}


		/// <summary>
		/// Extracts all pak files for a given project.
		/// </summary>
		/// <param name="SourceDirectory">Source file to read paks from</param>
		/// <param name="TargetDirectory">Destinaton path where the contents of the pak file will be written</param>
		/// <param name="CryptoKeysFilename">Path to the json bloc with keys if encryption is used</param>
		/// <param name="AdditionalArgs">Additional args to pass to UnrealPak</param>
		/// <param name="bExtractByLayers">Extract as layers. E.g. patch paks will be extracted after the base paks and will overwrite files</param>
		/// <param name="ProjectFile">Path to the project file. Can be null if UnrealPak is not built with project specific options or plugins</param>
		/// <returns></returns>
		public static void ExtractPakFiles(DirectoryInfo SourceDirectory, string TargetDirectory, string CryptoKeysFilename, string AdditionalArgs, bool bExtractByLayers, FileReference ProjectFile=null)
		{
			FileInfo[] PakFiles = SourceDirectory.GetFiles("*.pak", SearchOption.TopDirectoryOnly);

			if (bExtractByLayers)
			{
				List<string>[] PatchLayers;

				GetFileListByPatchLayers(PakFiles, out PatchLayers);

				int NumLayers = PatchLayers.Length;

				for (int layerIndex = 0; layerIndex < NumLayers; layerIndex++)
				{
					UnpakBuild(ProjectFile, SourceDirectory.FullName, PatchLayers[layerIndex], TargetDirectory, CryptoKeysFilename, AdditionalArgs);
				}
			}
			else
			{
				UnpakBuild(ProjectFile, SourceDirectory.FullName, PakFiles.Select(file => file.Name).ToList(), TargetDirectory, CryptoKeysFilename, AdditionalArgs);
			}
		}
	}
}
