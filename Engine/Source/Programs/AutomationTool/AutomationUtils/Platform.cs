// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.IO;
using System.Linq;
using System.Text;
using System.Reflection;
using UnrealBuildTool;
using EpicGames.Core;
using UnrealBuildBase;
using Microsoft.Extensions.Logging;

using static AutomationTool.CommandUtils;
using IdentityModel.Client;

namespace AutomationTool
{
	public interface ITurnkeyContext
	{
		string RetrieveFileSource(string Name, string InType = "Misc", string InPlatform = null, string SubType = null);
		string RetrieveFileSource(object HintObject);
		string GetVariable(string VariableName);
		int RunExternalCommand(string Command, string Params, bool bRequiresPrivilegeElevation, bool bUnattended, bool bCreateWindow);
		void Log(string Message);
		void ReportError(string Message);
		void PauseForUser(string Message);
		int ReadInputInt(string Prompt, List<string> Options, bool bIsCancellable, int DefaultValue = -1);
	}

	//public interface InputOutput
	//{
	//	string RetrieveByTags(string[] RequiredTags, string[] PreferredTags, Dictionary<string, string> ExtraVariables = null);
	//}

	public class DeviceInfo
	{
		public enum AutoSoftwareUpdateMode
		{
			Unknown,
			Disabled,
			Enabled
		}

		public DeviceInfo(UnrealTargetPlatform Platform)
		{
			this.Platform = Platform;
		}

		public DeviceInfo(UnrealTargetPlatform Platform, string Name, string Id, string SoftwareVersion, string Type, bool bIsDefault, bool bCanConnect, Dictionary<string, string> PlatformValues = null, AutoSoftwareUpdateMode AutoSoftwareUpdates = AutoSoftwareUpdateMode.Unknown)
		{
			this.Platform = Platform;
			this.Name = Name;
			this.Id = Id;
			this.SoftwareVersion = SoftwareVersion;
			this.Type = Type;
			this.bIsDefault = bIsDefault;
			this.bCanConnect = bCanConnect;
			this.AutoSoftwareUpdates = AutoSoftwareUpdates;
			if (PlatformValues != null)
			{
				this.PlatformValues = new Dictionary<string, string>(PlatformValues);
			}
		}

		public UnrealTargetPlatform Platform;
		public string Name;
		public string Id;
		public string SoftwareVersion;
		public string Type;
		public bool bIsDefault = false;
		// is the device able to be connected to (this is more about able to flash SDK or run, not about matching SDK version)
		// if false, any of the above fields are suspect, especially SoftwareVersion
		public bool bCanConnect = true;
		public AutoSoftwareUpdateMode AutoSoftwareUpdates = AutoSoftwareUpdateMode.Unknown;

		// case insensitive platform value dictionary. turnkey doesn't use this, but the platform can look up the device during deployment, etc to get this out
		public Dictionary<string, string> PlatformValues = new Dictionary<string, string>(StringComparer.InvariantCultureIgnoreCase);
	}

	/// <summary>
	/// Holds information for targeting specific platform (platform type + cook flavor)
	/// </summary>
	public struct TargetPlatformDescriptor
	{
		public UnrealTargetPlatform Type;
		public string CookFlavor;

		public TargetPlatformDescriptor(UnrealTargetPlatform InType)
		{
			Type = InType;
			CookFlavor = "";
		}
		public TargetPlatformDescriptor(UnrealTargetPlatform InType, string InCookFlavor)
		{
			Type = InType;
			CookFlavor = InCookFlavor ?? "";
		}

		public override string ToString()
		{
			return Type.ToString();
		}
	}

	/// <summary>
	/// Platform abstraction layer.
	/// </summary>
	public class Platform : CommandUtils
	{
		private static Dictionary<TargetPlatformDescriptor, Platform> AllPlatforms = new Dictionary<TargetPlatformDescriptor, Platform>();
		internal static void InitializePlatforms(HashSet<Assembly> AssembliesWithPlatforms)
		{
			Logger.LogDebug("Creating platforms.");

			// Create all available platforms.
			foreach (Assembly ScriptAssembly in AssembliesWithPlatforms)
			{
				CreatePlatformsFromAssembly(ScriptAssembly);
			}
			// Create dummy platforms for platforms we don't support
			foreach (UnrealTargetPlatform PlatformType in UnrealTargetPlatform.GetValidPlatforms())
			{
				TargetPlatformDescriptor TargetDesc = new TargetPlatformDescriptor(PlatformType);
				Platform ExistingInstance;
				if (AllPlatforms.TryGetValue(TargetDesc, out ExistingInstance) == false)
				{
					Logger.LogDebug("Creating placeholder platform for target: {TargetType}", TargetDesc.Type);
					AllPlatforms.Add(TargetDesc, new Platform(TargetDesc.Type));
				}
			}
		}

		private static void CreatePlatformsFromAssembly(Assembly ScriptAssembly)
		{
			Logger.LogDebug("Looking for platforms in {Location}", ScriptAssembly.Location);
			Type[] AllTypes = null;
			try
			{
				AllTypes = ScriptAssembly.GetTypes();
			}
			catch (Exception Ex)
			{
				Logger.LogError("Failed to get assembly types for {Location}", ScriptAssembly.Location);
				if (Ex is ReflectionTypeLoadException)
				{
					ReflectionTypeLoadException TypeLoadException = (ReflectionTypeLoadException)Ex;
					if (!IsNullOrEmpty(TypeLoadException.LoaderExceptions))
					{
						Logger.LogError("Loader Exceptions:");
						foreach (Exception LoaderException in TypeLoadException.LoaderExceptions)
						{
							Logger.LogError(LoaderException, "{Text}", LogUtils.FormatException(LoaderException));
						}
					}
					else
					{
						Logger.LogError("No Loader Exceptions available.");
					}
				}
				// Re-throw, this is still a critical error!
				throw;
			}
			foreach (Type PotentialPlatformType in AllTypes)
			{
				if (PotentialPlatformType != typeof(Platform) && typeof(Platform).IsAssignableFrom(PotentialPlatformType) && !PotentialPlatformType.IsAbstract)
				{
					Logger.LogDebug("Creating platform {Platform} from {Location}.", PotentialPlatformType.Name, ScriptAssembly.Location);
					Platform PlatformInstance = Activator.CreateInstance(PotentialPlatformType) as Platform;
					TargetPlatformDescriptor PlatformDesc = PlatformInstance.GetTargetPlatformDescriptor();

					Platform ExistingInstance;
					if (!AllPlatforms.TryGetValue(PlatformDesc, out ExistingInstance))
					{
						AllPlatforms.Add(PlatformDesc, PlatformInstance);
					}
					else
					{
						if (ExistingInstance.GetType() != PlatformInstance.GetType())
						{
							Logger.LogWarning("Platform {Platform} already exists", PotentialPlatformType.Name);
						}
					}
				}
			}
		}

		public UnrealTargetPlatform TargetPlatformType { get; protected set; }
		public UnrealTargetPlatform TargetIniPlatformType { get; protected set; }

		public Platform(UnrealTargetPlatform PlatformType)
		{
			TargetPlatformType = PlatformType;
			TargetIniPlatformType = PlatformType;

			Encoding.RegisterProvider(CodePagesEncodingProvider.Instance);
		}

		/// <summary>
		/// Allow the platform to alter the ProjectParams
		/// </summary>
		/// <param name="ProjParams"></param>
		public virtual void PlatformSetupParams(ref ProjectParams ProjParams)
		{

		}

		public virtual TargetPlatformDescriptor GetTargetPlatformDescriptor()
		{
			return new TargetPlatformDescriptor(TargetPlatformType, "");
		}

		/// <summary>
		/// Allows a platform to add runtime dependencies to UAT that may not be referenced in other ways, but are needed for staging UAT
		/// </summary>
		/// <param name="ProjectDirectory"></param>
		/// <param name="Dependencies"></param>
		public virtual void GetPlatformUATDependencies(DirectoryReference ProjectDirectory, List<FileReference> Dependencies)
		{

		}


		#region Turnkey

		public virtual DeviceInfo[] GetDevices()
		{
			return null;
		}

		public virtual DeviceInfo GetDeviceByName( string DeviceName )
		{
			DeviceInfo[] Devices = GetDevices();
			if (Devices == null)
			{
				return null;
			}
			// look by Id first
			DeviceInfo Device = Array.Find(Devices, x => string.Compare(x.Id, DeviceName, true) == 0);
			// if that fails, use Name
			if (Device == null)
			{
				Device = Array.Find(Devices, x => string.Compare(x.Name, DeviceName, true) == 0);
			}
			return Device;

		}

		public virtual bool InstallSDK(BuildCommand BuildCommand, ITurnkeyContext TurnkeyContext, DeviceInfo Device, bool bUnattended, bool bSdkAlreadyInstalled)
		{
			string Command, Params;

			bool bRequiresPrivilegeElevation = false;
			bool bCreateWindow = false;
			if (Device != null && GetDeviceUpdateSoftwareCommand(out Command, out Params, ref bRequiresPrivilegeElevation, ref bCreateWindow, TurnkeyContext, Device))
			{
				int ExitCode = TurnkeyContext.RunExternalCommand(Command, Params, bRequiresPrivilegeElevation, bUnattended, bCreateWindow);
				return OnSDKInstallComplete(ExitCode, TurnkeyContext, Device);
			}
			else if (Device == null && GetSDKInstallCommand(out Command, out Params, ref bRequiresPrivilegeElevation, ref bCreateWindow, TurnkeyContext, bSdkAlreadyInstalled))
			{
				int ExitCode = TurnkeyContext.RunExternalCommand(Command, Params, bRequiresPrivilegeElevation, bUnattended, bCreateWindow);
				return OnSDKInstallComplete(ExitCode, TurnkeyContext, null);
			}

			return false;
		}

		public virtual bool PostSDKSetup(ITurnkeyContext TurnkeyContext, bool bUnattended)
		{
			return true;
		}

		/// <summary>
		/// Return a list of versions that will be used to create "fake" FileSource objects which are used
		/// for install Sdks where no file downloads are needed
		/// </summary>
		/// <returns></returns>
		public virtual string[] GetCodeSpecifiedSdkVersions()
		{
			return new string[] { };
		}

		/// <summary>
		/// Return a list of versions that will be used to create "fake" FileSource objects which are used
		/// for flash/system updates where no file downloads are needed
		/// </summary>
		public virtual string[] GetCodeSpecifiedDeviceSoftwareUpdateVersions()
		{
			return new string[] { };
		}

		public virtual bool GetSDKInstallCommand(out string Command, out string Params, ref bool bRequiresPrivilegeElevation, ref bool bCreateWindow, ITurnkeyContext TurnkeyContext)
		{
			Command = null;
			Params = null;
			return false;
		}

		public virtual bool GetSDKInstallCommand(out string Command, out string Params, ref bool bRequiresPrivilegeElevation, ref bool bCreateWindow, ITurnkeyContext TurnkeyContext, bool bSdkAlreadyInstalled)
		{
			return GetSDKInstallCommand(out Command, out Params, ref bRequiresPrivilegeElevation, ref bCreateWindow, TurnkeyContext);
		}

		public virtual bool GetDeviceUpdateSoftwareCommand(out string Command, out string Params, ref bool bRequiresPrivilegeElevation, ref bool bCreateWindow, ITurnkeyContext TurnkeyContext, DeviceInfo Device = null)
		{
			Command = null;
			Params = null;
			return false;
		}

		/// <summary>
		/// Let's the platform handle the result of 
		/// </summary>
		/// <param name="ExitCode"></param>
		/// <param name="TurnkeyContext"></param>
		/// <param name="Device"></param>
		/// <returns>True if the installation was a success (defaults to ExitCode == 0)</returns>
		public virtual bool OnSDKInstallComplete(int ExitCode, ITurnkeyContext TurnkeyContext, DeviceInfo Device)
		{
			return ExitCode == 0;
		}

		public virtual void PersistSdkRootVar()
		{
		}

		public virtual string GetSDKCreationHelp()
		{
			return null;
		}

		public virtual bool UpdateHostPrerequisites(BuildCommand Command, ITurnkeyContext TurnkeyContext, bool bVerifyOnly)
		{
			return true;
		}

		public virtual bool UpdateDevicePrerequisites(DeviceInfo Device, BuildCommand Command, ITurnkeyContext TurnkeyContext, bool bVerifyOnly)
		{
			return true;
		}

		public virtual bool SetDeviceAutoSoftwareUpdateMode(DeviceInfo Device, bool bEnableAutoSoftwareUpdates)
		{
			Logger.LogWarning("{PlatformType} does not implement SetDeviceAutoSoftwareUpdateMode", PlatformType);
			return false;
		}

		#endregion

		/// <summary>
		/// Package files for the current platform.
		/// </summary>
		/// <param name="Params"></param>
		/// <param name="SC"></param>
		/// <param name="WorkingCL"></param>
		public virtual void Package(ProjectParams Params, DeploymentContext SC, int WorkingCL)
		{
			throw new AutomationException("{0} does not yet implement Packaging.", PlatformType);
		}

		/// <summary>
		/// Does the reverse of the output from the package process
		/// </summary>
		/// <param name="Params"></param>
		/// <param name="SourcePath"></param>
		/// <param name="DestinationPath"></param>
		public virtual void ExtractPackage(ProjectParams Params, string SourcePath, string DestinationPath)
		{
			throw new AutomationException("{0} does not yet implement ExtractPackage.", PlatformType);
		}

		/// <summary>
		/// Allow platform to do platform specific work on archived project before it's deployed.
		/// </summary>
		/// <param name="Params"></param>
		/// <param name="SC"></param>
		public virtual void ProcessArchivedProject(ProjectParams Params, DeploymentContext SC)
		{
		}

		/// <summary>
		/// Get all connected device names for this platform
		/// </summary>
		/// <param name="Params"></param>
		/// <param name="Devices"></param>
		public virtual void GetConnectedDevices(ProjectParams Params, out List<string> Devices)
		{
			Devices = null;
			Logger.LogWarning("{PlatformType} does not implement GetConnectedDevices", PlatformType);
		}

		/// <summary>
		/// Allow platform specific work prior to touching the staging directory
		/// </summary>
		/// <param name="Params"></param>
		/// <param name="SC"></param>
		public virtual void PreStage(ProjectParams Params, DeploymentContext SC)
		{
			// do nothing on most platforms
		}


		/// <summary>
		/// Deploy the application on the current platform
		/// </summary>
		/// <param name="Params"></param>
		/// <param name="SC"></param>
		public virtual void Deploy(ProjectParams Params, DeploymentContext SC)
		{
			Logger.LogWarning("{PlatformType} does not implement Deploy...", PlatformType);
		}

		/// <summary>
		/// Run the client application on the platform
		/// </summary>
		/// <param name="ClientRunFlags"></param>
		/// <param name="ClientApp"></param>
		/// <param name="ClientCmdLine"></param>
		/// <param name="Params"></param>
		/// <param name="SC"></param>
		public virtual IProcessResult RunClient(ERunOptions ClientRunFlags, string ClientApp, string ClientCmdLine, ProjectParams Params, DeploymentContext SC)
		{
			return RunClient(ClientRunFlags, ClientApp, ClientCmdLine, Params );
		}

		/// <summary>
		/// Run the client application on the platform
		/// </summary>
		/// <param name="ClientRunFlags"></param>
		/// <param name="ClientApp"></param>
		/// <param name="ClientCmdLine"></param>
		/// <param name="Params"></param>
		public virtual IProcessResult RunClient(ERunOptions ClientRunFlags, string ClientApp, string ClientCmdLine, ProjectParams Params)
		{
			PushDir(Path.GetDirectoryName(ClientApp));
			// Always start client process and don't wait for exit.
			IProcessResult ClientProcess = Run(ClientApp, ClientCmdLine, null, ClientRunFlags | ERunOptions.NoWaitForExit);
			PopDir();

			return ClientProcess;
		}

		/// <summary>
		/// Downloads file from target device to local pc
		/// </summary>
		/// <param name="RemoteFilePath"></param>
		/// <param name="LocalFile"></param>
		/// <param name="Params"></param>
		public virtual void GetTargetFile(string RemoteFilePath, string LocalFile, ProjectParams Params)
		{
			throw new NotImplementedException();
		}

		/// <summary>
		/// Allow platform specific clean-up or detection after client has run
		/// </summary>
		/// <param name="Result"></param>
		/// <param name="Params"></param>
		public virtual void PostRunClient(IProcessResult Result, ProjectParams Params)
		{
			// do nothing in the default case
		}

		/// <summary>
		/// Get the platform-specific name for the executable (with out the file extension)
		/// </summary>
		/// <param name="InExecutableName"></param>
		/// <returns></returns>
		public virtual string GetPlatformExecutableName(string InExecutableName)
		{
			return InExecutableName;
		}

		public virtual List<FileReference> GetExecutableNames(DeploymentContext SC)
		{
			List<FileReference> ExecutableNames = new List<FileReference>();
			foreach (StageTarget Target in SC.StageTargets)
			{
				foreach (BuildProduct Product in Target.Receipt.BuildProducts)
				{
					if (Product.Type == BuildProductType.Executable)
					{
						FileReference BuildProductFile = Product.Path;
						if (BuildProductFile.IsUnderDirectory(SC.ProjectRoot))
						{
							ExecutableNames.Add(FileReference.Combine(SC.RuntimeProjectRootDir, BuildProductFile.MakeRelativeTo(SC.ProjectRoot)));
						}
						else
						{
							ExecutableNames.Add(FileReference.Combine(SC.RuntimeRootDir, BuildProductFile.MakeRelativeTo(Unreal.RootDirectory)));
						}
					}
				}
			}
			return ExecutableNames;
		}

		/// <summary>
		/// Get the files to deploy, specific to this platform, typically binaries
		/// </summary>
		/// <param name="Params"></param>
		/// <param name="SC">Deployment Context</param>
		public virtual void GetFilesToDeployOrStage(ProjectParams Params, DeploymentContext SC)
		{
			throw new AutomationException("{0} does not yet implement GetFilesToDeployOrStage.", PlatformType);
		}

		/// <summary>
		/// Get additional platform specific files to stage when staging DLC
		/// </summary>
		/// <param name="Params"></param>
		/// <param name="SC">Deployment Context</param>
		public virtual void GetFilesToStageForDLC(ProjectParams Params, DeploymentContext SC)
		{
		}

		/// <summary>
		/// Called after CopyUsingStagingManifest.  Does anything platform specific that requires a final list of staged files.
		/// </summary>
		/// <param name="Params"></param>
		/// <param name="SC"></param>
		public virtual void PostStagingFileCopy(ProjectParams Params, DeploymentContext SC)
		{
		}

		/// <summary>
		/// Get the files to deploy, specific to this platform, typically binaries
		/// </summary>
		/// <param name="Params"></param>
		/// <param name="SC">Deployment Context</param>
		public virtual void GetFilesToArchive(ProjectParams Params, DeploymentContext SC)
		{
			SC.ArchiveFiles(SC.StageDirectory.FullName);
		}

		/// <summary>
		/// Gets cook platform name for this platform.
		/// </summary>
		/// <param name="bDedicatedServer">True if cooking for dedicated server</param>
		/// <param name="bIsClientOnly">True if cooking for client only</param>
		/// <returns>Cook platform string.</returns>
		public virtual string GetCookPlatform(bool bDedicatedServer, bool bIsClientOnly)
		{
			// this should get all cases, but a platform can override if needed

			string Suffix = bIsClientOnly ? "Client" : bDedicatedServer ? "Server" : "";
			string PlatformName = GetGenericPlatformName(TargetPlatformType);
			return $"{PlatformName}{Suffix}";
		}

		/// <summary>
		/// Gets extra cook commandline arguments for this platform.
		/// </summary>
		/// <param name="Params"> ProjectParams </param>
		/// <returns>Cook platform string.</returns>
		public virtual string GetCookExtraCommandLine(ProjectParams Params)
		{
			return "";
		}

		/// <summary>
		/// Gets extra maps needed on this platform.
		/// </summary>
		/// <returns>extra maps</returns>
		public virtual List<string> GetCookExtraMaps()
		{
			return new List<string>();
		}

		/// <summary>
		/// Get a release pak file path, if we are currently building a patch then get the previous release pak file path, if we are creating a new release this will be the output path
		/// </summary>
		/// <param name="SC"></param>
		/// <param name="Params"></param>
		/// <param name="PakName"></param>
		/// <returns></returns>
		public virtual string GetReleasePakFilePath(DeploymentContext SC, ProjectParams Params, string PakName)
		{
			if (Params.IsGeneratingPatch)
			{
				return CombinePaths(Params.GetBasedOnReleaseVersionPath(SC, Params.Client), PakName);
			}
			else
			{
				return CombinePaths(Params.GetCreateReleaseVersionPath(SC, Params.Client), PakName);
			}
		}

		/// <summary>
		/// Gets editor cook platform name for this platform. Cooking the editor is not useful, but this is used to fill the derived data cache
		/// </summary>
		/// <returns>Cook platform string.</returns>
		public virtual string GetEditorCookPlatform()
		{
			return GetCookPlatform(false, false);
		}

		/// <summary>
		/// return true if we need to change the case of filenames outside of pak files
		/// </summary>
		/// <param name="FileType">The staged file type to check (UFS vs SsytemNonUFS, etc)</param>
		/// <returns>true if files should be lower-cased during staging, for the given filetype</returns>
		public virtual bool DeployLowerCaseFilenames(StagedFileType FileType)
		{
			return false;
		}

		/// <summary>
		/// return true if we need to change the case of a particular file
		/// </summary>
		/// <param name="File"></param>
		/// <param name="FileType">The staged file type to check (UFS vs SsytemNonUFS, etc)</param>
		/// <returns>true if files should be lower-cased during staging, for the given filetype</returns>
		public virtual bool DeployLowerCaseFile(FileReference File, StagedFileType FileType)
		{
			return DeployLowerCaseFilenames(FileType);
		}


		/// <summary>
		/// Converts local path to target platform path.
		/// </summary>
		/// <param name="LocalPath">Local path.</param>
		/// <param name="LocalRoot">Local root.</param>
		/// <returns>Local path converted to device format path.</returns>
		public virtual string LocalPathToTargetPath(string LocalPath, string LocalRoot)
		{
			return LocalPath;
		}

		/// <summary>
		/// Update the build agenda for this platform
		/// </summary>
		/// <param name="Agenda">Agenda to update</param>
		/// <param name="ExtraBuildProducts">Any additional files that will be created</param>
		public virtual void MakeAgenda(UnrealBuild.BuildAgenda Agenda, List<string> ExtraBuildProducts)
		{
		}

		/// <summary>
		/// Returns a list of the compiler produced debug file extensions
		/// </summary>
		/// <returns>a list of the compiler produced debug file extensions</returns>
		public virtual List<string> GetDebugFileExtensions()
		{
			return new List<string>();
		}

		/// <summary>
		/// UnrealTargetPlatform type for this platform.
		/// </summary>
		public UnrealTargetPlatform PlatformType
		{
			get { return TargetPlatformType; }
		}

		/// <summary>
		/// UnrealTargetPlatform type for this platform.
		/// </summary>
		public UnrealTargetPlatform IniPlatformType
		{
			get { return TargetIniPlatformType; }
		}

		/// <summary>
		/// True if this platform is supported.
		/// </summary>
		public virtual bool IsSupported
		{
			get { return false; }
		}

		/// <summary>
		/// True if this platform requires UFE for deploying
		/// </summary>
		public virtual bool DeployViaUFE
		{
			get { return false; }
		}

		/// <summary>
		/// True if this platform requires UFE for launching
		/// </summary>
		public virtual bool LaunchViaUFE
		{
			get { return false; }
		}

		/// <summary>
		/// Gets extra launch commandline arguments for this platform.
		/// </summary>
		/// <param name="Params"> ProjectParams </param>
		/// <returns>Launch platform string.</returns>
		public virtual string GetLaunchExtraCommandLine(ProjectParams Params)
		{
			return "";
		}

		/// <summary>
		/// Modify or override the list of file host addresses for this platform.
		/// </summary>
		public virtual void ModifyFileHostAddresses(List<string> HostAddresses)
		{
		}

		/// <summary>
		/// True if this platform can write to the abslog path that's on the host desktop.
		/// </summary>
		public virtual bool UseAbsLog
		{
			get { return BuildHostPlatform.Current.Platform == PlatformType; }
		}

		/// <summary>
		/// return true if we need to call Remap of a specific file type
		/// </summary>
		/// <param name="FileType">The staged file type to check (UFS vs SsytemNonUFS, etc)</param>
		/// <returns>true if files should be remaped, for the given filetype</returns>
		public virtual bool RemapFileType(StagedFileType FileType)
		{
			return (FileType == StagedFileType.UFS || FileType == StagedFileType.NonUFS);
		}

		/// <summary>
		/// Remaps the given content directory to its final location
		/// </summary>
		public virtual StagedFileReference Remap(StagedFileReference Dest)
		{
			return Dest;
		}

		/// <summary>
		/// Tri-state - The intent is to override command line parameters for pak if needed per platform.
		/// </summary>
		public enum PakType { Always, Never, DontCare };

		public virtual PakType RequiresPak(ProjectParams Params)
		{
			return PakType.DontCare;
		}

		/// <summary>
		/// Returns platform specific command line options for UnrealPak
		/// </summary>
		public virtual string GetPlatformPakCommandLine(ProjectParams Params, DeploymentContext SC)
		{
			return "";
		}

		/// <summary>
		/// Returns platform specific command line options for the IoStore cmdlet
		/// </summary>
		public virtual string GetPlatformIoStoreCommandLine(ProjectParams Params, DeploymentContext SC)
		{
			return "";
		}

		/// <summary>
		/// Get the custom deployment handler for this platform,
		/// </summary>
		/// <param name="Params"></param>
		/// <param name="SC">Deployment Context</param>
		public virtual CustomDeploymentHandler GetCustomDeploymentHandler(ProjectParams Params, DeploymentContext SC)
		{
			return null;
		}

		/// <summary>
		/// True if this platform is supported.
		/// </summary>
		public virtual bool SupportsMultiDeviceDeploy
		{
			get { return false; }
		}

		/// <summary>
		/// Returns true if the platform wants patches to generate a small .pak file containing the difference
		/// of current data against a shipped pak file.
		/// </summary>
		/// <returns></returns>
		public virtual bool GetPlatformPatchesWithDiffPak(ProjectParams Params, DeploymentContext SC)
		{
			return true;
		}

		/// <summary>
		///  Returns whether the platform requires a package to deploy to a device
		/// </summary>
		public virtual bool RequiresPackageToDeploy(ProjectParams Params)
		{
			return false;
		}

		/// <summary>
		/// Returns whether the platform requires the Manifest_*_.txt files to be copied to the staged directory.
		/// </summary>
		public virtual bool RequiresManifestFiles
		{
			get { return true; }
		}

		public virtual HashSet<StagedFileReference> GetFilesForCRCCheck()
		{
			string CmdLine = "UECommandLine.txt";
			// using SystemNonUFS because that is how it's staged in CreateStagingManifest
			if (DeployLowerCaseFilenames(StagedFileType.SystemNonUFS))
			{
				CmdLine = CmdLine.ToLowerInvariant();
			}
			return new HashSet<StagedFileReference>() { new StagedFileReference(CmdLine) };
		}

		public virtual void StripSymbols(FileReference SourceFile, FileReference TargetFile)
		{
			if (SourceFile == TargetFile)
			{
				Logger.LogWarning("StripSymbols() has not been implemented for {Arg0}", PlatformType.ToString());
			}
			else
			{
				Logger.LogWarning("StripSymbols() has not been implemented for {Arg0}; copying files", PlatformType.ToString());
				File.Copy(SourceFile.FullName, TargetFile.FullName, true);
			}
		}

		public virtual void StripAllSymbols(FileReference SourceFile, FileReference TargetFile)
		{
			StripSymbols(SourceFile, TargetFile);
		}

		public virtual bool PublishSymbols(DirectoryReference SymbolStoreDirectory, List<FileReference> Files,
			bool bIndexSources, List<FileReference> SourceFiles,
			string Product, string Branch, int Change, string BuildVersion = null, string VFSMapping = null)
		{
			Logger.LogWarning("PublishSymbols() has not been implemented for {Arg0}", PlatformType.ToString());
			return false;
		}

		public virtual int GetExecutableSize(DirectoryReference BinariesDirectory, string ClientName, HashSet<FileReference> BuildProducts)
		{
			Logger.LogWarning("GetExecutableSize() has not been implemented for {Arg0}", PlatformType.ToString());
			return -1;
		}

		public virtual bool UpdatePatchPackagingParameters( string ProjectRoot, string BuildToUse, string AdditionalOptions )
		{
			Logger.LogWarning("UpdatePackagingParameters() not needed on platform {Arg0}", PlatformType.ToString());
			return false;
		}

		/// <summary>
		/// When overridden, returns the directory structure of the platform's symbol server.
		/// Each element is a semi-colon separated string of possible directory names.
		/// The * wildcard is allowed in any entry. {0} will be substituted for a custom filter string.
		/// </summary>
		public virtual string[] SymbolServerDirectoryStructure
		{
			get { return null; }
		}

		/// <summary>
		/// When overridden to return true, allows the AgeStoreTask to delete individual files in a single symbol folder,
		/// rather than requiring all files in a symbol folder to be out of date before deleting the entire directory.
		/// </summary>
		public virtual bool SymbolServerDeleteIndividualFiles
		{
			get { return false; }
		}

		/// <summary>
		/// When true, callers of PublishSymbols() must provide an explicit list of source files to create the index from.
		/// Some platforms discover the source files via other means, so it is possible to turn this step of the process
		/// off, since it can be slow.
		/// </summary>
		public virtual bool SymbolServerSourceIndexingRequiresListOfSourceFiles
		{
			get { return true; }
		}

		/// <summary>
		/// If true, indicates the platform's symbol server directory must be locked for
		/// exclusive access before any operation is performed on it. Platforms may override
		/// this to disable if their tools support concurrent access to the symbol server directory.
		/// </summary>
		public virtual bool SymbolServerRequiresLock
		{
			get { return true; }
		}

		public virtual void PreBuildAgenda(UnrealBuild Build, UnrealBuild.BuildAgenda Agenda, ProjectParams Params)
		{

		}

		/// <summary>
		/// Allows a platform to use the crash reporter from a different (built-in) platform
		/// </summary>
		public virtual UnrealTargetPlatform? CrashReportPlatform
		{
			get { return null; }
		}

		/// <summary>
		/// General purpose command to run generic string commands inside the platform interfeace
		/// </summary>
		/// <param name="Command"></param>
		public virtual int RunCommand(string Command)
		{
			return 0;
		}

		/// <summary>
		/// Determines whether we should stage a UECommandLine.txt for this platform
		/// </summary>
		public virtual bool ShouldStageCommandLine(ProjectParams Params, DeploymentContext SC)
		{
			return true;
		}

		/// <summary>
		/// Can host compile and cook for the platform
		/// </summary>
		public virtual bool CanHostPlatform(UnrealTargetPlatform Platform)
		{
			return false;
		}

		/// <summary>
		/// Allows some platforms to not be compiled, for instance when BuildCookRun -build is performed
		/// </summary>
		/// <returns><c>true</c> if this instance can be compiled; otherwise, <c>false</c>.</returns>
		public virtual bool CanBeCompiled()
		{
			return true;
		}

		public virtual bool RetrieveDeployedManifests(ProjectParams Params, DeploymentContext SC, string DeviceName, out List<string> UFSManifests, out List<string> NonUFSManifests)
		{
			UFSManifests = null;
			NonUFSManifests = null;
			return false;
		}

		public virtual bool SignExecutables(DeploymentContext SC, ProjectParams Params)
		{
			return true;
		}

		public virtual UnrealTargetPlatform[] GetStagePlatforms()
		{
			return new UnrealTargetPlatform[] { PlatformType };
		}

		public virtual DirectoryReference GetProjectRootForStage(DirectoryReference RuntimeRoot, StagedDirectoryReference RelativeProjectRootForStage)
		{
			return DirectoryReference.Combine(RuntimeRoot, RelativeProjectRootForStage.Name);
		}

		public virtual void PrepareForDebugging(string SourcePackage, string ProjectFilePath, string ClientPlatform)
		{
			Logger.LogError("Not implemented for the {Platform} platform.", ClientPlatform);
		}

		public virtual void SetSecondaryRemoteMac(string ProjectFilePath, string ClientPlatform)
		{
			Logger.LogError("Not implemented for this platform.");
		}

		// let the platform set the exe extension if it chooses (otherwise, use
		// the switch statement in GetExeExtension below)
		protected virtual string GetPlatformExeExtension()
		{
			return null;
		}

		public static string GetExeExtension(UnrealTargetPlatform Target)
		{
			Platform Plat = GetPlatform(Target);
			string PlatformExeExtension = Plat.GetPlatformExeExtension();
			if (!string.IsNullOrEmpty(PlatformExeExtension))
			{
				return PlatformExeExtension;
			}

			if (Target == UnrealTargetPlatform.Win64)
			{
				return ".exe";
			}
			if (Target == UnrealTargetPlatform.IOS)
			{
				return ".stub";
			}
			if (Target == UnrealTargetPlatform.Linux || Target == UnrealTargetPlatform.LinuxArm64)
			{
				return "";
			}
			if (Target == UnrealTargetPlatform.Mac)
			{
				return ".app";
			}

			return String.Empty;
		}

		public static Dictionary<TargetPlatformDescriptor, Platform> Platforms
		{
			get { return AllPlatforms; }
		}

		public static Platform GetPlatform(UnrealTargetPlatform PlatformType)
		{
			TargetPlatformDescriptor Desc = new TargetPlatformDescriptor(PlatformType);
			return AllPlatforms[Desc];
		}

		public static Platform GetPlatform(UnrealTargetPlatform PlatformType, string CookFlavor)
		{
			TargetPlatformDescriptor Desc = new TargetPlatformDescriptor(PlatformType, CookFlavor);
			return AllPlatforms[Desc];
		}

		public static bool IsValidTargetPlatform(TargetPlatformDescriptor PlatformDesc)
		{
			return AllPlatforms.ContainsKey(PlatformDesc);
		}

		public static List<TargetPlatformDescriptor> GetValidTargetPlatforms(UnrealTargetPlatform PlatformType, List<string> CookFlavors)
		{
			List<TargetPlatformDescriptor> ValidPlatforms = new List<TargetPlatformDescriptor>();
			if (!CommandUtils.IsNullOrEmpty(CookFlavors))
			{
				foreach (string CookFlavor in CookFlavors)
				{
					TargetPlatformDescriptor TargetDesc = new TargetPlatformDescriptor(PlatformType, CookFlavor);
					if (IsValidTargetPlatform(TargetDesc))
					{
						ValidPlatforms.Add(TargetDesc);
					}
				}
			}

			// In case there are no flavors specified or this platform type does not care/support flavors add it as generic platform
			if (ValidPlatforms.Count == 0)
			{
				TargetPlatformDescriptor TargetDesc = new TargetPlatformDescriptor(PlatformType);
				if (IsValidTargetPlatform(TargetDesc))
				{
					ValidPlatforms.Add(TargetDesc);
				}
			}

			return ValidPlatforms;
		}
	}
}
