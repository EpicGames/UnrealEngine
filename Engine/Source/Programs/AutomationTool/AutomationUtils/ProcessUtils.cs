// Copyright Epic Games, Inc. All Rights Reserved.
using System;
using System.Buffers;
using System.Collections;
using System.Collections.Generic;
using System.Linq;
using System.Text;
using System.IO;
using System.Threading;
using System.Diagnostics;
using System.Management;
using System.Runtime.InteropServices;
using System.Runtime.CompilerServices;
using EpicGames.Core;
using UnrealBuildTool;
using UnrealBuildBase;
using Microsoft.Extensions.Logging;

using static AutomationTool.CommandUtils;

namespace AutomationTool
{
	/// <summary>
	/// Reads from a StringBuilder line-by-line using pooled buffers to minimize allocations.
	/// This avoids the 2GB string limitation when processing large outputs.
	/// Uses GetChunks() for O(n) iteration instead of O(n²) from indexer access.
	/// </summary>
	public class StringBuilderReader : IDisposable
	{
		// Note: ChunkEnumerator is a struct - do NOT mark as readonly or MoveNext() won't work!
		private StringBuilder.ChunkEnumerator ChunkEnumerator;
		private ReadOnlyMemory<char> CurrentChunk;
		private int ChunkPosition;
		private bool HasMoreChunks;
		private char[] Buffer;

		public StringBuilderReader(StringBuilder builder)
		{
			if (builder == null)
			{
				throw new ArgumentNullException(nameof(builder));
			}
			ChunkEnumerator = builder.GetChunks();
			HasMoreChunks = ChunkEnumerator.MoveNext();
			if (HasMoreChunks)
			{
				CurrentChunk = ChunkEnumerator.Current;
			}
			ChunkPosition = 0;
			Buffer = ArrayPool<char>.Shared.Rent(4096);
		}

		/// <summary>
		/// Reads the next line as a ReadOnlySpan. The span is only valid until the next call to ReadLine.
		/// Returns false when there are no more lines to read.
		/// </summary>
		public bool ReadLine(out ReadOnlySpan<char> Line)
		{
			if (!HasMoreChunks || (ChunkPosition >= CurrentChunk.Length && !MoveToNextChunk()))
			{
				Line = default;
				return false;
			}

			int BufferPos = 0;

			// Read characters until we hit a newline or end of content
			while (HasMoreChunks)
			{
				ReadOnlySpan<char> ChunkSpan = CurrentChunk.Span;

				while (ChunkPosition < ChunkSpan.Length)
				{
					char C = ChunkSpan[ChunkPosition];

					if (C == '\r' || C == '\n')
					{
						// Skip the newline character(s)
						ChunkPosition++;
						// Handle \r\n sequence
						if (C == '\r')
						{
							// Check if next char is \n (may be in next chunk)
							if (ChunkPosition < ChunkSpan.Length)
							{
								if (ChunkSpan[ChunkPosition] == '\n')
								{
									ChunkPosition++;
								}
							}
							else if (MoveToNextChunk())
							{
								if (CurrentChunk.Span[0] == '\n')
								{
									ChunkPosition = 1;
								}
							}
						}
						goto Done;
					}

					// Ensure buffer is large enough
					if (BufferPos >= Buffer.Length)
					{
						char[] NewBuffer = ArrayPool<char>.Shared.Rent(Buffer.Length * 2);
						Buffer.AsSpan(0, BufferPos).CopyTo(NewBuffer);
						ArrayPool<char>.Shared.Return(Buffer, clearArray: true);
						Buffer = NewBuffer;
					}

					Buffer[BufferPos++] = C;
					ChunkPosition++;
				}

				// Move to next chunk
				if (!MoveToNextChunk())
				{
					break;
				}
			}

		Done:
			Line = Buffer.AsSpan(0, BufferPos);
			return true;
		}

		private bool MoveToNextChunk()
		{
			if (ChunkEnumerator.MoveNext())
			{
				CurrentChunk = ChunkEnumerator.Current;
				ChunkPosition = 0;
				return true;
			}
			HasMoreChunks = false;
			return false;
		}

		public void Dispose()
		{
			if (Buffer != null)
			{
				ArrayPool<char>.Shared.Return(Buffer, clearArray: true);
				Buffer = null;
			}
		}
	}

	public enum CtrlTypes
	{
		CTRL_C_EVENT = 0,
		CTRL_BREAK_EVENT,
		CTRL_CLOSE_EVENT,
		CTRL_LOGOFF_EVENT = 5,
		CTRL_SHUTDOWN_EVENT
	}

	public interface IProcess
	{
		void StopProcess(bool KillDescendants = true);
		bool HasExited { get; }
		string GetProcessName();
	}

	/// <summary>
	/// Tracks all active processes.
	/// </summary>
	public sealed class ProcessManager
	{
		public delegate bool CtrlHandlerDelegate(CtrlTypes EventType);

		[DllImport("Kernel32")]
		public static extern bool SetConsoleCtrlHandler(CtrlHandlerDelegate Handler, bool Add);

		/// <summary>
		/// List of active (running) processes.
		/// </summary>
		private static List<IProcess> ActiveProcesses = new List<IProcess>();
		/// <summary>
		/// Synchronization object
		/// </summary>
		private static object SyncObject = new object();


		/// <summary>
		/// Creates a new process and adds it to the tracking list.
		/// </summary>
		/// <returns>New Process objects</returns>
		public static IProcessResult CreateProcess(string AppName, bool bAllowSpew, bool bCaptureSpew, Dictionary<string, string> Env = null, LogEventType SpewVerbosity = LogEventType.Console, ProcessResult.SpewFilterCallbackType SpewFilterCallback = null, string WorkingDir = null)
		{
			Process NewProcess = HostPlatform.Current.CreateProcess(AppName);
			if (Env != null)
			{
				foreach (KeyValuePair<string, string> EnvPair in Env)
				{
					if (NewProcess.StartInfo.Environment.ContainsKey(EnvPair.Key))
					{
						NewProcess.StartInfo.Environment.Remove(EnvPair.Key);
					}
					if (!String.IsNullOrEmpty(EnvPair.Value))
					{
						NewProcess.StartInfo.Environment.Add(EnvPair.Key, EnvPair.Value);
					}
				}
			}
			if (WorkingDir != null)
			{
				NewProcess.StartInfo.WorkingDirectory = WorkingDir;
			}

			ProcessResult Result = new ProcessResult(AppName, NewProcess, bAllowSpew, bCaptureSpew, SpewVerbosity: SpewVerbosity, InSpewFilterCallback: SpewFilterCallback);
			AddProcess(Result);
			return Result;
		}

		public static void AddProcess(IProcess Proc)
		{
			lock (SyncObject)
			{
				ActiveProcesses.Add(Proc);
			}
		}

		public static void RemoveProcess(IProcess Proc)
		{
			lock (SyncObject)
			{
				ActiveProcesses.Remove(Proc);
			}
		}

		public static bool CanBeKilled(string ProcessName)
		{
			return !HostPlatform.Current.DontKillProcessList.Contains(ProcessName, StringComparer.InvariantCultureIgnoreCase);
		}

		/// <summary>
		/// Kills all running processes.
		/// </summary>
		public static void KillAll()
		{
			List<IProcess> ProcessesToKill = null;
			lock (SyncObject)
			{
				ProcessesToKill = new List<IProcess>(ActiveProcesses);
				ActiveProcesses.Clear();
			}

			// Remove processes that have exited or can't be killed
			for (int ProcessIndex = ProcessesToKill.Count - 1; ProcessIndex >= 0; --ProcessIndex )
			{
				IProcess Process =  ProcessesToKill[ProcessIndex];
				var ProcessName = string.Empty;
				bool bProcessHasExited = true;
				try
				{
					ProcessName = Process.GetProcessName();
					bProcessHasExited = Process.HasExited;
				}
				catch (InvalidOperationException Ex)
				{
					Logger.LogDebug("Exception accessing the Process properties:\n{Exception}", Ex.ToString());
				}
					
				if (bProcessHasExited)
				{
					ProcessesToKill.RemoveAt(ProcessIndex);
				}
				if (!String.IsNullOrEmpty(ProcessName) && !CanBeKilled(ProcessName))
				{
					Logger.LogDebug("Ignoring process \"{ProcessName}\" because it can't be killed.", ProcessName);
					ProcessesToKill.RemoveAt(ProcessIndex);
				}
			}
			if(ProcessesToKill.Count > 0)
			{
				Logger.LogDebug("Trying to kill {Arg0} spawned processes.", ProcessesToKill.Count);
				foreach (IProcess Proc in ProcessesToKill)
				{
					try
					{
						var ProcessName = Proc.GetProcessName();
						Logger.LogDebug("  {Arg0}", ProcessName);
					}
					catch (InvalidOperationException Ex)
					{
						Logger.LogDebug("Exception accessing the Process name:\n{Exception}", Ex.ToString());
					}
				}
				if (CommandUtils.IsBuildMachine)
				{
					for (int Cnt = 0; Cnt < 9; Cnt++)
					{
						bool AllDone = true;
						foreach (IProcess Proc in ProcessesToKill)
						{
							try
							{
								if (!Proc.HasExited)
								{
									AllDone = false;
									Logger.LogDebug("Waiting for process: {Arg0}", Proc.GetProcessName());
								}
							}
							catch (Exception)
							{
								Logger.LogWarning("Exception Waiting for process");
								AllDone = false;
							}
						}
						try
						{
							if (ProcessResult.HasAnyDescendants(Process.GetCurrentProcess()))
							{
								AllDone = false;
								Logger.LogInformation("Waiting for descendants of main process...");
							}
						}
						catch (Exception Ex)
						{
							Logger.LogWarning("{Text}", "Exception Waiting for descendants of main process. " + Ex);
							AllDone = false;
						}

						if (AllDone)
						{
							break;
						}
						Thread.Sleep(10000);
					}
				}
				foreach (IProcess Proc in ProcessesToKill)
				{
					var ProcName = string.Empty;
					try
					{
						ProcName = Proc.GetProcessName();
						if (!Proc.HasExited)
						{
							Logger.LogDebug("Killing process: {ProcName}", ProcName);
							Proc.StopProcess();
						}
					}
					catch (Exception Ex)
					{
						Logger.LogWarning("Exception while trying to kill process {ProcName}:", ProcName);
						Logger.LogWarning("{Text}", LogUtils.FormatException(Ex));
					}
				}
				try
				{
					if (CommandUtils.IsBuildMachine && ProcessResult.HasAnyDescendants(Process.GetCurrentProcess()))
					{
						Logger.LogDebug("current process still has descendants, trying to kill them...");
						ProcessResult.KillAllDescendants(Process.GetCurrentProcess());
					}
				}
				catch (Exception)
				{
					Logger.LogWarning("Exception killing descendants of main process");
				}
			}
		}
	}

	public interface IProcessResult : IProcess
	{
		void OnProcessExited();
		void DisposeProcess();
		void StdOut(object sender, DataReceivedEventArgs e);
		void StdErr(object sender, DataReceivedEventArgs e);
		int ExitCode { get; set; }
		bool bExitCodeSuccess { get; }
		string Output { get; }
		Process ProcessObject { get; }
		string ToString();
		void WaitForExit();
		FileReference WriteOutputToFile(string FileName);
		StringBuilderReader GetOutputReader() => throw new NotImplementedException("GetOutputReader is not supported on this platform");
	}
	
	/// <summary>
	/// Class containing the result of process execution.
	/// </summary>
	public class ProcessResult : IProcessResult
	{
		public delegate string SpewFilterCallbackType(string Message);

		private int ProcessExitCode = -1;
		private StringBuilder ProcessOutput = null;
		private bool AllowSpew = true;
		private LogEventType SpewVerbosity = LogEventType.Console;
		private SpewFilterCallbackType SpewFilterCallback = null;
		private string AppName = String.Empty;
		private Process Proc = null;
		private AutoResetEvent OutputWaitHandle = new AutoResetEvent(false);
		private AutoResetEvent ErrorWaitHandle = new AutoResetEvent(false);
		private bool bStdOutSignalReceived = false;
		private bool bStdErrSignalReceived = false;
		private object ProcSyncObject;

		public ProcessResult(string InAppName, Process InProc, bool bAllowSpew, bool bCaptureSpew = true, LogEventType SpewVerbosity = LogEventType.Console, SpewFilterCallbackType InSpewFilterCallback = null)
		{
			AppName = InAppName;
			ProcSyncObject = new object();
			Proc = InProc;
			AllowSpew = bAllowSpew;
			ProcessOutput = bCaptureSpew ? new StringBuilder() : null;
			if (!AllowSpew && !bCaptureSpew)
			{
				OutputWaitHandle.Set();
				ErrorWaitHandle.Set();
			}
			this.SpewVerbosity = SpewVerbosity;
			SpewFilterCallback = InSpewFilterCallback;
			if (Proc != null)
			{
				Proc.EnableRaisingEvents = false;
			}
		}

		~ProcessResult()
		{
			if (Proc != null)
			{
				Proc.Dispose();
			}
		}

		/// <summary>
		/// Removes a process from the list of tracked processes.
		/// </summary>
		public void OnProcessExited()
		{
			ProcessManager.RemoveProcess(this);
		}

		/// <summary>
		/// Log output of a remote process at a given severity.
		/// To pretty up the output, we use a custom source so it will say the source of the process instead of this method name.
		/// </summary>
		/// <param name="Verbosity"></param>
		/// <param name="Message"></param>
		private void LogOutput(LogEventType Verbosity, string Message)
		{
			Log.WriteLine(Verbosity, Message);
		}

		/// <summary>
		/// Manually dispose of Proc and set it to null.
		/// </summary>
		public void DisposeProcess()
		{
			if(Proc != null)
			{
				Proc.Dispose();
				Proc = null;
			}
		}

		/// <summary>
		/// Process.OutputDataReceived event handler.
		/// </summary>
		/// <param name="sender">Sender</param>
		/// <param name="e">Event args</param>
		public void StdOut(object sender, DataReceivedEventArgs e)
		{
			if (e.Data != null)
			{
				if (AllowSpew)
				{
					if (SpewFilterCallback != null)
					{
						string FilteredSpew = SpewFilterCallback(e.Data);
						if (FilteredSpew != null)
						{
							LogOutput(SpewVerbosity, FilteredSpew);
						}
					}
					else
					{
						LogOutput(SpewVerbosity, e.Data);
					}
				}

				if(ProcessOutput != null)
				{
					lock (ProcSyncObject)
					{
						ProcessOutput.Append(e.Data);
						ProcessOutput.Append(Environment.NewLine);
					}
				}
			}
			else
			{
				OutputWaitHandle.Set();
			}
		}

		/// <summary>
		/// Process.ErrorDataReceived event handler.
		/// </summary>
		/// <param name="sender">Sender</param>
		/// <param name="e">Event args</param>
		public void StdErr(object sender, DataReceivedEventArgs e)
		{
			if (e.Data != null)
			{
				if (AllowSpew)
				{
					if (SpewFilterCallback != null)
					{
						string FilteredSpew = SpewFilterCallback(e.Data);
						if (FilteredSpew != null)
						{
							LogOutput(SpewVerbosity, FilteredSpew);
						}
					}
					else
					{
						LogOutput(SpewVerbosity, e.Data);
					}
				}

				if(ProcessOutput != null)
				{
					lock (ProcSyncObject)
					{
						ProcessOutput.Append(e.Data);
						ProcessOutput.Append(Environment.NewLine);
					}
				}
			}
			else
			{
				ErrorWaitHandle.Set();
			}
		}

		/// <summary>
		/// Gets or sets the process exit code.
		/// </summary>
		public int ExitCode
		{
			get { return ProcessExitCode; }
			set { ProcessExitCode = value; }
		}

		public bool bExitCodeSuccess => ExitCode == 0;

		/// <summary>
		/// Gets all std output the process generated.
		/// </summary>
		public string Output
		{
			get
			{
				if (ProcessOutput == null)
				{
					return null;
				}
				lock (ProcSyncObject)
				{
					return ProcessOutput.ToString();
				}
			}
		}

		public bool HasExited
		{
			get 
			{
				bool bHasExited = true;
				lock (ProcSyncObject)
				{
					if (Proc != null)
					{
						bHasExited = Proc.HasExited;
						if (bHasExited)
						{
							ExitCode = Proc.ExitCode;
						}
					}
				}
				return bHasExited; 
			}
		}

		public Process ProcessObject
		{
			get { return Proc; }
		}

		/// <summary>
		/// Thread-safe way of getting the process name
		/// </summary>
		/// <returns>Name of the process this object represents</returns>
		public string GetProcessName()
		{
			string Name = null;
			lock (ProcSyncObject)
			{
				try
				{
					if (Proc != null && !Proc.HasExited)
					{
						Name = Proc.ProcessName;
					}
				}
				catch
				{
					// Ignore all exceptions
				}
			}
			if (String.IsNullOrEmpty(Name))
			{
				Name = "[EXITED] " + AppName;
			}
			return Name;
		}

		/// <summary>
		/// Object iterface.
		/// </summary>
		/// <returns>String representation of this object.</returns>
		public override string ToString()
		{
			return ExitCode.ToString();
		}

		public void WaitForExit()
		{
			bool bProcTerminated = false;
			// Make sure the process object is valid.
			lock (ProcSyncObject)
			{
				bProcTerminated = (Proc == null) || Proc.HasExited;
			}

			if (bProcTerminated)
			{
				Proc?.WaitForExit();
			}

			// Keep checking if we got all output messages until the process terminates.
			Stopwatch Watch = Stopwatch.StartNew();
			int MaxWaitUntilMessagesReceived = 60 * 1000;
			int WaitTimeout = 500;
			if (GlobalCommandLine.WaitForStdStreams >= 0)
			{
				MaxWaitUntilMessagesReceived = GlobalCommandLine.WaitForStdStreams;
			}
			if (MaxWaitUntilMessagesReceived > WaitTimeout)
			{
				WaitTimeout = 1 + (MaxWaitUntilMessagesReceived / 10);
			}
			while (!(bStdOutSignalReceived && bStdErrSignalReceived))
			{
				if (!bStdOutSignalReceived)
				{
					bStdOutSignalReceived = OutputWaitHandle.WaitOne(WaitTimeout);
				}
				if (!bStdErrSignalReceived)
				{
					bStdErrSignalReceived = ErrorWaitHandle.WaitOne(WaitTimeout);
				}
				// Check if the process terminated
				lock (ProcSyncObject)
				{
					bProcTerminated = (Proc == null) || Proc.HasExited;
				}

				if (bProcTerminated)
				{
					Proc?.WaitForExit(30000);
				}

				if (!bProcTerminated)
				{
					// Timeout starts when process has terminated
					Watch.Restart();
				}
				else
				{
					if (Watch.ElapsedMilliseconds > MaxWaitUntilMessagesReceived)
					{
						// Timeout passed, do not wait any longer
						break;
					}
				}
			}

			int waitTimeBeforeExit = Timeout.Infinite;

			if (!(bStdOutSignalReceived && bStdErrSignalReceived))
			{
				Logger.LogInformation("Waited for {Seconds:n2}s for output of {AppName}, some output may be missing; we gave up. Now waiting 5 minutes for process to exit.", Watch.Elapsed.TotalSeconds, AppName);
				waitTimeBeforeExit = 300000;
			}

			bool exited = true;

			if (Proc != null)
			{
				// wait until the process terminates or we hit our timeout
				exited = Proc.WaitForExit(waitTimeBeforeExit);
			}

			lock (ProcSyncObject)
			{
				// if we exited return the exit code
				if (exited)
				{
					if (Proc != null)
					{
						ExitCode = Proc.ExitCode;
					}
				}
				else
				{
					// if we failed to exit log a systemic warning and kill the process
					Logger.LogWarning(KnownLogEvents.Systemic, "Failed to exit process {AppName}.", AppName);

					try
					{
						Proc?.Kill();
					}
					catch
					{ 
					
					}
				}
			}
		}

		public FileReference WriteOutputToFile(string FileName)
		{
			using (StreamWriter writer = new StreamWriter(FileName))
			{
				writer.Write(ProcessOutput);
			}

			return new FileReference(FileName);
		}

		public StringBuilderReader GetOutputReader()
		{
			if (ProcessOutput == null)
			{
				return null;
			}
			return new StringBuilderReader(ProcessOutput);
		}

		/// <summary>
		/// Finds child processes of the current process.
		/// </summary>
		/// <param name="ProcessToKill"></param>
		/// <param name="PossiblyRelatedId"></param>
		/// <param name="VisitedPids"></param>
		/// <returns></returns>
		private static bool IsOurDescendant(Process ProcessToKill, int PossiblyRelatedId, HashSet<int> VisitedPids)
		{
			// check if we're the parent of it or its parent is our descendant
			try
			{
				VisitedPids.Add(PossiblyRelatedId);
				Process Parent = null;
				if (OperatingSystem.IsWindows())
				{
					using (ManagementObject ManObj = new ManagementObject(string.Format("win32_process.handle='{0}'", PossiblyRelatedId)))
					{
						ManObj.Get();
						int ParentId = Convert.ToInt32(ManObj["ParentProcessId"]);
						if (ParentId == 0 || VisitedPids.Contains(ParentId))
						{
							return false;
						}
						Parent = Process.GetProcessById(ParentId);  // will throw an exception if not spawned by us or not running
					}
				}
				if (Parent != null)
				{
					return Parent.Id == ProcessToKill.Id || IsOurDescendant(ProcessToKill, Parent.Id, VisitedPids);  // could use ParentId, but left it to make the above var used
				}
				else
				{
					return false;
				}
			}
			catch (Exception)
			{
				// This can happen if the pid is no longer valid which is ok.
				return false;
			}
		}

		/// <summary>
		/// Kills all child processes of the specified process.
		/// </summary>
		/// <param name="ProcessToKill"></param>
		public static void KillAllDescendants(Process ProcessToKill)
		{
			bool bKilledAChild;
			do
			{
				bKilledAChild = false;
				// For some reason Process.GetProcesses() sometimes returns the same process twice
				// So keep track of all processes we already tried to kill
				HashSet<int> KilledPids = new HashSet<int>();
				Process[] AllProcs = Process.GetProcesses();
				foreach (Process KillCandidate in AllProcs)
				{
					HashSet<int> VisitedPids = new HashSet<int>();
					if (ProcessManager.CanBeKilled(KillCandidate.ProcessName) && 
						!KilledPids.Contains(KillCandidate.Id) && 
						IsOurDescendant(ProcessToKill, KillCandidate.Id, VisitedPids))
					{
						KilledPids.Add(KillCandidate.Id);
						Logger.LogDebug("Trying to kill descendant pid={Arg0}, name={Arg1}", KillCandidate.Id, KillCandidate.ProcessName);
						try
						{
							KillCandidate.Kill();
							bKilledAChild = true;
						}
						catch (Exception Ex)
						{
							if(!KillCandidate.HasExited)
							{
								Logger.LogWarning("Failed to kill descendant:");
								Logger.LogWarning("{Text}", LogUtils.FormatException(Ex));
							}
						}
						break;  // exit the loop as who knows what else died, so let's get processes anew
					}
				}
			} while (bKilledAChild);
		}

		/// <summary>
		/// returns true if this process has any descendants
		/// </summary>
		/// <param name="ProcessToCheck">Process to check</param>
		public static bool HasAnyDescendants(Process ProcessToCheck)
		{
			Process[] AllProcs = Process.GetProcesses();
			foreach (Process KillCandidate in AllProcs)
			{
				// Silently skip InvalidOperationExceptions here, because it depends on the process still running. It may have terminated.
				string ProcessName;
				try
				{
					ProcessName = KillCandidate.ProcessName;
				}
				catch(InvalidOperationException)
				{
					continue;
				}

				// Check if it's still running
				HashSet<int> VisitedPids = new HashSet<int>();
				if (ProcessManager.CanBeKilled(ProcessName) && IsOurDescendant(ProcessToCheck, KillCandidate.Id, VisitedPids))
				{
					Logger.LogDebug("Descendant pid={Arg0}, name={ProcessName}", KillCandidate.Id, ProcessName);
					return true;
				}
			}
			return false;
		}

		public void StopProcess(bool KillDescendants = true)
		{
			if (Proc != null)
			{
				Process ProcToKill = null;
				// At this point any call to Proc memebers will result in an exception
				// so null the object.
				var ProcToKillName = GetProcessName();
				lock (ProcSyncObject)
				{
					ProcToKill = Proc;
					Proc = null;
				}
				// Now actually kill the process and all its descendants if requested
				try
				{
					ProcToKill.Kill(KillDescendants);
					ProcToKill.WaitForExit(60000);
					if (!ProcToKill.HasExited)
					{
						Logger.LogDebug("Process {ProcToKillName} failed to exit.", ProcToKillName);
					}
					else
					{
						ProcToKill.WaitForExit();
						ExitCode = ProcToKill.ExitCode;
						Logger.LogDebug("Process {ProcToKillName} successfully exited.", ProcToKillName);
						OnProcessExited();
					}
					ProcToKill.Close();					
				}
				catch (Exception Ex)
				{
					Logger.LogWarning("Exception while trying to kill process {ProcToKillName}:", ProcToKillName);
					Logger.LogWarning("{Text}", LogUtils.FormatException(Ex));
				}
			}
		}
	}

	public partial class CommandUtils
	{
		private static Dictionary<string, int> ExeToTimeInMs = new Dictionary<string, int>();

		public static void AddRunTime(string Exe, int TimeInMs)
		{
			lock(ExeToTimeInMs)
			{
				string Base = Path.GetFileName(Exe);
				if (!ExeToTimeInMs.ContainsKey(Base))
				{
					ExeToTimeInMs.Add(Base, TimeInMs);
				}
				else
				{
					ExeToTimeInMs[Base] += TimeInMs;
				}
			}
		}

		public static void PrintRunTime()
		{
			lock(ExeToTimeInMs)
			{
				foreach (KeyValuePair<string, int> Item in ExeToTimeInMs)
				{
					Logger.LogDebug("Total {Time}s to run {Exe}", Item.Value / 1000, Item.Key);
				}
				ExeToTimeInMs.Clear();
			}
		}

		[Flags]
		public enum ERunOptions
		{
			None = 0,
			/// <summary>
			/// If AllowSpew is set, then the redirected output from StdOut/StdErr is logged.
			/// Not relevant when NoStdOutRedirect is set.
			/// </summary>
			AllowSpew = 1 << 0,
			AppMustExist = 1 << 1,
			NoWaitForExit = 1 << 2,
			/// <summary>
			/// If NoStdOutRedirect is set, then StdOut/StdErr output is not redirected, logged or captured.
			/// Else, StdOut/StdErr is redirected and captured by default.
			/// </summary>
			NoStdOutRedirect = 1 << 3,
			NoLoggingOfRunCommand = 1 << 4,

			/// <summary>
			/// Output of the spawned process is expected to be encoded as UTF-8.
			/// </summary>
			UTF8Output = 1 << 5,

			/// When specified with AllowSpew, the output will be TraceEventType.Verbose instead of TraceEventType.Information
			SpewIsVerbose = 1 << 6,
			/// <summary>
			/// If NoLoggingOfRunCommand is set, it normally suppresses the run duration output. This turns it back on.
			/// </summary>
			LoggingOfRunDuration = 1 << 7,
			/// <summary>
			/// If set, a window is allowed to be created
			/// </summary>
			NoHideWindow = 1 << 8,
			/// <summary>
			/// If NoStdOutCapture is set, then the redirected output from StdOut/StdErr is not captured in the ProcessResult,
			/// and ProcessResult.Output will return null.
			/// Not relevant when NoStdOutRedirect is set.
			/// </summary>
			NoStdOutCapture = 1 << 9,

			/// <summary>
			/// Output of the spawned process is expected to be encoded as UTF-16 (System.Text.UnicodeEncoding, in .NET terminology)
			/// </summary>
			UTF16Output = 1 << 10,

			/// <summary>
			/// If set, then use the shell to create the process, else the process will be created directly from the executable.
			/// Will force NoStdOutRedirect since UseShellExecute is incompatible with redirection of stdout.
			/// </summary>
			UseShellExecute = 1 << 11,

			Default = AllowSpew | AppMustExist,
		}

		/// <summary>
		/// Resolves the passed in name using the path environment
		/// </summary>
		/// <param name="App"></param>
		/// <param name="Quiet"></param>
		/// <returns></returns>
		public static string WhichApp(string App, bool Quiet=true)
		{
			if (FileExists(Quiet, App))
			{
				return App;
			}

			if (HostPlatform.Current.HostEditorPlatform == UnrealTargetPlatform.Win64 && !Path.HasExtension(App))
			{
				App += ".exe";
			}

			string ResolvedPath = null;

			if (!App.Contains(Path.DirectorySeparatorChar) && !App.Contains(Path.AltDirectorySeparatorChar))
			{
				string[] PathDirectories = Environment.GetEnvironmentVariable("PATH").Split(Path.PathSeparator);
				foreach (string PathDirectory in PathDirectories)
				{
					try
					{
						string TryApp = Path.Combine(PathDirectory, App);
						if (FileExists(Quiet, TryApp))
						{
							ResolvedPath = TryApp;
							break;
						}
					}
					catch(ArgumentException)	// Path.Combine can throw an exception
					{
						Log.TraceWarningOnce("PATH variable contains invalid characters.");
					}
				}
			}

			if (ResolvedPath != null)
			{
				Logger.LogTrace("Resolved {App} to {ResolvedPath}", App, ResolvedPath);
			}
			else
			{
				Logger.LogDebug("Could not resolve app {App}", App);
			}

			return ResolvedPath;
		}

		/// <summary>
		/// Returns true if the given application executable is located withing the engine directory (as defined by CmdEnv.LocalRoot).
		/// This check may not be perfectly accurate because it only relies on the string paths.
		/// </summary>
		/// <param name="App"></param>
		/// <returns></returns>
		private static bool IsEngineExecutable(string App)
		{
			if (!Path.IsPathFullyQualified(App))
			{
				App = WhichApp(App);

				if (App == null)
				{
					return false;
				}
			}

			string NormalizedRoot = ConvertSeparators(PathSeparator.Default, Path.GetFullPath(CmdEnv.LocalRoot));
			string NormalizedApp  = ConvertSeparators(PathSeparator.Default, Path.GetFullPath(App));

			return NormalizedApp.StartsWith(NormalizedRoot);
		}

		/// <summary>
		/// Runs external program.
		/// </summary>
		/// <param name="App">Program filename.</param>
		/// <param name="CommandLine">Commandline</param>
		/// <param name="Input">Optional Input for the program (will be provided as stdin)</param>
		/// <param name="Options">Defines the options how to run. See ERunOptions.</param>
		/// <param name="Env">Environment to pass to program.</param>
		/// <param name="SpewFilterCallback">Callback to filter log spew before output.</param>
		/// <param name="Identifier"></param>
		/// <param name="WorkingDir"></param>
		/// <returns>Object containing the exit code of the program as well as it's stdout output.</returns>
		public static IProcessResult Run(string App, string CommandLine = null, string Input = null, ERunOptions Options = ERunOptions.Default, Dictionary<string, string> Env = null, ProcessResult.SpewFilterCallbackType SpewFilterCallback = null, string Identifier = null, string WorkingDir = null)
		{
			App = ConvertSeparators(PathSeparator.Default, App);

			HostPlatform.Current.SetupOptionsForRun(ref App, ref Options, ref CommandLine);
			if (App == "ectool" || App == "zip" || App == "xcodebuild")
			{
				Options &= ~ERunOptions.AppMustExist;
			}
			
			// Check if the application exists, including the PATH directories.
			if (Options.HasFlag(ERunOptions.AppMustExist) && !FileExists(Options.HasFlag(ERunOptions.NoLoggingOfRunCommand) ? true : false, App))
			{
				// in the case of something like "dotnet msbuild", split it up and put the msbuild on the commandline
				// this could be generalized, but would have to account for spaces in the App part
				if (App.StartsWith("dotnet "))
				{
					string[] Tokens = App.Split(" ", StringSplitOptions.RemoveEmptyEntries);
					App = Tokens[0];
					CommandLine = $"{Tokens[1]} {CommandLine}";
				}
				string ResolvedPath = WhichApp(App);

				if(string.IsNullOrEmpty(ResolvedPath))
				{
					throw new AutomationException("BUILD FAILED: Couldn't find the executable to run: {0}", App);
				}

				App = ResolvedPath;
			}
			DateTime StartTime = DateTime.UtcNow;

			LogEventType SpewVerbosity = Options.HasFlag(ERunOptions.SpewIsVerbose) ? LogEventType.Verbose : LogEventType.Console;
			if (!Options.HasFlag(ERunOptions.NoLoggingOfRunCommand))
			{
				LogWithVerbosity(SpewVerbosity,"Running: " + App + " " + (String.IsNullOrEmpty(CommandLine) ? "" : CommandLine));
			}

			if (!IsEngineExecutable(App))
			{
				Dictionary<string, string> FinalEnv = (Env != null) ? new Dictionary<string, string>(Env) : new Dictionary<string, string>();

				// Environment variables to preserve if explicitly set, otherwise clear
				string[] dotnetVars = [
					"DOTNET_ROOT",
					"DOTNET_MULTILEVEL_LOOKUP",
					"DOTNET_ROLL_FORWARD",
					"DOTNET_HOST_PATH",
					"DOTNET_MSBUILD_SDK_RESOLVER_CLI_DIR",
					"DOTNET_MSBUILD_SDK_RESOLVER_SDKS_DIR",
					"DOTNET_MSBUILD_SDK_RESOLVER_SDKS_VER",
				];

				foreach (string varName in dotnetVars)
				{
					if (Env == null || !Env.ContainsKey(varName))
					{
						FinalEnv[varName] = String.Empty;
					}
				}

				// Allow rolling forward to a later dotnet runtime if it's not already specified
				if (Env?.ContainsKey("DOTNET_ROLL_FORWARD") != true)
				{
					FinalEnv["DOTNET_ROLL_FORWARD"] = "Major";
				}

				// Clearing our variable as well
				FinalEnv["UE_DOTNET_VERSION"] = String.Empty;

				Env = FinalEnv;
			}

			bool bUseShellExecute = Options.HasFlag(ERunOptions.UseShellExecute);
			bool bRedirectStdOut = !bUseShellExecute && !Options.HasFlag(ERunOptions.NoStdOutRedirect);
			bool bAllowSpew = bRedirectStdOut && Options.HasFlag(ERunOptions.AllowSpew);
			bool bCaptureSpew = bRedirectStdOut && !Options.HasFlag(ERunOptions.NoStdOutCapture);
			IProcessResult Result = ProcessManager.CreateProcess(App, bAllowSpew, bCaptureSpew, Env, SpewVerbosity: SpewVerbosity, SpewFilterCallback: SpewFilterCallback, WorkingDir: WorkingDir);
			using (LogIndentScope Scope = Options.HasFlag(ERunOptions.AllowSpew) ? new LogIndentScope("  ") : null)
			{
				Process Proc = Result.ProcessObject;

				Proc.StartInfo.FileName = App;

				// Process Arguments follow windows conventions in .NET Core
				// Which means single quotes ' are not considered quotes.
				// see https://github.com/dotnet/runtime/issues/29857
				// also see UE-102580
				Proc.StartInfo.Arguments = String.IsNullOrEmpty(CommandLine) ? "" : CommandLine.Replace('\'', '\"');

				Proc.StartInfo.UseShellExecute = bUseShellExecute;
				if (bRedirectStdOut)
				{
					Proc.StartInfo.RedirectStandardOutput = true;
					Proc.StartInfo.RedirectStandardError = true;
					Proc.OutputDataReceived += Result.StdOut;
					Proc.ErrorDataReceived += Result.StdErr;
				}

				// By default the standard input stream uses the current terminal input encoding (`Console.InputEncoding`),
				// so let's make sure to set an explicit known input encoding (that doesn't produce a BOM).
				if (Input != null)
				{
					Proc.StartInfo.RedirectStandardInput = true;

					// Assume that if the application produces UTF-16, it also consumes UTF-16.
					if ((Options & ERunOptions.UTF16Output) == ERunOptions.UTF16Output)
					{
						Proc.StartInfo.StandardInputEncoding = new UnicodeEncoding(false, false, false);
					}
					else
					{
						Proc.StartInfo.StandardInputEncoding = new UTF8Encoding(false);
					}
				}

				Proc.StartInfo.CreateNoWindow = (Options & ERunOptions.NoHideWindow) == 0;
				if ((Options & ERunOptions.UTF8Output) == ERunOptions.UTF8Output)
				{
					Proc.StartInfo.StandardOutputEncoding = new System.Text.UTF8Encoding(false, false);
				}
				else if ((Options & ERunOptions.UTF16Output) == ERunOptions.UTF16Output)
				{
					Proc.StartInfo.StandardOutputEncoding = new System.Text.UnicodeEncoding(false, false, false);
				}

				Proc.Start();

				if (bRedirectStdOut)
				{
					Proc.BeginOutputReadLine();
					Proc.BeginErrorReadLine();
				}

				if (String.IsNullOrEmpty(Input) == false)
				{
					Proc.StandardInput.WriteLine(Input);
					Proc.StandardInput.Close();
				}

				if (!Options.HasFlag(ERunOptions.NoWaitForExit))
				{
					Result.WaitForExit();
				}
				else
				{
					Result.ExitCode = -1;
				}
			}

			if (!Options.HasFlag(ERunOptions.NoWaitForExit))
			{
				var BuildDuration = (DateTime.UtcNow - StartTime).TotalMilliseconds;
				//AddRunTime(App, (int)(BuildDuration));
				Process Proc = Result.ProcessObject;
				if (Proc != null)
				{
					Result.ExitCode = Proc.ExitCode;
				}
				if (!Options.HasFlag(ERunOptions.NoLoggingOfRunCommand) || Options.HasFlag(ERunOptions.LoggingOfRunDuration))
				{
					LogWithVerbosity(SpewVerbosity, "Took {0:n2}s to run {1}, ExitCode={2}", BuildDuration / 1000, Path.GetFileName(App), Result.ExitCode);
				}
				Result.OnProcessExited();
				Result.DisposeProcess();
			}

			return Result;
		}

		/// <summary>
		/// Gets a logfile name for a RunAndLog call
		/// </summary>
		/// <param name="Env">Environment to use.</param>
		/// <param name="App">Executable to run</param>
		/// <param name="LogName">Name of the logfile ( if null, executable name is used )</param>
		/// <returns>The log file name.</returns>
		public static string GetRunAndLogOnlyName(CommandEnvironment Env, string App, string LogName = null)
		{
			if (LogName == null)
			{
				LogName = Path.GetFileNameWithoutExtension(App);
			}
			return LogUtils.GetUniqueLogName(CombinePaths(Env.LogFolder, LogName));
		}

		/// <summary>
		/// Runs external program and writes the output to a logfile.
		/// </summary>
		/// <param name="Env">Environment to use.</param>
		/// <param name="App">Executable to run</param>
		/// <param name="CommandLine">Commandline to pass on to the executable</param>
		/// <param name="LogName">Name of the logfile ( if null, executable name is used )</param>
		/// <param name="MaxSuccessCode"></param>
		/// <param name="Input">Optional Input for the program (will be provided as stdin)</param>
		/// <param name="Options">Defines the options how to run. See ERunOptions.</param>
		/// <param name="EnvVars"></param>
		/// <param name="SpewFilterCallback">Callback to filter log spew before output.</param>
		public static void RunAndLog(CommandEnvironment Env, string App, string CommandLine, string LogName = null, int MaxSuccessCode = 0, string Input = null, ERunOptions Options = ERunOptions.Default, Dictionary<string, string> EnvVars = null, ProcessResult.SpewFilterCallbackType SpewFilterCallback = null)
		{
			RunAndLog(App, CommandLine, GetRunAndLogOnlyName(Env, App, LogName), MaxSuccessCode, Input, Options, EnvVars, SpewFilterCallback);
		}

		/// <summary>
		/// Exception class for child process commands failing
		/// </summary>
		public class CommandFailedException : AutomationException
		{
			public CommandFailedException(string Message) : base(Message)
			{
			}

			public CommandFailedException(ExitCode ExitCode, string Message) : base(ExitCode, Message)
			{
			}
		}

		/// <summary>
		/// Runs external program and writes the output to a logfile.
		/// </summary>
		/// <param name="App">Executable to run</param>
		/// <param name="CommandLine">Commandline to pass on to the executable</param>
		/// <param name="Logfile">Full path to the logfile, where the application output should be written to.</param>
		/// <param name="MaxSuccessCode"></param>
		/// <param name="Input">Optional Input for the program (will be provided as stdin)</param>
		/// <param name="Options">Defines the options how to run. See ERunOptions.</param>
		/// <param name="EnvVars"></param>
		/// <param name="SpewFilterCallback">Callback to filter log spew before output.</param>
		public static string RunAndLog(string App, string CommandLine, string Logfile = null, int MaxSuccessCode = 0, string Input = null, ERunOptions Options = ERunOptions.Default, Dictionary<string, string> EnvVars = null, ProcessResult.SpewFilterCallbackType SpewFilterCallback = null)
		{
			IProcessResult Result = Run(App, CommandLine, Input, Options, EnvVars, SpewFilterCallback);
			if (!String.IsNullOrEmpty(Result.Output) && Logfile != null)
			{
				WriteToFile(Logfile, Result.Output);
			}
			else if (Logfile == null)
			{
				Logfile = "[No logfile specified]";
			}
			else
			{
				Logfile = "[None!, no output produced]";
			}

			if (Result.ExitCode > MaxSuccessCode || Result.ExitCode < 0)
			{
				throw new CommandFailedException((ExitCode)Result.ExitCode, String.Format("Command failed (Result:{3}): {0} {1}. See logfile for details: '{2}' ",
												App, CommandLine, Path.GetFileName(Logfile), Result.ExitCode)){ OutputFormat = AutomationExceptionOutputFormat.Minimal };
			}
			if (!String.IsNullOrEmpty(Result.Output))
			{
				return Result.Output;
			}
			return "";
		}

		/// <summary>
		/// Runs external program and writes the output to a logfile.
		/// </summary>
		/// <param name="App">Executable to run</param>
		/// <param name="CommandLine">Commandline to pass on to the executable</param>
		/// <param name="SuccessCode"></param>
		/// <param name="Logfile">Full path to the logfile, where the application output should be written to.</param>
		/// <param name="EnvVars"></param>
		/// <param name="SpewFilterCallback">Callback to filter log spew before output.</param>
		/// <returns>Whether the program executed successfully or not.</returns>
		public static string RunAndLog(string App, string CommandLine, out int SuccessCode, string Logfile = null, Dictionary<string, string> EnvVars = null, ProcessResult.SpewFilterCallbackType SpewFilterCallback = null)
		{
			IProcessResult Result = Run(App, CommandLine, Env: EnvVars, SpewFilterCallback: SpewFilterCallback);
			SuccessCode = Result.ExitCode;
			if (Result.Output.Length > 0 && Logfile != null)
			{
				WriteToFile(Logfile, Result.Output);
			}
			if (!String.IsNullOrEmpty(Result.Output))
			{
				return Result.Output;
			}
			return "";
		}

		/// <summary>
		/// Runs external program and writes the output to a logfile.
		/// </summary>
		/// <param name="Env">Environment to use.</param>
		/// <param name="App">Executable to run</param>
		/// <param name="CommandLine">Commandline to pass on to the executable</param>
		/// <param name="SuccessCode"></param>
		/// <param name="LogName">Name of the logfile ( if null, executable name is used )</param>
		/// <param name="EnvVars"></param>
		/// <param name="SpewFilterCallback">Callback to filter log spew before output.</param>
		/// <returns>Whether the program executed successfully or not.</returns>
		public static string RunAndLog(CommandEnvironment Env, string App, string CommandLine, out int SuccessCode, string LogName = null, Dictionary<string, string> EnvVars = null, ProcessResult.SpewFilterCallbackType SpewFilterCallback = null)
		{
			return RunAndLog(App, CommandLine, out SuccessCode, GetRunAndLogOnlyName(Env, App, LogName), EnvVars, SpewFilterCallback);
		}

		/// <summary>
		/// Runs UAT recursively
		/// </summary>
		/// <param name="Env">Environment to use.</param>
		/// <param name="CommandLine">Commandline to pass on to the executable</param>
		/// <param name="Identifier">Log prefix for output</param>
		public static void RunUAT(CommandEnvironment Env, string CommandLine, string Identifier)
		{
			// Check if there are already log files which start with this prefix, and try to uniquify it if until there aren't.
			string DirOnlyName = Identifier;
			string LogSubdir = CombinePaths(CmdEnv.LogFolder, DirOnlyName, "");
			for(int Attempt = 1;;Attempt++)
			{
				string[] ExistingFiles = FindFiles(DirOnlyName + "*", false, CmdEnv.LogFolder);
				if (ExistingFiles.Length == 0)
				{
					break;
				}
				if (Attempt == 1000)
				{
					throw new AutomationException("Couldn't seem to create a log subdir {0}", LogSubdir);
				}
				DirOnlyName = String.Format("{0}_{1}", Identifier, Attempt + 1);
				LogSubdir = CombinePaths(CmdEnv.LogFolder, DirOnlyName, "");
			}

			// Get the stdout log file for this run, and create the subdirectory for all the other log output
			CreateDirectory(LogSubdir);

			// Run UAT with the log folder redirected through the environment
			Dictionary<string, string> EnvironmentVars = new Dictionary<string, string>();
			EnvironmentVars.Add(EnvVarNames.LogFolder, LogSubdir);
			EnvironmentVars.Add(EnvVarNames.FinalLogFolder, CombinePaths(CmdEnv.FinalLogFolder, DirOnlyName));
			EnvironmentVars.Add(EnvVarNames.DisableStartupMutex, "1");
			EnvironmentVars.Add(EnvVarNames.IsChildInstance, "1");
			if (!IsBuildMachine)
			{
				EnvironmentVars.Add(AutomationTool.EnvVarNames.LocalRoot, ""); // if we don't clear this out, it will think it is a build machine; it will rederive everything
			}

			IProcessResult Result = Run(Unreal.DotnetPath.FullName, $"\"{Env.AutomationToolDll}\" {CommandLine}", null, ERunOptions.Default, EnvironmentVars, Identifier: Identifier);
			if (Result.ExitCode != 0)
			{
				throw new CommandFailedException(String.Format("Recursive UAT command failed (exit code {0})", Result.ExitCode)){ OutputFormat = AutomationExceptionOutputFormat.Silent };
			}
		}

		protected delegate bool ProcessLog(string LogText);
		/// <summary>
		/// Keeps reading a log file as it's being written to by another process until it exits.
		/// </summary>
		/// <param name="LogFilename">Name of the log file.</param>
		/// <param name="LogProcess">Process that writes to the log file.</param>
		/// <param name="OnLogRead">Callback used to process the recently read log contents.</param>
		protected static void LogFileReaderProcess(string LogFilename, IProcessResult LogProcess, ProcessLog OnLogRead = null)
		{
			while (!FileExists(LogFilename) && !LogProcess.HasExited)
			{
				Logger.LogInformation("Waiting for logging process to start...");
				Thread.Sleep(2000);
			}
			Thread.Sleep(1000);

			using (FileStream ProcessLog = File.Open(LogFilename, FileMode.Open, FileAccess.Read, FileShare.ReadWrite))
			{
				StreamReader LogReader = new StreamReader(ProcessLog);
				bool bKeepReading = true;

				// Read until the process has exited.
				while (!LogProcess.HasExited && bKeepReading)
				{
					while (!LogReader.EndOfStream && bKeepReading)
					{
						string Output = LogReader.ReadToEnd();
						if (Output != null && OnLogRead != null)
						{
							bKeepReading = OnLogRead(Output);
						}
					}

					while (LogReader.EndOfStream && !LogProcess.HasExited && bKeepReading)
					{
						Thread.Sleep(250);
						// Tick the callback so that it can respond to external events
						if (OnLogRead != null)
						{
							bKeepReading = OnLogRead(null);
						}
					}
				}
			}
		}

	}
}