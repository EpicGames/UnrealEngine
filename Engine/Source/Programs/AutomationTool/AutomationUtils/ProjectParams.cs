// Copyright Epic Games, Inc. All Rights Reserved.
using System;
using System.Collections.Generic;
using System.Linq;
using System.Text;
using UnrealBuildTool;
using System.IO;
using System.Reflection;
using EpicGames.Core;
using UnrealBuildBase;
using Microsoft.Extensions.Logging;

using static AutomationTool.CommandUtils;

namespace AutomationTool
{
	[ParamHelp("platform", "Platform of the target", ParamType = typeof(string))]
	[ParamHelp("config", "Config of the target", ParamType = typeof(string))]
	[ParamHelp("project", "Config of the target", ParamType = typeof(string))]
	[ParamHelp("client", "Build, cook and run a client and a server, uses client target configuration", ParamType = typeof(bool))]
	[ParamHelp("noclient", "Do not run the client, just run the server", ParamType = typeof(bool))]
	[ParamHelp("server", "Is this a server target?", ParamType = typeof(bool))]
	[ParamHelp("build", "True if build step should be executed", ParamType = typeof(bool))]
	[ParamHelp("ubtargs", "Extra options to pass to ubt")]
	[ParamHelp("cook", "Determines if the build is going to use cooked data", ParamType = typeof(bool))]
	[ParamHelp("AdditionalCookerOptions", "Additional arguments sent to the cooking step", ParamType = typeof(string))]
	[ParamHelp("pak", "Generate a pak file", ParamType = typeof(bool))]
	[ParamHelp("deploy", "Deploy the project for the target platform", ParamType = typeof(bool))]
	[ParamHelp("stage", "Put this build in a stage director", ParamType = typeof(bool))]
	[ParamHelp("run", "Run the game after it is built (including server, if -server)", ParamType = typeof(bool))]
	[ParamHelp("zenstore", "Save cooked output data to the Zen storage server", ParamType = typeof(bool))]
	[ParamHelp("iterate", "Uses the iterative cooking/deploy", ParamType = typeof(bool))]
	[ParamHelp("snapshot", "Imports the most recently published project snapshot (if cooking, this forces the use of -forcerecook=false)", ParamType = typeof(bool))]
	[ParamHelp("cookincremental", "Cooks incrementally using previously-cooked data or a published snapshot as a base", ParamType = typeof(bool))]
	public interface IProjectParamsHelpers
	{
	}

	[Help("targetplatform=PlatformName", "target platform for building, cooking and deployment (also -Platform)")]
	[Help("servertargetplatform=PlatformName", "target platform for building, cooking and deployment of the dedicated server (also -ServerPlatform)")]
	public class ProjectParams
	{
		static ILogger Logger => Log.Logger;

		/// <summary>
		/// Gets a parameter from the command line if it hasn't been specified in the constructor. 
		/// If the command line is not available, default value will be used.
		/// </summary>
		/// <param name="Command">Command to parse the command line for. Can be null.</param>
		/// <param name="SpecifiedValue">Value specified in the constructor (or not)</param>
		/// <param name="Default">Default value.</param>
		/// <param name="ParamNames">Command line parameter names to parse.</param>
		/// <returns>Parameter value.</returns>
		bool GetParamValueIfNotSpecified(BuildCommand Command, bool? SpecifiedValue, bool Default, params string[] ParamNames)
		{
			if (SpecifiedValue.HasValue)
			{
				return SpecifiedValue.Value;
			}
			else if (Command != null)
			{
				foreach (var Param in ParamNames)
				{
					if (Command.ParseParam(Param))
					{
						return true;
					}
				}
			}
			return Default;
		}

		/// <summary>
		/// Gets optional parameter from the command line if it hasn't been specified in the constructor. 
		/// If the command line is not available or the command has not been specified in the command line, default value will be used.
		/// </summary>
		/// <param name="Command">Command to parse the command line for. Can be null.</param>
		/// <param name="SpecifiedValue">Value specified in the constructor (or not)</param>
		/// <param name="Default">Default value.</param>
		/// <param name="TrueParam">Name of a parameter that sets the value to 'true', for example: -clean</param>
		/// <param name="FalseParam">Name of a parameter that sets the value to 'false', for example: -noclean</param>
		/// <returns>Parameter value or default value if the paramater has not been specified</returns>
		bool GetOptionalParamValueIfNotSpecified(BuildCommand Command, bool? SpecifiedValue, bool Default, string TrueParam, string FalseParam)
		{
			return GetOptionalParamValueIfNotSpecified(Command, SpecifiedValue, (bool?)Default, TrueParam, FalseParam).Value;
		}

		/// <summary>
		/// Gets optional parameter from the command line if it hasn't been specified in the constructor. 
		/// If the command line is not available or the command has not been specified in the command line, default value will be used.
		/// </summary>
		/// <param name="Command">Command to parse the command line for. Can be null.</param>
		/// <param name="SpecifiedValue">Value specified in the constructor (or not)</param>
		/// <param name="Default">Default value.</param>
		/// <param name="TrueParam">Name of a parameter that sets the value to 'true', for example: -clean</param>
		/// <param name="FalseParam">Name of a parameter that sets the value to 'false', for example: -noclean</param>
		/// <returns>Parameter value or default value if the paramater has not been specified</returns>
		bool? GetOptionalParamValueIfNotSpecified(BuildCommand Command, bool? SpecifiedValue, bool? Default, string TrueParam, string FalseParam)
		{
			if (SpecifiedValue.HasValue)
			{
				return SpecifiedValue.Value;
			}
			else if (Command != null)
			{
				bool? Value = null;
				if (!String.IsNullOrEmpty(TrueParam) && Command.ParseParam(TrueParam))
				{
					Value = true;
				}
				else if (!String.IsNullOrEmpty(FalseParam) && Command.ParseParam(FalseParam))
				{
					Value = false;
				}
				if (Value.HasValue)
				{
					return Value;
				}
			}
			return Default;
		}

		/// <summary>
		/// Gets a parameter value from the command line if it hasn't been specified in the constructor. 
		/// If the command line is not available, default value will be used.
		/// </summary>
		/// <param name="Command">Command to parse the command line for. Can be null.</param>
		/// <param name="SpecifiedValue">Value specified in the constructor (or not)</param>
		/// <param name="ParamName">Command line parameter name to parse.</param>
		/// <param name="Default">Default value</param>
		/// <param name="bTrimQuotes">If set, the leading and trailing quotes will be removed, e.g. instead of "/home/User Name" it will return /home/User Name</param>
		/// <param name="ObsoleteParamName"></param>
		/// <param name="ObsoleteSpecifiedValue"></param>
		/// <returns>Parameter value.</returns>
		string ParseParamValueIfNotSpecified(BuildCommand Command, string SpecifiedValue, string ParamName, string Default = "", bool bTrimQuotes = false, string ObsoleteParamName = null, string ObsoleteSpecifiedValue = null)
		{
			string Result = Default;

			if (ObsoleteSpecifiedValue != null)
			{
				if (SpecifiedValue == null)
				{
					Logger.LogWarning("Value was provided for \"{ParamName}\" using obsolete name \"{ObsoleteParamName}\"", ParamName, ObsoleteParamName);
					Result = SpecifiedValue;
				}
				else
				{
					Logger.LogWarning("Value provided for obsolete name \"{ObsoleteParamName}\" will be ignored as \"{ParamName}\" was provided", ObsoleteParamName, ParamName);
				}
			}
			else if (SpecifiedValue != null)
			{
				Result = SpecifiedValue;
			}
			else if (Command != null)
			{
				string Parsed = Command.ParseParamValue(ParamName, null);

				if (ObsoleteParamName != null)
				{
					string ParsedObsolete = Command.ParseParamValue(ObsoleteParamName, null);
					if (Parsed == null)
					{
						// Didn't find the new name on the command line. If the obsolete name was found, use it, and warn.
						if (ParsedObsolete != null)
						{
							Logger.LogWarning("Obsolete argument \"{ObsoleteParamName}\" on command line - use \"{ParamName}\" instead", ObsoleteParamName, ParamName);
							Parsed = ParsedObsolete;
						}
					}
					else if (ParsedObsolete != null)
					{
						// Did find the new name on the command line - check for the obsolete name. If found, do not use it, and warn.
						Logger.LogWarning("Obsolete argument \"{ObsoleteParamName}\" will be ignored as \"{ParamName}\" was provided", ObsoleteParamName, ParamName);
					}
				}

				Result = Parsed ?? Default;
			}

			return bTrimQuotes ? Result.Trim( new char[]{'\"'} ) : Result;
		}

		/// <summary>
		/// Sets up platforms
		/// </summary>
        /// <param name="DependentPlatformMap">Set with a mapping from source->destination if specified on command line</param>
		/// <param name="Command">The command line to parse</param>
		/// <param name="OverrideTargetPlatforms">If passed use this always</param>
        /// <param name="DefaultTargetPlatforms">Use this if nothing is passed on command line</param>
		/// <param name="AllowPlatformParams">Allow raw -platform options</param>
		/// <param name="PlatformParamNames">Possible -parameters to check for</param>
		/// <returns>List of platforms parsed from the command line</returns>
		private List<TargetPlatformDescriptor> SetupTargetPlatforms(ref Dictionary<TargetPlatformDescriptor, TargetPlatformDescriptor> DependentPlatformMap, BuildCommand Command, List<TargetPlatformDescriptor> OverrideTargetPlatforms, List<TargetPlatformDescriptor> DefaultTargetPlatforms, bool AllowPlatformParams, params string[] PlatformParamNames)
		{
			List<TargetPlatformDescriptor> TargetPlatforms = null;
			if (CommandUtils.IsNullOrEmpty(OverrideTargetPlatforms))
			{
				if (Command != null)
				{
					// Parse the command line, we support the following params:
					// -'PlatformParamNames[n]'=Platform_1+Platform_2+...+Platform_k
					// or (if AllowPlatformParams is true)
					// -Platform_1 -Platform_2 ... -Platform_k
					string CmdLinePlatform = null;
					foreach (string ParamName in PlatformParamNames)
					{
						string ParamValue = Command.ParseParamValue(ParamName);
						if (!string.IsNullOrEmpty(ParamValue))
						{
							if (!string.IsNullOrEmpty(CmdLinePlatform))
							{
								CmdLinePlatform += "+";
							}

							CmdLinePlatform += ParamValue;
						}
					}

                    List<string> CookFlavors = null;
                    {
                        string CmdLineCookFlavor = Command.ParseParamValue("cookflavor");
                        if (!String.IsNullOrEmpty(CmdLineCookFlavor))
                        {
                            CookFlavors = new List<string>(CmdLineCookFlavor.Split(new char[] { '+' }, StringSplitOptions.RemoveEmptyEntries));
                        }
                    }

                    if (!String.IsNullOrEmpty(CmdLinePlatform))
					{
						// Get all platforms from the param value: Platform_1+Platform_2+...+Platform_k
						TargetPlatforms = new List<TargetPlatformDescriptor>();
						List<string> PlatformNames = (new HashSet<string>(CmdLinePlatform.Split(new char[] { '+' }, StringSplitOptions.RemoveEmptyEntries))).ToList();
						foreach (string PlatformName in PlatformNames)
						{
                            // Look for dependent platforms, Source_1.Dependent_1+Source_2.Dependent_2+Standalone_3
                            List<string> SubPlatformNames = new List<string>(PlatformName.Split(new char[] { '.' }, StringSplitOptions.RemoveEmptyEntries));

                            foreach (string SubPlatformName in SubPlatformNames)
                            {
								// Require this to be a valid platform name
								UnrealTargetPlatform NewPlatformType = UnrealTargetPlatform.Parse(SubPlatformName);

								// generate all valid platform descriptions for this platform type + cook flavors
								List<TargetPlatformDescriptor> PlatformDescriptors = Platform.GetValidTargetPlatforms(NewPlatformType, CookFlavors);
								TargetPlatforms.AddRange(PlatformDescriptors);
                                                              
								if (SubPlatformName != SubPlatformNames[0])
								{
									// This is not supported with cook flavors
									if (!CommandUtils.IsNullOrEmpty(CookFlavors))
									{
										throw new AutomationException("Cook flavors are not supported for dependent platforms!");
									}

									// We're a dependent platform so add ourselves to the map, pointing to the first element in the list
									UnrealTargetPlatform SubPlatformType;
									if (UnrealTargetPlatform.TryParse(SubPlatformNames[0], out SubPlatformType))
									{
										DependentPlatformMap.Add(new TargetPlatformDescriptor(NewPlatformType), new TargetPlatformDescriptor(SubPlatformType));
									}
								}
                            }
						}
					}
					else if (AllowPlatformParams)
					{
						// Look up platform names in the command line: -Platform_1 -Platform_2 ... -Platform_k
						TargetPlatforms = new List<TargetPlatformDescriptor>();
						foreach (UnrealTargetPlatform PlatType in UnrealTargetPlatform.GetValidPlatforms())
						{
							if (Command.ParseParam(PlatType.ToString()))
							{
                                List<TargetPlatformDescriptor> PlatformDescriptors = Platform.GetValidTargetPlatforms(PlatType, CookFlavors);
                                TargetPlatforms.AddRange(PlatformDescriptors);
							}
						}
					}
				}
			}
			else
			{
				TargetPlatforms = OverrideTargetPlatforms;
			}
			if (CommandUtils.IsNullOrEmpty(TargetPlatforms))
			{
				// Revert to single default platform - the current platform we're running
				TargetPlatforms = DefaultTargetPlatforms;
			}
			return TargetPlatforms;
		}

		public ProjectParams(ProjectParams InParams)
		{
			//
			//// Use this.Name with properties and fields!
			//

			this.RawProjectPath = InParams.RawProjectPath;
			this.RawProgramProjectOverride = InParams.RawProgramProjectOverride;
			this.MapsToCook = InParams.MapsToCook;
			this.MapIniSectionsToCook = InParams.MapIniSectionsToCook;
			this.DirectoriesToCook = InParams.DirectoriesToCook;
            this.DDCGraph = InParams.DDCGraph;
            this.InternationalizationPreset = InParams.InternationalizationPreset;
            this.CulturesToCook = InParams.CulturesToCook;
			this.OriginalReleaseVersion = InParams.OriginalReleaseVersion;
			this.BasedOnReleaseVersion = InParams.BasedOnReleaseVersion;
            this.CreateReleaseVersion = InParams.CreateReleaseVersion;
			this.ReferenceContainerGlobalFileName = InParams.ReferenceContainerGlobalFileName;
			this.ReferenceContainerAdditionalPath = InParams.ReferenceContainerAdditionalPath;
			this.ReferenceContainerChangesCSVFileName = InParams.ReferenceContainerChangesCSVFileName;
			this.ReferenceContainerCryptoKeys = InParams.ReferenceContainerCryptoKeys;
            this.GeneratePatch = InParams.GeneratePatch;
			this.AddPatchLevel = InParams.AddPatchLevel;
			this.DLCFile = InParams.DLCFile;
			this.DiscVersion = InParams.DiscVersion;
            this.DLCIncludeEngineContent = InParams.DLCIncludeEngineContent;
			this.DLCActLikePatch = InParams.DLCActLikePatch;
			this.DLCSkipNamedPak = InParams.DLCSkipNamedPak;
			this.DLCPakPluginFile = InParams.DLCPakPluginFile;
			this.DLCOverrideCookedSubDir = InParams.DLCOverrideCookedSubDir;
			this.DLCOverrideStagedSubDir = InParams.DLCOverrideStagedSubDir;
			this.DLCOverrideCryptoSettings = InParams.DLCOverrideCryptoSettings;
			this.DLCOverrideEncryptionKeyGuid = InParams.DLCOverrideEncryptionKeyGuid;
			this.StageOnlyCookedLocalizationForGameFeatures = InParams.StageOnlyCookedLocalizationForGameFeatures;
			this.DiffCookedContentPath = InParams.DiffCookedContentPath;
            this.AdditionalBuildOptions = InParams.AdditionalBuildOptions;
            this.AdditionalCookerOptions = InParams.AdditionalCookerOptions;
			this.ClientCookedTargets = InParams.ClientCookedTargets;
			this.ServerCookedTargets = InParams.ServerCookedTargets;
			this.EditorTargets = InParams.EditorTargets;
			this.ProgramTargets = InParams.ProgramTargets;
			this.TargetNames = new List<string>(InParams.TargetNames);
			this.ClientTargetPlatforms = InParams.ClientTargetPlatforms;
            this.ClientDependentPlatformMap = InParams.ClientDependentPlatformMap;
			this.ServerTargetPlatforms = InParams.ServerTargetPlatforms;
            this.ServerDependentPlatformMap = InParams.ServerDependentPlatformMap;
			this.ConfigOverrideParams = InParams.ConfigOverrideParams;
			this.Build = InParams.Build;
			this.SkipBuildClient = InParams.SkipBuildClient;
			this.SkipBuildEditor = InParams.SkipBuildEditor;
			this.Run = InParams.Run;
			this.Cook = InParams.Cook;
			this.IncrementalCook = InParams.IncrementalCook;
			this.IterateSharedCookedBuild = InParams.IterateSharedCookedBuild;
			this.IterateSharedBuildUsePrecompiledExe = InParams.IterateSharedBuildUsePrecompiledExe;
            this.CookAll = InParams.CookAll;
			this.CookPartialGC = InParams.CookPartialGC;
			this.CookInEditor = InParams.CookInEditor; 
			this.CookOutputDir = InParams.CookOutputDir;
			this.CookMapsOnly = InParams.CookMapsOnly;
			this.SkipCook = InParams.SkipCook;
			this.SkipCookOnTheFly = InParams.SkipCookOnTheFly;
            this.Prebuilt = InParams.Prebuilt;
            this.RunTimeoutSeconds = InParams.RunTimeoutSeconds;
			this.Clean = InParams.Clean;
			this.Pak = InParams.Pak;
			this.IgnorePaksFromDifferentCookSource = InParams.IgnorePaksFromDifferentCookSource;
			this.IoStore = InParams.IoStore;
			this.ZenStore = InParams.ZenStore;
			this.NoZenAutoLaunch = InParams.NoZenAutoLaunch;
			this.GenerateOptimizationData = InParams.GenerateOptimizationData;
			this.SignPak = InParams.SignPak;
			this.SignedPak = InParams.SignedPak;
			this.PakAlignForMemoryMapping = InParams.PakAlignForMemoryMapping;
			this.RehydrateAssets = InParams.RehydrateAssets;
			this.SkipPak = InParams.SkipPak;
            this.PrePak = InParams.PrePak;
            this.NoXGE = InParams.NoXGE;
			this.CookOnTheFly = InParams.CookOnTheFly;
            this.CookOnTheFlyStreaming = InParams.CookOnTheFlyStreaming;
            this.UnversionedCookedContent = InParams.UnversionedCookedContent;
			this.OptionalContent = InParams.OptionalContent;
			this.SkipCookingEditorContent = InParams.SkipCookingEditorContent;
			this.SkipCookingErrorSummary = InParams.SkipCookingErrorSummary;
			this.FileServer = InParams.FileServer;
			this.DedicatedServer = InParams.DedicatedServer;
			this.Client = InParams.Client;
			this.NoClient = InParams.NoClient;
			this.LogWindow = InParams.LogWindow;
			this.Stage = InParams.Stage;
			this.SkipStage = InParams.SkipStage;
			this.RunChunkAssigner = InParams.RunChunkAssigner;
			this.PreStaged = InParams.PreStaged;
            this.StageDirectoryParam = InParams.StageDirectoryParam;
			this.Manifests = InParams.Manifests;
            this.CreateChunkInstall = InParams.CreateChunkInstall;
			this.SkipEncryption = InParams.SkipEncryption;
			this.SpecifiedUnrealExe = InParams.SpecifiedUnrealExe;
			this.NoDebugInfo = InParams.NoDebugInfo;
			this.SeparateDebugInfo = InParams.SeparateDebugInfo;
			this.MapFile = InParams.MapFile;
			this.NoCleanStage = InParams.NoCleanStage;
			this.MapToRun = InParams.MapToRun;
			this.AdditionalServerMapParams = InParams.AdditionalServerMapParams;
			this.Foreign = InParams.Foreign;
			this.ForeignCode = InParams.ForeignCode;
			this.StageCommandline = InParams.StageCommandline;
            this.BundleName = InParams.BundleName;
			this.RunCommandline = InParams.RunCommandline;
			this.ServerCommandline = InParams.ServerCommandline;
            this.ClientCommandline = InParams.ClientCommandline;
            this.Package = InParams.Package;
			this.SkipPackage = InParams.SkipPackage;
			this.NeverPackage = InParams.NeverPackage;
			this.ForcePackageData = InParams.ForcePackageData;
			this.Deploy = InParams.Deploy;
			this.DeployFolder = InParams.DeployFolder;
			this.GetFile = InParams.GetFile;
			this.IterativeDeploy = InParams.IterativeDeploy;
			this.IgnoreCookErrors = InParams.IgnoreCookErrors;
			this.KeepFileOpenLog = InParams.KeepFileOpenLog;
			this.FastCook = InParams.FastCook;
			this.Snapshot = InParams.Snapshot;
			this.Devices = InParams.Devices;
			this.DeviceNames = InParams.DeviceNames;
			this.ServerDevice = InParams.ServerDevice;
            this.NullRHI = InParams.NullRHI;
			this.WriteBackMetadataToAssetRegistry = InParams.WriteBackMetadataToAssetRegistry;
            this.FakeClient = InParams.FakeClient;
            this.EditorTest = InParams.EditorTest;
            this.RunAutomationTests = InParams.RunAutomationTests;
            this.RunAutomationTest = InParams.RunAutomationTest;
            this.CrashIndex = InParams.CrashIndex;
            this.Port = InParams.Port;
			this.SkipServer = InParams.SkipServer;
			this.Unattended = InParams.Unattended;
            this.ServerDeviceAddress = InParams.ServerDeviceAddress;
            this.DeviceUsername = InParams.DeviceUsername;
            this.DevicePassword = InParams.DevicePassword;
            this.CrashReporter = InParams.CrashReporter;
			this.ClientConfigsToBuild = InParams.ClientConfigsToBuild;
			this.ServerConfigsToBuild = InParams.ServerConfigsToBuild;
			this.NumClients = InParams.NumClients;
			this.Compressed = InParams.Compressed;
			this.ForceUncompressed = InParams.ForceUncompressed;
			this.UseMergedModules = InParams.UseMergedModules;
			this.AdditionalPakOptions = InParams.AdditionalPakOptions;
			this.AdditionalIoStoreOptions = InParams.AdditionalIoStoreOptions;
			this.ForceOodleDllVersion = InParams.ForceOodleDllVersion;
			this.Archive = InParams.Archive;
			this.ArchiveDirectoryParam = InParams.ArchiveDirectoryParam;
			this.ArchiveMetaData = InParams.ArchiveMetaData;
			this.CreateAppBundle = InParams.CreateAppBundle;
			this.Distribution = InParams.Distribution;
			this.PackageEncryptionKeyFile = InParams.PackageEncryptionKeyFile;
			this.Prereqs = InParams.Prereqs;
			this.AppLocalDirectory = InParams.AppLocalDirectory;
			this.CustomDeploymentHandler = InParams.CustomDeploymentHandler;
			this.NoBootstrapExe = InParams.NoBootstrapExe;
            this.Prebuilt = InParams.Prebuilt;
            this.RunTimeoutSeconds = InParams.RunTimeoutSeconds;
			this.bIsCodeBasedProject = InParams.bIsCodeBasedProject;
			this.bCodeSign = InParams.bCodeSign;
			this.TitleID = InParams.TitleID;
			this.bTreatNonShippingBinariesAsDebugFiles = InParams.bTreatNonShippingBinariesAsDebugFiles;
			this.bUseExtraFlavor = InParams.bUseExtraFlavor;
			this.AdditionalPackageOptions = InParams.AdditionalPackageOptions;
			this.Trace = InParams.Trace;
			this.TraceHost = InParams.TraceHost;
			this.TraceFile = InParams.TraceFile;
			this.SessionLabel = InParams.SessionLabel;
			this.ProjectDescriptor = InParams.ProjectDescriptor;
			this.Upload = InParams.Upload;
			this.ApplyIoStoreOnDemand = InParams.ApplyIoStoreOnDemand;
			this.OnDemandSettings = InParams.OnDemandSettings;
			this.GenerateOptionalPakForNonChunkedBuild = InParams.GenerateOptionalPakForNonChunkedBuild;
			this.XcodeBuildOptions = InParams.XcodeBuildOptions;
			this.bMakeMacNative = InParams.bMakeMacNative;
			this.DeviceLockedTimeout = InParams.DeviceLockedTimeout;
		}

		/// <summary>
		/// Constructor. Be sure to use this.ParamName to set the actual property name as parameter names and property names
		/// overlap here.
		/// If a parameter value is not set, it will be parsed from the command line; if the command is null, the default value will be used.
		/// </summary>
		public ProjectParams(			
			FileReference RawProjectPath,
			FileReference RawProgramProjectOverride = null,
			BuildCommand Command = null,
			string Device = null,			
			string MapToRun = null,	
			string AdditionalServerMapParams = null,
			ParamList<string> Port = null,
			string RunCommandline = null,						
			string StageCommandline = null,
            string BundleName = null,
            string StageDirectoryParam = null,
			string UnrealExe = null,
			string UE4Exe = null, // remove this when deprecated HostParams.UE4Exe is removed
			string SignPak = null,
			List<UnrealTargetConfiguration> ClientConfigsToBuild = null,
			List<UnrealTargetConfiguration> ServerConfigsToBuild = null,
			ParamList<string> MapsToCook = null,
			ParamList<string> MapIniSectionsToCook = null,
			ParamList<string> DirectoriesToCook = null,
            string DDCGraph = null,
            string InternationalizationPreset = null,
            ParamList<string> CulturesToCook = null,
			ParamList<string> ClientCookedTargets = null,
			ParamList<string> EditorTargets = null,
			ParamList<string> ServerCookedTargets = null,
			List<TargetPlatformDescriptor> ClientTargetPlatforms = null,
            Dictionary<TargetPlatformDescriptor, TargetPlatformDescriptor> ClientDependentPlatformMap = null,
			List<TargetPlatformDescriptor> ServerTargetPlatforms = null,
            Dictionary<TargetPlatformDescriptor, TargetPlatformDescriptor> ServerDependentPlatformMap = null,
			List<string> ConfigOverrideParams = null,
			bool? Build = null,
			bool? SkipBuildClient = null,
			bool? SkipBuildEditor = null,
			bool? Cook = null,
			bool? Run = null,
			bool? SkipServer = null,
			bool? Clean = null,
			bool? Compressed = null,
			bool? ForceUncompressed = null,
			bool? UseMergedModules = null,
			string AdditionalPakOptions = null,
			string AdditionalIoStoreOptions = null,
			string ForceOodleDllVersion = null,
			bool? IncrementalCook = null,
			string IterateSharedCookedBuild = null,
			bool? IterateSharedBuildUsePrecompiledExe = null,
			bool? CookAll = null,
			bool? CookPartialGC = null,
			bool? CookInEditor = null,
			string CookOutputDir = null,
			bool? CookMapsOnly = null,
            bool? CookOnTheFly = null,
            bool? CookOnTheFlyStreaming = null,
            bool? UnversionedCookedContent = null,
			bool? OptionalContent = null,
			bool? EncryptIniFiles = null,
            bool? EncryptPakIndex = null,
			bool? EncryptEverything = null,
			bool? SkipCookingEditorContent = null,
			bool? SkipCookingErrorSummary = null,
            string AdditionalCookerOptions = null,
			string OriginalReleaseVersion = null,
			string BasedOnReleaseVersion = null,
            string CreateReleaseVersion = null,
			string CreateReleaseVersionBasePath = null,
			string BasedOnReleaseVersionBasePath = null,
			string ReferenceContainerGlobalFileName = null,
			string ReferenceContainerAdditionalPath = null,
			string ReferenceContainerChangesCSVFileName = null,
			string ReferenceContainerCryptoKeys = null,
			bool? GeneratePatch = null,
			bool? AddPatchLevel = null,
            string DiscVersion = null,
            string DLCName = null,
			string DLCOverrideCookedSubDir = null,
			string DLCOverrideStagedSubDir = null,
			FileReference DLCOverrideCryptoSettings = null,
			string DLCOverrideEncryptionKeyGuid = null,
			string DiffCookedContentPath = null,
            bool? DLCIncludeEngineContent = null,
			bool? DLCPakPluginFile = null,
			bool? DLCActLikePatch = null,
			bool? DLCSkipNamedPak = null,
			bool? StageOnlyCookedLocalizationForGameFeatures = null,
			bool? CrashReporter = null,
			bool? DedicatedServer = null,
			bool? Client = null,
			bool? Deploy = null,
			string DeployFolder = null,
			string GetFile = null,
			bool? FileServer = null,
			bool? Foreign = null,
			bool? ForeignCode = null,
			bool? LogWindow = null,
			bool? NoCleanStage = null,
			bool? NoClient = null,
			bool? NoDebugInfo = null,
			bool? SeparateDebugInfo = null,
			bool? MapFile = null,
			bool? NoXGE = null,
			bool? SkipPackage = null,
			bool? NeverPackage = null,
			bool? Package = null,
			bool? Pak = null,
			bool? IgnorePaksFromDifferentCookSource = null,
			bool? IoStore = null,
			bool? ZenStore = null,
			string NoZenAutoLaunch = null,
			string ZenWorkspaceSharePath = null,
			bool? ZenOmitPakFiles = null,
			bool? SkipIoStore = null,
			bool? GenerateOptimizationData = null,
			bool? Prereqs = null,
			string AppLocalDirectory = null,
			string CustomDeploymentHandler = null,
			bool? NoBootstrapExe = null,
            bool? SignedPak = null,
			bool? PakAlignForMemoryMapping = null,
			bool? RehydrateAssets = null,
			bool? NullRHI = null,
            bool? FakeClient = null,
            bool? EditorTest = null,
            bool? RunAutomationTests = null,
            string RunAutomationTest = null,
            int? CrashIndex = null,
			bool? SkipCook = null,
			bool? SkipCookOnTheFly = null,
			bool? SkipPak = null,
            bool? PrePak = null,
            bool? SkipStage = null,
			bool? RunChunkAssigner = null,
			bool? Stage = null,
			bool? PreStaged = null,
			bool? Manifests = null,
            bool? CreateChunkInstall = null,
			bool? SkipEncryption = null,
			bool? Unattended = null,
			int? NumClients = null,
			bool? Archive = null,
			string ArchiveDirectoryParam = null,
			bool? ArchiveMetaData = null,
			bool? CreateAppBundle = null,
			string SpecifiedClientTarget = null,
			string SpecifiedServerTarget = null,
			ParamList<string> ProgramTargets = null,
			bool? Distribution = null,
			string PackageEncryptionKeyFile = null,
			bool? Prebuilt = null,
            int? RunTimeoutSeconds = null,
			string SpecifiedArchitecture = null,
			string ServerArchitecture = null,
			string EditorArchitecture = null,
			string ClientArchitecture = null,
			string ProgramArchitecture = null,
			string UbtArgs = null,
			string AdditionalPackageOptions = null,
			bool? IterativeDeploy = null,
			bool? FastCook = null,
			bool? Snapshot = null,
			bool? IgnoreCookErrors = null,
			bool? KeepFileOpenLog = null,
			bool? CodeSign = null,
			bool? TreatNonShippingBinariesAsDebugFiles = null,
			bool? UseExtraFlavor = null,
			string Provision = null,
			string Certificate = null,
		    string Team = null,
		    bool AutomaticSigning = false,
			string Trace = null,
			string TraceHost = null,
			string TraceFile = null,
			string SessionLabel = null,
			ParamList<string> InMapsToRebuildLightMaps = null,
			ParamList<string> InMapsToRebuildHLOD = null,
			ParamList<string> TitleID = null,
			string Upload = null,
			IoStoreOnDemandSettings OnDemandSettings = null,
			bool? GenerateOptionalPakForNonChunkedBuild = null,
			string XcodeBuildOptions = null,
			bool bMakeMacNative = false
			)
		{
			//
			//// Use this.Name with properties and fields!
			//

			this.RawProjectPath = RawProjectPath;
			try
			{
				this.ProjectDescriptor = ProjectDescriptor.FromFile(RawProjectPath);
			}
			catch { this.ProjectDescriptor = new ProjectDescriptor(); }

			if(RawProgramProjectOverride != null)
			{
				this.RawProgramProjectOverride = RawProgramProjectOverride;
			}

			if (DirectoriesToCook != null)
			{
				this.DirectoriesToCook = DirectoriesToCook;
			}
			this.DDCGraph = ParseParamValueIfNotSpecified(Command, DDCGraph, "ddc");
            this.InternationalizationPreset = ParseParamValueIfNotSpecified(Command, InternationalizationPreset, "i18npreset");

            // If not specified in parameters, check commandline.
            if (CulturesToCook == null)
            {
                if (Command != null)
                {
                    var CookCulturesString = Command.ParseParamValue("CookCultures");
                    if (CookCulturesString != null)
                    {
                        this.CulturesToCook = new ParamList<string>(CookCulturesString.Split(new char[] { ',' }, StringSplitOptions.RemoveEmptyEntries));
                    }
                }
            }
            else
            {
                this.CulturesToCook = CulturesToCook;
            }

			if (ClientCookedTargets != null)
			{
				this.ClientCookedTargets = ClientCookedTargets;
			}
			if (ServerCookedTargets != null)
			{
				this.ServerCookedTargets = ServerCookedTargets;
			}
			if (EditorTargets != null)
			{
				this.EditorTargets = EditorTargets;
			}
			if (ProgramTargets != null)
			{
				this.ProgramTargets = ProgramTargets;
			}

			// Parse command line params for client platforms "-TargetPlatform=Win64+Mac", "-Platform=Win64+Mac" and also "-Win64", "-Mac" etc.
            if (ClientDependentPlatformMap != null)
            {
                this.ClientDependentPlatformMap = ClientDependentPlatformMap;
            }

            List<TargetPlatformDescriptor> DefaultTargetPlatforms = new ParamList<TargetPlatformDescriptor>(new TargetPlatformDescriptor(HostPlatform.Current.HostEditorPlatform));
            this.ClientTargetPlatforms = SetupTargetPlatforms(ref this.ClientDependentPlatformMap, Command, ClientTargetPlatforms, DefaultTargetPlatforms, true, "TargetPlatform", "Platform");

            // Parse command line params for server platforms "-ServerTargetPlatform=Win64+Mac", "-ServerPlatform=Win64+Mac". "-Win64" etc is not allowed here
            if (ServerDependentPlatformMap != null)
            {
                this.ServerDependentPlatformMap = ServerDependentPlatformMap;
            }
            this.ServerTargetPlatforms = SetupTargetPlatforms(ref this.ServerDependentPlatformMap, Command, ServerTargetPlatforms, this.ClientTargetPlatforms, false, "ServerTargetPlatform", "ServerPlatform");

			this.Build = GetParamValueIfNotSpecified(Command, Build, this.Build, "build");
			bool bSkipBuild = GetParamValueIfNotSpecified(Command, null, false, "skipbuild");
			if (bSkipBuild)
			{
				this.Build = false;
			}

			this.SkipBuildClient = GetParamValueIfNotSpecified(Command, SkipBuildClient, this.SkipBuildClient, "skipbuildclient");
			this.SkipBuildEditor = GetParamValueIfNotSpecified(Command, SkipBuildEditor, this.SkipBuildEditor, "skipbuildeditor", "nocompileeditor");
			this.Run = GetParamValueIfNotSpecified(Command, Run, this.Run, "run");
			this.Cook = GetParamValueIfNotSpecified(Command, Cook, this.Cook, "cook");
			this.CreateReleaseVersionBasePath = ParseParamValueIfNotSpecified(Command, CreateReleaseVersionBasePath, "createreleaseversionroot", String.Empty);
			this.BasedOnReleaseVersionBasePath = ParseParamValueIfNotSpecified(Command, BasedOnReleaseVersionBasePath, "basedonreleaseversionroot", String.Empty);
			this.ReferenceContainerGlobalFileName = ParseParamValueIfNotSpecified(Command, ReferenceContainerGlobalFileName, "ReferenceContainerGlobalFileName", String.Empty);
			this.ReferenceContainerAdditionalPath = ParseParamValueIfNotSpecified(Command, ReferenceContainerAdditionalPath, "ReferenceContainerAdditionalPath", String.Empty);
			this.ReferenceContainerChangesCSVFileName = ParseParamValueIfNotSpecified(Command, ReferenceContainerChangesCSVFileName, "ReferenceContainerChangesCSVFileName", String.Empty);
			this.ReferenceContainerCryptoKeys = ParseParamValueIfNotSpecified(Command, ReferenceContainerCryptoKeys, "ReferenceContainerCryptoKeys", String.Empty); 
			this.OriginalReleaseVersion = ParseParamValueIfNotSpecified(Command, OriginalReleaseVersion, "originalreleaseversion", String.Empty);
			this.CreateReleaseVersion = ParseParamValueIfNotSpecified(Command, CreateReleaseVersion, "createreleaseversion", String.Empty);
            this.BasedOnReleaseVersion = ParseParamValueIfNotSpecified(Command, BasedOnReleaseVersion, "basedonreleaseversion", String.Empty);
            this.GeneratePatch = GetParamValueIfNotSpecified(Command, GeneratePatch, this.GeneratePatch, "GeneratePatch");
            this.AddPatchLevel = GetParamValueIfNotSpecified(Command, AddPatchLevel, this.AddPatchLevel, "AddPatchLevel");
			this.DiscVersion = ParseParamValueIfNotSpecified(Command, DiscVersion, "DiscVersion", String.Empty);
			this.AdditionalCookerOptions = ParseParamValueIfNotSpecified(Command, AdditionalCookerOptions, "AdditionalCookerOptions", String.Empty);
		
			DLCName = ParseParamValueIfNotSpecified(Command, DLCName, "DLCName", String.Empty);
			if (!String.IsNullOrEmpty(DLCName))
			{
				// is it fully specified already (look for having a uplugin extension)
				if (string.Equals(Path.GetExtension(DLCName), ".uplugin", StringComparison.InvariantCultureIgnoreCase))
				{
					this.DLCFile = new FileReference(DLCName);
				}
				else
				{
					List<PluginInfo> CandidatePlugins = Plugins.ReadAvailablePlugins(Unreal.EngineDirectory,
						DirectoryReference.FromFile(RawProjectPath), AdditionalPluginDirectories);
					PluginInfo DLCPlugin = CandidatePlugins.FirstOrDefault(x => String.Equals(x.Name, DLCName, StringComparison.InvariantCultureIgnoreCase));
					if (DLCPlugin == null)
					{
						this.DLCFile = FileReference.Combine(RawProjectPath.Directory, "Plugins", DLCName, DLCName + ".uplugin");
					}
					else
					{
						this.DLCFile = DLCPlugin.File;
					}
				}
			}

			this.DiffCookedContentPath = ParseParamValueIfNotSpecified(Command, DiffCookedContentPath, "DiffCookedContentPath", String.Empty);
            this.DLCIncludeEngineContent = GetParamValueIfNotSpecified(Command, DLCIncludeEngineContent, this.DLCIncludeEngineContent, "DLCIncludeEngineContent");
			this.DLCPakPluginFile = GetParamValueIfNotSpecified(Command, DLCPakPluginFile, this.DLCPakPluginFile, "DLCPakPluginFile");
			this.DLCActLikePatch = GetParamValueIfNotSpecified(Command, DLCActLikePatch, this.DLCActLikePatch, "DLCActLikePatch");
			this.DLCSkipNamedPak = GetParamValueIfNotSpecified(Command, DLCSkipNamedPak, this.DLCSkipNamedPak, "DLCSkipNamedPak");
			this.DLCOverrideCookedSubDir = ParseParamValueIfNotSpecified(Command, DLCOverrideCookedSubDir, "DLCOverrideCookedSubDir", null);
			this.DLCOverrideStagedSubDir = ParseParamValueIfNotSpecified(Command, DLCOverrideStagedSubDir, "DLCOverrideStagedSubDir", null);

			this.StageOnlyCookedLocalizationForGameFeatures = GetParamValueIfNotSpecified(Command, StageOnlyCookedLocalizationForGameFeatures, this.StageOnlyCookedLocalizationForGameFeatures, "StageOnlyCookedLocalizationForGameFeatures");

			this.SkipCook = GetParamValueIfNotSpecified(Command, SkipCook, this.SkipCook, "skipcook");
			if (this.SkipCook)
			{
				this.Cook = true;
			}
			this.Clean = GetOptionalParamValueIfNotSpecified(Command, Clean, this.Clean, "clean", null);
			this.SignPak = ParseParamValueIfNotSpecified(Command, SignPak, "signpak", String.Empty);
			this.SignedPak = !String.IsNullOrEmpty(this.SignPak) || GetParamValueIfNotSpecified(Command, SignedPak, this.SignedPak, "signedpak");
			if (string.IsNullOrEmpty(this.SignPak) && RawProjectPath != null)
			{
				this.SignPak = Path.Combine(RawProjectPath.Directory.FullName, @"Restricted\NoRedist\Build\Keys.txt");
				if (!File.Exists(this.SignPak))
				{
					this.SignPak = null;
				}
			}
			this.PakAlignForMemoryMapping = GetParamValueIfNotSpecified(Command, PakAlignForMemoryMapping, this.PakAlignForMemoryMapping, "PakAlignForMemoryMapping");
			this.RehydrateAssets = GetParamValueIfNotSpecified(Command, RehydrateAssets, this.RehydrateAssets, "RehydrateAssets");		
			this.Pak = GetParamValueIfNotSpecified(Command, Pak, this.Pak, "pak");
			this.IgnorePaksFromDifferentCookSource = GetParamValueIfNotSpecified(Command, IgnorePaksFromDifferentCookSource, this.IgnorePaksFromDifferentCookSource, "IgnorePaksFromDifferentCookSource");
			this.IoStore = GetParamValueIfNotSpecified(Command, IoStore, this.IoStore, "iostore");
			this.SkipIoStore = GetParamValueIfNotSpecified(Command, SkipIoStore, this.SkipIoStore, "skipiostore");
			this.ZenStore = GetParamValueIfNotSpecified(Command, ZenStore, this.ZenStore, "zenstore");
			if (this.ZenStore && this.Cook && !this.SkipCook)
			{
				this.AdditionalCookerOptions += " -ZenStore";
			}
			this.NoZenAutoLaunch = ParseParamValueIfNotSpecified(Command, NoZenAutoLaunch, "NoZenAutoLaunch", String.Empty);
			if (string.IsNullOrEmpty(this.NoZenAutoLaunch) && GetParamValueIfNotSpecified(Command, null, false, "NoZenAutoLaunch"))
			{
				this.NoZenAutoLaunch = "127.0.0.1";
			}
			if (!string.IsNullOrEmpty(this.NoZenAutoLaunch) && this.Cook && !this.SkipCook)
			{
				this.AdditionalCookerOptions += string.Format(" -NoZenAutoLaunch={0}", this.NoZenAutoLaunch);
			}
			this.ZenWorkspaceSharePath = ParseParamValueIfNotSpecified(Command, ZenWorkspaceSharePath, "ZenWorkspaceSharePath", String.Empty);
			this.ZenOmitPakFiles = GetParamValueIfNotSpecified(Command, ZenOmitPakFiles, false, "ZenOmitPakFiles");
			this.GenerateOptimizationData = GetParamValueIfNotSpecified(Command, GenerateOptimizationData, this.GenerateOptimizationData, "makebinaryconfig");
			
			this.SkipPak = GetParamValueIfNotSpecified(Command, SkipPak, this.SkipPak, "skippak");
			if (this.SkipPak)
			{
				this.Pak = true;
			}
            this.PrePak = GetParamValueIfNotSpecified(Command, PrePak, this.PrePak, "prepak");
            if (this.PrePak)
            {
                this.Pak = true;
                this.SkipCook = true;
            }
            this.NoXGE = GetParamValueIfNotSpecified(Command, NoXGE, this.NoXGE, "noxge");
			this.CookOnTheFly = GetParamValueIfNotSpecified(Command, CookOnTheFly, this.CookOnTheFly, "cookonthefly");
            if (this.CookOnTheFly && this.SkipCook)
            {
                this.Cook = false;
            }
            this.CookOnTheFlyStreaming = GetParamValueIfNotSpecified(Command, CookOnTheFlyStreaming, this.CookOnTheFlyStreaming, "cookontheflystreaming");
            this.UnversionedCookedContent = GetOptionalParamValueIfNotSpecified(Command, UnversionedCookedContent, this.UnversionedCookedContent, "UnversionedCookedContent", "VersionCookedContent");
			this.OptionalContent = GetOptionalParamValueIfNotSpecified(Command, OptionalContent, this.OptionalContent, "editoroptional", "noeditoroptional");
			this.SkipCookingEditorContent = GetParamValueIfNotSpecified(Command, SkipCookingEditorContent, this.SkipCookingEditorContent, "SkipCookingEditorContent");
			this.SkipCookingErrorSummary = GetParamValueIfNotSpecified(Command, SkipCookingErrorSummary, this.SkipCookingErrorSummary, "SkipCookingErrorSummary");
			this.Compressed = GetParamValueIfNotSpecified(Command, Compressed, this.Compressed, "compressed");
			this.ForceUncompressed = GetParamValueIfNotSpecified(Command, ForceUncompressed, this.ForceUncompressed, "ForceUncompressed");
			this.UseMergedModules = GetParamValueIfNotSpecified(Command, UseMergedModules, this.UseMergedModules, "UseMergedModules");
			this.AdditionalPakOptions = ParseParamValueIfNotSpecified(Command, AdditionalPakOptions, "AdditionalPakOptions");
			if (!string.IsNullOrEmpty(this.NoZenAutoLaunch))
			{
				this.AdditionalPakOptions += string.Format(" -NoZenAutoLaunch={0}", this.NoZenAutoLaunch);
			}
			this.AdditionalIoStoreOptions = ParseParamValueIfNotSpecified(Command, AdditionalIoStoreOptions, "AdditionalIoStoreOptions");
			this.ForceOodleDllVersion = ParseParamValueIfNotSpecified(Command, ForceOodleDllVersion, "ForceOodleDllVersion");
			this.IncrementalCook = GetParamValueIfNotSpecified(Command, IncrementalCook, this.IncrementalCook, new string[] { "iterativecooking", "iterate", "cookincremental" });
			this.IterateSharedCookedBuild = GetParamValueIfNotSpecified(Command, false, false, "iteratesharedcookedbuild") ? "usesyncedbuild" : null;
			this.IterateSharedCookedBuild = ParseParamValueIfNotSpecified(Command, IterateSharedCookedBuild, "IterateSharedCookedBuild", String.Empty);
			this.IterateSharedBuildUsePrecompiledExe = GetParamValueIfNotSpecified(Command, IterateSharedBuildUsePrecompiledExe, this.IterateSharedBuildUsePrecompiledExe, new string[] { "IterateSharedBuildUsePrecompiledExe" });
			
			this.SkipCookOnTheFly = GetParamValueIfNotSpecified(Command, SkipCookOnTheFly, this.SkipCookOnTheFly, "skipcookonthefly");
			this.CookAll = GetParamValueIfNotSpecified(Command, CookAll, this.CookAll, "CookAll");
			this.CookPartialGC = GetParamValueIfNotSpecified(Command, CookPartialGC, this.CookPartialGC, "CookPartialGC");
			this.CookInEditor = GetParamValueIfNotSpecified(Command, CookInEditor, this.CookInEditor, "CookInEditor");
			this.CookOutputDir = ParseParamValueIfNotSpecified(Command, CookOutputDir, "CookOutputDir", String.Empty, true);
			this.CookMapsOnly = GetParamValueIfNotSpecified(Command, CookMapsOnly, this.CookMapsOnly, "CookMapsOnly");
			this.FileServer = GetParamValueIfNotSpecified(Command, FileServer, this.FileServer, "fileserver");
			this.DedicatedServer = GetParamValueIfNotSpecified(Command, DedicatedServer, this.DedicatedServer, "dedicatedserver", "server");
			this.Client = GetParamValueIfNotSpecified(Command, Client, this.Client, "client");
			/*if( this.Client )
			{
				this.DedicatedServer = true;
			}*/
			this.NoClient = GetParamValueIfNotSpecified(Command, NoClient, this.NoClient, "noclient");

			if(Command != null)
			{
				if(TargetNames == null)
				{
					TargetNames = new List<string>();
				}
				foreach(string TargetParam in Command.ParseParamValues("target"))
				{
					TargetNames.AddRange(TargetParam.Split('+'));
				}
			}

			this.LogWindow = GetParamValueIfNotSpecified(Command, LogWindow, this.LogWindow, "logwindow");
			string ExtraTargetsToStageWithClientString = null;
			ExtraTargetsToStageWithClientString = ParseParamValueIfNotSpecified(Command, ExtraTargetsToStageWithClientString, "ExtraTargetsToStageWithClient", null);
			if (!string.IsNullOrEmpty(ExtraTargetsToStageWithClientString))
			{
				this.ExtraTargetsToStageWithClient = new ParamList<string>(ExtraTargetsToStageWithClientString.Split('+'));
			}
			this.Stage = GetParamValueIfNotSpecified(Command, Stage, this.Stage, "stage");
			this.SkipStage = GetParamValueIfNotSpecified(Command, SkipStage, this.SkipStage, "skipstage");
			if (this.SkipStage)
			{
				this.Stage = true;
			}
			this.RunChunkAssigner = GetParamValueIfNotSpecified(Command, RunChunkAssigner, this.RunChunkAssigner, "RunChunkAssigner");
			this.PreStaged = GetParamValueIfNotSpecified(Command, PreStaged, this.PreStaged, "prestaged");
			if (this.PreStaged)
			{
				this.Build = false;
				this.Cook = true;
				this.SkipCook = true;
				this.Stage = true;
				this.SkipStage = true;
			}

			this.StageDirectoryParam = ParseParamValueIfNotSpecified(Command, StageDirectoryParam, "stagingdirectory", String.Empty, true);
			this.OptionalFileStagingDirectory = ParseParamValueIfNotSpecified(Command, OptionalFileStagingDirectory, "optionalfilestagingdirectory", String.Empty, true);
			this.OptionalFileInputDirectory = ParseParamValueIfNotSpecified(Command, OptionalFileInputDirectory, "optionalfileinputdirectory", String.Empty, true);
			this.CookerSupportFilesSubdirectory = ParseParamValueIfNotSpecified(Command, CookerSupportFilesSubdirectory, "CookerSupportFilesSubdirectory", String.Empty, true);
			this.bCodeSign = GetOptionalParamValueIfNotSpecified(Command, CodeSign, IsEpicBuildMachine(), "CodeSign", "NoCodeSign");
			this.bTreatNonShippingBinariesAsDebugFiles = GetParamValueIfNotSpecified(Command, TreatNonShippingBinariesAsDebugFiles, false, "TreatNonShippingBinariesAsDebugFiles");
			this.bUseExtraFlavor = GetParamValueIfNotSpecified(Command, UseExtraFlavor, false, "UseExtraFlavor");
			this.Manifests = GetParamValueIfNotSpecified(Command, Manifests, this.Manifests, "manifests");
            this.CreateChunkInstall = GetParamValueIfNotSpecified(Command, CreateChunkInstall, this.CreateChunkInstall, "createchunkinstall");
			this.SkipEncryption = GetParamValueIfNotSpecified(Command, SkipEncryption, this.SkipEncryption, "skipencryption");
			this.ChunkInstallDirectory = ParseParamValueIfNotSpecified(Command, ChunkInstallDirectory, "chunkinstalldirectory", String.Empty, true);
			this.ChunkInstallVersionString = ParseParamValueIfNotSpecified(Command, ChunkInstallVersionString, "chunkinstallversion", String.Empty, true);
            this.ChunkInstallReleaseString = ParseParamValueIfNotSpecified(Command, ChunkInstallReleaseString, "chunkinstallrelease", String.Empty, true);
            if (string.IsNullOrEmpty(this.ChunkInstallReleaseString))
            {
                this.ChunkInstallReleaseString = this.ChunkInstallVersionString;
            }
			this.Archive = GetParamValueIfNotSpecified(Command, Archive, this.Archive, "archive");
			this.ArchiveDirectoryParam = ParseParamValueIfNotSpecified(Command, ArchiveDirectoryParam, "archivedirectory", String.Empty, true);
			this.ArchiveMetaData = GetParamValueIfNotSpecified(Command, ArchiveMetaData, this.ArchiveMetaData, "archivemetadata");
			this.CreateAppBundle = GetParamValueIfNotSpecified(Command, CreateAppBundle, true, "createappbundle");
			this.Distribution = GetParamValueIfNotSpecified(Command, Distribution, this.Distribution, "distribution");
			this.PackageEncryptionKeyFile = ParseParamValueIfNotSpecified(Command, PackageEncryptionKeyFile, "packageencryptionkeyfile", null);
			this.Prereqs = GetParamValueIfNotSpecified(Command, Prereqs, this.Prereqs, "prereqs");
			this.AppLocalDirectory = ParseParamValueIfNotSpecified(Command, AppLocalDirectory, "applocaldirectory", String.Empty, true);
			this.CustomDeploymentHandler = ParseParamValueIfNotSpecified(Command, CustomDeploymentHandler, "customdeployment", String.Empty, true );
			this.NoBootstrapExe = GetParamValueIfNotSpecified(Command, NoBootstrapExe, this.NoBootstrapExe, "nobootstrapexe");
            this.Prebuilt = GetParamValueIfNotSpecified(Command, Prebuilt, this.Prebuilt, "prebuilt");
            if (this.Prebuilt)
            {
                this.SkipCook = true;
                /*this.SkipPak = true;
                this.SkipStage = true;
                this.Pak = true;
                this.Stage = true;*/
                this.Cook = true;
                this.Archive = true;
                
                this.Deploy = true;
                this.Run = true;
                //this.StageDirectoryParam = this.PrebuiltDir;
            }
            this.NoDebugInfo = GetParamValueIfNotSpecified(Command, NoDebugInfo, this.NoDebugInfo, "nodebuginfo");
			this.SeparateDebugInfo = GetParamValueIfNotSpecified(Command, SeparateDebugInfo, this.SeparateDebugInfo, "separatedebuginfo");
			this.MapFile = GetParamValueIfNotSpecified(Command, MapFile, this.MapFile, "mapfile");
			this.NoCleanStage = GetParamValueIfNotSpecified(Command, NoCleanStage, this.NoCleanStage, "nocleanstage");
			this.MapToRun = ParseParamValueIfNotSpecified(Command, MapToRun, "map", String.Empty);
			this.AdditionalServerMapParams = ParseParamValueIfNotSpecified(Command, AdditionalServerMapParams, "AdditionalServerMapParams", String.Empty);
			this.Foreign = GetParamValueIfNotSpecified(Command, Foreign, this.Foreign, "foreign");
			this.ForeignCode = GetParamValueIfNotSpecified(Command, ForeignCode, this.ForeignCode, "foreigncode");
			this.StageCommandline = ParseParamValueIfNotSpecified(Command, StageCommandline, "cmdline");
			this.BundleName = ParseParamValueIfNotSpecified(Command, BundleName, "bundlename");
			this.RunCommandline = ParseParamValueIfNotSpecified(Command, RunCommandline, "addcmdline");
			this.RunCommandline = this.RunCommandline.Replace('\'', '\"'); // replace any single quotes with double quotes
			this.ServerCommandline = ParseParamValueIfNotSpecified(Command, ServerCommandline, "servercmdline");
			this.ServerCommandline = this.ServerCommandline.Replace('\'', '\"'); // replace any single quotes with double quotes
            this.ClientCommandline = ParseParamValueIfNotSpecified(Command, ClientCommandline, "clientcmdline");
            this.ClientCommandline = this.ClientCommandline.Replace('\'', '\"'); // replace any single quotes with double quotes
            this.Package = GetParamValueIfNotSpecified(Command, Package, this.Package, "package");
			this.SkipPackage = GetParamValueIfNotSpecified(Command, SkipPackage, this.SkipPackage, "skippackage");
			this.NeverPackage = GetParamValueIfNotSpecified(Command, NeverPackage, this.NeverPackage, "neverpackage");
			this.ForcePackageData = GetParamValueIfNotSpecified(Command, Package, this.ForcePackageData, "forcepackagedata");

			this.Deploy = GetParamValueIfNotSpecified(Command, Deploy, this.Deploy, "deploy");
			this.DeployFolder = ParseParamValueIfNotSpecified(Command, DeployFolder, "deploy", null);

			// always set the default deploy folder, so that it is available in -skipdeploy scenarios too
			if (string.IsNullOrEmpty(this.DeployFolder))
			{
				this.DeployFolder = UnrealBuildTool.DeployExports.GetDefaultDeployFolder(this.ShortProjectName);
			}
			else
			{
				this.Deploy = true;
			}

			// If the user specified archive without a param, set to the default. That way logging will be correct and other code doesn't
			// need to do this check and fallback
			if (this.Archive && string.IsNullOrEmpty(this.ArchiveDirectoryParam))
			{
				this.ArchiveDirectoryParam = BaseArchiveDirectory;
			}

			this.GetFile = ParseParamValueIfNotSpecified(Command, GetFile, "getfile", null);

			this.IterativeDeploy = GetParamValueIfNotSpecified(Command, IterativeDeploy, this.IterativeDeploy, new string[] {"iterativedeploy", "iterate" } );
			this.FastCook = GetParamValueIfNotSpecified(Command, FastCook, this.FastCook, "FastCook");
			this.Snapshot = GetParamValueIfNotSpecified(Command, Snapshot, this.Snapshot, "Snapshot");

			// -snapshot implies the use of -cookincremental for cooking.
			this.IncrementalCook = this.Snapshot || this.IncrementalCook;
			this.IgnoreCookErrors = GetParamValueIfNotSpecified(Command, IgnoreCookErrors, this.IgnoreCookErrors, "IgnoreCookErrors");
			this.KeepFileOpenLog = GetParamValueIfNotSpecified(Command, KeepFileOpenLog, this.KeepFileOpenLog, "KeepFileOpenLog");

            string DeviceString = ParseParamValueIfNotSpecified(Command, Device, "device", String.Empty).Trim(new char[] { '\"' });
            if(DeviceString == "")
            {
                this.Devices = new ParamList<string>("");
                this.DeviceNames = new ParamList<string>("");
            }
            else
            {
                this.Devices = new ParamList<string>(DeviceString.Split(new char[] { '+' }, StringSplitOptions.RemoveEmptyEntries));
                this.DeviceNames = new ParamList<string>();
                foreach (var d in this.Devices)
                {
                    // strip the platform prefix the specified device.
                    if (d.Contains("@"))
                    {
                        this.DeviceNames.Add(d.Substring(d.IndexOf("@") + 1));
                    }
                    else
                    {
                        this.DeviceNames.Add(d);
                    }
                }
            }

			this.Provision = ParseParamValueIfNotSpecified(Command, Provision, "provision", String.Empty, true);
			this.Certificate = ParseParamValueIfNotSpecified(Command, Certificate, "certificate", String.Empty, true);
			this.Team = ParseParamValueIfNotSpecified(Command, Team, "team", String.Empty, true);
			this.AutomaticSigning = GetParamValueIfNotSpecified(Command, AutomaticSigning, this.AutomaticSigning, "AutomaticSigning");

			this.ServerDevice = ParseParamValueIfNotSpecified(Command, ServerDevice, "serverdevice", this.Devices.Count > 0 ? this.Devices[0] : "");
			this.NullRHI = GetParamValueIfNotSpecified(Command, NullRHI, this.NullRHI, "nullrhi");
			this.FakeClient = GetParamValueIfNotSpecified(Command, FakeClient, this.FakeClient, "fakeclient");
			this.EditorTest = GetParamValueIfNotSpecified(Command, EditorTest, this.EditorTest, "editortest");
            this.RunAutomationTest = ParseParamValueIfNotSpecified(Command, RunAutomationTest, "RunAutomationTest");
            this.RunAutomationTests = this.RunAutomationTest != "" || GetParamValueIfNotSpecified(Command, RunAutomationTests, this.RunAutomationTests, "RunAutomationTests");
            this.SkipServer = GetParamValueIfNotSpecified(Command, SkipServer, this.SkipServer, "skipserver");
			this.SpecifiedUnrealExe = ParseParamValueIfNotSpecified(Command, UnrealExe, "unrealexe", null, ObsoleteSpecifiedValue: UE4Exe, ObsoleteParamName: "ue4exe");
			this.Unattended = GetParamValueIfNotSpecified(Command, Unattended, this.Unattended, "unattended");
			this.DeviceUsername = ParseParamValueIfNotSpecified(Command, DeviceUsername, "deviceuser", String.Empty);
			this.DevicePassword = ParseParamValueIfNotSpecified(Command, DevicePassword, "devicepass", String.Empty);
			this.CrashReporter = GetParamValueIfNotSpecified(Command, CrashReporter, this.CrashReporter, "crashreporter");
			this.UbtArgs = ParseParamValueIfNotSpecified(Command, UbtArgs, "ubtargs", String.Empty);
			this.AdditionalPackageOptions = ParseParamValueIfNotSpecified(Command, AdditionalPackageOptions, "AdditionalPackageOptions", String.Empty);
			this.WriteBackMetadataToAssetRegistry = ParseParamValueIfNotSpecified(Command, WriteBackMetadataToAssetRegistry, "WriteBackMetadataToAssetRegistry", String.Empty);

			string SpecifiedArchString, ServerArchString, EditorArchString, ClientArchString, ProgramArchString;
			SpecifiedArchString = ParseParamValueIfNotSpecified(Command, SpecifiedArchitecture, "specifiedarchitecture", null);
			// if SpecifiedArchitecture is used, then set them all to it, and then allow comandline to override specific ones
			ServerArchString = EditorArchString = ClientArchString = ProgramArchString = SpecifiedArchString;
			ServerArchString = ParseParamValueIfNotSpecified(Command, ServerArchitecture, "serverarchitecture", ServerArchString);
			EditorArchString = ParseParamValueIfNotSpecified(Command, EditorArchitecture, "editorarchitecture", EditorArchString);
			ClientArchString = ParseParamValueIfNotSpecified(Command, ClientArchitecture, "clientarchitecture", ClientArchString);
			ProgramArchString = ParseParamValueIfNotSpecified(Command, ProgramArchitecture, "programarchitecture", ProgramArchString);
			this.SpecifiedArchitecture = UnrealArchitectures.FromString(SpecifiedArchString, null);
			this.ServerArchitecture = UnrealArchitectures.FromString(ServerArchString, null);
			this.EditorArchitecture = UnrealArchitectures.FromString(EditorArchString, null);
			this.ClientArchitecture = UnrealArchitectures.FromString(ClientArchString, null);
			this.ProgramArchitecture = UnrealArchitectures.FromString(ProgramArchString, null);

			// -trace can be used with or without a value
			if (Trace != null || GetParamValueIfNotSpecified(Command, null, false, "trace"))
			{
				this.Trace += "-trace";
				string Value = ParseParamValueIfNotSpecified(Command, Trace, "trace", null);
				if (!String.IsNullOrWhiteSpace(Value))
				{
					this.Trace += "=" + Value;
				}
			}

			// -tracehost can be used with or without a value
			if (TraceHost != null || GetParamValueIfNotSpecified(Command, null, false, "tracehost"))
			{
				this.TraceHost += "-tracehost";
				string Value = ParseParamValueIfNotSpecified(Command, TraceHost, "tracehost", null);
				if (!String.IsNullOrWhiteSpace(Value))
				{
					this.TraceHost += "=" + Value;
				}
			}

			// -tracefile can be used with or without a value
			if (TraceFile != null || GetParamValueIfNotSpecified(Command, null, false, "tracefile"))
			{
				this.TraceFile += "-tracefile";
				string Value = ParseParamValueIfNotSpecified(Command, TraceFile, "tracefile", null);
				if (!String.IsNullOrWhiteSpace(Value))
				{
					this.TraceFile += "=" + Value;
				}
			}

			SessionLabel = Command.ParseParamValue("sessionlabel");

			if (SessionLabel!=null)
			{ 
				this.SessionLabel += "-sessionlabel";
				this.SessionLabel += "=" + SessionLabel;	
			}

			this.Upload = Command.ParseParamValue("upload");

			this.ApplyIoStoreOnDemand = Command.ParseParam("ApplyIoStoreOnDemand");
			if (this.ApplyIoStoreOnDemand)
			{
				// Compile the IoStoreOnDemand module when uploading content for streaming
				this.AdditionalBuildOptions += " -CompileIoStoreOnDemand";

				// Required to force the project to use chunk manifests
				this.Manifests = true;
			}

			this.OnDemandSettings = OnDemandSettings ?? new IoStoreOnDemandSettings(); // Make sure we have a default (OnDemand disabled) settings if none were provided.
			this.OnDemandSettings.ReadFromCommandLine(Command);

			this.GenerateOptionalPakForNonChunkedBuild = GetParamValueIfNotSpecified(Command, GenerateOptionalPakForNonChunkedBuild, false, "GenerateOptionalPakForNonChunkedBuild");

			this.XcodeBuildOptions = ParseParamValueIfNotSpecified(Command, XcodeBuildOptions, "xcodebuildoptions", null);
			this.bMakeMacNative = GetParamValueIfNotSpecified(Command, null, this.bMakeMacNative, "macnative");

			if (ClientConfigsToBuild == null)
			{
				if (Command != null)
				{
					string ClientConfig = Command.ParseParamValue("clientconfig");

                    if (ClientConfig == null)
                        ClientConfig = Command.ParseParamValue("config");

					if (ClientConfig == null)
						ClientConfig = Command.ParseParamValue("configuration");

					if (ClientConfig != null)
					{
						this.ClientConfigsToBuild = new List<UnrealTargetConfiguration>();
						ParamList<string> Configs = new ParamList<string>(ClientConfig.Split(new char[] { '+' }, StringSplitOptions.RemoveEmptyEntries));
						foreach (string ConfigName in Configs)
						{
							this.ClientConfigsToBuild.Add(ParseConfig(ConfigName));
						}
					}
				}
			}
			else
			{
				this.ClientConfigsToBuild = ClientConfigsToBuild;
			}

            if (Port == null)
            {
                if( Command != null )
                {
                    this.Port = new ParamList<string>();

                    var PortString = Command.ParseParamValue("port");
                    if (String.IsNullOrEmpty(PortString) == false)
                    {
						ParamList<string> Ports = new ParamList<string>(PortString.Split(new char[] { '+' }, StringSplitOptions.RemoveEmptyEntries));
                        foreach (var P in Ports)
                        {
                            this.Port.Add(P);
                        }
                    }
                    
                }
            }
            else
            {
                this.Port = Port;
            }

            if (MapsToCook == null)
            {
                if (Command != null)
                {
                    this.MapsToCook = new ParamList<string>();

                    var MapsString = Command.ParseParamValue("MapsToCook");
                    if (String.IsNullOrEmpty(MapsString) == false)
                    {
						ParamList<string> MapNames = new ParamList<string>(MapsString.Split(new char[] { '+' }, StringSplitOptions.RemoveEmptyEntries));
                        foreach ( var M in MapNames ) 
                        {
                            this.MapsToCook.Add( M );
                        }
                    }
                }
            }
            else
            {
                this.MapsToCook = MapsToCook;
            }

			if (MapIniSectionsToCook == null)
			{
				if (Command != null)
				{
					this.MapIniSectionsToCook = new ParamList<string>();

					var MapsString = Command.ParseParamValue("MapIniSectionsToCook");
					if (String.IsNullOrEmpty(MapsString) == false)
					{
						ParamList<string> MapNames = new ParamList<string>(MapsString.Split(new char[] { '+' }, StringSplitOptions.RemoveEmptyEntries));
						foreach (var M in MapNames)
						{
							this.MapIniSectionsToCook.Add(M);
						}
					}
				}
			}
			else
			{
				this.MapIniSectionsToCook = MapIniSectionsToCook;
			}

			if (String.IsNullOrEmpty(this.MapToRun) == false)
            {
                this.MapsToCook.Add(this.MapToRun);
            }

			// if the user specified multiple -map arguments, just use the first one
			if (!String.IsNullOrEmpty(this.MapToRun))
			{
				int DelimiterIndex = this.MapToRun.IndexOf('+');
				if (DelimiterIndex != -1)
				{
					this.MapToRun = this.MapToRun.Remove(DelimiterIndex);
				}
			}

			if (InMapsToRebuildLightMaps == null)
			{
				if (Command != null)
				{
					this.MapsToRebuildLightMaps = new ParamList<string>();

					var MapsString = Command.ParseParamValue("MapsToRebuildLightMaps");
					if (String.IsNullOrEmpty(MapsString) == false)
					{
						ParamList<string> MapNames = new ParamList<string>(MapsString.Split(new char[] { '+' }, StringSplitOptions.RemoveEmptyEntries));
						foreach (var M in MapNames)
						{
							this.MapsToRebuildLightMaps.Add(M);
						}
					}
				}
			}
			else
			{
				this.MapsToRebuildLightMaps = InMapsToRebuildLightMaps;
			}

            if (InMapsToRebuildHLOD == null)
            {
                if (Command != null)
                {
                    this.MapsToRebuildHLODMaps = new ParamList<string>();

                    var MapsString = Command.ParseParamValue("MapsToRebuildHLODMaps");
                    if (String.IsNullOrEmpty(MapsString) == false)
                    {
						ParamList<string> MapNames = new ParamList<string>(MapsString.Split(new char[] { '+' }, StringSplitOptions.RemoveEmptyEntries));
                        foreach (var M in MapNames)
                        {
                            this.MapsToRebuildHLODMaps.Add(M);
                        }
                    }
                }
            }
            else
            {
                this.MapsToRebuildHLODMaps = InMapsToRebuildHLOD;
            }

            if (TitleID == null)
			{
				if (Command != null)
				{
					this.TitleID = new ParamList<string>();

					var TitleString = Command.ParseParamValue("TitleID");
					if (String.IsNullOrEmpty(TitleString) == false)
					{
						ParamList<string> TitleIDs = new ParamList<string>(TitleString.Split(new char[] { '+' }, StringSplitOptions.RemoveEmptyEntries));
						foreach (var T in TitleIDs)
						{
							this.TitleID.Add(T);
						}
					}
				}
			}
			else
			{
				this.TitleID = TitleID;
			}

			if (ServerConfigsToBuild == null)
			{
				if (Command != null)
				{
					string ServerConfig = Command.ParseParamValue("serverconfig");

                    if (ServerConfig == null)
                        ServerConfig = Command.ParseParamValue("config");

					if (ServerConfig == null)
						ServerConfig = Command.ParseParamValue("configuration");

					if (ServerConfig != null)
					{
						this.ServerConfigsToBuild = new List<UnrealTargetConfiguration>();
						ParamList<string> Configs = new ParamList<string>(ServerConfig.Split(new char[] { '+' }, StringSplitOptions.RemoveEmptyEntries));
						foreach (string ConfigName in Configs)
						{
							this.ServerConfigsToBuild.Add(ParseConfig(ConfigName));
						}
					}
				}
			}
			else
			{
				this.ServerConfigsToBuild = ServerConfigsToBuild;
			}
			if (NumClients.HasValue)
			{
				this.NumClients = NumClients.Value;
			}
			else if (Command != null)
			{
				this.NumClients = Command.ParseParamInt("numclients");
			}
			if (Command != null)
			{
				this.DeviceLockedTimeout = Command.ParseParamInt("DeviceLockedTimeout", 120);
			}
            if (CrashIndex.HasValue)
            {
                this.CrashIndex = CrashIndex.Value;
            }
            else if (Command != null)
            {
                this.CrashIndex = Command.ParseParamInt("CrashIndex");
            }
            if (RunTimeoutSeconds.HasValue)
            {
                this.RunTimeoutSeconds = RunTimeoutSeconds.Value;
            }
            else if (Command != null)
            {
                this.RunTimeoutSeconds = Command.ParseParamInt("runtimeoutseconds");
            }

			if (ConfigOverrideParams != null)
			{
				this.ConfigOverrideParams = ConfigOverrideParams;
			}
			// Gather up any '-ini:' arguments and save them. We'll pass these along to other tools that may be spawned in a new process as part of the command.
			if(Command != null)
			{
				foreach (string Param in Command.Params)
				{
					if (Param.StartsWith("ini:", StringComparison.InvariantCultureIgnoreCase))
					{
						this.ConfigOverrideParams.Add(Param);
					}
				}
			}

			AutodetectSettings(false);
			ValidateAndLog();
            if (this.PrePak)
            {
                if (!CommandUtils.P4Enabled)
                {
                    throw new AutomationException("-PrePak requires -P4");
                }
                if (CommandUtils.P4Env.Changelist < 1000)
                {
                    throw new AutomationException("-PrePak requires a CL from P4 and we have {0}", CommandUtils.P4Env.Changelist);
                }

                string SrcBuildPath = CommandUtils.CombinePaths(CommandUtils.RootBuildStorageDirectory(), ShortProjectName);
                string SrcBuildPath2 = CommandUtils.CombinePaths(CommandUtils.RootBuildStorageDirectory(), ShortProjectName.Replace("Game", "").Replace("game", ""));

                if (!InternalUtils.SafeDirectoryExists(SrcBuildPath))
                {
                    if (!InternalUtils.SafeDirectoryExists(SrcBuildPath2))
                    {
                        throw new AutomationException("PrePak: Neither {0} nor {1} exists.", SrcBuildPath, SrcBuildPath2);
                    }
                    SrcBuildPath = SrcBuildPath2;
                }
                string SrcCLPath = CommandUtils.CombinePaths(SrcBuildPath, CommandUtils.EscapePath(CommandUtils.P4Env.Branch) + "-CL-" + CommandUtils.P4Env.Changelist.ToString());
                if (!InternalUtils.SafeDirectoryExists(SrcCLPath))
                {
                    throw new AutomationException("PrePak: {0} does not exist.", SrcCLPath);
                }
            }
        }

		static UnrealTargetConfiguration ParseConfig(string ConfigName)
		{
			UnrealTargetConfiguration ConfigValue;
			if (!Enum.TryParse(ConfigName, true, out ConfigValue))
			{
				throw new AutomationException("Invalid configuration '{0}'. Valid configurations are '{1}'.", ConfigName, String.Join("', '", Enum.GetNames(typeof(UnrealTargetConfiguration)).Where(x => x != nameof(UnrealTargetConfiguration.Unknown))));
			}
			return ConfigValue;
		}

		static bool IsEpicBuildMachine()
		{
			return CommandUtils.IsBuildMachine 
				&& FileReference.Exists(FileReference.Combine(Unreal.EngineDirectory, "Restricted", "NotForLicensees", "Build", "EpicInternal.txt"));
		}

		/// <summary>
		/// Shared: Full path to the .uproject file
		/// </summary>
		public FileReference RawProjectPath { private set; get; }

		/// <summary>
		/// Shared: The current project is a foreign project, commandline: -foreign
		/// </summary>
		[Help("foreign", "Generate a foreign uproject from blankproject and use that")]
		public bool Foreign { private set; get; }

		/// <summary>
		/// Shared: The current project is a foreign project, commandline: -foreign
		/// </summary>
		[Help("foreigncode", "Generate a foreign code uproject from platformergame and use that")]
		public bool ForeignCode { private set; get; }

		/// <summary>
		/// Shared: true if we should build crash reporter
		/// </summary>
		[Help("CrashReporter", "true if we should build crash reporter")]
		public bool CrashReporter { private set; get; }

		/// <summary>
		/// Shared: Determines if the build is going to use cooked data, commandline: -cook, -cookonthefly
		/// </summary>	
		[Help("cook, -cookonthefly", "Determines if the build is going to use cooked data")]
		public bool Cook { private set; get; }

		/// <summary>
		/// Shared: Determines if the build is going to use cooked data, commandline: -cook, -cookonthefly
		/// </summary>	
		[Help("skipcook", "use a cooked build, but we assume the cooked data is up to date and where it belongs, implies -cook")]
		public bool SkipCook { private set; get; }

		/// <summary>
		/// Shared: In a cookonthefly build, used solely to pass information to the package step. This is necessary because you can't set cookonthefly and cook at the same time, and skipcook sets cook.
		/// </summary>	
		[Help("skipcookonthefly", "in a cookonthefly build, used solely to pass information to the package step")]
		public bool SkipCookOnTheFly { private set; get; }

		/// <summary>
		/// Shared: Determines if the intermediate folders will be wiped before building, commandline: -clean
		/// </summary>
		[Help("clean", "wipe intermediate folders before building")]
		public bool? Clean { private set; get; }

		/// <summary>
		/// Shared: Assumes no user is sitting at the console, so for example kills clients automatically, commandline: -Unattended
		/// </summary>
		[Help("unattended", "assumes no operator is present, always terminates without waiting for something.")]
		public bool Unattended { private set; get; }

		/// <summary>
        /// Shared: Sets platforms to build for non-dedicated servers. commandline: -TargetPlatform
		/// </summary>
		public List<TargetPlatformDescriptor> ClientTargetPlatforms = new List<TargetPlatformDescriptor>();

        /// <summary>
        /// Shared: Dictionary that maps client dependent platforms to "source" platforms that it should copy data from. commandline: -TargetPlatform=source.dependent
        /// </summary>
        public Dictionary<TargetPlatformDescriptor, TargetPlatformDescriptor> ClientDependentPlatformMap = new Dictionary<TargetPlatformDescriptor, TargetPlatformDescriptor>();

		/// <summary>
        /// Shared: Sets platforms to build for dedicated servers. commandline: -ServerTargetPlatform
		/// </summary>
		public List<TargetPlatformDescriptor> ServerTargetPlatforms = new List<TargetPlatformDescriptor>();

        /// <summary>
        /// Shared: Dictionary that maps server dependent platforms to "source" platforms that it should copy data from: -ServerTargetPlatform=source.dependent
        /// </summary>
        public Dictionary<TargetPlatformDescriptor, TargetPlatformDescriptor> ServerDependentPlatformMap = new Dictionary<TargetPlatformDescriptor, TargetPlatformDescriptor>();

		/// <summary>
		/// Shared: True if pak file should be generated.
		/// </summary>
		[Help("pak", "generate a pak file")]
		public bool Pak { private set; get; }

		/// <summary>
		/// Stage: True if we should disable trying to re-use pak files from another staged build when we've specified a different cook source platform
		/// </summary>
		[Help("pak", "disable reuse of pak files from the alternate cook source folder, if specified")]
		public bool IgnorePaksFromDifferentCookSource { get; private set; }

		/// <summary>
		/// Shared: True if container file(s) should be generated with ZenPak.
		/// </summary>
		[Help("iostore", "generate I/O store container file(s)")]
		public bool IoStore { private set; get; }
		
		/// <summary>
		/// Shared: True if the cooker should store the cooked output to the Zen storage server
		/// </summary>
		[Help("zenstore", "save cooked output data to the Zen storage server")]
		public bool ZenStore { private set; get; }

		/// <summary>
		/// Shared: URL to a running Zen server
		/// </summary>
		[Help("nozenautolaunch", "URL to a running Zen server")]
		public string NoZenAutoLaunch { private set; get; }

		/// <summary>
		/// Shared: Global path to a Zen workspace share to be passed to the client. Used for Zen pak streaming
		/// </summary>
		public string ZenWorkspaceSharePath { private set; get; }

		/// <summary>
		/// Should omit pak files when deploying build to target for Zen pak streaming
		/// </summary>
		public bool ZenOmitPakFiles { private set; get; }

		/// <summary>
		/// Shared: True if optimization data is generated during staging that can improve loadtimes
		/// </summary>
		[Help("makebinaryconfig", "generate optimized config data during staging to improve loadtimes")]
		public bool GenerateOptimizationData { private set; get; }

		/// <summary>
		/// 
		/// </summary>
		public bool UsePak(Platform PlatformToCheck)
		{
			return Pak || PlatformToCheck.RequiresPak(this) == Platform.PakType.Always;
		}

		private string SignPakInternal { get; set; }

		/// <summary>
		/// Shared: Encryption keys used for signing the pak file.
		/// </summary>
		[Help("signpak=keys", "sign the generated pak file with the specified key, i.e. -signpak=C:\\Encryption.keys. Also implies -signedpak.")]
		public string SignPak 
		{ 
			private set
			{
				if (string.IsNullOrEmpty(value) || value.StartsWith("0x", StringComparison.InvariantCultureIgnoreCase))
				{
					SignPakInternal = value;
				}
				else
				{
					SignPakInternal = Path.GetFullPath(value);
				}
			}
			get
			{
				return SignPakInternal;
			}
		}

        /// <summary>
        /// Shared: true if this build is staged, command line: -stage
        /// </summary>
        [Help("prepak", "attempt to avoid cooking and instead pull pak files from the network, implies pak and skipcook")]
        public bool PrePak { private set; get; }

        /// <summary>
        /// Shared: the game will use only signed content.
        /// </summary>
        [Help("signed", "the game should expect to use a signed pak file.")]
		public bool SignedPak { private set; get; }

		/// <summary>
		/// Shared: The game will be set up for memory mapping bulk data.
		/// </summary>
		[Help("PakAlignForMemoryMapping", "The game will be set up for memory mapping bulk data.")]
		public bool PakAlignForMemoryMapping { private set; get; }
		
		/// <summary>
		/// Shared: true if we want to rehydrate virtualized assets when staging.
		/// </summary>
		[Help("rehydrateassets", "Should virtualized assets be rehydrated?")]
		public bool RehydrateAssets { get; set; }

		/// <summary>
		/// Shared: true if this build is staged, command line: -stage
		/// </summary>
		[Help("skippak", "use a pak file, but assume it is already built, implies pak")]
		public bool SkipPak { private set; get; }

		/// <summary>
		/// Shared: true if we want to skip iostore, even if -iostore is specified
		/// </summary>
		[Help("skipiostore", "override the -iostore commandline option to not run it")]
		public bool SkipIoStore { private set; get; }

		/// <summary>
		/// Shared: true if this build is staged, command line: -stage
		/// </summary>
		[Help("stage", "put this build in a stage directory")]
		public bool Stage { private set; get; }

		/// <summary>
		/// Shared: true if this build is staged, command line: -stage
		/// </summary>
		[Help("skipstage", "uses a stage directory, but assumes everything is already there, implies -stage")]
		public bool SkipStage { private set; get; }

		/// <summary>
		/// Staging: run OplogChunkAssigner before staging to generate pak chunk manifests from the Zen oplog.
		/// </summary>
		[Help("RunChunkAssigner", "Run OplogChunkAssigner before staging to generate pak chunk manifests from the Zen oplog")]
		public bool RunChunkAssigner { private set; get; }

		/// <summary>
		/// Shared: true if the staged build contains a build that may not have been created locally, command line: -prestaged (experimental)
		/// </summary>
		[Help("prestaged", "the staged directory contains a build that wasn't necessarily created locally, implies -skipbuild -skipcook -skipstage (experimental)")]
		public bool PreStaged { private set; get; }

		/// <summary>
		/// Shared: true if this build is using streaming install manifests, command line: -manifests
		/// </summary>
		[Help("manifests", "generate streaming install manifests when cooking data")]
		public bool Manifests { private set; get; }

        /// <summary>
        /// Shared: true if this build chunk install streaming install data, command line: -createchunkinstalldata
        /// </summary>
        [Help("createchunkinstall", "generate streaming install data from manifest when cooking data, requires -stage & -manifests")]
        public bool CreateChunkInstall { private set; get; }

		[Help("skipencryption", "skips encrypting pak files even if crypto keys are provided")]
		public bool SkipEncryption { private set; get; }

		/// <summary>
		/// Shared: Directory to use for built chunk install data, command line: -chunkinstalldirectory=
		/// </summary>
		public string ChunkInstallDirectory { set; get; }

		/// <summary>
		/// Shared: Version string to use for built chunk install data, command line: -chunkinstallversion=
		/// </summary>
		public string ChunkInstallVersionString { set; get; }

		/// <summary>
        /// Shared: Release string to use for built chunk install data, command line: -chunkinstallrelease=
        /// </summary>
        public string ChunkInstallReleaseString { set; get; }

        /// <summary>
		/// Shared: Directory to copy the client to, command line: -stagingdirectory=
		/// </summary>	
		public string BaseStageDirectory
		{
			get
			{
                if( !String.IsNullOrEmpty(StageDirectoryParam ) )
                {
                    return Path.GetFullPath( StageDirectoryParam );
                }
                if ( HasDLCName )
                {
                     return Path.GetFullPath( CommandUtils.CombinePaths( DLCFile.Directory.FullName, "Saved", "StagedBuilds" ) );
                }
                // default return the project saved\stagedbuilds directory
                return Path.GetFullPath( CommandUtils.CombinePaths(Path.GetDirectoryName(RawProjectPath.FullName), "Saved", "StagedBuilds") );
			}
		}

		[Help("stagingdirectory=Path", "Directory to copy the builds to, i.e. -stagingdirectory=C:\\Stage")]
		public string StageDirectoryParam;

		[Help("optionalfilestagingdirectory=Path", "Directory to copy the optional files to, i.e. -optionalfilestagingdirectory=C:\\StageOptional")]
		public string OptionalFileStagingDirectory;

		[Help("optionalfileinputdirectory=Path", "Directory to read the optional files from, i.e. -optionalfileinputdirectory=C:\\StageOptional")]
		public string OptionalFileInputDirectory;

		[Help("CookerSupportFilesSubdirectory=subdir", "Subdirectory under staging to copy CookerSupportFiles (as set in Build.cs files). -CookerSupportFilesSubdirectory=SDK")]
		public string CookerSupportFilesSubdirectory;
		
		[Help("unrealexe=ExecutableName", "Name of the Unreal Editor executable, i.e. -unrealexe=UnrealEditor.exe")]
		private string SpecifiedUnrealExe = null;

		public string UnrealExe
		{
			get
			{
				if (SpecifiedUnrealExe == null)
				{
					SpecifiedUnrealExe = ProjectUtils.GetEditorForProject(RawProjectPath).FullName;
				}

				return SpecifiedUnrealExe;
			}
			set 
			{
				// allow code override
				SpecifiedUnrealExe = value; 
			}
		}

		/// <summary>
		/// Shared: true if this build is archived, command line: -archive
		/// </summary>
		[Help("archive", "put this build in an archive directory")]
		public bool Archive { private set; get; }

		/// <summary>
		/// Shared: Directory to archive the client to, command line: -archivedirectory=
		/// </summary>	
		public string BaseArchiveDirectory
		{
			get
			{
                return Path.GetFullPath(String.IsNullOrEmpty(ArchiveDirectoryParam) ? CommandUtils.CombinePaths(Path.GetDirectoryName(RawProjectPath.FullName), "ArchivedBuilds") : ArchiveDirectoryParam);
			}
		}

		[Help("archivedirectory=Path", "Directory to archive the builds to, i.e. -archivedirectory=C:\\Archive")]
		public string ArchiveDirectoryParam;

		/// <summary>
		/// Whether the project should use non monolithic staging
		/// </summary>
		[Help("archivemetadata", "Archive extra metadata files in addition to the build (e.g. build.properties)")]
		public bool ArchiveMetaData;

		/// <summary>
		/// When archiving for Mac, set this to true to package it in a .app bundle instead of normal loose files
		/// </summary>
		[Help("createappbundle", "When archiving for Mac, set this to true to package it in a .app bundle instead of normal loose files")]
		public bool CreateAppBundle;

		/// <summary>
		/// Keeps track of any '-ini:type:[section]:value' arguments on the command line. These will override cached config settings for the current process, and can be passed along to other tools.
		/// </summary>
		public List<string> ConfigOverrideParams = new List<string>();

        /// <summary>
        /// Build: True if build step should be executed, command: -build
        /// </summary>
        [Help("build", "True if build step should be executed")]
		public bool Build { private set; get; }

		/// <summary>
		/// SkipBuildClient if true then don't build the client exe
		/// </summary>
		public bool SkipBuildClient { private set; get; }

		/// <summary>
		/// SkipBuildEditor if true then don't build the editor exe
		/// </summary>
		public bool SkipBuildEditor { private set; get; }

		/// <summary>
		/// Build: True if XGE should NOT be used for building.
		/// </summary>
		[Help("noxge", "True if XGE should NOT be used for building")]
		public bool NoXGE { private set; get; }

		/// <summary>
		/// Build: List of editor build targets.
		/// </summary>	
		private ParamList<string> EditorTargetsList = null;
		public ParamList<string> EditorTargets
		{
			set { EditorTargetsList = value; }
			get
			{
				if (EditorTargetsList == null)
				{
					// Lazy auto-initialization
					AutodetectSettings(false);
				}
				return EditorTargetsList;
			}
		}

		/// <summary>
		/// Build: List of program build targets.
		/// </summary>	
		private ParamList<string> ProgramTargetsList = null;
		public ParamList<string> ProgramTargets
		{
			set { ProgramTargetsList = value; }
			get
			{
				if (ProgramTargetsList == null)
				{
					// Lazy auto-initialization
					AutodetectSettings(false);
				}
				return ProgramTargetsList;
			}
		}

		/// <summary>
		/// Build: List of client configurations
		/// </summary>
		public List<UnrealTargetConfiguration> ClientConfigsToBuild = new List<UnrealTargetConfiguration>() { UnrealTargetConfiguration.Development };

		///<summary>
		/// Build: List of Server configurations
		/// </summary>
		public List<UnrealTargetConfiguration> ServerConfigsToBuild = new List<UnrealTargetConfiguration>() { UnrealTargetConfiguration.Development };

		/// <summary>
		/// Build: List of client cooked build targets.
		/// </summary>
		private ParamList<string> ClientCookedTargetsList = null;
		public ParamList<string> ClientCookedTargets
		{
			set { ClientCookedTargetsList = value; }
			get
			{
				if (ClientCookedTargetsList == null)
				{
					// Lazy auto-initialization
					AutodetectSettings(false);
				}
				return ClientCookedTargetsList;
			}
		}

		/// <summary>
		/// Build: List of Server cooked build targets.
		/// </summary>
		private ParamList<string> ServerCookedTargetsList = null;
		public ParamList<string> ServerCookedTargets
		{
			set { ServerCookedTargetsList = value; }
			get
			{
				if (ServerCookedTargetsList == null)
				{
					// Lazy auto-initialization
					AutodetectSettings(false);
				}
				return ServerCookedTargetsList;
			}
		}

		/// <summary>
		/// Build: Specifies the names of targets to build
		/// </summary>
		private List<string> TargetNames;

		/// <summary>
		/// Cook: List of maps to cook.
		/// </summary>
		public ParamList<string> MapsToCook = new ParamList<string>();


		/// <summary>
		/// Cook: List of map inisections to cook (see allmaps)
		/// </summary>
		public ParamList<string> MapIniSectionsToCook = new ParamList<string>();
		

		/// <summary>
		/// Cook: List of directories to cook.
		/// </summary>
		public ParamList<string> DirectoriesToCook = new ParamList<string>();

        /// <summary>
        /// Cook: Which ddc graph to use when cooking.
        /// </summary>
        public string DDCGraph;

        /// <summary>
        /// Cook: Internationalization preset to cook.
        /// </summary>
        public string InternationalizationPreset;

		/// <summary>
		/// Include all tag mapping ini files when staging
		/// </summary>
		[Help("StageTagMappingConfigFiles", "Include all tag mapping ini files when staging")]
		public bool StageTagMappingConfigFiles;

		/// <summary>
		/// Cook: While cooking clean up packages as we go along rather then cleaning everything (and potentially having to reload some of it) when we run out of space
		/// </summary>
		[Help("CookPartialgc", "while cooking clean up packages as we are done with them rather then cleaning everything up when we run out of space")]
		public bool CookPartialGC { private set; get; }

		/// <summary>
		/// Stage: Did we cook in the editor instead of from UAT (cook in editor uses a different staging directory)
		/// </summary>
		[Help("CookInEditor", "Did we cook in the editor instead of in UAT")]
		public bool CookInEditor { private set; get; }

		/// <summary>
		/// Cook: Output directory for cooked data
		/// </summary>
		public string CookOutputDir;

		/// <summary>
		/// Cook: Create a cooked release version.  Also, the version. e.g. 1.0
		/// </summary>
		public string CreateReleaseVersion;

		/// <summary>
		/// Cook: Base this cook of a already released version of the cooked data
		/// </summary>
		public string BasedOnReleaseVersion;

		/// <summary>
		/// The version of the originally released build. This is required by some platforms when generating patches.
		/// </summary>
		public string OriginalReleaseVersion;

		/// <summary>
		/// Cook: Path to the root of the directory where we store released versions of the game for a given version
		/// </summary>
		public string BasedOnReleaseVersionBasePath;

		/// <summary>
		/// Cook: Path to the root of the directory to create a new released version of the game.
		/// </summary>
		public string CreateReleaseVersionBasePath;

		/// <summary>
		/// Stage: Path to the global.utoc file for a directory of iostore containers to use as a source of compressed
		/// chunks when writing new containers. See -ReferenceContainerGlobalFileName in IoStoreUtilities.cpp.
		/// </summary>
		public string ReferenceContainerGlobalFileName;

		/// <summary>
		/// Stage: A directory to look for additional IoStore containers to use during staging (e.g. optional segment containers)
		/// </summary>
		public string ReferenceContainerAdditionalPath;

		/// <summary>
		/// Stage: If ReferenceContainerChangesCSVFileName is specified, this provides a file name to write the list of changed
		/// packages/chunks to.
		/// </summary>
		public string ReferenceContainerChangesCSVFileName;

		/// <summary>
		/// Stage: Path to the crypto.json file to use for decrypting ReferenceContainerGlobalFileName, if needed.
		/// </summary>
		public string ReferenceContainerCryptoKeys;


		/// <summary>
		/// Are we generating a patch, generate a patch from a previously released version of the game (use CreateReleaseVersion to create a release). 
		/// this requires BasedOnReleaseVersion
		/// see also CreateReleaseVersion, BasedOnReleaseVersion
		/// </summary>
		public bool GeneratePatch;

		/// <summary>
		/// Required when building remaster package
		/// </summary>
		public string DiscVersion;

		/// <summary>
        /// </summary>
        public bool AddPatchLevel;

		/// <summary>
		/// Name of dlc to cook and package (if this parameter is supplied cooks the dlc and packages it into the dlc directory)
		/// </summary>
		public FileReference DLCFile;

        /// <summary>
        /// Enable cooking of engine content when cooking dlc 
        ///  not included in original release but is referenced by current cook
        /// </summary>
        public bool DLCIncludeEngineContent;


		/// <summary>
		/// Enable packaging up the uplugin file inside the dlc pak.  This is sometimes desired if you want the plugin to be standalone
		/// </summary>
		public bool DLCPakPluginFile;

		/// <summary>
		/// DLC will remap it's files to the game directory and act like a patch.  This is useful if you want to override content in the main game along side your main game.
		/// For example having different main game content in different DLCs
		/// </summary>
		public bool DLCActLikePatch;

		/// <summary>
		/// Sometimes a DLC may get cooked to a subdirectory of where is expected, so this can tell the staging what the subdirectory of the cooked out
		/// to find the DLC files (for instance Metadata)
		/// </summary>
		public string DLCOverrideCookedSubDir;

		/// <summary>
		/// Controls where under the staged directory to output to (in case the plugin subdirectory is not desired under the StagingDirectory location)
		/// </summary>
		public string DLCOverrideStagedSubDir;

		/// <summary>
		/// Specifies keys to use rather than those generated from ini files when using a staging manifest (ignored when using chunk manifests)
		/// </summary>
		public EncryptionAndSigning.CryptoSettings DLCOverrideCryptoSettings;

		/// <summary>
		/// Specifies an encryption key for UnrealPak to use when using a staging manifest (not a chunk manifest)
		/// </summary>
		public string DLCOverrideEncryptionKeyGuid;

		/// <summary>
		/// When true the DLC Name will not be added to the generated pak files.
		/// </summary>
		public bool DLCSkipNamedPak = false;

		/// <summary>
		/// When true only the GameFeaturePlugins that are cooked will have thier GameFeaturePlugin data staged.
		/// This check is performed by looking for a cooked GameFeatureData asset within the GameFeaturePlugin expected folder.
		/// </summary>
		public bool StageOnlyCookedLocalizationForGameFeatures = false;

		/// <summary>
		/// After cook completes diff the cooked content against another cooked content directory.
		///  report all errors to the log
		/// </summary>
		public string DiffCookedContentPath;

		/// <summary>
		/// Build: Additional build options to include on the build commandline
		/// </summary>
		public string AdditionalBuildOptions;

		/// <summary>
		/// Cook: Additional cooker options to include on the cooker commandline
		/// </summary>
		public string AdditionalCookerOptions;

        /// <summary>
        /// Cook: List of cultures to cook.
        /// </summary>
        public ParamList<string> CulturesToCook;

        /// <summary>
        /// Compress packages during cook.
        /// </summary>
        public bool Compressed;

		/// <summary>
		/// Do not compress packages during cook, override game ProjectPackagingSettings to force it off
		/// </summary>
		public bool ForceUncompressed;

		/// <summary>
		/// Assume that executables will be built with merged modules
		/// </summary>
		public bool UseMergedModules;
		
		/// <summary>
		/// Compress packages during cook, override game ProjectPackagingSettings and Platform Hardware Compression settings to force it on
		/// </summary>
		public bool ForceCompressed;

		/// <summary>
		/// Additional parameters when generating the PAK file
		/// </summary>
		public string AdditionalPakOptions;

		/// <summary>
		/// Additional parameters when generating iostore container files
		/// </summary>
		public string AdditionalIoStoreOptions;

		/// <summary>
		/// If not empty, this is the dll file to use for Oodle compression. Can be "Latest" to use latest version.
		/// </summary>
		public string ForceOodleDllVersion;

        /// <summary>
        /// Cook: Do not include a version number in the cooked content
        /// </summary>
        public bool UnversionedCookedContent = true;

		/// <summary>
		/// Cook: Cook with optional data enabled
		/// </summary>
		public bool OptionalContent = false;

		/// <summary>
		/// Cook: Cooks incrementally using previously-cooked data or a published snapshot as a base
		/// </summary>
		[Help("cookincremental", "Cooks incrementally using previously-cooked data or a published snapshot as a base")]
		public bool IncrementalCook;

		/// <summary>
		/// Cook: Iterate from a build on the network
		/// </summary>
		[Help("Iteratively cook from a shared cooked build")]
		public string IterateSharedCookedBuild;

		/// <summary>
		/// Build: Don't build the game instead use the prebuild exe (requires iterate shared cooked build
		/// </summary>
		[Help("Iteratively cook from a shared cooked build")]
		public bool IterateSharedBuildUsePrecompiledExe;

		/// <summary>
		/// Cook: Only cook maps (and referenced content) instead of cooking everything only affects -cookall flag
		/// </summary>
		[Help("CookMapsOnly", "Cook only maps this only affects usage of -cookall the flag")]
        public bool CookMapsOnly;

        /// <summary>
        /// Cook: Only cook maps (and referenced content) instead of cooking everything only affects cookall flag
        /// </summary>
        [Help("CookAll", "Cook all the things in the content directory for this project")]
        public bool CookAll;


        /// <summary>
        /// Cook: Skip cooking editor content 
        /// </summary>
		[Help("SkipCookingEditorContent", "Skips content under /Engine/Editor when cooking")]
        public bool SkipCookingEditorContent;

		[Help("SkipCookingErrorSummary", "Skips error summary when cooking")]
		public bool SkipCookingErrorSummary;

		/// <summary>
		/// Cook: Uses the iterative deploy, command line: -iterativedeploy or -iterate
		/// </summary>
		[Help("IterativeDeploy", "Runs UnrealPak and targetplatform install iteratively, command line: -iterativedeploy or -iterate")]
		public bool IterativeDeploy;

		[Help("FastCook", "Uses fast cook path if supported by target")]
		public bool FastCook;

		/// <summary>
		/// Imports the most recently published project snapshot (if cooking, this forces the use of -cookincremental)
		/// </summary>
		[Help("snapshot", "Imports the most recently published project snapshot (if cooking, this forces the use of -cookincremental)")]
		public bool Snapshot;

		/// <summary>
		/// Cook: Ignores cook errors and continues with packaging etc.
		/// </summary>
		[Help("IgnoreCookErrors", "Ignores cook errors and continues with packaging etc")]
		public bool IgnoreCookErrors { private set; get; }

		/// <summary>
		/// Cook: Commandline: -fileopenlog
		/// </summary>
		[Help("KeepFileOpenLog", "Keeps a log of all files opened, commandline: -fileopenlog")]
		public bool KeepFileOpenLog { private set; get; } = true;

		/// <summary>
		/// Stage: Commandline: -nodebuginfo
		/// </summary>
		[Help("nodebuginfo", "do not copy debug files to the stage")]
		public bool NoDebugInfo { private set; get; }

		/// <summary>
		/// Stage: Commandline: -separatedebuginfo
		/// </summary>
		[Help("separatedebuginfo", "output debug info to a separate directory")]
		public bool SeparateDebugInfo { private set; get; }

		/// <summary>
		/// Stage: Commandline: -mapfile
		/// </summary>
		[Help("MapFile", "generates a *.map file")]
		public bool MapFile { private set; get; }

		/// <summary>
		/// true if the staging directory is to be cleaned: -cleanstage (also true if -clean is specified)
		/// </summary>
		[Help("nocleanstage", "skip cleaning the stage directory")]
		public bool NoCleanStage { set { bNoCleanStage = value; } get { return SkipStage || PreStaged || bNoCleanStage; } }
		private bool bNoCleanStage;

		/// <summary>
		/// Stage: If non-empty, the contents will be put into the stage
		/// </summary>
		[Help("cmdline", "command line to put into the stage in UECommandLine.txt")]
		public string StageCommandline;

        /// <summary>
		/// Stage: If non-empty, the contents will be used for the bundle name
		/// </summary>
		[Help("bundlename", "string to use as the bundle name when deploying to mobile device")]
        public string BundleName;

		/// <summary>
		/// Stage: Specifies a list of extra targets that should be staged along with a client
		/// </summary>
		public ParamList<string> ExtraTargetsToStageWithClient = new ParamList<string>();

		/// <summary>
		/// Stage: Optional callback that a build script can use to modify a deployment context immediately after it is created
		/// </summary>
		public Action<ProjectParams, DeploymentContext> PreModifyDeploymentContextCallback = null;

		/// <summary>
		/// Stage: Optional callback that a build script can use to modify a deployment context before it is applied (and before it is finalized)
		/// </summary>
		public Action<ProjectParams, DeploymentContext> ModifyDeploymentContextCallback = null;

		/// <summary>
		/// Stage: Optional callback that a build script can use to finalize a deployment context before it is applied
		/// </summary>
		public Action<ProjectParams, DeploymentContext> FinalizeDeploymentContextCallback = null;

		/// <summary>
		/// Name of the custom deployment handler to change how the build packaged, staged and deployed - for example, when packaging for a specific game store
		/// </summary>
		public string CustomDeploymentHandler { get; set; }

		/// <summary>
		/// On Windows, adds an executable to the root of the staging directory which checks for prerequisites being 
		/// installed and launches the game with a path to the .uproject file.
		/// </summary>
		public bool NoBootstrapExe { get; set; }

		/// <summary>
		/// By default we don't code sign unless it is required or requested
		/// </summary>
		public bool bCodeSign = false;

		/// <summary>
		/// Provision to use
		/// </summary>
		public string Provision = null;

		/// <summary>
		/// Certificate to use
		/// </summary>
		public string Certificate = null;

		/// <summary>
		/// Team ID to use
		/// </summary>
		public string Team = null;

		/// <summary>
		/// true if provisioning is automatically managed
		/// </summary>
		public bool AutomaticSigning = false;

		/// <summary>
		/// TitleID to package
		/// </summary>
		public ParamList<string> TitleID = new ParamList<string>();

		/// <summary>
		/// If true, non-shipping binaries will be considered DebugUFS files and will appear on the debugfiles manifest
		/// </summary>
		public bool bTreatNonShippingBinariesAsDebugFiles = false;

		/// <summary>
		/// If true, use chunk manifest files generated for extra flavor
		/// </summary>
		public bool bUseExtraFlavor = false;

		/// <summary>
		/// Run: True if the Run step should be executed, command: -run
		/// </summary>
		[Help("run", "run the game after it is built (including server, if -server)")]
		public bool Run { private set; get; }

		/// <summary>
		/// Run: The client runs with cooked data provided by cook on the fly server, command line: -cookonthefly
		/// </summary>
		[Help("cookonthefly", "run the client with cooked data provided by cook on the fly server")]
		public bool CookOnTheFly { private set; get; }

        /// <summary>
        /// Run: The client should run in streaming mode when connecting to cook on the fly server
        /// </summary>
        [Help("Cookontheflystreaming", "run the client in streaming cook on the fly mode (don't cache files locally instead force reget from server each file load)")]
        public bool CookOnTheFlyStreaming { private set; get; }



		/// <summary>
		/// Run: The client runs with cooked data provided by UnrealFileServer, command line: -fileserver = CookByTheBook with ZenServer (zenstore)
		/// </summary>
		[Help("fileserver", "run the client with cooked data provided by UnrealFileServer")]
		public bool FileServer { private set; get; }

		/// <summary>
		/// Run: The client connects to dedicated server to get data, command line: -dedicatedserver
		/// </summary>
		[Help("dedicatedserver", "build, cook and run both a client and a server (also -server)")]
		public bool DedicatedServer { private set; get; }

		/// <summary>
		/// Run: Uses a client target configuration, also implies -dedicatedserver, command line: -client
		/// </summary>
		[Help( "client", "build, cook and run a client and a server, uses client target configuration" )]
		public bool Client { private set; get; }

		/// <summary>
		/// Run: Whether the client should start or not, command line (to disable): -noclient
		/// </summary>
		[Help("noclient", "do not run the client, just run the server")]
		public bool NoClient { private set; get; }

		/// <summary>
		/// Run: Client should create its own log window, command line: -logwindow
		/// </summary>
		[Help("logwindow", "create a log window for the client")]
		public bool LogWindow { private set; get; }

		/// <summary>
		/// Run: Map to run the game with.
		/// </summary>
		[Help("map", "map to run the game with")]
		public string MapToRun;

		/// <summary>
		/// Run: Additional server map params.
		/// </summary>
		[Help("AdditionalServerMapParams", "Additional server map params, i.e ?param=value")]
		public string AdditionalServerMapParams;

		/// <summary>
		/// Run: The target device to run the game on.  Comes in the form platform@devicename.
		/// </summary>
		[Help("device", "Devices to run the game on")]
		public ParamList<string> Devices;

		/// <summary>
		/// Run: The target device to run the game on.  No platform prefix.
		/// </summary>
		[Help("device", "Device names without the platform prefix to run the game on")]
		public ParamList<string> DeviceNames;

		/// <summary>
		/// Run: the target device to run the server on
		/// </summary>
		[Help("serverdevice", "Device to run the server on")]
		public string ServerDevice;

		/// <summary>
		/// Run: The indicated server has already been started
		/// </summary>
		[Help("skipserver", "Skip starting the server")]
		public bool SkipServer;

		/// <summary>
		/// Run: The indicated server has already been started
		/// </summary>
		[Help("numclients=n", "Start extra clients, n should be 2 or more")]
		public int NumClients;

		[Help("DeviceLockedTimeout=n", "Seconds to wait for a locked iOS device before failing deploy (default 120, 0 to disable)")]
		public int DeviceLockedTimeout = 120;

		/// <summary>
		/// Run: Additional command line arguments to pass to the program
		/// </summary>
		[Help("addcmdline", "Additional command line arguments for the program, which will not be staged in UECommandLine.txt in most cases")]
		public string RunCommandline;

        /// <summary>
		/// Run: Additional command line arguments to pass to the server
		/// </summary>
		[Help("servercmdline", "Additional command line arguments for the program")]
		public string ServerCommandline;

        /// <summary>
		/// Run: Override command line arguments to pass to the client, if set it will not try to guess at IPs or settings
		/// </summary>
		[Help("clientcmdline", "Override command line arguments to pass to the client")]
        public string ClientCommandline;

        /// <summary>
        /// Run:adds -nullrhi to the client commandline
        /// </summary>
        [Help("nullrhi", "add -nullrhi to the client commandlines")]
        public bool NullRHI;

        [Help("WriteBackMetadataToAssetRegistry", "Passthru to iostore staging, see IoStoreUtilities.cpp")]
        public string WriteBackMetadataToAssetRegistry;

        /// <summary>
        /// Run:adds ?fake to the server URL
        /// </summary>
        [Help("fakeclient", "adds ?fake to the server URL")]
        public bool FakeClient;

        /// <summary>
        /// Run:adds ?fake to the server URL
        /// </summary>
        [Help("editortest", "rather than running a client, run the editor instead")]
        public bool EditorTest;

        /// <summary>
        /// Run:when running -editortest or a client, run all automation tests, not compatible with -server
        /// </summary>
        [Help("RunAutomationTests", "when running -editortest or a client, run all automation tests, not compatible with -server")]
        public bool RunAutomationTests;

        /// <summary>
        /// Run:when running -editortest or a client, run all automation tests, not compatible with -server
        /// </summary>
        [Help("RunAutomationTests", "when running -editortest or a client, run a specific automation tests, not compatible with -server")]
        public string RunAutomationTest;

        /// <summary>
        /// Run: Adds commands like debug crash, debug rendercrash, etc based on index
        /// </summary>
        [Help("Crash=index", "when running -editortest or a client, adds commands like debug crash, debug rendercrash, etc based on index")]
        public int CrashIndex;

        public ParamList<string> Port;

        /// <summary>
        /// Run: Linux username for unattended key genereation
        /// </summary>
        [Help("deviceuser", "Linux username for unattended key genereation")]
        public string DeviceUsername;

        /// <summary>
        /// Run: Linux password for unattended key genereation
        /// </summary>
        [Help("devicepass", "Linux password")]
        public string DevicePassword;

        /// <summary>
        /// Run: Server device IP address
        /// </summary>
        public string ServerDeviceAddress;

		[Help("package", "package the project for the target platform")]
		public bool Package { get; set; }
		
		[Help("skippackage", "Skips packaging the project for the target platform")]
		public bool SkipPackage { get; set; }

		[Help("neverpackage", "Skips preparing data that would be used during packaging, in earlier stages. Different from skippackage which is used to optimize later stages like archive, which still was packaged at some point")]
		public bool NeverPackage { get; set; }

		[Help("package", "Determine whether data is packaged. This can be an iteration optimization for platforms that require packages for deployment")]
		public bool ForcePackageData { get; set; }

		[Help("distribution", "package for distribution the project")]
		public bool Distribution { get; set; }

		[Help("PackageEncryptionKeyFile", "Path to file containing encryption key to use in packaging")]
		public string PackageEncryptionKeyFile { get; set; }

		[Help("prereqs", "stage prerequisites along with the project")]
		public bool Prereqs { get; set; }

		[Help("applocaldir", "location of prerequisites for applocal deployment")]
		public string AppLocalDirectory { get; set; }

		[Help("Prebuilt", "this is a prebuilt cooked and packaged build")]
        public bool Prebuilt { get; private set; }

        [Help("RunTimeoutSeconds", "timeout to wait after we lunch the game")]
        public int RunTimeoutSeconds;

		[Help("SpecifiedArchitecture", "Architecture to use for building any executables (see EditorArchitecture, etc for specific target type control)")]
		private UnrealArchitectures SpecifiedArchitecture;

		[Help("EditorArchitecture", "Architecture to use for building editor executables")]
		public UnrealArchitectures EditorArchitecture;

		[Help("ServerArchitecture", "Architecture to use for building server executables")]
		public UnrealArchitectures ServerArchitecture;

		[Help("ClientArchitecture", "Architecture to use for building client/game executables")]
		public UnrealArchitectures ClientArchitecture;

		[Help("ProgramArchitecture", "Architecture to use for building program executables")]
		public UnrealArchitectures ProgramArchitecture;

		[Help("UbtArgs", "extra options to pass to ubt")]
		public string UbtArgs;

		[Help("AdditionalPackageOptions", "extra options to pass to the platform's packager")]
		public string AdditionalPackageOptions { get; set; }

		[Help("deploy", "deploy the project for the target platform")]
		public bool Deploy { get; set; }

		[Help("deploy", "Location to deploy to on the target platform")]
		public string DeployFolder { get; set; }

		[Help("getfile", "download file from target after successful run")]
		public string GetFile { get; set; }

		[Help("MapsToRebuildLightMaps", "List of maps that need light maps rebuilding")]
		public ParamList<string> MapsToRebuildLightMaps = new ParamList<string>();

        [Help("MapsToRebuildHLODMaps", "List of maps that need HLOD rebuilding")]
        public ParamList<string> MapsToRebuildHLODMaps = new ParamList<string>();

        [Help("IgnoreLightMapErrors", "Whether Light Map errors should be treated as critical")]
		public bool IgnoreLightMapErrors { get; set; }

		[Help("trace", "The list of trace channels to enable")]
		public string Trace { get; set; }

		[Help("tracehost", "The host address of the trace recorder")]
		public string TraceHost { get; set; }
		
		[Help("tracefile", "The file where the trace will be recorded")]
		public string TraceFile { get; set; }

		[Help("sessionlabel", "A label to pass to analytics")]
		public string SessionLabel { get; set; }

		[Help("upload", "Arguments for uploading on demand content")]
		public string Upload { get; set; }

		[Help("applyiostoreondemand", "Forces IoStoreOnDemand to be enabled for the project even if it is not set up for it")]
		public bool ApplyIoStoreOnDemand { get; }

		[Help("ondemandsettings", "The settings for OnDemand data to apply to the project")]
		public IoStoreOnDemandSettings OnDemandSettings { get; }

		[Help("GenerateOptionalPakForNonChunkedBuild", "When true non chunked builds will attempt to generate an optional pak for optional bulk data.")]
		public bool GenerateOptionalPakForNonChunkedBuild { get; }

		[Help("XcodeBuildOptions", "Extra options to pass to xcodebuild")]
		public string XcodeBuildOptions { get; set; }

		[Help("macnative", "Generate a \"Designed for iPad\" .app that can be run natively on a silicon Mac")]
		public bool bMakeMacNative { get; set; }

		private List<SingleTargetProperties> DetectedTargets;
		private Dictionary<UnrealTargetPlatform, ConfigHierarchy> LoadedEngineConfigs;
		private Dictionary<UnrealTargetPlatform, ConfigHierarchy> LoadedGameConfigs;
		private ProjectDescriptor ProjectDescriptor;

		private List<String> TargetNamesOfType(TargetType DesiredType)
		{
			return DetectedTargets.FindAll(Target => Target.Rules.Type == DesiredType).ConvertAll(Target => Target.TargetName);
		}

		private String ChooseTarget(List<String> Targets, TargetType Type)
		{
			switch (Targets.Count)
			{
				case 1:
					return Targets.First();
				case 0:
					throw new AutomationException("{0} target not found!", Type);
				default:
					throw new AutomationException("More than one {0} target found. Specify which one to use with the -Target= option.", Type);
			}
		}

		private void SelectDefaultTarget(List<string> AvailableTargets, TargetType Type, ref string Target)
		{
			string DefaultTarget;

			string ConfigKey = String.Format($"Default{Type}Target");

			if (EngineConfigs[BuildHostPlatform.Current.Platform].GetString("/Script/BuildSettings.BuildSettings", ConfigKey, out DefaultTarget))
			{
				if (!AvailableTargets.Contains(DefaultTarget))
				{
					throw new AutomationException(string.Format($"A default {Type} target '{DefaultTarget}' was specified in engine.ini but does not exist. Available targets: {string.Join(", ", AvailableTargets)}"));
				}

				Target = DefaultTarget;
			}
			else
			{
				if (AvailableTargets.Count > 1)
				{
					throw new AutomationException($"Project contains multiple {Type} targets ({string.Join(", ", AvailableTargets)}) but no {ConfigKey} is set in the [/Script/BuildSettings.BuildSettings] section of DefaultEngine.ini");
				}

				if (AvailableTargets.Count > 0)
				{
					Target = AvailableTargets.First();
				}
			}
		}

		private void AutodetectSettings(bool bReset)
		{
			if (bReset)
			{
				EditorTargetsList = null;
				ClientCookedTargetsList = null;
				ServerCookedTargetsList = null;
				ProgramTargetsList = null;
				ProjectPlatformBinariesPaths = null;
				ProjectExePaths = null;
			}

			List<UnrealTargetPlatform> ClientTargetPlatformTypes = ClientTargetPlatforms.ConvertAll(x => x.Type).Distinct().ToList();
			// @todo (wip) - Removing Blueprint nativization as a feature.
			bool bRunAssetNativization = false;// this.RunAssetNativization;
			ProjectProperties Properties = ProjectUtils.GetProjectProperties(RawProjectPath, ClientTargetPlatformTypes, ClientConfigsToBuild, bRunAssetNativization);

			bIsCodeBasedProject = Properties.bIsCodeBasedProject;
			DetectedTargets = Properties.Targets;
			LoadedEngineConfigs = Properties.EngineConfigs;
			LoadedGameConfigs = Properties.GameConfigs;

			var GameTarget = String.Empty;
			var EditorTarget = String.Empty;
			var ServerTarget = String.Empty;
			var ProgramTarget = String.Empty;

			TargetType ProjectType = TargetType.Game;

			if (!bIsCodeBasedProject)
			{
				GameTarget = Client ? "UnrealClient" : "UnrealGame";
				EditorTarget = "UnrealEditor";
				ServerTarget = "UnrealServer";
			}
			else if (TargetNames.Count > 0)
			{
				// Resolve all the targets that need to be built
				List<SingleTargetProperties> Targets = new List<SingleTargetProperties>();
				foreach (string TargetName in TargetNames)
				{
					SingleTargetProperties Target = DetectedTargets.FirstOrDefault(x => String.Equals(x.TargetName, TargetName, StringComparison.OrdinalIgnoreCase));
					if(Target == null)
					{
						throw new AutomationException("Unable to find target '{0}'", TargetName);
					}
					Targets.Add(Target);
				}

				// Make sure we haven't specified game and clients together
				if (Targets.Any(x => x.Rules.Type == TargetType.Client) && Targets.Any(x => x.Rules.Type == TargetType.Game))
				{
					throw new AutomationException("Cannot specify client and game targets to be built together");
				}

				// Create the lists to receive all the target types
				if (ClientCookedTargetsList == null)
				{
					ClientCookedTargetsList = new ParamList<string>();
				}
				if (ServerCookedTargetsList == null)
				{
					ServerCookedTargetsList = new ParamList<string>();
				}
				if (ProgramTargetsList == null)
				{
					ProgramTargetsList = new ParamList<string>();
				}

				// Add them to the appropriate lists
				bool bHasGameTarget = false;
				foreach (SingleTargetProperties Target in Targets)
				{
					if (Target.Rules.Type == TargetType.Game && !ClientCookedTargetsList.Contains(Target.TargetName))
					{
						ClientCookedTargetsList.Add(Target.TargetName);
						bHasGameTarget = true;
					}
					else if (Target.Rules.Type == TargetType.Client && !ClientCookedTargetsList.Contains(Target.TargetName))
					{
						ClientCookedTargetsList.Add(Target.TargetName);
						Client = true;
					}
					else if (Target.Rules.Type == TargetType.Server && !ServerCookedTargetsList.Contains(Target.TargetName))
					{
						ServerCookedTargetsList.Add(Target.TargetName);
						DedicatedServer = true;
					}
					else if (!ProgramTargetsList.Contains(Target.TargetName))
					{
						ProgramTargetsList.Add(Target.TargetName);
						ProjectType = TargetType.Program;
					}
				}

				// If we don't have any game/client targets, don't stage any client executable
				if (ClientCookedTargetsList.Count == 0)
				{
					NoClient = true;
				}
				else
				{
					GameTarget = ClientCookedTargetsList[0];
				}

				// Validate all the settings
				if (Client && bHasGameTarget)
				{
					throw new AutomationException("Cannot mix game and client targets");
				}

				// Find the editor target name
				SelectDefaultTarget(TargetNamesOfType(TargetType.Editor), TargetType.Editor, ref EditorTarget);
			}
			else if (!CommandUtils.IsNullOrEmpty(Properties.Targets))
			{
				List<String> AvailableGameTargets = TargetNamesOfType(TargetType.Game);
				List<String> AvailableClientTargets = TargetNamesOfType(TargetType.Client);
				List<String> AvailableServerTargets = TargetNamesOfType(TargetType.Server);
				List<String> AvailableEditorTargets = TargetNamesOfType(TargetType.Editor);

				// That should cover all detected targets; Program targets are handled separately.
				System.Diagnostics.Debug.Assert(DetectedTargets.Count == (AvailableGameTargets.Count + AvailableClientTargets.Count + AvailableServerTargets.Count + AvailableEditorTargets.Count));

				if (Client)
				{
					SelectDefaultTarget(AvailableClientTargets, TargetType.Client, ref GameTarget);
					ProjectType = TargetType.Client;
				}
				else if (AvailableGameTargets.Count > 0 && !NoClient)
				{
					SelectDefaultTarget(AvailableGameTargets, TargetType.Game, ref GameTarget);
				}

				SelectDefaultTarget(AvailableEditorTargets, TargetType.Editor, ref EditorTarget);

				if (AvailableServerTargets.Count > 0 && DedicatedServer) // only if server is needed
				{
					SelectDefaultTarget(AvailableServerTargets, TargetType.Server, ref ServerTarget);
				}
			}
			else if (!CommandUtils.IsNullOrEmpty(Properties.Programs))
			{
				SingleTargetProperties TargetData = Properties.Programs[0];

				ProjectType = TargetType.Program;
				ProgramTarget = TargetData.TargetName;
				GameTarget = TargetData.TargetName;
			}
			else if (!this.Build)
			{
				var ShortName = ProjectUtils.GetShortProjectName(RawProjectPath);
				GameTarget = Client ? (ShortName + "Client") : ShortName;
				EditorTarget = ShortName + "Editor";
				ServerTarget = ShortName + "Server";
			}
			else
			{
				throw new AutomationException("{0} does not look like uproject file but no targets have been found!", RawProjectPath);
			}

			IsProgramTarget = ProjectType == TargetType.Program;

			if (String.IsNullOrEmpty(EditorTarget) && !IsProgramTarget && CommandUtils.IsNullOrEmpty(EditorTargetsList))
			{
				if (Properties.bWasGenerated)
				{
					EditorTarget = "UnrealEditor";
				}
				else
				{
					throw new AutomationException("Editor target not found!");
				}
			}

			if (EditorTargetsList == null)
			{
				if (!IsProgramTarget && !String.IsNullOrEmpty(EditorTarget))
				{
					EditorTargetsList = new ParamList<string>(EditorTarget);
				}
			}

			if (ProgramTargetsList == null)
			{
				if (IsProgramTarget)
				{
					ProgramTargetsList = new ParamList<string>(ProgramTarget);
				}
				else
				{
					ProgramTargetsList = new ParamList<string>();
				}
			}

			// Compile a client if it was asked for (-client) or we're cooking and require a client
			if (ClientCookedTargetsList == null)
			{
				if (!NoClient && (Cook || CookOnTheFly || Prebuilt || Client))
				{
					if (String.IsNullOrEmpty(GameTarget))
					{
						throw new AutomationException("Game target not found. Game target is required with -cook or -cookonthefly");
					}

					ClientCookedTargetsList = new ParamList<string>(GameTarget);
					
					if (ExtraTargetsToStageWithClient != null)
					{
						ClientCookedTargetsList.AddRange(ExtraTargetsToStageWithClient);
					}
				}
				else
				{
					ClientCookedTargetsList = new ParamList<string>();
				}
			}

			// Compile a server if it was asked for (-server) or we're cooking and require a server
			if (ServerCookedTargetsList == null)
			{
				/* Simplified from previous version which makes less sense.
				   TODO: tease out the actual dependencies between -cook and -server options, fix properly
				if (DedicatedServer && (Cook || CookOnTheFly || DedicatedServer)) */
				if (DedicatedServer)
				{
					if (String.IsNullOrEmpty(ServerTarget))
					{
						throw new AutomationException("Server target not found. Server target is required with -server and -cook or -cookonthefly");
					}

					ServerCookedTargetsList = new ParamList<string>(ServerTarget);
				}
				else
				{
					ServerCookedTargetsList = new ParamList<string>();
				}
			}

			if (ProjectPlatformBinariesPaths == null || ProjectExePaths == null)
			{
				ProjectPlatformBinariesPaths = new Dictionary<UnrealTargetPlatform, DirectoryReference>();
				ProjectExePaths = new Dictionary<UnrealTargetPlatform, FileReference>();

				DirectoryReference ProjectClientBinariesPath = ProjectUtils.GetClientProjectBinariesRootPath(RawProjectPath, ProjectType, Properties.bIsCodeBasedProject);

				foreach (TargetPlatformDescriptor TargetPlatform in ClientTargetPlatforms)
				{
					DirectoryReference BinariesPath = ProjectUtils.GetProjectClientBinariesFolder(ProjectClientBinariesPath, TargetPlatform.Type);
					ProjectPlatformBinariesPaths[TargetPlatform.Type] = BinariesPath;
					ProjectExePaths[TargetPlatform.Type] = FileReference.Combine(BinariesPath, GameTarget + Platform.GetExeExtension(TargetPlatform.Type));
				}
			}
		}

		public bool HasEditorTargets
		{
			get { return !CommandUtils.IsNullOrEmpty(EditorTargets); }
		}

		public bool HasCookedTargets
		{
			get { return !CommandUtils.IsNullOrEmpty(ClientCookedTargets) || !CommandUtils.IsNullOrEmpty(ServerCookedTargets); }
		}

		public bool HasServerCookedTargets
		{
			get { return !CommandUtils.IsNullOrEmpty(ServerCookedTargets); }
		}

		public bool HasClientCookedTargets
		{
			get { return !CommandUtils.IsNullOrEmpty(ClientCookedTargets); }
		}

		public bool HasProgramTargets
		{
			get { return !CommandUtils.IsNullOrEmpty(ProgramTargets); }
		}

		public bool HasMapsToCook
		{
			get { return !CommandUtils.IsNullOrEmpty(MapsToCook); }
		}

		public bool HasMapIniSectionsToCook
		{
			get { return !CommandUtils.IsNullOrEmpty(MapIniSectionsToCook); }
		}

		public bool HasDirectoriesToCook
		{
			get { return !CommandUtils.IsNullOrEmpty(DirectoriesToCook); }
		}

		public bool HasIterateSharedCookedBuild
		{
			get { return !String.IsNullOrEmpty(IterateSharedCookedBuild);  }
		}

        public bool HasDDCGraph
        {
            get { return !String.IsNullOrEmpty(DDCGraph); }
        }

        public bool HasInternationalizationPreset
        {
            get { return !String.IsNullOrEmpty(InternationalizationPreset); }
        }

		public bool HasCreateReleaseVersion
        {
            get { return !String.IsNullOrEmpty(CreateReleaseVersion); }
        }

        public bool HasBasedOnReleaseVersion
        {
            get { return !String.IsNullOrEmpty(BasedOnReleaseVersion); }
        }

		public bool HasOriginalReleaseVersion
		{
			get { return !String.IsNullOrEmpty(OriginalReleaseVersion); }
		}

		public bool HasAdditionalCookerOptions
        {
            get { return !String.IsNullOrEmpty(AdditionalCookerOptions); }
        }

        public bool HasDLCName
        {
            get { return DLCFile != null; }
        }

        public bool HasDiffCookedContentPath
        {
            get { return !String.IsNullOrEmpty(DiffCookedContentPath); }
        }

        public bool HasCulturesToCook
        {
            get { return CulturesToCook != null; }
        }

		public bool HasGameTargetDetected
		{
			get { return ProjectTargets.Exists(Target => Target.Rules.Type == TargetType.Game); }
		}

		public bool HasClientTargetDetected
		{
			get { return ProjectTargets.Exists(Target => Target.Rules.Type == TargetType.Client); }
		}

		public bool HasDedicatedServerAndClient
		{
			get { return Client && DedicatedServer; }
		}

		/// <summary>
		/// Project name (name of the uproject file without extension or directory name where the project is localed)
		/// </summary>
		public string ShortProjectName
		{
			get { return ProjectUtils.GetShortProjectName(RawProjectPath); }
		}

		/// <summary>
		/// AdditionalPluginDirectories from the project.uproject file
		/// </summary>
		public List<DirectoryReference> AdditionalPluginDirectories
		{
			get { return ProjectDescriptor.AdditionalPluginDirectories; }
		}

		/// <summary>
		/// Get the relative path to the DLC plugin's cooked output from the deployment
		/// root of the DLC. e.g. $(ProjectName)\Plugins\$(PluginName) for plugins under the Project's plugin
		/// directories.
		/// </summary>
		public string FindPluginRelativePathFromPlatformCookDir(FileReference PluginFile,
			DirectoryReference ProjectRoot, DirectoryReference EngineRoot, DirectoryReference LocalRoot, string ShortProjectName)
		{
			if (DLCOverrideCookedSubDir != null)
			{
				return DLCOverrideCookedSubDir;
			}

			foreach (DirectoryReference AdditionalPluginDir in AdditionalPluginDirectories)
			{
				if (PluginFile.IsUnderDirectory(AdditionalPluginDir))
				{
					// This is a plugin that lives outside of the Engine/Plugins or Game/Plugins directory so needs to be remapped for staging/packaging
					// The deployment path for plugins in AdditionalPluginDirectories is RemappedPlugins\PluginName
					return String.Format("RemappedPlugins/{0}", PluginFile.GetFileNameWithoutExtension());
				}
			}

			DirectoryReference DLCRoot = PluginFile.Directory;
			if (DLCRoot.IsUnderDirectory(EngineRoot))
			{
				return Path.Combine("Engine", DLCRoot.MakeRelativeTo(EngineRoot));
			}
			else if (DLCRoot.IsUnderDirectory(ProjectRoot))
			{
				return Path.Combine(ShortProjectName, DLCRoot.MakeRelativeTo(ProjectRoot));
			}
			else
			{
				return DLCRoot.MakeRelativeTo(LocalRoot);
			}
		}

		/// <summary>
		/// True if this project contains source code.
		/// </summary>	
		public bool IsCodeBasedProject
		{
			get
			{
				return bIsCodeBasedProject;
			}
		}
		private bool bIsCodeBasedProject;

		public FileReference CodeBasedUprojectPath
		{
            get 
			{ 
				if(RawProgramProjectOverride != null) { return RawProgramProjectOverride; }
				return IsCodeBasedProject ? RawProjectPath : null; 
			}
		}
		/// <summary>
		/// True if this project is a program.
		/// </summary>
		public bool IsProgramTarget { get; private set; }

		/// <summary>
		/// Returns override for Program Project files located in non-engine projects.
		/// </summary>
		public FileReference RawProgramProjectOverride { get; set; }

		/// <summary>
		/// Path where the project's game (or program) binaries are built for the given target platform.
		/// </summary>
		public DirectoryReference GetProjectBinariesPathForPlatform(UnrealTargetPlatform InPlatform)
		{
			DirectoryReference Result = null;
			ProjectPlatformBinariesPaths.TryGetValue(InPlatform, out Result);
			return Result;
		}

		private Dictionary<UnrealTargetPlatform, DirectoryReference> ProjectPlatformBinariesPaths;

		/// <summary>
		/// Filename of the target game exe (or program exe) for the given target platform
		/// </summary>
		public FileReference GetProjectExeForPlatform(UnrealTargetPlatform InPlatform)
		{
			FileReference Result = null;
			ProjectExePaths.TryGetValue(InPlatform, out Result);
			return Result;
		}

		private Dictionary<UnrealTargetPlatform, FileReference> ProjectExePaths;

		/// <summary>
		/// Override for the computed based on release version path
		/// </summary>
		public string BasedOnReleaseVersionPathOverride = null;

		/// <summary>
		/// Get the path to the directory of the version we are basing a diff or a patch on.  
		/// </summary>				
		public String GetBasedOnReleaseVersionPath(DeploymentContext SC, bool bIsClientOnly)
		{
			if (!string.IsNullOrEmpty(BasedOnReleaseVersionPathOverride))
			{
				return BasedOnReleaseVersionPathOverride;
			}

			String BasePath = BasedOnReleaseVersionBasePath;
			String Platform = SC.StageTargetPlatform.GetCookPlatform(SC.DedicatedServer, bIsClientOnly);
			if (String.IsNullOrEmpty(BasePath))
			{
                BasePath = CommandUtils.CombinePaths(SC.ProjectRoot.FullName, "Releases", BasedOnReleaseVersion, Platform);
			}
			else
			{
				BasePath = CommandUtils.CombinePaths(BasePath, BasedOnReleaseVersion, Platform);
			}

			return BasePath;
		}

		/// <summary>
		/// Get the path to the target directory for creating a new release version
		/// </summary>
		public String GetCreateReleaseVersionPath(DeploymentContext SC, bool bIsClientOnly)
		{
			String BasePath = CreateReleaseVersionBasePath;
			String Platform = SC.StageTargetPlatform.GetCookPlatform(SC.DedicatedServer, bIsClientOnly);
			if (String.IsNullOrEmpty(BasePath))
			{
				BasePath = CommandUtils.CombinePaths(SC.ProjectRoot.FullName, "Releases", CreateReleaseVersion, Platform);
			}
			else
			{
				BasePath = CommandUtils.CombinePaths(BasePath, CreateReleaseVersion, Platform);
			}

			return BasePath;
		}

		/// <summary>
		/// Get the path to the directory of the originally released version we're using to generate a patch.
		/// Only required by some platforms.
		/// </summary>				
		public String GetOriginalReleaseVersionPath(DeploymentContext SC, bool bIsClientOnly)
		{
			String BasePath = BasedOnReleaseVersionBasePath;
			String Platform = SC.StageTargetPlatform.GetCookPlatform(SC.DedicatedServer, bIsClientOnly);
			if (String.IsNullOrEmpty(BasePath))
			{
				BasePath = CommandUtils.CombinePaths(SC.ProjectRoot.FullName, "Releases", OriginalReleaseVersion, Platform);
			}
			else
			{
				BasePath = CommandUtils.CombinePaths(BasePath, OriginalReleaseVersion, Platform);
			}

			return BasePath;
		}

		/// <summary>
		/// True if we are generating a patch
		/// </summary>
		public bool IsGeneratingPatch
        {
            get { return GeneratePatch; }
        }

		/// <summary>
        /// True if we are generating a new patch pak tier
        /// </summary>
        public bool ShouldAddPatchLevel
        {
            get { return AddPatchLevel; }
        }

		public List<Platform> ClientTargetPlatformInstances
		{
			get
			{
				List<Platform> ClientPlatformInstances = new List<Platform>();
				foreach (TargetPlatformDescriptor ClientPlatform in ClientTargetPlatforms )
				{
					ClientPlatformInstances.Add(Platform.Platforms[ClientPlatform]);
				}
				return ClientPlatformInstances;
			}
		}

        public TargetPlatformDescriptor GetCookedDataPlatformForClientTarget(TargetPlatformDescriptor TargetPlatformDesc)
        {
            if (ClientDependentPlatformMap.ContainsKey(TargetPlatformDesc))
            {
                return ClientDependentPlatformMap[TargetPlatformDesc];
            }
            return TargetPlatformDesc;
        }

		public List<Platform> ServerTargetPlatformInstances
		{
			get
			{
				List<Platform> ServerPlatformInstances = new List<Platform>();
				foreach (TargetPlatformDescriptor ServerPlatform in ServerTargetPlatforms)
				{
					ServerPlatformInstances.Add(Platform.Platforms[ServerPlatform]);
				}
				return ServerPlatformInstances;
			}
		}

        public TargetPlatformDescriptor GetCookedDataPlatformForServerTarget(TargetPlatformDescriptor TargetPlatformType)
        {
            if (ServerDependentPlatformMap.ContainsKey(TargetPlatformType))
            {
                return ServerDependentPlatformMap[TargetPlatformType];
            }
            return TargetPlatformType;
        }

		/// <summary>
		/// All auto-detected targets for this project
		/// </summary>
		public List<SingleTargetProperties> ProjectTargets
		{
			get
			{
				if (DetectedTargets == null)
				{
					AutodetectSettings(false);
				}
				return DetectedTargets;
			}
		}

		/// <summary>
		/// List of all Engine ini files for this project
		/// </summary>
		public Dictionary<UnrealTargetPlatform, ConfigHierarchy> EngineConfigs
		{
			get
			{
				if (LoadedEngineConfigs == null)
				{
					AutodetectSettings(false);
				}
				return LoadedEngineConfigs;
			}
		}

		/// <summary>
		/// List of all Game ini files for this project
		/// </summary>
		public Dictionary<UnrealTargetPlatform, ConfigHierarchy> GameConfigs
		{
			get
			{
				if (LoadedGameConfigs == null)
				{
					AutodetectSettings(false);
				}
				return LoadedGameConfigs;
			}
		}

		public void Validate()
		{
			if (RawProjectPath == null)
			{
				throw new AutomationException("RawProjectPath can't be empty.");
			}
            if (!RawProjectPath.HasExtension(".uproject"))
            {
                throw new AutomationException("RawProjectPath {0} must end with .uproject", RawProjectPath);
            }
            if (!CommandUtils.FileExists(RawProjectPath.FullName))
            {
                throw new AutomationException("RawProjectPath {0} file must exist", RawProjectPath);
            }

			if (RawProgramProjectOverride != null && !RawProgramProjectOverride.HasExtension(".uproject"))
			{
				throw new AutomationException("RawProgramProjectPath {0} must end with .uproject", RawProgramProjectOverride);
			}
			if (RawProgramProjectOverride != null && !CommandUtils.FileExists(RawProgramProjectOverride.FullName))
			{
				throw new AutomationException("RawProgramProjectOverride {0} file must exist", RawProgramProjectOverride);
			}

			if (FileServer && !Cook && !CookInEditor)
			{
				throw new AutomationException("Only cooked builds can use a fileserver, use -cook or -CookInEditor");
			}

			if (Stage && !SkipStage && !Cook && !CookOnTheFly && !IsProgramTarget)
			{
				throw new AutomationException("Only cooked builds or programs can be staged, use -cook, -cookonthefly or -skipcook.");
			}

			if (Manifests && !Cook && !Stage && !Pak)
			{
				throw new AutomationException("Only staged pakd and cooked builds can generate streaming install manifests");
			}

			if (Pak && !Stage)
			{
				throw new AutomationException("Only staged builds can be paked, use -stage or -skipstage.");
			}

            if (Deploy && !Stage)
            {
                throw new AutomationException("Only staged builds can be deployed, use -stage or -skipstage.");
            }

            if ((Pak || Stage || Cook || CookOnTheFly || FileServer || DedicatedServer) && EditorTest)
            {
                throw new AutomationException("None of pak, stage, cook, CookOnTheFly or DedicatedServer can be used with EditorTest");
            }

            if (DedicatedServer && RunAutomationTests)
            {
                throw new AutomationException("DedicatedServer cannot be used with RunAutomationTests");
            }

			if (NoClient && !DedicatedServer && !CookOnTheFly)
			{
				throw new AutomationException("-noclient can only be used with -server or -cookonthefly.");
			}

			if (Build && !HasCookedTargets && !HasEditorTargets && !HasProgramTargets)
			{
				throw new AutomationException("-build is specified but there are no targets to build.");
			}

			if (Pak && FileServer)
			{
				throw new AutomationException("Can't use -pak and -fileserver at the same time.");
			}

			if (Cook && CookOnTheFly)
			{
				throw new AutomationException("Can't use both -cook and -cookonthefly.");
			}

            if (!HasDLCName && DLCIncludeEngineContent)
            {
                throw new AutomationException("DLCIncludeEngineContent flag is only valid when building DLC.");
            }
			if (!HasDLCName && DLCPakPluginFile )
			{
				throw new AutomationException("DLCPakPluginFile flag is only valid when building DLC.");
            }

            if ((IsGeneratingPatch || HasDLCName || ShouldAddPatchLevel) && !HasBasedOnReleaseVersion)
            {
                throw new AutomationException("Require based on release version to build patches or dlc");
            }

            if (ShouldAddPatchLevel && !IsGeneratingPatch)
            {
                throw new AutomationException("Creating a new patch tier requires patch generation");
            }

			if (HasCreateReleaseVersion && HasDLCName)
            {
                throw new AutomationException("Can't create a release version at the same time as creating dlc.");
            }

            if (HasBasedOnReleaseVersion && (IncrementalCook || IterativeDeploy || HasIterateSharedCookedBuild))
            {
                throw new AutomationException("Can't use iterative cooking / deploy on dlc or patching or creating a release");
            }

            /*if (Compressed && !Pak)
            {
                throw new AutomationException("-compressed can only be used with -pak");
            }*/

            if (CreateChunkInstall && (!(Manifests || HasDLCName) || !Stage))
            {
                throw new AutomationException("-createchunkinstall can only be used with -manifests & -stage"); 
            }

			if (CreateChunkInstall && String.IsNullOrEmpty(ChunkInstallDirectory))
			{
				throw new AutomationException("-createchunkinstall must specify the chunk install data directory with -chunkinstalldirectory=");
			}

			if (CreateChunkInstall && String.IsNullOrEmpty(ChunkInstallVersionString))
			{
				throw new AutomationException("-createchunkinstall must specify the chunk install data version string with -chunkinstallversion=");
			}
		}

		public FileReference FindZenProjectStoreMarker(DeploymentContext SC)
		{
			DirectoryReference ProjectStoreDir = null;
			if (Stage)
			{
				if (SC.PlatformCookDir == null)
				{
					throw new AutomationException("FindZenProjectStoreMarker called before PlatformCookDir was populated");
				}
				ProjectStoreDir = SC.PlatformCookDir;
			}
			else if (Deploy)
			{
				ProjectStoreDir = SC.StageDirectory;
			}
			if (ProjectStoreDir == null)
			{
				return null;
			}
			// Check for stage with zenstore without PAK?
			FileReference PackageStoreManifestFile = FileReference.Combine(ProjectStoreDir, "ue.projectstore");
			System.IO.FileInfo PackageStoreManifestFileInfo = PackageStoreManifestFile.ToFileInfo();
			if (PackageStoreManifestFileInfo.Exists)
			{
				return PackageStoreManifestFile;
			}
			return null;
		}

		public bool ShouldTreatAsFileServer(DeploymentContext SC)
		{
			if (FileServer)
			{
				return true;
			}
			if (ZenStore)
			{
				return true;
			}
			FileReference ZenStoreMarkerFile = FindZenProjectStoreMarker(SC);
			bool UseZenServerHost = (ZenStoreMarkerFile != null) && !UsePak(SC.StageTargetPlatform);
			return UseZenServerHost;
		}

		protected bool bLogged = false;
		public virtual void ValidateAndLog()
		{
			// Avoid spamming, log only once
			if (!bLogged)
			{
				// In alphabetical order.
				Logger.LogDebug("Project Params **************");

				Logger.LogDebug("AdditionalServerMapParams={AdditionalServerMapParams}", AdditionalServerMapParams);
				Logger.LogDebug("Archive={Archive}", Archive);
				Logger.LogDebug("ArchiveMetaData={ArchiveMetaData}", ArchiveMetaData);
				Logger.LogDebug("CreateAppBundle={CreateAppBundle}", CreateAppBundle);
				Logger.LogDebug("BaseArchiveDirectory={BaseArchiveDirectory}", BaseArchiveDirectory);
				Logger.LogDebug("BaseStageDirectory={BaseStageDirectory}", BaseStageDirectory);
				Logger.LogDebug("ConfigOverrideParams=-{Arg0}", string.Join(" -", ConfigOverrideParams));
				Logger.LogDebug("Build={Build}", Build);
				Logger.LogDebug("SkipBuildClient={SkipBuildClient}", SkipBuildClient);
				Logger.LogDebug("SkipBuildEditor={SkipBuildEditor}", SkipBuildEditor);
				Logger.LogDebug("Cook={Cook}", Cook);
				Logger.LogDebug("Clean={Clean}", Clean);
				Logger.LogDebug("Client={Client}", Client);
				Logger.LogDebug("ClientConfigsToBuild={Arg0}", string.Join(",", ClientConfigsToBuild));
				Logger.LogDebug("ClientCookedTargets={Arg0}", ClientCookedTargets.ToString());
				Logger.LogDebug("ClientTargetPlatform={Arg0}", string.Join(",", ClientTargetPlatforms));
				Logger.LogDebug("Compressed={Compressed}", Compressed);
				Logger.LogDebug("ForceUncompressed={ForceUncompressed}", ForceUncompressed);
				Logger.LogDebug("AdditionalPakOptions={AdditionalPakOptions}", AdditionalPakOptions);
				Logger.LogDebug("AdditionalIoStoreOptions={AdditionalIoStoreOptions}", AdditionalIoStoreOptions);
				Logger.LogDebug("ForceOodleDllVersion={ForceOodleDllVersion}", ForceOodleDllVersion);
				Logger.LogDebug("CookOnTheFly={CookOnTheFly}", CookOnTheFly);
				Logger.LogDebug("CookOnTheFlyStreaming={CookOnTheFlyStreaming}", CookOnTheFlyStreaming);
				Logger.LogDebug("UnversionedCookedContent={UnversionedCookedContent}", UnversionedCookedContent);
				Logger.LogDebug("OptionalContent={OptionalContent}", OptionalContent);
				Logger.LogDebug("SkipCookingEditorContent={SkipCookingEditorContent}", SkipCookingEditorContent);
				Logger.LogDebug("SkipCookingErrorSummary={SkipCookingErrorSummary}", SkipCookingErrorSummary);
                Logger.LogDebug("GeneratePatch={GeneratePatch}", GeneratePatch);
				Logger.LogDebug("AddPatchLevel={AddPatchLevel}", AddPatchLevel);
				Logger.LogDebug("DiscVersion={DiscVersion}", DiscVersion);
				Logger.LogDebug("CreateReleaseVersion={CreateReleaseVersion}", CreateReleaseVersion);
                Logger.LogDebug("BasedOnReleaseVersion={BasedOnReleaseVersion}", BasedOnReleaseVersion);
				Logger.LogDebug("OriginalReleaseVersion={OriginalReleaseVersion}", OriginalReleaseVersion);
				Logger.LogDebug("DLCFile={DLCFile}", DLCFile);
                Logger.LogDebug("DLCIncludeEngineContent={DLCIncludeEngineContent}", DLCIncludeEngineContent);
				Logger.LogDebug("DLCPakPluginFile={DLCPakPluginFile}", DLCPakPluginFile);
				Logger.LogDebug("DLCOverrideCookedSubDir={DLCOverrideCookedSubDir}", DLCOverrideCookedSubDir);
				Logger.LogDebug("DLCOverrideStagedSubDir={DLCOverrideStagedSubDir}", DLCOverrideStagedSubDir);
				Logger.LogDebug("DiffCookedContentPath={DiffCookedContentPath}", DiffCookedContentPath);
                Logger.LogDebug("AdditionalCookerOptions={AdditionalCookerOptions}", AdditionalCookerOptions);
				Logger.LogDebug("DedicatedServer={DedicatedServer}", DedicatedServer);
				Logger.LogDebug("DirectoriesToCook={Arg0}", DirectoriesToCook.ToString());
                Logger.LogDebug("DDCGraph={DDCGraph}", DDCGraph);
                Logger.LogDebug("CulturesToCook={Arg0}", CommandUtils.IsNullOrEmpty(CulturesToCook) ? "<Not Specified> (Use Defaults)" : CulturesToCook.ToString());
				Logger.LogDebug("EditorTargets={Arg0}", EditorTargets?.ToString());
				Logger.LogDebug("Foreign={Foreign}", Foreign);
				Logger.LogDebug("IsCodeBasedProject={Arg0}", IsCodeBasedProject.ToString());
				Logger.LogDebug("IsProgramTarget={Arg0}", IsProgramTarget.ToString());
				Logger.LogDebug("IncrementalCook={IncrementalCook}", IncrementalCook);
				Logger.LogDebug("IterateSharedCookedBuild={IterateSharedCookedBuild}", IterateSharedCookedBuild);
				Logger.LogDebug("IterateSharedBuildUsePrecompiledExe={IterateSharedBuildUsePrecompiledExe}", IterateSharedBuildUsePrecompiledExe);
				Logger.LogDebug("CookAll={CookAll}", CookAll);
				Logger.LogDebug("CookPartialGC={CookPartialGC}", CookPartialGC);
				Logger.LogDebug("CookInEditor={CookInEditor}", CookInEditor);
				Logger.LogDebug("CookMapsOnly={CookMapsOnly}", CookMapsOnly);
                Logger.LogDebug("Deploy={Deploy}", Deploy);
				Logger.LogDebug("IterativeDeploy={IterativeDeploy}", IterativeDeploy);
				Logger.LogDebug("FastCook={FastCook}", FastCook);
				Logger.LogDebug("LogWindow={LogWindow}", LogWindow);
				Logger.LogDebug("Manifests={Manifests}", Manifests);
				Logger.LogDebug("MapToRun={MapToRun}", MapToRun);
				Logger.LogDebug("NoClient={NoClient}", NoClient);
				Logger.LogDebug("NumClients={NumClients}", NumClients);
				Logger.LogDebug("NoDebugInfo={NoDebugInfo}", NoDebugInfo);
				Logger.LogDebug("SeparateDebugInfo={SeparateDebugInfo}", SeparateDebugInfo);
				Logger.LogDebug("MapFile={MapFile}", MapFile);
				Logger.LogDebug("NoCleanStage={NoCleanStage}", NoCleanStage);
				Logger.LogDebug("NoXGE={NoXGE}", NoXGE);
				Logger.LogDebug("MapsToCook={Arg0}", MapsToCook.ToString());
				Logger.LogDebug("MapIniSectionsToCook={Arg0}", MapIniSectionsToCook.ToString());
				Logger.LogDebug("Pak={Pak}", Pak);
				Logger.LogDebug("IgnorePaksFromDifferentCookSource={IgnorePaksFromDifferentCookSource}", IgnorePaksFromDifferentCookSource);
				Logger.LogDebug("IoStore={IoStore}", IoStore);
				Logger.LogDebug("SkipIoStore={SkipIoStore}", SkipIoStore);
				Logger.LogDebug("ZenStore={ZenStore}", ZenStore);
				Logger.LogDebug("NoZenAutoLaunch={NoZenAutoLaunch}", NoZenAutoLaunch);
				Logger.LogDebug("SkipEncryption={SkipEncryption}", SkipEncryption);
				Logger.LogDebug("GenerateOptimizationData={GenerateOptimizationData}", GenerateOptimizationData);
				Logger.LogDebug("SkipPackage={SkipPackage}", SkipPackage);
				Logger.LogDebug("NeverPackage={NeverPackage}", NeverPackage);
				Logger.LogDebug("Package={Package}", Package);
				Logger.LogDebug("ForcePackageData={ForcePackageData}", ForcePackageData);
				Logger.LogDebug("NullRHI={NullRHI}", NullRHI);
				Logger.LogDebug("WriteBackMetadataToAssetRegistry={WriteBackMetadataToAssetRegistry}", WriteBackMetadataToAssetRegistry);
				Logger.LogDebug("FakeClient={FakeClient}", FakeClient);
                Logger.LogDebug("EditorTest={EditorTest}", EditorTest);
                Logger.LogDebug("RunAutomationTests={RunAutomationTests}", RunAutomationTests);
                Logger.LogDebug("RunAutomationTest={RunAutomationTest}", RunAutomationTest);
                Logger.LogDebug("RunTimeoutSeconds={RunTimeoutSeconds}", RunTimeoutSeconds);
                Logger.LogDebug("CrashIndex={CrashIndex}", CrashIndex);
				Logger.LogDebug("ProgramTargets={Arg0}", ProgramTargets.ToString());
				Logger.LogDebug("ProjectPlatformBinariesPaths={Arg0}", string.Join(",", ProjectPlatformBinariesPaths));
				Logger.LogDebug("ProjectExePaths={Arg0}", string.Join(",", ProjectExePaths));
				Logger.LogDebug("Distribution={Distribution}", Distribution);
				Logger.LogDebug("PackageEncryptionKeyFile={PackageEncryptionKeyFile}", PackageEncryptionKeyFile);
				Logger.LogDebug("Prebuilt={Prebuilt}", Prebuilt);
				Logger.LogDebug("Prereqs={Prereqs}", Prereqs);
				Logger.LogDebug("AppLocalDirectory={AppLocalDirectory}", AppLocalDirectory);
				Logger.LogDebug("NoBootstrapExe={NoBootstrapExe}", NoBootstrapExe);
				Logger.LogDebug("RawProjectPath={RawProjectPath}", RawProjectPath);
				Logger.LogDebug("RawProgramProjectOverride={RawProgramProjectOverride}", RawProgramProjectOverride);
				Logger.LogDebug("Run={Run}", Run);
				Logger.LogDebug("ServerConfigsToBuild={Arg0}", string.Join(",", ServerConfigsToBuild));
				Logger.LogDebug("ServerCookedTargets={Arg0}", ServerCookedTargets.ToString());
				Logger.LogDebug("ServerTargetPlatform={Arg0}", string.Join(",", ServerTargetPlatforms));
				Logger.LogDebug("ShortProjectName={Arg0}", ShortProjectName.ToString());
				Logger.LogDebug("SignedPak={SignedPak}", SignedPak);
				Logger.LogDebug("SignPak={SignPak}", SignPak);
				Logger.LogDebug("SkipCook={SkipCook}", SkipCook);
				Logger.LogDebug("SkipCookOnTheFly={SkipCookOnTheFly}", SkipCookOnTheFly);
				Logger.LogDebug("SkipPak={SkipPak}", SkipPak);
                Logger.LogDebug("PrePak={PrePak}", PrePak);
                Logger.LogDebug("SkipStage={SkipStage}", SkipStage);
				Logger.LogDebug("Stage={Stage}", Stage);
				Logger.LogDebug("RehydrateAssets={RehydrateAssets}", RehydrateAssets);
				Logger.LogDebug("bTreatNonShippingBinariesAsDebugFiles={bTreatNonShippingBinariesAsDebugFiles}", bTreatNonShippingBinariesAsDebugFiles);
				Logger.LogDebug("bUseExtraFlavor={bUseExtraFlavor}", bUseExtraFlavor);
                Logger.LogDebug("StageDirectoryParam={StageDirectoryParam}", StageDirectoryParam);
				Logger.LogDebug("AdditionalPackageOptions={AdditionalPackageOptions}", AdditionalPackageOptions);
				Logger.LogDebug("Trace={Trace}", Trace);
				Logger.LogDebug("TraceHost={TraceHost}", TraceHost);
				Logger.LogDebug("TraceFile={TraceFile}", TraceFile);
				Logger.LogDebug("Project Params **************");
			}
			bLogged = true;

			Validate();
		}
	}
}
