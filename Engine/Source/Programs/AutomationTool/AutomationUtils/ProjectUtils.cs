// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Linq;
using System.Text;
using System.IO;
using UnrealBuildTool;
using System.Diagnostics;
using EpicGames.Core;
using System.Reflection;
using UnrealBuildBase;
using System.Runtime.Serialization;
using System.Collections;
using Microsoft.Extensions.Logging;

using static AutomationTool.CommandUtils;

namespace AutomationTool
{
	public class SingleTargetProperties
	{
		public string TargetName;
		public string TargetClassName;
		public TargetRules Rules;
	}

	/// <summary>
	/// Autodetected project properties.
	/// </summary>
	public class ProjectProperties
	{
		/// <summary>
		/// Full Project path. Must be a .uproject file
		/// </summary>
		public FileReference RawProjectPath;

		/// <summary>
		/// True if the uproject contains source code.
		/// </summary>
		public bool bIsCodeBasedProject;

		/// <summary>
		/// List of all targets detected for this project.
		/// </summary>
		public List<SingleTargetProperties> Targets = new List<SingleTargetProperties>();

		/// <summary>
		/// List of all scripts that were compiled to create the list of Targets
		/// </summary>
		public List<FileReference> TargetScripts = new List<FileReference>();

		/// <summary>
		/// List of all Engine ini files for this project
		/// </summary>
		public Dictionary<UnrealTargetPlatform, ConfigHierarchy> EngineConfigs = new Dictionary<UnrealTargetPlatform,ConfigHierarchy>();

		/// <summary>
		/// List of all Game ini files for this project
		/// </summary>
		public Dictionary<UnrealTargetPlatform, ConfigHierarchy> GameConfigs = new Dictionary<UnrealTargetPlatform, ConfigHierarchy>();

		/// <summary>
		/// List of all programs detected for this project.
		/// </summary>
		public List<SingleTargetProperties> Programs = new List<SingleTargetProperties>();

		/// <summary>
		/// Specifies if the target files were generated
		/// </summary>
		public bool bWasGenerated = false;

		internal ProjectProperties()
		{
		}
	}

	/// <summary>
	/// Project related utility functions.
	/// </summary>
	public class ProjectUtils
	{

		/// <summary>
		/// Struct that acts as a key for the project property cache. Based on these attributes 
		/// DetectProjectProperties may return different answers, e.g. Some platforms require a 
		///  codebased project for targets
		/// </summary>
		struct PropertyCacheKey : IEquatable<PropertyCacheKey>
		{
			string ProjectName;

			UnrealTargetPlatform[] TargetPlatforms;

			UnrealTargetConfiguration[] TargetConfigurations;

			public PropertyCacheKey(string InProjectName, IEnumerable<UnrealTargetPlatform> InTargetPlatforms, IEnumerable<UnrealTargetConfiguration> InTargetConfigurations)
			{
				ProjectName = InProjectName.ToLower();
				TargetPlatforms = InTargetPlatforms != null ? InTargetPlatforms.ToArray() : new UnrealTargetPlatform[0];
				TargetConfigurations = InTargetConfigurations != null ? InTargetConfigurations.ToArray() : new UnrealTargetConfiguration[0];
			}

			public bool Equals(PropertyCacheKey Other)
			{
				return ProjectName == Other.ProjectName &&
						StructuralComparisons.StructuralEqualityComparer.Equals(TargetPlatforms, Other.TargetPlatforms) &&
						StructuralComparisons.StructuralEqualityComparer.Equals(TargetConfigurations, Other.TargetConfigurations);
			}

			public override bool Equals(object Other)
			{
				return Other is PropertyCacheKey OtherKey && Equals(OtherKey);
			}

			public override int GetHashCode()
			{
				return HashCode.Combine(
					ProjectName.GetHashCode(),
					StructuralComparisons.StructuralEqualityComparer.GetHashCode(TargetPlatforms),
					StructuralComparisons.StructuralEqualityComparer.GetHashCode(TargetConfigurations));
			}

			public static bool operator==(PropertyCacheKey A, PropertyCacheKey B)
			{
				return A.Equals(B);
			}

			public static bool operator!=(PropertyCacheKey A, PropertyCacheKey B)
			{
				return !(A == B);
			}
		}

		private static ILogger Logger => Log.Logger;
		private static Dictionary<PropertyCacheKey, ProjectProperties> PropertiesCache = new Dictionary<PropertyCacheKey, ProjectProperties>();

		/// <summary>
		/// Gets a short project name (QAGame, Elemental, etc)
		/// </summary>
		/// <param name="RawProjectPath">Full project path.</param>
		/// <returns>Short project name</returns>
		public static string GetShortProjectName(FileReference RawProjectPath)
		{
			return CommandUtils.GetFilenameWithoutAnyExtensions(RawProjectPath.FullName);
		}

		/// <summary>
		/// Gets a short alphanumeric identifier for the project path.
		/// </summary>
		/// <param name="RawProjectPath">Full project path.</param>
		/// <returns>Project path identifier</returns>
		public static string GetProjectPathId(FileReference RawProjectPath)
		{
			string UniformProjectPath = FileReference.FindCorrectCase(RawProjectPath).ToNormalizedPath();
			string ProjectPathHash = ContentHash.MD5(Encoding.UTF8.GetBytes(UniformProjectPath)).ToString();
			return String.Format("{0}.{1}", GetShortProjectName(RawProjectPath), ProjectPathHash.Substring(0, 8));
		}

		/// <summary>
		/// Gets project properties.
		/// </summary>
		/// <param name="RawProjectPath">Full project path.</param>
		/// <param name="ClientTargetPlatforms"></param>
		/// <param name="ClientTargetConfigurations"></param>
		/// <param name="AssetNativizationRequested"></param>
		/// <returns>Properties of the project.</returns>
		public static ProjectProperties GetProjectProperties(FileReference RawProjectPath, List<UnrealTargetPlatform> ClientTargetPlatforms = null, List<UnrealTargetConfiguration> ClientTargetConfigurations = null, bool AssetNativizationRequested = false)
		{
			string ProjectKey = "UE4";
			if (RawProjectPath != null)
			{
				ProjectKey = CommandUtils.ConvertSeparators(PathSeparator.Slash, RawProjectPath.FullName);
			}
			ProjectProperties Properties;
			PropertyCacheKey PropertyKey = new PropertyCacheKey(ProjectKey, ClientTargetPlatforms, ClientTargetConfigurations);

			if (PropertiesCache.TryGetValue(PropertyKey, out Properties) == false)
			{
                Properties = DetectProjectProperties(RawProjectPath, ClientTargetPlatforms, ClientTargetConfigurations, AssetNativizationRequested);
				PropertiesCache.Add(PropertyKey, Properties);
			}
			return Properties;
		}

		/// <summary>
		/// Checks if the project is a UProject file with source code.
		/// </summary>
		/// <param name="RawProjectPath">Full project path.</param>
		/// <param name="ClientTargetPlatforms"></param>
		/// <param name="ClientTargetConfigurations"></param>
		/// <returns>True if the project is a UProject file with source code.</returns>
		public static bool IsCodeBasedUProjectFile(FileReference RawProjectPath, List<UnrealTargetPlatform> ClientTargetPlatforms = null, List < UnrealTargetConfiguration> ClientTargetConfigurations = null)
		{
			return GetProjectProperties(RawProjectPath, ClientTargetPlatforms, ClientTargetConfigurations).bIsCodeBasedProject;
		}

		/// <summary>
		/// Checks if the project is a UProject file with source code.
		/// </summary>
		/// <param name="RawProjectPath">Full project path.</param>
		/// <param name="ClientTargetPlatform"></param>
		/// <param name="ClientTargetConfigurations"></param>
		/// <returns>True if the project is a UProject file with source code.</returns>
		public static bool IsCodeBasedUProjectFile(FileReference RawProjectPath, UnrealTargetPlatform ClientTargetPlatform, List<UnrealTargetConfiguration> ClientTargetConfigurations = null)
		{
			return GetProjectProperties(RawProjectPath, new List<UnrealTargetPlatform>() { ClientTargetPlatform }, ClientTargetConfigurations).bIsCodeBasedProject;
		}

		/// <summary>
		/// Returns a path to the client binaries folder.
		/// </summary>
		/// <param name="ProjectClientBinariesPath"></param>
		/// <param name="Platform">Platform type.</param>
		/// <returns>Path to the binaries folder.</returns>
		public static DirectoryReference GetProjectClientBinariesFolder(DirectoryReference ProjectClientBinariesPath, UnrealTargetPlatform Platform)
		{
			ProjectClientBinariesPath = DirectoryReference.Combine(ProjectClientBinariesPath, Platform.ToString());
			return ProjectClientBinariesPath;
		}

		/// <summary>
		/// Attempts to autodetect project properties.
		/// </summary>
		/// <param name="RawProjectPath">Full project path.</param>
		/// <param name="ClientTargetPlatforms"></param>
		/// <param name="ClientTargetConfigurations"></param>
		/// <param name="AssetNativizationRequested"></param>
		/// <returns>Project properties.</returns>
		private static ProjectProperties DetectProjectProperties(FileReference RawProjectPath, List<UnrealTargetPlatform> ClientTargetPlatforms, List<UnrealTargetConfiguration> ClientTargetConfigurations, bool AssetNativizationRequested)
		{
			ProjectProperties Properties = new ProjectProperties();
			Properties.RawProjectPath = RawProjectPath;

			// detect if the project is content only, but has non-default build settings
			List<string> ExtraSearchPaths = new();
			if (RawProjectPath != null)
			{
				// no Target file, now check to see if build settings have changed
				List<UnrealTargetPlatform> TargetPlatforms = ClientTargetPlatforms;
				if (ClientTargetPlatforms == null || ClientTargetPlatforms.Count < 1)
				{
					// No client target platforms, add all in
					TargetPlatforms = new List<UnrealTargetPlatform>();
					foreach (UnrealTargetPlatform TargetPlatformType in UnrealTargetPlatform.GetValidPlatforms())
					{
						TargetPlatforms.Add(TargetPlatformType);
					}
				}

				List<UnrealTargetConfiguration> TargetConfigurations = ClientTargetConfigurations;
				if (TargetConfigurations == null || TargetConfigurations.Count < 1)
				{
					// No client target configurations, add all in
					TargetConfigurations = new List<UnrealTargetConfiguration>();
					foreach (UnrealTargetConfiguration TargetConfigurationType in Enum.GetValues(typeof(UnrealTargetConfiguration)))
					{
						if (TargetConfigurationType != UnrealTargetConfiguration.Unknown)
						{
							TargetConfigurations.Add(TargetConfigurationType);
						}
					}
				}

				if (NativeProjects.ConditionalMakeTempTargetForHybridProject(RawProjectPath, Logger))
				{
					Properties.bWasGenerated = true;
					string TempTargetDir = CommandUtils.CombinePaths(Path.GetDirectoryName(RawProjectPath.FullName), "Intermediate", "Source");
					ExtraSearchPaths.Add(TempTargetDir);
				}
            }

			if (CommandUtils.CmdEnv.HasCapabilityToCompile)
			{
				DetectTargetsForProject(Properties, ExtraSearchPaths);
				Properties.bIsCodeBasedProject = !CommandUtils.IsNullOrEmpty(Properties.Targets) || !CommandUtils.IsNullOrEmpty(Properties.Programs);
			}
			else
			{
				// should never ask for engine targets if we can't compile
				if (RawProjectPath == null)
				{
					throw new AutomationException("Cannot determine engine targets if we can't compile.");
				}

				Properties.bIsCodeBasedProject = Properties.bWasGenerated;
				// if there's a Source directory with source code in it, then mark us as having source code
				string SourceDir = CommandUtils.CombinePaths(Path.GetDirectoryName(RawProjectPath.FullName), "Source");
				if (Directory.Exists(SourceDir))
				{
					string[] CppFiles = Directory.GetFiles(SourceDir, "*.cpp", SearchOption.AllDirectories);
					string[] HFiles = Directory.GetFiles(SourceDir, "*.h", SearchOption.AllDirectories);
					Properties.bIsCodeBasedProject |= (CppFiles.Length > 0 || HFiles.Length > 0);
				}
			}

			// check to see if the uproject loads modules, only if we haven't already determined it is a code based project
			if (!Properties.bIsCodeBasedProject && RawProjectPath != null)
			{
				string uprojectStr = File.ReadAllText(RawProjectPath.FullName);
				Properties.bIsCodeBasedProject = uprojectStr.Contains("\"Modules\"");
			}

			// Get all ini files
			if (RawProjectPath != null)
			{
				Logger.LogDebug("Loading ini files for {RawProjectPath}", RawProjectPath);

				foreach (UnrealTargetPlatform TargetPlatformType in UnrealTargetPlatform.GetValidPlatforms())
				{
					ConfigHierarchy EngineConfig = ConfigCache.ReadHierarchy(ConfigHierarchyType.Engine, RawProjectPath.Directory, TargetPlatformType);
					Properties.EngineConfigs.Add(TargetPlatformType, EngineConfig);

					ConfigHierarchy GameConfig = ConfigCache.ReadHierarchy(ConfigHierarchyType.Game, RawProjectPath.Directory, TargetPlatformType);
					Properties.GameConfigs.Add(TargetPlatformType, GameConfig);
				}
			}

			return Properties;
		}


		/// <summary>
		/// Gets the project's root binaries folder.
		/// </summary>
		/// <param name="RawProjectPath">Full project path.</param>
		/// <param name="TargetType">Target type.</param>
		/// <param name="bIsCodeBasedProject">True if a code based project.</param>
		/// <returns>Binaries path.</returns>
		public static DirectoryReference GetClientProjectBinariesRootPath(FileReference RawProjectPath, TargetType TargetType, bool bIsCodeBasedProject)
		{
			DirectoryReference BinPath = null;
			switch (TargetType)
			{
				case TargetType.Program:
					BinPath = DirectoryReference.Combine(Unreal.RootDirectory, "Engine", "Binaries");
					break;
				case TargetType.Client:
				case TargetType.Game:
					if (!bIsCodeBasedProject)
					{
						BinPath = DirectoryReference.Combine(Unreal.RootDirectory, "Engine", "Binaries");
					}
					else
					{
						BinPath = DirectoryReference.Combine(RawProjectPath.Directory, "Binaries");
					}
					break;
			}
			return BinPath;
		}

		/// <summary>
		/// Gets the location where all rules assemblies should go
		/// </summary>
		private static string GetRulesAssemblyFolder()
		{
			string RulesFolder;
			if (Unreal.IsEngineInstalled())
			{
				RulesFolder = CommandUtils.CombinePaths(Path.GetTempPath(), "UAT", CommandUtils.EscapePath(CommandUtils.CmdEnv.LocalRoot), "Rules"); 
			}
			else
			{
				RulesFolder = CommandUtils.CombinePaths(CommandUtils.CmdEnv.EngineSavedFolder, "Rules");
			}
			return RulesFolder;
		}

		/// <summary>
		/// Finds all targets for the project.
		/// </summary>
		/// <param name="Properties">Project properties.</param>
		/// <param name="ExtraSearchPaths">Additional search paths.</param>
		private static void DetectTargetsForProject(ProjectProperties Properties, List<string> ExtraSearchPaths = null)
		{
			Properties.Targets = new List<SingleTargetProperties>();
			FileReference TargetsDllFilename;
			string FullProjectPath = null;

			List<DirectoryReference> GameFolders = new List<DirectoryReference>();
			DirectoryReference RulesFolder = new DirectoryReference(GetRulesAssemblyFolder());
			List<DirectoryReference> ExtraSearchDirectories = (ExtraSearchPaths == null) ? null : ExtraSearchPaths.Select(x => new DirectoryReference(x)).ToList();
			if (Properties.RawProjectPath != null)
			{
				Logger.LogDebug("Looking for targets for project {Arg0}", Properties.RawProjectPath);

				TargetsDllFilename = FileReference.Combine(RulesFolder, String.Format("UATRules-{0}.dll", ContentHash.MD5(Properties.RawProjectPath.FullName.ToUpperInvariant()).ToString()));

				FullProjectPath = CommandUtils.GetDirectoryName(Properties.RawProjectPath.FullName).Replace("\\", "/");

				// Extra search root uproject folder if we have /tests/
				if (FullProjectPath.Contains("/Tests/"))
				{
					ExtraSearchDirectories.Add(new DirectoryReference(FullProjectPath));
				}

				// there is a special case of Programs, where the uproject doesn't align with the Source directory, so we redirect to where
				// the program's target.cs file(s) are
				if (FullProjectPath.Contains("/Programs/"))
				{
					FullProjectPath = FullProjectPath.Replace("/Programs/", "/Source/Programs/");
				}

				GameFolders.Add(new DirectoryReference(FullProjectPath));
				Logger.LogDebug("Searching for target rule files in {FullProjectPath}", FullProjectPath);
			}
			else
			{
				TargetsDllFilename = FileReference.Combine(RulesFolder, String.Format("UATRules{0}.dll", "_BaseEngine_"));
			}

			// the UBT code assumes a certain CWD, but artists don't have this CWD.
			string SourceDir = CommandUtils.CombinePaths(CommandUtils.CmdEnv.LocalRoot, "Engine", "Source");
			bool DirPushed = false;
			if (CommandUtils.DirectoryExists_NoExceptions(SourceDir))
			{
				CommandUtils.PushDir(SourceDir);
				DirPushed = true;
			}

			List<FileReference> TargetScripts = Rules.FindAllRulesSourceFiles(Rules.RulesFileType.Target, GameFolders: GameFolders, ForeignPlugins: null, AdditionalSearchPaths: ExtraSearchDirectories);
			if (DirPushed)
			{
				CommandUtils.PopDir();
			}

			if (!CommandUtils.IsNullOrEmpty(TargetScripts))
			{
				// We only care about project target script so filter out any scripts not in the project folder, or take them all if we are just doing engine stuff
				List<FileReference> ProjectTargetScripts = new List<FileReference>();
				foreach (FileReference TargetScript in TargetScripts)
				{
					if (FullProjectPath == null || TargetScript.IsUnderDirectory(new DirectoryReference(FullProjectPath)))
					{
						// skip target rules that are platform extension or platform group specializations (don't treat _<Platform> targets as extensions if not under a <Platform> directory)
						string[] TargetPathSplit = TargetScript.GetFileNameWithoutAnyExtensions().Split(new char[]{'_'}, StringSplitOptions.RemoveEmptyEntries );
						if (TargetPathSplit.Length > 1 && 
							(UnrealTargetPlatform.IsValidName(TargetPathSplit.Last()) || UnrealPlatformGroup.IsValidName(TargetPathSplit.Last())) &&
							// platform extension targets will always be under a directory of that platform/group name
							TargetScript.ContainsName(TargetPathSplit.Last(), 0))
						{
							continue;
						}

						ProjectTargetScripts.Add(TargetScript);
					}
				}
				TargetScripts = ProjectTargetScripts;
			}

			if (!CommandUtils.IsNullOrEmpty(TargetScripts))
			{
				Logger.LogDebug("Found {Arg0} target rule files:", TargetScripts.Count);
				foreach (FileReference Filename in TargetScripts)
				{
					Logger.LogDebug("  {Filename}", Filename);
				}

				// Check if the scripts require compilation
				bool DoNotCompile = false;

				if (!CommandUtils.IsBuildMachine && !CheckIfScriptAssemblyIsOutOfDate(TargetsDllFilename, TargetScripts))
				{
					Logger.LogDebug("Targets DLL {Filename} is up to date.", TargetsDllFilename);
					DoNotCompile = true;
				}
				if (!DoNotCompile && CommandUtils.FileExists_NoExceptions(TargetsDllFilename.FullName))
				{
					if (!CommandUtils.DeleteFile_NoExceptions(TargetsDllFilename.FullName, true))
					{
						DoNotCompile = true;
						Logger.LogDebug("Could not delete {TargetsDllFilename} assuming it is up to date and reusable for a recursive UAT call.", TargetsDllFilename);
					}
				}

				CompileAndLoadTargetsAssembly(Properties, TargetsDllFilename, DoNotCompile, TargetScripts);
			}
		}

		/// <summary>
		/// Optionally compiles and loads target rules assembly.
		/// </summary>
		/// <param name="Properties"></param>
		/// <param name="TargetsDllFilename"></param>
		/// <param name="DoNotCompile"></param>
		/// <param name="TargetScripts"></param>
		private static void CompileAndLoadTargetsAssembly(ProjectProperties Properties, FileReference TargetsDllFilename, bool DoNotCompile, List<FileReference> TargetScripts)
		{
			Properties.TargetScripts = new List<FileReference>(TargetScripts);

			Logger.LogDebug("Compiling targets DLL: {TargetsDllFilename}", TargetsDllFilename);

			List<string> ReferencedAssemblies = new List<string>() 
					{ 
						typeof(UnrealBuildTool.PlatformExports).Assembly.Location
					};
			List<string> PreprocessorDefinitions = RulesAssembly.GetPreprocessorDefinitions();
			Assembly TargetsDLL = DynamicCompilation.CompileAndLoadAssembly(TargetsDllFilename, new HashSet<FileReference>(TargetScripts), Log.Logger, ReferencedAssemblies, PreprocessorDefinitions, DoNotCompile);
			Type[] AllCompiledTypes = TargetsDLL.GetTypes();
			foreach (Type TargetType in AllCompiledTypes)
			{
				// Find TargetRules but skip all "UnrealEditor", "UnrealGame" targets.
				if (typeof(TargetRules).IsAssignableFrom(TargetType) && !TargetType.IsAbstract)
				{
					string TargetName = GetTargetName(TargetType);

					TargetInfo DummyTargetInfo = new TargetInfo(TargetName, BuildHostPlatform.Current.Platform, UnrealTargetConfiguration.Development, null, Properties.RawProjectPath, null);

					// Create an instance of this type
					Logger.LogDebug("Creating target rules object: {Arg0}", TargetType.Name);
					TargetRules Rules = TargetRules.Create(TargetType, DummyTargetInfo, null, null, null, null, Log.Logger);
					Logger.LogDebug("Adding target: {Arg0} ({Arg1})", TargetType.Name, Rules.Type);

					SingleTargetProperties TargetData = new SingleTargetProperties();
					TargetData.TargetName = GetTargetName(TargetType);
					TargetData.TargetClassName = TargetType.FullName;
					TargetData.Rules = Rules;
					if (Rules.Type == global::UnrealBuildTool.TargetType.Program)
					{
						Properties.Programs.Add(TargetData);
					}
					else
					{
						Properties.Targets.Add(TargetData);
					}
				}
			}
		}

		/// <summary>
		/// Checks if any of the script files in newer than the generated assembly.
		/// </summary>
		/// <param name="TargetsDllFilename"></param>
		/// <param name="TargetScripts"></param>
		/// <returns>True if the generated assembly is out of date.</returns>
		private static bool CheckIfScriptAssemblyIsOutOfDate(FileReference TargetsDllFilename, List<FileReference> TargetScripts)
		{
			bool bOutOfDate = false;
			FileInfo AssemblyInfo = new FileInfo(TargetsDllFilename.FullName);
			if (AssemblyInfo.Exists)
			{
				foreach (FileReference ScriptFilename in TargetScripts)
				{
					FileInfo ScriptInfo = new FileInfo(ScriptFilename.FullName);
					if (ScriptInfo.Exists && ScriptInfo.LastWriteTimeUtc > AssemblyInfo.LastWriteTimeUtc)
					{
						bOutOfDate = true;
						break;
					}
				}
			}
			else
			{
				bOutOfDate = true;
			}
			return bOutOfDate;
		}

		/// <summary>
		/// Converts class type name (usually ends with Target) to a target name (without the postfix).
		/// </summary>
		/// <param name="TargetRulesType">Tagert class.</param>
		/// <returns>Target name</returns>
		private static string GetTargetName(Type TargetRulesType)
		{
			const string TargetPostfix = "Target";
			string Name = TargetRulesType.Name;
			if (Name.EndsWith(TargetPostfix, StringComparison.InvariantCultureIgnoreCase))
			{
				Name = Name.Substring(0, Name.Length - TargetPostfix.Length);
			}
			return Name;
		}

		/// <summary>
		/// Performs initial cleanup of target rules folder
		/// </summary>
		public static void CleanupFolders()
		{
			Logger.LogDebug("Cleaning up project rules folder");
			string RulesFolder = GetRulesAssemblyFolder();
			if (CommandUtils.DirectoryExists(RulesFolder))
			{
				CommandUtils.DeleteDirectoryContents(RulesFolder);
			}
		}

		/// <summary>
		/// Takes a game name (e.g "ShooterGame") and tries to find the path to the project file
		/// </summary>
		/// <param name="GameName"></param>
		/// <returns></returns>
		public static FileReference FindProjectFileFromName(string GameName)
		{
			// if they passed in a path then easy.
			if (File.Exists(GameName))
			{
				return new FileReference(GameName);
			}

			// Start with the gamename regardless of what they passed in
			GameName = Path.GetFileNameWithoutExtension(GameName);

			// Turn Foo into Foo.uproject
			string ProjectFile = GameName;

			if (string.IsNullOrEmpty(Path.GetExtension(ProjectFile)))
			{
				// if project was specified but had no extension then just add it.
				ProjectFile = Path.ChangeExtension(GameName, ".uproject");
			}

			// Turn Foo.uproject into Foo/Foo.uproject
			ProjectFile = Path.Combine(GameName, ProjectFile);

			GameName = Path.GetFileNameWithoutExtension(GameName);

			// check for sibling to engine
			if (File.Exists(ProjectFile))
			{
				return new FileReference(ProjectFile);
			}

			// Search NativeProjects (sibling folders).
			IEnumerable<FileReference> Projects = NativeProjects.EnumerateProjectFiles(Log.Logger);

			FileReference ProjectPath = Projects.Where(R => string.Equals(R.GetFileName(), ProjectFile, StringComparison.OrdinalIgnoreCase)).FirstOrDefault();

			if (ProjectPath == null)
			{
				// read .uprojectdirs
				List<string> SearchPaths = new List<string>();
				SearchPaths.Add("");
				string ProjectDirsFile = Directory.EnumerateFiles(Environment.CurrentDirectory, "*.uprojectdirs").FirstOrDefault();
				if (ProjectDirsFile != null)
				{
					foreach (string FilePath in File.ReadAllLines(ProjectDirsFile))
					{
						string Trimmed = FilePath.Trim();
						if (!Trimmed.StartsWith("./", StringComparison.OrdinalIgnoreCase) &&
							!Trimmed.StartsWith(";", StringComparison.OrdinalIgnoreCase) &&
							Trimmed.IndexOfAny(Path.GetInvalidPathChars()) < 0)
						{
							SearchPaths.Add(Trimmed);
						}
					}

					string ResolvedFile = SearchPaths.Select(P => Path.Combine(P, ProjectFile))
											.Where(P => File.Exists(P))
											.FirstOrDefault();

					if (ResolvedFile != null)
					{
						ProjectPath = new FileReference(ResolvedFile);
					}
				}
			}
						
			// either valid or we're out of ideas...
			return ProjectPath;
		}

		/// <summary>
		/// Full path to the Project executable for the current platform.
		/// </summary>
		/// <param name="ProjectFile">Path to Project file</param>
		/// <param name="TargetType">Target type</param>
		/// <param name="TargetPlatform">Target platform</param>
		/// <param name="TargetConfiguration">Target build configuration</param>
		/// <param name="Cmd">Do you want the console subsystem/commandlet executable?</param>
		/// <returns></returns>
		public static FileSystemReference GetProjectTarget(FileReference ProjectFile, UnrealBuildTool.TargetType TargetType, UnrealBuildTool.UnrealTargetPlatform TargetPlatform, UnrealBuildTool.UnrealTargetConfiguration TargetConfiguration = UnrealBuildTool.UnrealTargetConfiguration.Development, bool Cmd = false)
		{
			ProjectProperties Properties = ProjectUtils.GetProjectProperties(ProjectFile);
			List<SingleTargetProperties> Targets = Properties.Targets.Where(x => x.Rules.Type == TargetType).ToList();
			string TargetName = null;
			switch (Targets.Count)
			{
				case 0:
					return null;
				case 1:
					TargetName = Targets.First().TargetName;
					break;
				default:
					Properties.EngineConfigs[TargetPlatform].GetString("/Script/BuildSettings.BuildSettings", "DefaultEditorTarget", out TargetName);
					break;
			}

			FileReference TargetReceiptFileName = UnrealBuildTool.TargetReceipt.GetDefaultPath(ProjectFile.Directory, TargetName, TargetPlatform, TargetConfiguration, null);
			UnrealBuildTool.TargetReceipt TargetReceipt = UnrealBuildTool.TargetReceipt.Read(TargetReceiptFileName);

			if (Cmd)
			{
				return TargetReceipt.LaunchCmd;
			}
			
			if (TargetPlatform == UnrealTargetPlatform.Mac)
			{
				// Remove trailing "/Contents/MacOS/UnrealEngine" to get back to .app directory
				return TargetReceipt.Launch.Directory.ParentDirectory.ParentDirectory;
			}
			
			return TargetReceipt.Launch;
		}

		// Helper function to get the editor executable to run for cooking, etc the given project
		public static FileSystemReference GetEditorForProject(FileReference ProjectFile)
		{
			FileSystemReference EditorExe = null;
			if (ProjectFile != null)
			{
				EditorExe = ProjectUtils.GetProjectTarget(ProjectFile, UnrealBuildTool.TargetType.Editor, BuildHostPlatform.Current.Platform, UnrealTargetConfiguration.Development, true);
			}

			// if we haven't gotten an editor yet, use the engine shared binary
			if (EditorExe == null)
			{
				EditorExe = new FileReference(HostPlatform.Current.GetUnrealExePath("UnrealEditor-Cmd.exe"));
			}

			return EditorExe;
		}
	}

    public class BranchInfo
    {
		[DebuggerDisplay("{GameName}")]
        public class BranchUProject
        {
            public string GameName;
            public FileReference FilePath;

			private ProjectProperties CachedProperties;
			
			public ProjectProperties Properties
			{
				get
				{
					if(CachedProperties == null)
					{
						CachedProperties = ProjectUtils.GetProjectProperties(FilePath);
					}
					return CachedProperties;
				}
			}

            public BranchUProject(FileReference ProjectFile)
            {
                GameName = ProjectFile.GetFileNameWithoutExtension();

                //not sure what the heck this path is relative to
                FilePath = ProjectFile;

                if (!CommandUtils.FileExists_NoExceptions(FilePath.FullName))
                {
                    throw new AutomationException("Could not resolve relative path corrctly {0} -> {1} which doesn't exist.", ProjectFile, FilePath);
                }
            }
        }

		public List<BranchUProject> AllProjects = new List<BranchUProject>();

        public BranchInfo()
        {
            IEnumerable<FileReference> ProjectFiles = UnrealBuildTool.NativeProjects.EnumerateProjectFiles(Log.Logger);
			foreach (FileReference InfoEntry in ProjectFiles)
			{
				AllProjects.Add(new BranchUProject(InfoEntry));
			}

			Logger.LogDebug("  {Arg0} projects:", AllProjects.Count);
			foreach (BranchUProject Proj in AllProjects)
			{
				Logger.LogDebug(" {Arg0}: {Arg1}", Proj.GameName, Proj.FilePath);
			}
        }

        public BranchUProject FindGame(string GameName)
        {
			foreach (BranchUProject Proj in AllProjects)
            {
                if (Proj.GameName.Equals(GameName, StringComparison.InvariantCultureIgnoreCase))
                {
                    return Proj;
                }
            }
            return null;
        }

		public BranchUProject FindGameChecked(string GameName)
		{
			BranchUProject Project = FindGame(GameName);
			if(Project == null)
			{
				throw new AutomationException("Cannot find project '{0}' in branch", GameName);
			}
			return Project;
		}
    }
}
