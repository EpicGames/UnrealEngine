// Copyright Epic Games, Inc. All Rights Reserved.
using System;
using System.Collections.Generic;
using System.Linq;
using System.IO;
using System.Reflection;
using System.Diagnostics;
using UnrealBuildTool;
using EpicGames.Core;
using System.Threading.Tasks;
using UnrealBuildBase;
using Microsoft.Extensions.Logging;

using static AutomationTool.CommandUtils;

namespace AutomationTool
{
	/// <summary>
	/// Compiles and loads script assemblies.
	/// </summary>
	public static class ScriptManager
	{
		private static Dictionary<string, Type> ScriptCommands;
		public static readonly HashSet<Assembly> AllScriptAssemblies = new HashSet<Assembly>();
		public static HashSet<FileReference> BuildProducts { get; private set; } = new HashSet<FileReference>();

		/// <summary>
		/// Populates ScriptCommands
		/// </summary>
		static void EnumerateScriptCommands(ILogger Logger)
		{
			ScriptCommands = new Dictionary<string, Type>(StringComparer.InvariantCultureIgnoreCase);
			foreach (Assembly CompiledScripts in AllScriptAssemblies)
			{
				try
				{
					foreach (Type ClassType in CompiledScripts.GetTypes())
					{
						if (ClassType.IsSubclassOf(typeof(BuildCommand)) && ClassType.IsAbstract == false)
						{
							if (ScriptCommands.ContainsKey(ClassType.Name) == false)
							{
								ScriptCommands.Add(ClassType.Name, ClassType);
							}
							else
							{
								bool IsSame = string.Equals(ClassType.AssemblyQualifiedName, ScriptCommands[ClassType.Name].AssemblyQualifiedName);

								if (IsSame == false)
								{
									Logger.LogWarning("Unable to add command {ClassName} twice. Previous: {OldQualifiedName}, Current: {NewQualifiedName}", ClassType.Name,
										ClassType.AssemblyQualifiedName, ScriptCommands[ClassType.Name].AssemblyQualifiedName);
								}
							}
						}
					}
				}
				catch (ReflectionTypeLoadException LoadEx)
				{
					foreach (Exception SubEx in LoadEx.LoaderExceptions)
					{
						Logger.LogWarning(SubEx, "Got type loader exception: {SubEx}", SubEx.ToString());
					}
					throw new AutomationException("Failed to add commands from {0}. {1}", CompiledScripts, LoadEx);
				}
				catch (Exception Ex)
				{
					throw new AutomationException("Failed to add commands from {0}. {1}", CompiledScripts, Ex);
				}
			}

		}

		/// <summary>
		/// Enumerate the contents of the output directories
		/// </summary>
		static void EnumerateBuildProducts()
		{
			BuildProducts = new HashSet<FileReference>();

			HashSet<DirectoryReference> OutputDirs = new HashSet<DirectoryReference>();

			foreach (Assembly CompiledAssembly in AllScriptAssemblies)
			{
				if (string.IsNullOrWhiteSpace(CompiledAssembly.Location))
				{
					continue;
				}

				DirectoryReference AssemblyDirectory = new FileReference(CompiledAssembly.Location).Directory;
				AssemblyDirectory = DirectoryReference.FindCorrectCase(AssemblyDirectory);

				if (OutputDirs.Add(AssemblyDirectory))
				{
					BuildProducts.UnionWith(DirectoryReference.EnumerateFiles(AssemblyDirectory));

					// If there's "runtimes" sub-directory, include all the files contained therein
					foreach (DirectoryReference SubDir in DirectoryReference.EnumerateDirectories(AssemblyDirectory))
					{
						if (String.Equals(SubDir.GetDirectoryName(), "runtimes"))
						{
							BuildProducts.UnionWith(DirectoryReference.EnumerateFiles(SubDir, "*", SearchOption.AllDirectories));
						}
					}
				}
			}

		}

		/// <summary>
		/// Loads all precompiled assemblies (DLLs that end with *Scripts.dll).
		/// </summary>
		/// <param name="AssemblyPaths"></param>
		/// <param name="Logger"></param>
		/// <returns>List of compiled assemblies</returns>
		public static void LoadScriptAssemblies(IEnumerable<FileReference> AssemblyPaths, ILogger Logger)
		{
			foreach (FileReference AssemblyLocation in AssemblyPaths)
			{
				// Load the assembly into our app domain
				Logger.LogDebug("Loading script DLL: {AssemblyLocation}", AssemblyLocation);
				try
				{
					AssemblyUtils.AddFileToAssemblyCache(AssemblyLocation.FullName);
					// Add a resolver for the Assembly directory, so that its dependencies may be found alongside it
					AssemblyUtils.InstallRecursiveAssemblyResolver(AssemblyLocation.Directory.FullName);
					Assembly Assembly = AppDomain.CurrentDomain.Load(AssemblyName.GetAssemblyName(AssemblyLocation.FullName));
					AllScriptAssemblies.Add(Assembly);
				}
				catch (Exception Ex)
				{
					throw new AutomationException("Failed to load script DLL: {0}: {1}", AssemblyLocation, Ex.Message);
				}
			}

			Platform.InitializePlatforms(AllScriptAssemblies);

			EnumerateScriptCommands(Logger);

			EnumerateBuildProducts();
		}

		public static Dictionary<string, Type> Commands
		{
			get { return ScriptCommands; }
		}
	}
}
