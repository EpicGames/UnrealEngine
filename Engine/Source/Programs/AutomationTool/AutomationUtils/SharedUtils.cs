// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Linq;
using System.Text;
using System.IO;
using System.Threading;
using System.Diagnostics;
using System.Reflection;
// NOTE: This file is shared with AutomationToolLauncher. It can NOT have any references to non-system libraries

namespace AutomationTool
{
	public class SharedUtils
	{
		/// <summary>
		/// Parses the command line string and returns an array of passed arguments. Unlike the default parsing algorithm, this will treat \r\n characters just like spaces.
		/// </summary>
		/// <param name="CmdLine">The command line to parse</param>
		/// <returns>List of command line arguments.</returns>
		public static string[] ParseCommandLine(string CmdLine)
		{
			List<string> Args = new List<string>();
			StringBuilder Arg = new StringBuilder(CmdLine.Length);
			bool bQuote = false;
			int bEscape = 0;
			for (int Index = 0; Index < CmdLine.Length; ++Index)
			{
				bool bCanAppend = true;
				char C = CmdLine[Index];
				if (!bQuote && Char.IsWhiteSpace(C))
				{
					if (Arg.Length > 0)
					{
						Args.Add(Arg.ToString());
						Arg.Clear();
					}
					bCanAppend = false;
				}
				else if (C == '\\' && Index < (CmdLine.Length - 1))
				{
					// Escape character
					bEscape++;
					bCanAppend = false;
				}
				else if(bEscape == 0 && C == '^' && Index + 1 < CmdLine.Length && CmdLine[Index + 1] == '&')
				{
					// Visual studio seems to escape ampersands as if for a batch file when running UAT *without* the debugger attached (ie. ctrl-f5)
					Index++;
					C = CmdLine[Index];
				}
				else if (C == '\"')
				{
					bCanAppend = bEscape > 0;
					if (bEscape == 0)
					{
						bQuote = !bQuote;
					}
					else
					{
						// Consume the scape character
						bEscape--;
					}
				}
				if (bCanAppend)
				{
					if (bEscape > 0)
					{
						// Unused escape character.
						Arg.Append('\\', bEscape);
						bEscape = 0;
					}
					Arg.Append(C);
				}
			}
			if (Arg.Length > 0)
			{
				Args.Add(Arg.ToString());
			}
			return Args.ToArray();
		}

		/// <summary>
		/// Implements the same functionality as ParseCommandLine(), but removes the executable as the first argument.
		/// </summary>
		/// <param name="CmdLine">The command line to parse</param>
		/// <returns>Array of command line arguments.</returns>
		public static string[] ParseCommandLineAndRemoveExe(string CmdLine)
		{
			return ParseCommandLine(CmdLine).Skip(1).ToArray();
		}
	}
}
