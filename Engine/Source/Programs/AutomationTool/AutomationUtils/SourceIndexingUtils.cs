// Copyright Epic Games, Inc. All Rights Reserved.

using AutomationTool;
using EpicGames.Core;
using Microsoft.Extensions.Logging;
using System;
using System.Collections.Generic;
using System.Linq;
using UnrealBuildBase;

namespace AutomationUtils
{
	using VFSPathMapping = (DirectoryReference Local, DirectoryReference Virtualized);
	using VFSPathMappingList = List<(DirectoryReference Local, DirectoryReference Virtualized)>;

	public static class SourceIndexingUtils
	{
		/// <summary>
		/// Read a list of Virtual File System path transformations from a single string.
		/// The string may have multiple entries separate by semicolons.
		/// Each entry is an assignment: LocalPath=VirtualPath.
		/// LocalPath may use [Root] which is replaced by by the current workspace root (Unreal.RootDirectory).
		/// </summary>
		/// <param name="VFSMapping">List of path transformations.</param>
		/// <returns></returns>
		public static VFSPathMappingList ParseVFSMapping(string VFSMapping)
		{
			VFSPathMappingList Result = new();

			if (string.IsNullOrEmpty(VFSMapping))
			{
				return Result;
			}

			foreach (string EntryRaw in VFSMapping.Split(';', StringSplitOptions.RemoveEmptyEntries))
			{
				string Entry = EntryRaw.Trim();

				if (Entry.Length == 0)
				{
					continue;
				}

				int EqualPosition = Entry.IndexOf('=');

				if (EqualPosition < 1 || EqualPosition == Entry.Length - 1)
				{
					continue;
				}

				string Local = Entry.Substring(0, EqualPosition).Trim();
				string Virtualized = Entry.Substring(EqualPosition + 1).Trim();

				Local = Local.Replace("[Root]", Unreal.RootDirectory.FullName);

				Result.Add((new DirectoryReference(Local), new DirectoryReference(Virtualized)));
			}

			return Result;
		}

		/// <summary>
		/// Helper function printing the given filesystem path mapping to the log for debug purposes.
		/// </summary>
		/// <param name="VFSMapping"></param>
		public static void OutputVFSMapping(VFSPathMappingList VFSMapping)
		{
			if (VFSMapping.Count > 0)
			{
				CommandUtils.Logger.LogInformation("Applying the following Virtual File System path transformations to source indexing information:");

				foreach ((DirectoryReference Local, DirectoryReference Virtualized) VFSMappingEntry in VFSMapping)
				{
					CommandUtils.Logger.LogInformation("\t{Local} -> {Virtualized}", VFSMappingEntry.Local.FullName, VFSMappingEntry.Virtualized.FullName);
				}
			}
		}

		/// <summary>
		/// Converts a local path to a virtualized path (as generated by Unreal Build Tool when using Virtual File System i.e. compiling with -vfs switch).
		/// Only the first matching rule from VFSMapping is applied.
		/// </summary>
		/// <param name="Local">Local path to a source file e.g. C:\UE\Main\Engine\Source\Foo.cpp</param>
		/// <param name="VFSMapping">List of path substitutions (generated by ParseVFSMapping).</param>
		/// <returns>Virtualized path e.g. Z:\UEVFS\Root\Engine\Source\Foo.cpp. If no VFSMapping rules matched, Local is returned.</returns>
		public static string ConvertLocalToVirtualized(string Local, VFSPathMappingList VFSMapping)
		{
			FileReference LocalFile = new FileReference(Local);

			foreach (VFSPathMapping VFSMappingEntry in VFSMapping)
			{
				if (LocalFile.IsUnderDirectory(VFSMappingEntry.Local))
				{
					return FileReference.Combine(VFSMappingEntry.Virtualized, LocalFile.MakeRelativeTo(VFSMappingEntry.Local)).FullName;
				}
			}

			return Local;
		}

		/// <summary>
		/// Build a database of source code files in the current Perforce workspace.
		/// By relying on the standard layout for UE projects i.e. the fact that source code is in Source directories,
		/// we may very significantly reduce the about of data sent to us from the server.
		/// </summary>
		/// <param name="Pattern">Perforce pattern path to query e.g. //UE/Branch/.../Source/...</param>
		/// <returns></returns>
		public static Dictionary<string, P4HaveRecord> BuildSourceDatabase(string Pattern, string VFSMappingString)
		{
			List<P4HaveRecord> Files = null;

			P4Connection DefaultConnection = new P4Connection(User: null, Client: null, ServerAndPort: null);

			try
			{
				Files = DefaultConnection.HaveFiles(Pattern);
			}
			catch (P4Exception e)
			{
				CommandUtils.Logger.LogError("Failed to fetch source code information from Perforce for '{Pattern}' ({Message}).", Pattern, e.Message);

				return null;
			}

			if (!string.IsNullOrEmpty(VFSMappingString))
			{
				VFSPathMappingList VFSMapping = ParseVFSMapping(VFSMappingString);

				if (VFSMapping.Count > 0)
				{
					OutputVFSMapping(VFSMapping);

					foreach (P4HaveRecord File in Files)
					{
						File.ClientFile = ConvertLocalToVirtualized(File.ClientFile, VFSMapping);
					}
				}
			}

			return Files.ToDictionary(file => file.ClientFile, file => file, StringComparer.InvariantCultureIgnoreCase);
		}
	}
}
