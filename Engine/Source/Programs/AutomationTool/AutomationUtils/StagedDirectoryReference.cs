// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Linq;
using System.Text;
using System.Threading.Tasks;
using EpicGames.Core;

public class StagedDirectoryReference : StagedFileSystemReference, IEquatable<StagedDirectoryReference>
{
	/// <summary>
	/// Shared instance representing the root directory
	/// </summary>
	public static readonly StagedDirectoryReference Root = new StagedDirectoryReference("");

	/// <summary>
	/// Constructor
	/// </summary>
	/// <param name="Name">The directory being referred to. Either type of directory separator is permitted and will be normalized. Empty path fragments and leading/trailing slashes are not permitted.</param>
	public StagedDirectoryReference(string Name) : base(Name)
	{
	}

	/// <summary>
	/// Create a staged directory reference by concatenating multiple strings
	/// </summary>
	/// <param name="BaseDir">The base directory</param>
	/// <param name="Fragments">The fragments to append</param>
	/// <returns>Directory reference formed by concatenating the arguments</returns>
	public static StagedDirectoryReference Combine(string BaseDir, params string[] Fragments)
	{
		return new StagedDirectoryReference(CombineStrings(BaseDir, Fragments));
	}

	/// <summary>
	/// Create a staged directory reference by concatenating multiple strings
	/// </summary>
	/// <param name="BaseDir">The base directory</param>
	/// <param name="Fragments">The fragments to append</param>
	/// <returns>Directory reference formed by concatenating the arguments</returns>
	public static StagedDirectoryReference Combine(StagedDirectoryReference BaseDir, params string[] Fragments)
	{
		return new StagedDirectoryReference(CombineStrings(BaseDir.Name, Fragments));
	}

	/// <summary>
	/// Compares two directory references for equality
	/// </summary>
	/// <param name="A">First directory reference</param>
	/// <param name="B">Second directory reference</param>
	/// <returns>True if the two directories are identical. Case is ignored.</returns>
	public static bool operator ==(StagedDirectoryReference A, StagedDirectoryReference B)
	{
		if ((object)A == null)
		{
			return (object)B == null;
		}
		else
		{
			return (object)B != null && A.Name.Equals(B.Name, FileSystemReference.Comparison);
		}
	}

	/// <summary>
	/// Compares two directory references for inequality
	/// </summary>
	/// <param name="A">First directory reference</param>
	/// <param name="B">Second directory reference</param>
	/// <returns>True if the two directories are not identical. Case is ignored.</returns>
	public static bool operator !=(StagedDirectoryReference A, StagedDirectoryReference B)
	{
		return !(A == B);
	}

	/// <summary>
	/// Compares against another object for equality
	/// </summary>
	/// <param name="Obj">Object to compare against</param>
	/// <returns>True if the other object is a directory reference and is identical. Case is ignored.</returns>
	public override bool Equals(object Obj)
	{
		StagedDirectoryReference Other = Obj as StagedDirectoryReference;
		return Other != null && Other.Name.Equals(Name, FileSystemReference.Comparison);
	}

	/// <summary>
	/// Compares against another object for equality
	/// </summary>
	/// <param name="Other">Directory reference to compare against</param>
	/// <returns>True if the two directories are identical. Case is ignored.</returns>
	public bool Equals(StagedDirectoryReference Other)
	{
		return Name.Equals(Other.Name, FileSystemReference.Comparison);
	}

	/// <summary>
	/// Gets a hash code for this reference.
	/// </summary>
	/// <returns>Hash code for the current object.</returns>
	public override int GetHashCode()
	{
		return FileSystemReference.Comparer.GetHashCode(Name);
	}
}

