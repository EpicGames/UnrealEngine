// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Linq;
using System.Text;
using System.Threading.Tasks;
using EpicGames.Core;
using UnrealBuildTool;

/// <summary>
/// Location of a staged file. Stored in a normalized format with forward slashes as directory separators.
/// </summary>
public class StagedFileReference : StagedFileSystemReference, IEquatable<StagedFileReference>
{
	/// <summary>
	/// Constructor
	/// </summary>
	/// <param name="Name">The file being referred to. Either type of directory separator is permitted and will be normalized. Empty path fragments and leading/trailing slashes are not permitted.</param>
	public StagedFileReference(string Name) : base(Name)
	{
	}

	/// <summary>
	/// Protected constructor. Initializes to the given parameters without validation.
	/// </summary>
	/// <param name="Name">The name of this entity.</param>
	/// <param name="Sanitize"></param>
	private StagedFileReference(string Name, Sanitize Sanitize) : base(Name, Sanitize)
	{
	}

	/// <summary>
	/// Accessor for the directory containing this file
	/// </summary>
	public StagedDirectoryReference Directory
	{
		get
		{
			int LastSlashIdx = Name.LastIndexOf('/');
			if(LastSlashIdx == -1)
			{
				return StagedDirectoryReference.Root;
			}
			else
			{
				return new StagedDirectoryReference(Name.Substring(0, LastSlashIdx));
			}
		}
	}

	/// <summary>
	/// Determines if the name contains the given fragment
	/// </summary>
	/// <param name="OtherName"></param>
	/// <returns></returns>
	public bool ContainsName(string OtherName)
	{
		int StartIdx = 0;
		for(;;)
		{
			int Idx = Name.IndexOf(OtherName, StartIdx, FileSystemReference.Comparison);
			if(Idx == -1)
			{
				return false;
			}
			if (Idx == 0 || Name[Idx - 1] == '/')
			{
				int EndIdx = Idx + OtherName.Length;
				if(EndIdx < Name.Length && Name[EndIdx] == '/')
				{
					return true;
				}
			}
			StartIdx = Idx + 1;
		}
	}

	/// <summary>
	/// Attempts to remap this file reference from one directory to another
	/// </summary>
	/// <param name="InputFile"></param>
	/// <param name="SourceDir">Directory to map from</param>
	/// <param name="TargetDir">Directory to map to</param>
	/// <param name="RemappedFile">On success, receives the new staged file location</param>
	/// <returns>True if the file was remapped, false otherwise</returns>
	public static bool TryRemap(StagedFileReference InputFile, StagedDirectoryReference SourceDir, StagedDirectoryReference TargetDir, out StagedFileReference RemappedFile)
	{
		if (InputFile.Name.StartsWith(SourceDir.Name, FileSystemReference.Comparison) && InputFile.Name.Length > SourceDir.Name.Length && InputFile.Name[SourceDir.Name.Length] == '/')
		{
			RemappedFile = new StagedFileReference(TargetDir.Name + InputFile.Name.Substring(SourceDir.Name.Length), Sanitize.None);
			return true;
		}
		else
		{
			RemappedFile = null;
			return false;
		}
	}

	/// <summary>
	/// Create a staged file reference by concatenating multiple strings
	/// </summary>
	/// <param name="BaseDir">The base directory</param>
	/// <param name="Fragments">The fragments to append</param>
	/// <returns>File reference formed by concatenating the arguments</returns>
	public static StagedFileReference Combine(string BaseDir, params string[] Fragments)
	{
		return new StagedFileReference(CombineStrings(BaseDir, Fragments));
	}

	/// <summary>
	/// Create a staged file reference by concatenating multiple strings
	/// </summary>
	/// <param name="BaseDir">The base directory</param>
	/// <param name="Fragments">The fragments to append</param>
	/// <returns>File reference formed by concatenating the arguments</returns>
	public static StagedFileReference Combine(StagedDirectoryReference BaseDir, params string[] Fragments)
	{
		return new StagedFileReference(CombineStrings(BaseDir.Name, Fragments));
	}

	/// <summary>
	/// Converts this file reference to a lowercase invariant form.
	/// </summary>
	/// <returns>Lowercase version of this file reference.</returns>
	public StagedFileReference ToLowerInvariant()
	{
		return new StagedFileReference(Name.ToLowerInvariant(), Sanitize.None);
	}

	/// <summary>
	/// Compares two file references for equality
	/// </summary>
	/// <param name="A">First file reference</param>
	/// <param name="B">Second file reference</param>
	/// <returns>True if the two files are identical. Case is ignored.</returns>
	public static bool operator ==(StagedFileReference A, StagedFileReference B)
	{
		if ((object)A == null)
		{
			return (object)B == null;
		}
		else
		{
			return (object)B != null && A.Name.Equals(B.Name, FileSystemReference.Comparison);
		}
	}

	/// <summary>
	/// Compares two file references for inequality
	/// </summary>
	/// <param name="A">First file reference</param>
	/// <param name="B">Second file reference</param>
	/// <returns>True if the two files are not identical. Case is ignored.</returns>
	public static bool operator !=(StagedFileReference A, StagedFileReference B)
	{
		return !(A == B);
	}

	/// <summary>
	/// Compares against another object for equality
	/// </summary>
	/// <param name="Obj">Object to compare against</param>
	/// <returns>True if the other object is a directory reference and is identical. Case is ignored.</returns>
	public override bool Equals(object Obj)
	{
		StagedFileReference Other = Obj as StagedFileReference;
		return Other != null && Other.Name.Equals(Name, FileSystemReference.Comparison);
	}

	/// <summary>
	/// Compares against another object for equality
	/// </summary>
	/// <param name="Other">Directory reference to compare against</param>
	/// <returns>True if the two directories are identical. Case is ignored.</returns>
	public bool Equals(StagedFileReference Other)
	{
		return Name.Equals(Other.Name, FileSystemReference.Comparison);
	}

	/// <summary>
	/// Gets a hash code for this reference.
	/// </summary>
	/// <returns>Hash code for the current object.</returns>
	public override int GetHashCode()
	{
		return FileSystemReference.Comparer.GetHashCode(Name);
	}
}

