// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Linq;
using System.Text;
using System.Threading.Tasks;
using EpicGames.Core;
using UnrealBuildTool;

/// <summary>
/// Location of a staged file or directory. Stored in a normalized format with forward slashes as directory separators.
/// </summary>
public abstract class StagedFileSystemReference
{
	/// <summary>
	/// Special value used to invoke the non-sanitizing constructor overload
	/// </summary>
	public enum Sanitize
	{
		None
	}

	/// <summary>
	/// The name of this file system entity
	/// </summary>
	public readonly string Name;

	/// <summary>
	/// Constructor
	/// </summary>
	/// <param name="InName">The file system reference referred to. Either type of directory separator is permitted and will be normalized. Empty path fragments and leading/trailing slashes are not permitted.</param>
	public StagedFileSystemReference(string InName)
	{
		Name = InName.Replace('\\', '/');

		// Make sure it's not an absolute path
		if (Name.Length >= 2 && (Name[1] == ':' || Name[0] == '/'))
		{
			throw new ArgumentException(String.Format("Attempt to construct staged filesystem reference from absolute path ({0}). Staged paths are always relative to the staging root.", InName));
		}

		// Make sure it doesn't end in a directory separator
		if(Name.Length > 0 && Name[Name.Length - 1] == '/')
		{
			throw new ArgumentException(String.Format("Staged filesystem references cannot end with path separators ({0}).", InName));
		}

		// Remove any relative paths
		for(int FragmentIdx = 0; FragmentIdx < Name.Length; )
		{
			// Scan to the end of the current path fragment
			int FragmentEndIdx = FragmentIdx;
			while(FragmentEndIdx != Name.Length && Name[FragmentEndIdx] != '/')
			{
				FragmentEndIdx++;
			}

			// If it's an empty fragment (including if it's a path separator at the start of the string), it's invalid
			if(FragmentEndIdx == FragmentIdx)
			{
				throw new ArgumentException(String.Format("Staged filesystem reference may not start with a path separator or contain an empty path fragment ({0})", InName));
			}

			// Remove references to the current directory (".")
			if(FragmentEndIdx == FragmentIdx + 1 && Name[FragmentIdx] == '.')
			{
				if(FragmentEndIdx < Name.Length)
				{
					Name = Name.Remove(FragmentIdx, (FragmentEndIdx + 1) - FragmentIdx);
					continue;
				}
				else if(FragmentIdx > 0)
				{
					Name = Name.Remove(FragmentIdx - 1, FragmentEndIdx - (FragmentIdx - 1));
					break;
				}
				else
				{
					Name = "";
					break;
				}
			}

			// Remove references to the parent directory ("..")
			if(FragmentEndIdx == FragmentIdx + 2 && Name[FragmentIdx] == '.' && Name[FragmentIdx + 1] == '.')
			{
				// Make sure it's not right at the start
				if(FragmentIdx == 0)
				{
					throw new ArgumentException(String.Format("Staged filesystem reference cannot reference outside the staging root ({0})", InName));
				}

				// Get the start of the last fragment
				FragmentIdx = Name.LastIndexOf('/', FragmentIdx - 2) + 1;
				if(FragmentEndIdx < Name.Length)
				{
					Name = Name.Remove(FragmentIdx, (FragmentEndIdx + 1) - FragmentIdx);
					continue;
				}
				else if(FragmentIdx > 0)
				{
					Name = Name.Remove(FragmentIdx - 1, FragmentEndIdx - (FragmentIdx - 1));
					break;
				}
				else
				{
					Name = "";
					break;
				}
			}

			// Move to the next fragment
			FragmentIdx = FragmentEndIdx + 1;
		}
	}

	/// <summary>
	/// Protected constructor. Initializes to the given parameters without validation.
	/// </summary>
	/// <param name="Name">The name of this entity.</param>
	/// <param name="Sanitize">Dummy argument to force overload resolution to use this implementation.</param>
	public StagedFileSystemReference(string Name, Sanitize Sanitize)
	{
		this.Name = Name;
	}

	/// <summary>
	/// Checks whether this name has the given extension.
	/// </summary>
	/// <param name="Extension">The extension to check</param>
	/// <returns>True if this name has the given extension, false otherwise</returns>
	public bool HasExtension(string Extension)
	{
		if (Extension.Length > 0 && Extension[0] != '.')
		{
			return Name.Length >= Extension.Length + 1 && Name[Name.Length - Extension.Length - 1] == '.' && Name.EndsWith(Extension, FileSystemReference.Comparison);
		}
		else
		{
			return Name.EndsWith(Extension, FileSystemReference.Comparison);
		}
	}

	/// <summary>
	/// Checks if this path is equal to or under the given directory
	/// </summary>
	/// <param name="Directory">The directory to check against</param>
	/// <returns>True if the path is under the given directory</returns>
	public bool IsUnderDirectory(StagedDirectoryReference Directory)
	{
		return Name.StartsWith(Directory.Name, FileSystemReference.Comparison) && (Name.Length == Directory.Name.Length || Name[Directory.Name.Length] == '/');
	}

	/// <summary>
	/// Create a full path by concatenating multiple strings
	/// </summary>
	/// <returns></returns>
	static protected string CombineStrings(string BaseDirectory, params string[] Fragments)
	{
		StringBuilder NewName = new StringBuilder(BaseDirectory);
		foreach (string Fragment in Fragments)
		{
			if (NewName.Length > 0 && (NewName[NewName.Length - 1] != '/' && NewName[NewName.Length - 1] != '\\'))
			{
				NewName.Append('/');
			}
			NewName.Append(Fragment);
		}
		return NewName.ToString();
	}

	/// <summary>
	/// Returns the name of this entity
	/// </summary>
	/// <returns>Path to the file or directory</returns>
	public override string ToString()
	{
		return Name;
	}
}
