// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.IO;
using System.Linq;
using System.Text;
using System.Threading.Tasks;

namespace AutomationTool
{
	/// <summary>
	/// Extension methods for Stream classes
	/// </summary>
	public static class StreamUtils
	{
		/// <summary>
		/// Read a stream into another, buffering in 4K chunks.
		/// </summary>
		/// <param name="output">this</param>
		/// <param name="input">the Stream to read from</param>
		/// <returns>same stream for expression chaining.</returns>
		public static Stream ReadFrom(this Stream output, Stream input)
		{
			long bytesRead;
			return output.ReadFrom(input, out bytesRead);
		}

		/// <summary>
		/// Read a stream into another, buffering in 4K chunks.
		/// </summary>
		/// <param name="output">this</param>
		/// <param name="input">the Stream to read from</param>
		/// <param name="totalBytesRead">returns bytes read</param>
		/// <returns>same stream for expression chaining.</returns>
		public static Stream ReadFrom(this Stream output, Stream input, out long totalBytesRead)
		{
			totalBytesRead = 0;
			const int BytesToRead = 4096;
			byte[] buf = new byte[BytesToRead];
			int bytesReadThisTime = 0;
			do
			{
				bytesReadThisTime = input.Read(buf, 0, BytesToRead);
				totalBytesRead += bytesReadThisTime;
				output.Write(buf, 0, bytesReadThisTime);
			} while (bytesReadThisTime != 0);
			return output;
		}

		/// <summary>
		/// Read stream into a new MemoryStream. Useful for chaining together expressions.
		/// </summary>
		/// <param name="input">Stream to read from.</param>
		/// <returns>memory stream that contains the stream contents.</returns>
		public static MemoryStream ReadIntoMemoryStream(this Stream input)
		{
			MemoryStream data = new MemoryStream(4096);
			data.ReadFrom(input);
			return data;
		}

		/// <summary>
		/// Writes the entire contents of a byte array to the stream.
		/// </summary>
		/// <param name="stream"></param>
		/// <param name="arr"></param>
		/// <returns></returns>
		public static Stream Write(this Stream stream, byte[] arr)
		{
			stream.Write(arr, 0, arr.Length);
			return stream;
		}
	}
}
