// Copyright Epic Games, Inc. All Rights Reserved.
using System;
using System.Collections.Generic;
using System.Text;
using System.IO;
using System.Xml;
using System.Diagnostics;
using UnrealBuildTool;
using EpicGames.Core;
using System.Linq;
using System.Reflection;
using OpenTracing;
using OpenTracing.Util;
using UnrealBuildBase;

using CommandLine = UnrealBuildBase.CommandLine;
using Microsoft.Extensions.Logging;

using static AutomationTool.CommandUtils;

namespace AutomationTool
{
	class UnrealBuildException : AutomationException
	{
		public UnrealBuildException(string Message)
			: base(ExitCode.Error_UBTFailure, "BUILD FAILED: " + Message)
		{
			OutputFormat = AutomationExceptionOutputFormat.MinimalError;
		}

		public UnrealBuildException(string Format, params object[] Args)
			: this(String.Format(Format, Args))
		{
		}
	}

	[Help("ForceMonolithic", "Toggle to combined the result into one executable")]
	[Help("ForceDebugInfo", "Forces debug info even in development builds")]
	[Help("NoXGE", "Toggle to disable the distributed build process")]
	[Help("ForceNonUnity", "Toggle to disable the unity build system")]
	[Help("ForceUnity", "Toggle to force enable the unity build system")]
	[Help("Licensee", "If set, this build is being compiled by a licensee")]
	public class UnrealBuild
	{
		private BuildCommand OwnerCommand;

		/// <summary>
		/// If true we will let UBT build UHT
		/// </summary>
		[Obsolete]
		public bool AlwaysBuildUHT { get; set; }

		public void AddBuildProduct(string InFile)
		{
			string File = CommandUtils.CombinePaths(InFile);
			if (!CommandUtils.FileExists(File) && !CommandUtils.DirectoryExists(File))
			{
				throw new UnrealBuildException("Specified file to AddBuildProduct {0} does not exist.", File);
			}
			BuildProductFiles.Add(File);
		}

		static bool IsBuildReceipt(string FileName)
		{
			return FileName.EndsWith(".version", StringComparison.InvariantCultureIgnoreCase)
				|| FileName.EndsWith(".target", StringComparison.InvariantCultureIgnoreCase) 
				|| FileName.EndsWith(".modules", StringComparison.InvariantCultureIgnoreCase)
				|| FileName.EndsWith("buildid.txt", StringComparison.InvariantCultureIgnoreCase);
		}

		BuildManifest AddBuildProductsFromManifest(FileReference ManifestFile)
		{
			if (!FileReference.Exists(ManifestFile))
			{
				throw new UnrealBuildException("UBT Manifest {0} does not exist.", ManifestFile);
			}

			BuildManifest Manifest = CommandUtils.ReadManifest(ManifestFile);
			foreach (string Item in Manifest.BuildProducts)
			{
				if (!CommandUtils.FileExists_NoExceptions(Item) && !CommandUtils.DirectoryExists_NoExceptions(Item))
				{
					throw new UnrealBuildException($"AddBuildProductsFromManifest: {Item} was in manifest \"{ManifestFile}\" but could not be found.");
				}
				AddBuildProduct(Item);
			}
			return Manifest;
		}
		
		private void PrepareUBT()
		{
			if (!FileReference.Exists(UnrealBuildToolDll))
			{
				throw new UnrealBuildException($"UnrealBuildTool.dll does not exist at {UnrealBuildToolDll}");
			}

			if (!FileReference.Exists(Unreal.DotnetPath))
			{
				throw new UnrealBuildException($"dotnet executable does not exist at {Unreal.DotnetPath}");
			}
		}

		public void CleanWithUBT(string TargetName, UnrealTargetPlatform Platform, UnrealTargetConfiguration Config, FileReference UprojectPath, string InAddArgs = "")
		{
			string AddArgs = "";
			if (UprojectPath != null)
			{
				AddArgs += " " + CommandUtils.MakePathSafeToUseWithCommandLine(UprojectPath.FullName);
			}
			AddArgs += " -NoUBTMakefiles";
			AddArgs += " " + InAddArgs;

			PrepareUBT();
			using (IScope Scope = GlobalTracer.Instance.BuildSpan("Compile").StartActive())
			{
				Scope.Span.SetTag("target", TargetName);
				Scope.Span.SetTag("platform", Platform.ToString());
				Scope.Span.SetTag("config", Config.ToString());
				CommandUtils.RunUBT(CommandUtils.CmdEnv, UnrealBuildToolDll: UnrealBuildToolDll, Project: UprojectPath, Target: TargetName, Platform: Platform, Config: Config, AdditionalArgs: "-Clean -NoHotReload" + AddArgs);
			}
        }

		List<FileReference> GetManifestFiles(List<BuildTarget> Targets)
		{
			List<FileReference> ManifestFiles = new List<FileReference>();
			for (int Idx = 0; Idx < Targets.Count; Idx++)
			{
				BuildTarget Target = Targets[Idx];
				DirectoryReference ManifestDir = GetManifestDir(Target.UprojectPath);
				ManifestFiles.Add(FileReference.Combine(ManifestDir, $"Manifest-{Idx + 1}-{Target.TargetName}-{Target.Platform}-{Target.Config}.xml"));
			}

			return ManifestFiles;
		}

		FileReference GetManifestFile(BuildTarget Target, int Index)
		{
			DirectoryReference ManifestDir = GetManifestDir(Target.UprojectPath);
			if (Index == -1)
			{
				return FileReference.Combine(ManifestDir, "Manifest.xml");
			}
			else
			{
				return FileReference.Combine(ManifestDir, $"Manifest-{Index + 1}-{Target.TargetName}-{Target.Platform}-{Target.Config}.xml");
			}
		}

		void BuildWithUBT(List<BuildTarget> Targets, Dictionary<BuildTarget, BuildManifest> TargetToManifest, bool DisableXGE, bool AllCores, bool SkipBuild)
		{
			List<FileReference> ManifestFiles = TargetToManifest != null ? new List<FileReference>(Targets.Count) : null;

			StringBuilder FullCommandLine = new StringBuilder();
			if (Targets.Count == 1)
			{
				FileReference ManifestFile = null;
				if (TargetToManifest != null)
				{
					ManifestFiles.Add(GetManifestFile(Targets[0], -1));
					ManifestFile = ManifestFiles[0];
				}
				FullCommandLine.Append(GetTargetArguments(Targets[0], ManifestFile));
			}
			else
			{
				List<string> Arguments = new List<string>();
				for (int Idx = 0; Idx < Targets.Count; Idx++)
				{
					FileReference ManifestFile = null;
					if (TargetToManifest != null)
					{
						ManifestFiles.Add(GetManifestFile(Targets[Idx], Idx));
						ManifestFile = ManifestFiles[Idx];
					}
					Arguments.Add("-Target=" + GetTargetArguments(Targets[Idx], ManifestFile));
				}
				FullCommandLine.Append(CommandLine.FormatCommandLine(Arguments));
			}

			if (DisableXGE)
			{
				FullCommandLine.Append(" -NoXGE");
			}
			if (AllCores)
			{
				FullCommandLine.Append(" -AllCores");
			}
			if (SkipBuild)
			{
				FullCommandLine.Append(" -SkipBuild");
			}

			PrepareUBT();

			if (TargetToManifest != null)
			{
				for (int Idx = 0; Idx < ManifestFiles.Count; Idx++)
				{
					CommandUtils.DeleteFile(ManifestFiles[Idx]);
				}
			}

			CommandUtils.RunUBT(CommandUtils.CmdEnv, UnrealBuildToolDll, FullCommandLine.ToString());

			if (TargetToManifest != null)
			{
				for (int Idx = 0; Idx < Targets.Count; Idx++)
				{
					FileReference ManifestFile = ManifestFiles[Idx];
					BuildManifest Manifest = AddBuildProductsFromManifest(ManifestFile);
					CommandUtils.DeleteFile(ManifestFile);
					TargetToManifest?.Add(Targets[Idx], Manifest);
				}
			}
		}

		string GetTargetArguments(BuildTarget Target, FileReference ManifestFile)
		{
			List<string> Arguments = new List<string> { Target.TargetName, Target.Platform.ToString(), Target.Config.ToString() };
			if (Target.UprojectPath != null)
			{
				Arguments.Add($"-Project={Target.UprojectPath}");
			}
			if (ManifestFile != null)
			{
				Arguments.Add($"-Manifest={ManifestFile}");
			}

			string CmdLine = CommandLine.FormatCommandLine(Arguments);
			if (!String.IsNullOrEmpty(Target.UBTArgs))
			{
				CmdLine = $"{CmdLine} {Target.UBTArgs}";
			}

			return CmdLine;
		}

		string[] DotNetProductExtenstions()
		{
			return new string[] 
			{
				".dll",
				".pdb",
				".exe.config",
				".exe",
				"exe.mdb"
			};
		}

		string[] SwarmBuildProducts()
		{
			return new string[]
            {
                "AgentInterface",
                "SwarmAgent",
                "SwarmCoordinator",
                "SwarmCoordinatorInterface",
                "SwarmInterface",
				"SwarmCommonUtils"
            };
		}

		void AddBuildProductsForCSharpProj(string CsProj)
		{
			string BaseOutput = CommandUtils.CmdEnv.LocalRoot + @"/Engine/Binaries/DotNET/" + Path.GetFileNameWithoutExtension(CsProj);
			foreach (var Ext in DotNetProductExtenstions())
			{
				if (CommandUtils.FileExists(BaseOutput + Ext))
				{
					AddBuildProduct(BaseOutput + Ext);
				}
			}
		}

		void AddIOSBuildProductsForCSharpProj(string CsProj)
		{
			string BaseOutput = CommandUtils.CmdEnv.LocalRoot + @"/Engine/Binaries/DotNET/IOS/" + Path.GetFileNameWithoutExtension(CsProj);
			foreach (var Ext in DotNetProductExtenstions())
			{
				if (CommandUtils.FileExists(BaseOutput + Ext))
				{
					AddBuildProduct(BaseOutput + Ext);
				}
			}
		}

		void AddSwarmBuildProducts()
		{
			foreach (var SwarmProduct in SwarmBuildProducts())
			{
				string DotNETOutput = CommandUtils.CmdEnv.LocalRoot + @"/Engine/Binaries/DotNET/" + SwarmProduct;
				string Win64Output = CommandUtils.CmdEnv.LocalRoot + @"/Engine/Binaries/Win64/" + SwarmProduct;
				foreach (var Ext in DotNetProductExtenstions())
				{
					if (CommandUtils.FileExists(DotNETOutput + Ext))
					{
						AddBuildProduct(DotNETOutput + Ext);
					}
				}
				foreach (var Ext in DotNetProductExtenstions())
				{
					if (CommandUtils.FileExists(Win64Output + Ext))
					{
						AddBuildProduct(Win64Output + Ext);
					}
				}
			}
		}

		/// <summary>
		/// Updates the engine version files
		/// </summary>
		public List<FileReference> UpdateVersionFiles(bool ActuallyUpdateVersionFiles = true, int? ChangelistNumberOverride = null, int? CompatibleChangelistNumberOverride = null, string Build = null, string BuildURL = null, bool? IsPromotedOverride = null)
		{
			bool bIsLicenseeVersion = ParseParam("Licensee") || !FileReference.Exists(FileReference.Combine(Unreal.EngineDirectory, "Restricted", "NotForLicensees", "Build", "EpicInternal.txt"));
			bool bIsPromotedBuild = IsPromotedOverride.HasValue? IsPromotedOverride.Value : (ParseParamInt("Promoted", 1) != 0);
			bool bDoUpdateVersionFiles = CommandUtils.P4Enabled && ActuallyUpdateVersionFiles;		
			int ChangelistNumber = 0;
			if (bDoUpdateVersionFiles)
			{
				ChangelistNumber = ChangelistNumberOverride.HasValue? ChangelistNumberOverride.Value : CommandUtils.P4Env.Changelist;
			}
			int CompatibleChangelistNumber = 0;
			if(bDoUpdateVersionFiles && CompatibleChangelistNumberOverride.HasValue)
			{
				CompatibleChangelistNumber = CompatibleChangelistNumberOverride.Value;
			}

			string Branch = OwnerCommand.ParseParamValue("Branch");
			if (String.IsNullOrEmpty(Branch))
			{
				Branch = CommandUtils.P4Enabled ? CommandUtils.EscapePath(CommandUtils.P4Env.Branch) : "";
			}

			return StaticUpdateVersionFiles(ChangelistNumber, CompatibleChangelistNumber, Branch, Build, BuildURL, bIsLicenseeVersion, bIsPromotedBuild, bDoUpdateVersionFiles);
		}

		public static List<FileReference> StaticUpdateVersionFiles(int ChangelistNumber, int CompatibleChangelistNumber, string Branch, string Build, string BuildURL, bool bIsLicenseeVersion, bool bIsPromotedBuild, bool bDoUpdateVersionFiles)
		{
			FileReference BuildVersionFile = BuildVersion.GetDefaultFileName();

			// Get the revision to sync files to before 
			if(CommandUtils.P4Enabled && ChangelistNumber > 0 && !CommandUtils.IsBuildMachine)
			{
				CommandUtils.P4.Sync(String.Format("-f \"{0}@{1}\"", BuildVersionFile, ChangelistNumber), false, false);
			}

			BuildVersion Version;
			if(!BuildVersion.TryRead(BuildVersionFile, out Version))
			{
				Version = new BuildVersion();
			}

			List<FileReference> Result = new List<FileReference>(1);
			{
				if (bDoUpdateVersionFiles)
				{
					Logger.LogDebug("Updating {BuildVersionFile} with:", BuildVersionFile);
					Logger.LogDebug("  Changelist={ChangelistNumber}", ChangelistNumber);
					Logger.LogDebug("  CompatibleChangelist={CompatibleChangelistNumber}", CompatibleChangelistNumber);
					Logger.LogDebug("  IsLicenseeVersion={Arg0}", bIsLicenseeVersion? 1 : 0);
					Logger.LogDebug("  IsPromotedBuild={Arg0}", bIsPromotedBuild? 1 : 0);
					Logger.LogDebug("  BranchName={Branch}", Branch);
					Logger.LogDebug("  BuildVersion={Build}", Build);
					Logger.LogDebug("  BuildURL={BuildURL}", BuildURL);

					Version.Changelist = ChangelistNumber;
					if(CompatibleChangelistNumber > 0)
					{
						Version.CompatibleChangelist = CompatibleChangelistNumber;
					}
					else if(bIsLicenseeVersion != Version.IsLicenseeVersion)
					{
						Version.CompatibleChangelist = 0; // Clear out the compatible changelist number; it corresponds to a different P4 server.
					}
					Version.IsLicenseeVersion = bIsLicenseeVersion;
					Version.IsPromotedBuild = bIsPromotedBuild;
					Version.BranchName = Branch;
					Version.BuildVersionString = Build;
					Version.BuildURL = BuildURL;

					if (File.Exists(BuildVersionFile.FullName))
					{
						VersionFileUpdater.MakeFileWriteable(BuildVersionFile.FullName, true);
					}

					Version.Write(BuildVersionFile);
				}
				else
				{
					Logger.LogDebug("{BuildVersionFile} will not be updated because P4 is not enabled.", BuildVersionFile);
				}
				Result.Add(BuildVersionFile);
			}

			return Result;
		}

		[DebuggerDisplay("{TargetName} {Platform} {Config}")]
		public class BuildTarget
		{
			/// <summary>
			/// Name of the target
			/// </summary>
			public string TargetName;

			/// <summary>
			/// For code-based projects with a .uproject, the TargetName isn't enough for UBT to find the target, this will point UBT to the target
			/// </summary>
			public FileReference UprojectPath;

			/// <summary>
			/// Platform to build
			/// </summary>
			public UnrealTargetPlatform Platform;

			/// <summary>
			/// Configuration to build
			/// </summary>
			public UnrealTargetConfiguration Config;

			/// <summary>
			/// Extra UBT args
			/// </summary>
			public string UBTArgs;

			/// <summary>
			/// Whether to clean this target. If not specified, the target will be cleaned if -Clean is on the command line.
			/// </summary>
			public bool? Clean;

			/// <summary>
			/// Format as string
			/// </summary>
			/// <returns></returns>
			public override string ToString()
			{
				return string.Format("{0} {1} {2}", TargetName, Platform, Config);
			}
		}


		public class BuildAgenda
		{
			/// <summary>
			/// .NET .csproj files that will be compiled and included in the build.  Currently we assume the output
			/// binary file names match the solution file base name, but with various different binary file extensions.
			/// </summary>
			public List<string> DotNetProjects = new List<string>();

			public string SwarmAgentProject = "";
			public string SwarmCoordinatorProject = "";

			/// <summary>
			/// List of targets to build.  These can be various Unreal projects, programs or libraries in various configurations
			/// </summary>
			public List<BuildTarget> Targets = new List<BuildTarget>();

			/// <summary>
			/// Adds a target with the specified configuration.
			/// </summary>
			/// <param name="TargetName">Name of the target</param>
			/// <param name="InPlatform">Platform</param>
			/// <param name="InConfiguration">Configuration</param>
			/// <param name="InUprojectPath">Path to optional uproject file</param>
			/// <param name="InAddArgs">Specifies additional arguments for UBT</param>
			public void AddTarget(string TargetName, UnrealTargetPlatform InPlatform, UnrealTargetConfiguration InConfiguration, FileReference InUprojectPath = null, string InAddArgs = "")
			{
				// Is this platform a compilable target?
				if (!Platform.GetPlatform(InPlatform).CanBeCompiled())
				{
					return;
				}

				Targets.Add(new BuildTarget()
				{
					TargetName = TargetName,
					Platform = InPlatform,
					Config = InConfiguration,
					UprojectPath = InUprojectPath,
					UBTArgs = InAddArgs,
				});
			}

			/// <summary>
			/// Adds multiple targets with the specified configuration.
			/// </summary>
			/// <param name="TargetNames">List of targets.</param>
			/// <param name="InPlatform">Platform</param>
			/// <param name="InConfiguration">Configuration</param>
			/// <param name="InUprojectPath">Path to optional uproject file</param>
			/// <param name="InAddArgs">Specifies additional arguments for UBT</param>
			public void AddTargets(string[] TargetNames, UnrealTargetPlatform InPlatform, UnrealTargetConfiguration InConfiguration, FileReference InUprojectPath = null, string InAddArgs = "")
			{
				// Is this platform a compilable target?
				if (!Platform.GetPlatform(InPlatform).CanBeCompiled())
				{
					return;
				}

				foreach (var Target in TargetNames)
				{
					Targets.Add(new BuildTarget()
					{
						TargetName = Target,
						Platform = InPlatform,
						Config = InConfiguration,
						UprojectPath = InUprojectPath,
						UBTArgs = InAddArgs,
					});
				}
			}
		}


		public UnrealBuild(BuildCommand Command)
		{
			OwnerCommand = Command;
			BuildProductFiles.Clear();
		}

		private bool ParseParam(string Name)
		{
			return OwnerCommand != null && OwnerCommand.ParseParam(Name);
		}

		private string ParseParamValue(string Name)
		{
			return (OwnerCommand != null)? OwnerCommand.ParseParamValue(Name) : null;
		}

		private int ParseParamInt(string Name, int Default = 0)
		{
			return (OwnerCommand != null)? OwnerCommand.ParseParamInt(Name, Default) : Default;
		}

		/// <summary>
		/// Executes a build.
		/// </summary>
		/// <param name="InAgenda">Build agenda.</param>
		/// <param name="InDeleteBuildProducts">if specified, determines if the build products will be deleted before building. If not specified -clean parameter will be used,</param>
		/// <param name="InUpdateVersionFiles">True if the version files are to be updated </param>
		/// <param name="InForceNoXGE">If true will force XGE off</param>
		/// <param name="InAllCores">If true AND XGE not present or not being used then ensure UBT uses all available cores</param>
		/// <param name="InChangelistNumberOverride"></param>
		/// <param name="InTargetToManifest"></param>
		/// <param name="InSkipBuild"></param>
		public void Build(BuildAgenda InAgenda, bool? InDeleteBuildProducts = null, bool InUpdateVersionFiles = true, bool InForceNoXGE = false, bool InAllCores = false, int? InChangelistNumberOverride = null, Dictionary<BuildTarget, BuildManifest> InTargetToManifest = null, bool InSkipBuild = false)
		{
			if (!CommandUtils.CmdEnv.HasCapabilityToCompile)
			{
				throw new UnrealBuildException("You are attempting to compile on a machine that does not have a supported compiler!");
			}
			bool DeleteBuildProducts = InDeleteBuildProducts.HasValue ? InDeleteBuildProducts.Value : ParseParam("Clean");
			if (InUpdateVersionFiles)
			{
				UpdateVersionFiles(ActuallyUpdateVersionFiles: true, ChangelistNumberOverride: InChangelistNumberOverride);
			}

			//////////////////////////////////////

			// make a set of unique platforms involved
			List<UnrealTargetPlatform> UniquePlatforms = new List<UnrealTargetPlatform>();
			foreach (BuildTarget Target in InAgenda.Targets)
			{
				if (!UniquePlatforms.Contains(Target.Platform))
				{
					UniquePlatforms.Add(Target.Platform);
				}
			}

			if (InAgenda.SwarmAgentProject != "")
			{
				string SwarmAgentSolution = Path.Combine(CommandUtils.CmdEnv.LocalRoot, InAgenda.SwarmAgentProject);
				CommandUtils.BuildSolution(CommandUtils.CmdEnv, SwarmAgentSolution, "Development", "Mixed Platforms");
				AddSwarmBuildProducts();
			}

			if (InAgenda.SwarmCoordinatorProject != "")
			{
				string SwarmCoordinatorSolution = Path.Combine(CommandUtils.CmdEnv.LocalRoot, InAgenda.SwarmCoordinatorProject);
				CommandUtils.BuildSolution(CommandUtils.CmdEnv, SwarmCoordinatorSolution, "Development", "Mixed Platforms");
				AddSwarmBuildProducts();
			}
				
			foreach (var DotNetProject in InAgenda.DotNetProjects)
			{
				string CsProj = Path.Combine(CommandUtils.CmdEnv.LocalRoot, DotNetProject);
				CommandUtils.BuildCSharpProject(CommandUtils.CmdEnv, CsProj);
				AddBuildProductsForCSharpProj(CsProj);
			}

			string XGEConsole = null;
			bool bDisableXGE = ParseParam("NoXGE") || InForceNoXGE;
			bool bCanUseXGE = !bDisableXGE && PlatformExports.TryGetXgConsoleExecutable(out XGEConsole);

			Logger.LogDebug("************************* UnrealBuild:");
			Logger.LogDebug("************************* UseXGE: {bCanUseXGE}", bCanUseXGE);

			// Clean all the targets
			foreach (BuildTarget Target in InAgenda.Targets)
			{
				bool bClean = Target.Clean ?? DeleteBuildProducts;
				if (bClean)
				{
					CleanWithUBT(Target.TargetName, Target.Platform, Target.Config, Target.UprojectPath, Target.UBTArgs);
				}
			}

			if (InAgenda.Targets.Count == 0)
			{
				return;
			}

			// Build all the targets
			BuildWithUBT(InAgenda.Targets, InTargetToManifest, bDisableXGE, InAllCores, InSkipBuild);
		}

		/// <summary>
		/// Checks to make sure there was at least one build product, and that all files exist.  Also, logs them all out.
		/// </summary>
		/// <param name="BuildProductFiles">List of files</param>
		public static void CheckBuildProducts(HashSet<string> BuildProductFiles)
		{
			// Check build products
			{
				Logger.LogDebug("Build products *******");
				if (BuildProductFiles.Count < 1)
				{
					Logger.LogInformation("No build products were made");
				}
				else
				{
					foreach (var Product in BuildProductFiles)
					{
						if (!CommandUtils.FileExists(Product) && !CommandUtils.DirectoryExists(Product))
						{
							throw new UnrealBuildException("{0} was a build product but no longer exists", Product);
						}
						Logger.LogDebug("{Text}", Product);
					}
				}
				Logger.LogDebug("End Build products *******");
			}
		}


		/// <summary>
		/// Adds or edits existing files at head revision, expecting an exclusive lock, resolving by clobbering any existing version
		/// </summary>
		/// <param name="WorkingCL"></param>
		/// <param name="Files">List of files to check out</param>
		public static void AddBuildProductsToChangelist(int WorkingCL, IEnumerable<string> Files)
		{
			Logger.LogInformation("Adding {Arg0} build products to changelist {WorkingCL}...", Files.Count(), WorkingCL);
			foreach (var File in Files)
			{
				CommandUtils.P4.Sync("-f -k " + CommandUtils.MakePathSafeToUseWithCommandLine(File) + "#head"); // sync the file without overwriting local one
				if (!CommandUtils.FileExists(File))
				{
					throw new UnrealBuildException("{0} was a build product but no longer exists", File);
				}

				CommandUtils.P4.ReconcileNoDeletes(WorkingCL, CommandUtils.MakePathSafeToUseWithCommandLine(File));

				// Change file type on binary files to be always writeable.
				P4FileStat FileStats = CommandUtils.P4.FStat(File);

                if (CommandUtils.IsProbablyAMacOrIOSExe(File))
                {
                    if (FileStats.Type == P4FileType.Binary && (FileStats.Attributes & (P4FileAttributes.Executable | P4FileAttributes.Writeable)) != (P4FileAttributes.Executable | P4FileAttributes.Writeable))
                    {
                        CommandUtils.P4.ChangeFileType(File, (P4FileAttributes.Executable | P4FileAttributes.Writeable));
                    }
                }
                else
                {
					if (IsBuildProduct(File, FileStats) && (FileStats.Attributes & P4FileAttributes.Writeable) != P4FileAttributes.Writeable)
                    {
                        CommandUtils.P4.ChangeFileType(File, P4FileAttributes.Writeable);
                    }

                }                    
			}
		}

		/// <summary>
		/// Determines if this file is a build product.
		/// </summary>
		/// <param name="File">File path</param>
		/// <param name="FileStats">P4 file stats.</param>
		/// <returns>True if this is a Windows build product. False otherwise.</returns>
		private static bool IsBuildProduct(string File, P4FileStat FileStats)
		{
			if(FileStats.Type == P4FileType.Binary || IsBuildReceipt(File))
			{
				return true;
			}

			return FileStats.Type == P4FileType.Text && File.EndsWith(".exe.config", StringComparison.InvariantCultureIgnoreCase);
		}

		/// <summary>
		/// Add UBT files to build products
		/// </summary>
		public void AddUBTFilesToBuildProducts()
		{
			string UBTLocation = UnrealBuildToolDll.Directory.FullName;
			// copy all the files from the UBT output directory
			string[] UBTFiles = CommandUtils.FindFiles_NoExceptions("*.*", true, UBTLocation);

			foreach (string UBTFile in UBTFiles)
			{
				AddBuildProduct(UBTFile);
			}
		}

		/// <summary>
		/// Copy the UAT files to their precompiled location, and add them as build products
		/// </summary>
		public void AddUATFilesToBuildProducts()
		{
			// All scripts are expected to exist in DotNET/AutomationScripts subfolder.
			foreach (FileReference BuildProduct in ScriptManager.BuildProducts)
			{
				string UATScriptFilePath = BuildProduct.FullName;
				if (!CommandUtils.FileExists_NoExceptions(UATScriptFilePath))
				{
					throw new UnrealBuildException("Cannot add UAT to the build products because {0} does not exist.", UATScriptFilePath);
				}
				AddBuildProduct(UATScriptFilePath);
			}
		}

		DirectoryReference GetManifestDir(FileReference ProjectFile)
		{
			// Can't write to Engine directory on installed builds
			if (Unreal.IsEngineInstalled() && ProjectFile != null)
			{
				return DirectoryReference.Combine(ProjectFile.Directory, "Intermediate", "Build");
			}
			else
			{
				return DirectoryReference.Combine(Unreal.EngineDirectory, "Intermediate", "Build");
			}
			}

		FileReference GetManifestFile(FileReference ProjectFile)
		{
			return FileReference.Combine(GetManifestDir(ProjectFile), "Manifest.xml");
		}

		[Obsolete("Deprecated in UE5.1; use UnrealBuildToolDll")]
		public static string GetUBTExecutable()
		{
			return CommandUtils.CombinePaths(CommandUtils.CmdEnv.LocalRoot, @"Engine/Binaries/DotNET/UnrealBuildTool/UnrealBuildTool" + (OperatingSystem.IsWindows() ? ".exe" : ""));
		}

		[Obsolete("Deprecated in UE5.1; use UnrealBuildToolDll")]
		public string UBTExecutable
		{
			get
			{
				return GetUBTExecutable();							
			}
		}

		public static FileReference UnrealBuildToolDll => 
			new(CommandUtils.CombinePaths(CommandUtils.CmdEnv.LocalRoot, "Engine/Binaries/DotNET/UnrealBuildTool/UnrealBuildTool.dll"));

		// List of everything we built so far
		public readonly HashSet<string> BuildProductFiles = new HashSet<string>(StringComparer.InvariantCultureIgnoreCase);
	}
}
