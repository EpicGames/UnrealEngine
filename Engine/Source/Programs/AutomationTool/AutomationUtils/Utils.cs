// Copyright Epic Games, Inc. All Rights Reserved.
using System;
using System.Collections.Generic;
using System.Linq;
using System.Text;
using System.IO;
using System.Threading;
using System.Diagnostics;
using System.Reflection;
using UnrealBuildTool;
using System.Text.RegularExpressions;
using EpicGames.Core;
using UnrealBuildBase;
using Microsoft.Extensions.Logging;

using static AutomationTool.CommandUtils;
using System.Management;

namespace AutomationTool
{
	/// <summary>
	/// AutomationTool internal Utilities.
	/// </summary>
	public static class InternalUtils
	{
		static ILogger Logger => Log.Logger;

		/// <summary>
		/// Gets environment variable value.
		/// </summary>
		/// <param name="VarName">Variable name.</param>
		/// <param name="Default">Default value to be returned if the variable does not exist.</param>
		/// <param name="bQuiet"></param>
		/// <returns>Variable value or the default value if the variable did not exist.</returns>
		public static string GetEnvironmentVariable(string VarName, string Default, bool bQuiet = false)
		{
			var Value = Environment.GetEnvironmentVariable(VarName);
			if (Value == null)
			{
				Value = Default;
			}
			if (!bQuiet)
			{
				Logger.LogDebug("GetEnvironmentVariable {VarName}={Value}", VarName, Value);
			}
			return Value;
		}

		/// <summary>
		/// Creates a directory.
		/// @todo: this function should not exchange exception context for error codes that can be ignored.
		/// </summary>
		/// <param name="Path">Directory name.</param>
		/// <param name="bQuiet"></param>
		/// <returns>True if the directory was created, false otherwise.</returns>
		public static bool SafeCreateDirectory(string Path, bool bQuiet = false)
		{
			if(!bQuiet)
			{
				Logger.LogDebug("SafeCreateDirectory {Path}", Path);
			}

			const int MaxAttempts = 10;
			int Attempts = 0;
			bool Result = true;
			Exception LastException = null;
			do
			{
				Result = Directory.Exists(Path);
				if (!Result)
				{
					try
					{
						Result = Directory.CreateDirectory(Path).Exists;
					}
					catch (Exception Ex)
					{
						if (!Directory.Exists(Path))
						{
							Thread.Sleep(3000);
						}
						Result = Directory.Exists(Path);
						LastException = Ex;
					}
				}
			} while (Result == false && ++Attempts < MaxAttempts);

			if (Result == false && LastException != null)
			{
				if (bQuiet)
				{
					Logger.LogDebug("Failed to create directory {Path} in {MaxAttempts} attempts.", Path, MaxAttempts);
				}
				else
				{
					Logger.LogWarning("Failed to create directory {Path} in {MaxAttempts} attempts.", Path, MaxAttempts);
					Logger.LogWarning("{Text}", LogUtils.FormatException(LastException));
				}
			}
			return Result;
		}

		/// <summary>
		/// Deletes a file (will remove read-only flag if necessary).
		/// </summary>
		/// <param name="Path">Filename</param>
		/// <param name="bQuiet">if true, then do not print errors, also in quiet mode do not retry</param>
		/// <returns>True if the file does not exist, false otherwise.</returns>
		public static bool SafeDeleteFile(string Path, bool bQuiet = false)
		{
			if (!bQuiet)
			{
				Logger.LogDebug("SafeDeleteFile {Path}", Path);
			}
			int MaxAttempts = bQuiet ? 1 : 10;
			int Attempts = 0;
			bool Result = true;
			Exception LastException = null;

			do
			{
				Result = true;
				try
				{
					if (File.Exists(Path))
					{
						FileAttributes Attributes = File.GetAttributes(Path);
						if ((Attributes & FileAttributes.ReadOnly) != 0)
						{
							File.SetAttributes(Path, Attributes & ~FileAttributes.ReadOnly);
						}
						File.Delete(Path);
						FileItem.ResetCachedInfo(Path);
					}
				}
				catch (Exception Ex)
				{
					if (File.Exists(Path))
					{
						Result = false;
					}
					LastException = Ex;
				}
				if (Result == false && Attempts + 1 < MaxAttempts)
				{
					Thread.Sleep(1000);
				}
			} while (Result == false && ++Attempts < MaxAttempts);

			if (Result == false && LastException != null)
			{
				if (bQuiet)
				{
					Logger.LogDebug("Failed to delete file {Path} in {MaxAttempts} attempts.", Path, MaxAttempts);
				}
				else
				{
					Logger.LogWarning("Failed to delete file {Path} in {MaxAttempts} attempts.", Path, MaxAttempts);
					Logger.LogWarning("{Text}", LogUtils.FormatException(LastException));
				}
			}

			return Result;
		}

		/// <summary>
		/// Recursively deletes a directory and all its files and subdirectories.
		/// </summary>
		/// <param name="Path">Path to delete.</param>
		/// <param name="bQuiet"></param>
		/// <returns>Whether the deletion was succesfull.</returns>
		private static bool RecursivelyDeleteDirectory(string Path, bool bQuiet = false)
		{
			if (!bQuiet)
			{
				Logger.LogDebug("RecursivelyDeleteDirectory {Path}", Path);
			}
			// Delete all files. This will also delete read-only files.
			IEnumerable<string> FilesInDirectory = Directory.EnumerateFiles(Path);
			foreach (string Filename in FilesInDirectory)
			{
				if (SafeDeleteFile(Filename, bQuiet) == false)
				{
					return false;
				}
			}

			// Recursively delete all files from sub-directories.
			IEnumerable<string> FoldersInDirectory = Directory.EnumerateDirectories(Path);
			foreach (string Folder in FoldersInDirectory)
			{
				if (RecursivelyDeleteDirectory(Folder, bQuiet) == false)
				{
					return false;
				}
			}

			// At this point there should be no read-only files in any of the directories and
			// this directory should be empty too.
			return SafeDeleteEmptyDirectory(Path, bQuiet);
		}

		/// <summary>
		/// Deletes an empty directory.
		/// </summary>
		/// <param name="Path">Path to the Directory.</param>
		/// <param name="bQuiet"></param>
		/// <returns>True if deletion was successful, otherwise false.</returns>
		public static bool SafeDeleteEmptyDirectory(string Path, bool bQuiet = false)
		{
			if (!bQuiet)
			{
				Logger.LogDebug("SafeDeleteEmptyDirectory {Path}", Path);
			}
			const int MaxAttempts = 10;
			int Attempts = 0;
			bool Result = true;
			Exception LastException = null;
			do
			{
				Result = !Directory.Exists(Path);
				if (!Result)
				{
					try
					{
						Directory.Delete(Path, true);
						DirectoryItem.ResetCachedInfo(Path);
					}
					catch (Exception Ex)
					{
						if (Directory.Exists(Path))
						{
							Thread.Sleep(3000);
						}
						Result = !Directory.Exists(Path);
						LastException = Ex;
					}
				}
			} while (Result == false && ++Attempts < MaxAttempts);

			if (Result == false && LastException != null)
			{
				Logger.LogWarning("Failed to delete directory {Path} in {MaxAttempts} attempts.", Path, MaxAttempts);
				Logger.LogWarning("{Text}", LogUtils.FormatException(LastException));
			}

			return Result;
		}

		/// <summary>
		/// Deletes a directory and all its contents. Will delete read-only files.
		/// </summary>
		/// <param name="Path">Directory name.</param>
		/// <param name="bQuiet"></param>
		/// <returns>True if the directory no longer exists, false otherwise.</returns>
		public static bool SafeDeleteDirectory(string Path, bool bQuiet = false)
		{
			if (!bQuiet)
			{
				Logger.LogDebug("SafeDeleteDirectory {Path}", Path);
			}
			if (Directory.Exists(Path))
			{
				return RecursivelyDeleteDirectory(Path, bQuiet);
			}
			else
			{
				return true;
			}
		}


		/// <summary>
		/// Renames/moves a directory and all its contents
		/// </summary>
		/// <param name="OldName"></param>
		/// <param name="NewName"></param>
		/// <param name="bQuiet"></param>
		/// <param name="bRetry"></param>
		/// <param name="bThrow"></param>
		/// <returns>True if the directory was moved, false otehrwise</returns>
		public static bool SafeRenameDirectory(string OldName, string NewName, bool bQuiet = false, bool bRetry = true, bool bThrow = false)
		{
			if (!bQuiet)
			{
				Logger.LogDebug("SafeRenameDirectory {OldName} {NewName}", OldName, NewName);
			}
			const int MaxAttempts = 10;
			int Attempts = 0;

			bool Result = true;
			do
			{
				Result = true;
				try
				{
					Directory.Move(OldName, NewName);
				}
				catch (Exception Ex)
				{
					if (Directory.Exists(OldName) == true || Directory.Exists(NewName) == false)
					{
						if (!bQuiet)
						{
							Logger.LogWarning("Failed to rename {OldName} to {NewName}", OldName, NewName);
							Logger.LogWarning("{Text}", LogUtils.FormatException(Ex));
						}
						Result = false;
					}

					++Attempts;
					if (Attempts == MaxAttempts)
					{
						if (bThrow)
						{
							throw;
						}

						break;
					}
				}
			}
			while (Result == false && bRetry);

			return Result;
		}

		/// <summary>
		/// Renames/moves a file.
		/// </summary>
		/// <param name="OldName">Old name</param>
		/// <param name="NewName">New name</param>
		/// <param name="bQuiet"></param>
		/// <returns>True if the operation was successful, false otherwise.</returns>
		public static bool SafeRenameFile(string OldName, string NewName, bool bQuiet = false)
		{
			if( !bQuiet )
			{
				Logger.LogDebug("SafeRenameFile {OldName} {NewName}", OldName, NewName);
			}
			const int MaxAttempts = 10;
			int Attempts = 0;

			bool Result = true;
			do
			{
				Result = true;
				try
				{
					if (File.Exists(OldName))
					{
						FileAttributes Attributes = File.GetAttributes(OldName);
						if ((Attributes & FileAttributes.ReadOnly) != 0)
						{
							File.SetAttributes(OldName, Attributes & ~FileAttributes.ReadOnly);
						}
					}
					File.Move(OldName, NewName);
				}
				catch (Exception Ex)
				{
					if (File.Exists(OldName) == true || File.Exists(NewName) == false)
					{
						Logger.LogWarning("Failed to rename {OldName} to {NewName}", OldName, NewName);
						Logger.LogWarning("{Text}", LogUtils.FormatException(Ex));
						Result = false;
					}
				}
			}
			while (Result == false && ++Attempts < MaxAttempts);

			return Result;
		}

		// Characters that can appear at the start of
		private static char[] IgnoredIniValuePrefixes = { '+', '-', ' ', '\t' };

		private static void FilterIniFile(string SourceName, string TargetName, List<string> IniKeyDenyList, List<string> InSectionDenyList, List<string> InSectionAllowList)
		{
			Encoding encoding = FileUtils.GetEncoding(SourceName);
			string[] Lines = File.ReadAllLines(SourceName, encoding);
			StringBuilder NewLines = new StringBuilder("");
			bool bFilteringSection = false;

			foreach (string OriginalLine in Lines)
			{
				string Line = OriginalLine.Trim();

				if (Line.StartsWith(";"))
				{
					continue;
				}

				bool bFiltered = bFilteringSection;

				// look for each filter on each line
				if (!bFiltered && IniKeyDenyList != null)
				{
					string TrimmedLine = Line.TrimStart(IgnoredIniValuePrefixes);
					foreach (string Filter in IniKeyDenyList)
					{
						if (TrimmedLine.StartsWith(Filter + "="))
						{
							bFiltered = true;
							break;
						}
					}
				}

				if (InSectionDenyList != null || InSectionAllowList != null)
				{
					if (Line.StartsWith("[") && Line.EndsWith("]"))
					{
						string SectionName = Line.Substring(1, Line.Length - 2);
						bFilteringSection = bFiltered = ((InSectionDenyList != null && InSectionAllowList == null && InSectionDenyList.Contains(SectionName)) ||
						                                 (InSectionAllowList != null && !InSectionAllowList.Contains(SectionName)));

						if (bFilteringSection)
						{
							Logger.LogDebug("Filtering config section '{SectionName}'", SectionName);
						}
					}
				}

				// write out if it's not filtered out
				if (!bFiltered)
				{
					NewLines.AppendLine(Line);
				}
			}

			// now write out the final .ini file
			if (File.Exists(TargetName))
			{
				File.Delete(TargetName);
			}
			File.WriteAllText(TargetName, NewLines.ToString(), encoding);

			// other code assumes same timestamp for source and dest
			File.SetLastWriteTimeUtc(TargetName, File.GetLastWriteTimeUtc(SourceName));
		}

		/// <summary>
		/// Copies a file.
		/// </summary>
		/// <param name="SourceName">Source name</param>
		/// <param name="TargetName">Target name</param>
		/// <param name="bQuiet"></param>
		/// <param name="OverrideCopyHandler"></param>
		/// <param name="IniKeyDenyList"></param>
		/// <param name="IniSectionDenyList"></param>
		/// <param name="IniSectionAllowList"></param>
		/// <param name="bSafeCreateDirectory"></param>
		/// <returns>True if the operation was successful, false otherwise.</returns>
		public static bool SafeCopyFile(string SourceName, string TargetName, bool bQuiet = false, OverrideCopyDelegate OverrideCopyHandler = null, List<string> IniKeyDenyList = null, List<string> IniSectionDenyList = null, List<string> IniSectionAllowList = null, bool bSafeCreateDirectory = false)
		{
			if (!bQuiet)
			{
				Logger.LogDebug("SafeCopyFile {SourceName} {TargetName}", SourceName, TargetName);
			}

			if (bSafeCreateDirectory)
			{
				SafeCreateDirectory(Path.GetDirectoryName(TargetName), bQuiet);
			}

			const int MaxAttempts = 10;
			int Attempts = 0;

			bool Result = true;
			do
			{
				Result = true;
				bool Retry = true;
				try
				{
					bool bSkipSizeCheck = false;
					// BinaryConfig.ini is a special case with binary data, but with same extension to handle all ini 
					// file chunking/packaging/etc rules
					string FileName = Path.GetFileName(SourceName);
					bool bIsNotBinaryConfigIni = string.Equals(Path.GetExtension(SourceName), ".ini", StringComparison.OrdinalIgnoreCase) && Path.GetFileName(SourceName).IndexOf("BinaryConfig", StringComparison.OrdinalIgnoreCase) == -1;
					if ((IniKeyDenyList != null || IniSectionDenyList != null || IniSectionAllowList != null) && bIsNotBinaryConfigIni)
					{
						FilterIniFile(SourceName, TargetName, IniKeyDenyList, IniSectionDenyList, IniSectionAllowList);
						// ini files may change size, don't check
						bSkipSizeCheck = true;
					}
					else if (OverrideCopyHandler != null && OverrideCopyHandler.Invoke(Logger, SourceName, TargetName))
					{
						// handlers may change the file sizes, don't check
						bSkipSizeCheck = true;
					}
					else
					{
						if (File.Exists(SourceName))
						{
							File.Copy(SourceName, TargetName, overwrite: true);
							if (!OperatingSystem.IsWindows())
							{
								// Permissions may not be the same as the source file if the umask filters it, and/or if the file is overwritten
								File.SetUnixFileMode(TargetName, File.GetUnixFileMode(SourceName));
							}
						}
						else
						{
							Logger.LogInformation("Skip copying file {SourceName} because it doesn't exist.", SourceName);
						}
					}
					Retry = !File.Exists(TargetName);
					if (!Retry)
					{
						FileInfo SourceInfo = new FileInfo(SourceName);
						FileInfo TargetInfo = new FileInfo(TargetName);

						if (SourceInfo.Attributes.HasFlag(FileAttributes.ReparsePoint))
						{
							if (SourceInfo.ResolveLinkTarget(true) is FileInfo ResolvedInfo && ResolvedInfo != null)
							{
								SourceInfo = ResolvedInfo;
							}
						}

						if (!bSkipSizeCheck && SourceInfo.Length != TargetInfo.Length)
						{
							Logger.LogInformation("Size mismatch {SourceName} = {SourceLength} to {TargetName} = {TargetLength}", SourceName, SourceInfo.Length, TargetName, TargetInfo.Length);
							Retry = true;
						}
						// Timestamps should be no more than 2 seconds out - assuming this as exFAT filesystems store timestamps at 2 second intervals:
						// http://ntfs.com/exfat-time-stamp.htm
						if (!((SourceInfo.LastWriteTimeUtc - TargetInfo.LastWriteTimeUtc).TotalSeconds < 2 && (SourceInfo.LastWriteTimeUtc - TargetInfo.LastWriteTimeUtc).TotalSeconds > -2))
						{
							// Copy on some networks have lag for updating timestamps, so sleep and retry the check
							Thread.Sleep(2000);
							SourceInfo.Refresh();
							TargetInfo.Refresh();
							if (!((SourceInfo.LastWriteTimeUtc - TargetInfo.LastWriteTimeUtc).TotalSeconds < 2 && (SourceInfo.LastWriteTimeUtc - TargetInfo.LastWriteTimeUtc).TotalSeconds > -2))
							{
								Logger.LogInformation("Date mismatch {SourceName} = {SourceTime} to {TargetName} = {TargetTime}", SourceName, SourceInfo.LastWriteTimeUtc, TargetName, TargetInfo.LastWriteTimeUtc);
								Retry = true;
							}
						}
					}
				}
				catch (Exception Ex)
				{
					Logger.LogInformation("SafeCopyFile Exception was {Ex}", LogUtils.FormatException(Ex));
					Retry = true;
				}

				if (Retry)
				{
					if (Attempts + 1 < MaxAttempts)
					{
						Logger.LogInformation("Failed to copy {SourceName} to {TargetName}, deleting, waiting 10s and retrying.", SourceName, TargetName);
						if (File.Exists(TargetName))
						{
							SafeDeleteFile(TargetName);
						}
						Thread.Sleep(10000);
					}
					else
					{
						Logger.LogError("Failed to copy {SourceName} to {TargetName}", SourceName, TargetName);
					}
					Result = false;
				}
			}
			while (Result == false && ++Attempts < MaxAttempts);

			return Result;
		}


		/// <summary>
		/// Reads all lines from a file.
		/// </summary>
		/// <param name="Filename">Filename</param>
		/// <returns>An array containing all lines read from the file or null if the file could not be read.</returns>
		public static string[] SafeReadAllLines(string Filename)
		{
			Logger.LogDebug("SafeReadAllLines {Filename}", Filename);
			string[] Result = null;
			try
			{
				Result = File.ReadAllLines(Filename);
			}
			catch (Exception Ex)
			{
				Logger.LogWarning("Failed to load {Filename}", Filename);
				Logger.LogWarning(Ex, "{Text}", LogUtils.FormatException(Ex));
			}
			return Result;
		}

		/// <summary>
		/// Reads all text from a file.
		/// </summary>
		/// <param name="Filename">Filename</param>
		/// <returns>String containing all text read from the file or null if the file could not be read.</returns>
		public static string SafeReadAllText(string Filename)
		{
			Logger.LogDebug("SafeReadAllLines {Filename}", Filename);
			string Result = null;
			try
			{
				Result = File.ReadAllText(Filename);
			}
			catch (Exception Ex)
			{
				Logger.LogWarning("Failed to load {Filename}", Filename);
				Logger.LogWarning(Ex, "{Text}", LogUtils.FormatException(Ex));
			}
			return Result;
		}

		/// <summary>
		/// Finds files in the specified path.
		/// </summary>
		/// <param name="Path">Path</param>
		/// <param name="SearchPattern">Search pattern</param>
		/// <param name="Recursive">Whether to search recursively or not.</param>
		/// <param name="bQuiet"></param>
		/// <returns>List of all files found (can be empty) or null if the operation failed.</returns>
		public static string[] FindFiles(string Path, string SearchPattern, bool Recursive, bool bQuiet = false)
		{
			if (!bQuiet)
			{
				Logger.LogDebug("FindFiles {Path} {SearchPattern} {Recursive}", Path, SearchPattern, Recursive);
			}

			// On Linux, filter out symlinks since we (usually) create them to fix mispelled case-sensitive filenames in content, and if they aren't filtered, 
			// UAT picks up both the symlink and the original file and considers them duplicates when packaging (pak files are case-insensitive).
			// Windows needs the symlinks though because that's how deduplication works on Windows server, 
			// see https://answers.unrealengine.com/questions/212888/automated-buildjenkins-failing-due-to-symlink-chec.html
			// FIXME: ZFS, JFS and other fs that can be case-insensitive on Linux should use the faster path as well.
			if (BuildHostPlatform.Current.Platform != UnrealTargetPlatform.Linux)
			{
				return Directory.GetFiles(Path, SearchPattern, Recursive ? SearchOption.AllDirectories : SearchOption.TopDirectoryOnly);
			}
			else
			{
				List<string> FileNames = new List<string>();
				DirectoryInfo DirInfo = new DirectoryInfo(Path);
				foreach( FileInfo File in DirInfo.EnumerateFiles(SearchPattern, Recursive ? SearchOption.AllDirectories : SearchOption.TopDirectoryOnly))
				{
					if (File.Attributes.HasFlag(FileAttributes.ReparsePoint))
					{
						if (!bQuiet)
						{
							Logger.LogWarning("Ignoring symlink {Path}", File.FullName);
						}
						continue;
					}
					
					FileNames.Add(File.FullName);
				}
				
				return FileNames.ToArray();
			}
		}

		/// <summary>
		/// Finds directories in the specified path.
		/// </summary>
		/// <param name="Path">Path</param>
		/// <param name="SearchPattern">Search pattern</param>
		/// <param name="Recursive">Whether to search recursively or not.</param>
		/// <param name="bQuiet"></param>
		/// <returns>List of all directories found (can be empty) or null if the operation failed.</returns>
		public static string[] FindDirectories(string Path, string SearchPattern, bool Recursive, bool bQuiet = false)
		{
			if (!bQuiet)
			{
				Logger.LogDebug("FindDirectories {Path} {SearchPattern} {Recursive}", Path, SearchPattern, Recursive);
			}
			return Directory.GetDirectories(Path, SearchPattern, Recursive ? SearchOption.AllDirectories : SearchOption.TopDirectoryOnly);
		}

		/// <summary>
		/// Finds files in the specified path.
		/// </summary>
		/// <param name="Path">Path</param>
		/// <param name="SearchPattern">Search pattern</param>
		/// <param name="Recursive">Whether to search recursively or not.</param>
		/// <param name="bQuiet"></param>
		/// <returns>List of all files found (can be empty) or null if the operation failed.</returns>
		public static string[] SafeFindFiles(string Path, string SearchPattern, bool Recursive, bool bQuiet = false)
		{
			if (!bQuiet)
			{
				Logger.LogDebug("SafeFindFiles {Path} {SearchPattern} {Recursive}", Path, SearchPattern, Recursive);
			}
			string[] Files = null;
			try
			{
				Files = FindFiles(Path, SearchPattern, Recursive, bQuiet);
			}
			catch (Exception Ex)
			{
				Logger.LogWarning("Unable to Find Files in {Path}", Path);
				Logger.LogWarning(Ex, "{Text}", LogUtils.FormatException(Ex));
			}
			return Files;
		}

		/// <summary>
		/// Finds directories in the specified path.
		/// </summary>
		/// <param name="Path">Path</param>
		/// <param name="SearchPattern">Search pattern</param>
		/// <param name="Recursive">Whether to search recursively or not.</param>
		/// <param name="bQuiet"></param>
		/// <returns>List of all files found (can be empty) or null if the operation failed.</returns>
		public static string[] SafeFindDirectories(string Path, string SearchPattern, bool Recursive, bool bQuiet = false)
		{
			if (!bQuiet)
			{
				Logger.LogDebug("SafeFindDirectories {Path} {SearchPattern} {Recursive}", Path, SearchPattern, Recursive);
			}
			string[] Directories = null;
			try
			{
				Directories = FindDirectories(Path, SearchPattern, Recursive, bQuiet);
			}
			catch (Exception Ex)
			{
				Logger.LogWarning("Unable to Find Directories in {Path}", Path);
				Logger.LogWarning(Ex, "{Text}", LogUtils.FormatException(Ex));
			}
			return Directories;
		}

		/// <summary>
		/// Checks if a file exists.
		/// </summary>
		/// <param name="Path">Filename</param>
		/// <param name="bQuiet">if true, do not print a message</param>
		/// <returns>True if the file exists, false otherwise.</returns>
		public static bool SafeFileExists(string Path, bool bQuiet = false)
		{
			bool Result = false;
			try
			{
				Result = File.Exists(Path);
				if (!bQuiet)
				{
					Logger.LogDebug("SafeFileExists {Path}={Result}", Path, Result);
				}
			}
			catch (Exception Ex)
			{
				Logger.LogWarning("Unable to check if file {Path} exists.", Path);
				Logger.LogWarning(Ex, "{Text}", LogUtils.FormatException(Ex));
			}
			return Result;
		}


		/// <summary>
		/// Checks if a directory exists.
		/// </summary>
		/// <param name="Path">Directory</param>
		/// <param name="bQuiet">if true, no longging</param>
		/// <returns>True if the directory exists, false otherwise.</returns>
		public static bool SafeDirectoryExists(string Path, bool bQuiet = false)
		{
			bool Result = false;
			try
			{
				Result = Directory.Exists(Path);
				if (!bQuiet)
				{
					Logger.LogDebug("SafeDirectoryExists {Path}={Result}", Path, Result);
				}
			}
			catch (Exception Ex)
			{
				Logger.LogWarning("Unable to check if directory {Path} exists.", Path);
				Logger.LogWarning(Ex, "{Text}", LogUtils.FormatException(Ex));
			}
			return Result;
		}

		/// <summary>
		/// Writes lines to a file.
		/// </summary>
		/// <param name="Path">Filename</param>
		/// <param name="Text">Text</param>
		/// <returns>True if the operation was successful, false otherwise.</returns>
		public static bool SafeWriteAllLines(string Path, string[] Text)
		{
			Logger.LogDebug("SafeWriteAllLines {Path}", Path);
			bool Result = false;
			try
			{
				File.WriteAllLines(Path, Text);
				Result = true;
			}
			catch (Exception Ex)
			{
				Logger.LogWarning("Unable to write text to {Path}", Path);
				Logger.LogWarning(Ex, "{Text}", LogUtils.FormatException(Ex));
			}
			return Result;
		}

		/// <summary>
		/// Writes text to a file.
		/// </summary>
		/// <param name="Path">Filename</param>
		/// <param name="Text">Text</param>
		/// <returns>True if the operation was successful, false otherwise.</returns>
		public static bool SafeWriteAllText(string Path, string Text)
		{
			Logger.LogDebug("SafeWriteAllText {Path}", Path);
			bool Result = false;
			try
			{
				File.WriteAllText(Path, Text);
				Result = true;
			}
			catch (Exception Ex)
			{
				Logger.LogWarning("Unable to write text to {Path}", Path);
				Logger.LogWarning(Ex, "{Text}", LogUtils.FormatException(Ex));
			}
			return Result;
		}

		/// <summary>
		/// Writes text to a file.
		/// </summary>
		/// <param name="Path">Filename</param>
		/// <param name="Bytes"></param>
		/// <returns>True if the operation was successful, false otherwise.</returns>
		public static bool SafeWriteAllBytes(string Path, byte[] Bytes)
		{
			Logger.LogDebug("SafeWriteAllBytes {Path}", Path);
			bool Result = false;
			try
			{
				File.WriteAllBytes(Path, Bytes);
				Result = true;
			}
			catch (Exception Ex)
			{
				Logger.LogWarning("Unable to write text to {Path}", Path);
				Logger.LogWarning(Ex, "{Text}", LogUtils.FormatException(Ex));
			}
			return Result;
		}

	    public static void Robust_CopyFile(string InputFileName, string OutputFileName)
	    {
	        if (OutputFileName.StartsWith("/Volumes/"))
	        {
	            int Retry = 0;
	            int NumRetries = 60;
	            bool bCopied = false;
	            while (!bCopied && Retry < NumRetries)
	            {
	                if (Retry > 0)
	                {
                        //@todo: These retries should be reported so we can track how often they are occurring.
                        Logger.LogInformation("*** Mac temp storage retry {OutputFileName}", OutputFileName);
	                    System.Threading.Thread.Sleep(1000);
	                }
	                bCopied = CommandUtils.CopyFile_NoExceptions(InputFileName, OutputFileName, true);
	                Retry++;
	            }
	        }
	        else
	        {
	            CommandUtils.CopyFile(InputFileName, OutputFileName);
	        }
	    }

	    public static void Robust_FileExists(string Filename, string Message)
	    {
	        Robust_FileExists(false, Filename, Message);
	    }

	    public static void Robust_FileExists(bool bQuiet, string Filename, string Message)
	    {
	        if (!CommandUtils.FileExists_NoExceptions(bQuiet, Filename))
	        {
	            bool bFound = false;
	            // mac is terrible on shares, this isn't a solution, but a stop gap
	            if (Filename.StartsWith("/Volumes/"))
	            {
	                int Retry = 0;
	                while (!bFound && Retry < 60)
	                {
                        //@todo: These retries should be reported so we can track how often they are occurring.
                        Logger.LogInformation("*** Mac temp storage retry {Filename}", Filename);
	                    System.Threading.Thread.Sleep(10000);
	                    bFound = CommandUtils.FileExists_NoExceptions(bQuiet, Filename);
	                    Retry++;
	                }
	            }
	            if (!bFound)
	            {
	                throw new AutomationException(Message, Filename);
	            }
	        }
	    }

	    public static bool Robust_DirectoryExists_NoExceptions(string Directoryname, string Message)
	    {
	        bool bFound = false;
	        if (!CommandUtils.DirectoryExists_NoExceptions(Directoryname))
	        {				
	            // mac is terrible on shares, this isn't a solution, but a stop gap
	            if (Directoryname.StartsWith("/Volumes/"))
	            {
	                int Retry = 0;
	                while (!bFound && Retry < 60)
	                {
                        //@todo: These retries should be reported so we can track how often they are occurring.
                        Logger.LogInformation("*** Mac temp storage retry {Directoryname}", Directoryname);
	                    System.Threading.Thread.Sleep(10000);
	                    bFound = CommandUtils.DirectoryExists_NoExceptions(Directoryname);
	                    Retry++;
	                }
	            }				
	        }
	        else
	        {
	            bFound = true;
	        }
	        return bFound;
	    }

	    public static bool Robust_DirectoryExistsAndIsWritable_NoExceptions(string Directoryname)
	    {
	        bool bFound = false;
	        if (!CommandUtils.DirectoryExistsAndIsWritable_NoExceptions(Directoryname))
	        {
	            // mac is terrible on shares, this isn't a solution, but a stop gap
	            if (Directoryname.StartsWith("/Volumes/"))
	            {
	                int Retry = 0;
	                int NumRetries = 60;
	                if(!Directoryname.Contains("UE"))
	                {
	                    NumRetries = 2;
	                }
	                while (!bFound && Retry < NumRetries)
	                {
                        //@todo: These retries should be reported so we can track how often they are occurring.
                        Logger.LogInformation("*** Mac temp storage retry {Directoryname}", Directoryname);
	                    System.Threading.Thread.Sleep(1000);
	                    bFound = CommandUtils.DirectoryExistsAndIsWritable_NoExceptions(Directoryname);
	                    Retry++;
	                }
	            }
	        }
	        else
	        {
	            bFound = true;
	        }
	        return bFound;
	    }

		/// <summary>
		/// Attempts to get the shared network path for a given mapped network drive
		/// e.g.  X:\Shared\Path\   -> \\MyServer\Root\Shared\Path\
		/// </summary>
		/// <param name="InPath">Source path</param>
		/// <param name="UNCPath">Receives the UNC path</param>
		/// <returns>true on success</returns>
		public static bool TryResolveMappedNetworkPath( string InPath, out string UNCPath )
		{
			UNCPath = "";

			// only works on Windows at the moment
			if (HostPlatform.Current.HostEditorPlatform != UnrealTargetPlatform.Win64)
			{
				return false;
			}

			// path is already a network shared path
			if (InPath.StartsWith("\\\\"))
			{
				UNCPath = InPath;
				return true;
			}

			// path has no drive
			if (!Path.IsPathRooted(InPath))
			{
				return false;
			}

			// see if the path is on a network drive & try to resolve the target
			try
			{
				DriveInfo Drive = new(InPath);
				if (Drive.DriveType == DriveType.Network)
				{
					if (OperatingSystem.IsWindows())
					{
						using (ManagementObject ManObj = new($"Win32_LogicalDisk='{Drive.Name.TrimEnd(Path.DirectorySeparatorChar)}'")) // Win32_LogicalDisk='X:'
						{
							string UNCRoot = ManObj["ProviderName"].ToString(); // e.g. \\MyServer\Root
							string SharedPathFragment = InPath.Replace(Drive.Name, "", StringComparison.InvariantCultureIgnoreCase);

							UNCPath = Path.Combine(UNCRoot, SharedPathFragment);
							return true;
						}
					}
				}
			}
			catch
			{
			}

			// something failed, or the drive isn't a network drive
			return false;
		}


		/// <summary>
		/// Efficient iterator for walking over a string line by line.
		/// </summary>
		/// <param name="Str">string to walk over</param>
		/// <returns>enumerable of each line in the string.</returns>
		public static IEnumerable<string> EnumerateLines(this string Str)
		{
			if (Str == null)
			{
				yield break;
			}

			using (StringReader Reader = new StringReader(Str))
			{
				string line = Reader.ReadLine();
				while (line != null)
				{
					yield return line;
					line = Reader.ReadLine();
				}
			}
		}

		/// <summary>
		/// Indicates whether the Internet Protocol (IP) address is valid to appear in a Domain Name System (DNS) server database.
		///
		/// Addresses in the range 169.254.0.0 to 169.254.255.255 are not DNS eligible. These addresses are reserved for Automatic Private IP Addressing (APIPA).
		/// https://docs.microsoft.com/en-us/dotnet/api/system.net.networkinformation.ipaddressinformation.isdnseligible?view=netcore-3.1
		/// 
		/// NET Core 3.1 does not include a usable implementation of UnicastIPAddressInformation.IsDnsEligible() for Linux or Mac
		/// </summary>
		/// <param name="AddressInformation">Information about the address to evaluate for DNS eligibility</param>
		/// <returns></returns>
		public static bool IsDnsEligible(System.Net.NetworkInformation.IPAddressInformation AddressInformation)
		{
			byte[] AddressBytes = AddressInformation.Address.GetAddressBytes();
			return !(AddressBytes[0] == 169 && AddressBytes[1] == 254);
		}
	}


	/// <summary>
	/// This is to ensure that UAT can produce version strings precisely compatible
	/// with FEngineVersion.
	/// 
	/// WARNING: If FEngineVersion compatibility changes, this code needs to be updated.
	/// </summary>
	public class FEngineVersionSupport
    {
        /// <summary>
        /// The version info read from the Version header. The populated fields will be Major, Minor, and Build from the MAJOR, MINOR, and PATCH lines, respectively.
        /// Expects lines like:
        /// #define APP_MAJOR_VERSION 0
        /// #define APP_MINOR_VERSION 0
        /// #define APP_PATCH_VERSION 0
        /// </summary>
        public readonly Version Version;

        /// <summary>
        /// The changelist this version is associated with
        /// </summary>
        public readonly int Changelist;

        /// <summary>
        /// The Branch name associated with the version
        /// </summary>
        public readonly string BranchName;

        /// <summary>
        /// Reads a Version.h file, looking for macros that define the MAJOR/MINOR/PATCH version fields. Expected to match the Version.h in the engine.
        /// </summary>
        /// <param name="Filename">Version.h file to read.</param>
        /// <returns>Version that puts the Major/Minor/Patch fields in the Major/Minor/Build fields, respectively.</returns>
        public static Version ReadVersionFromFile(string Filename)
        {
			Regex regex = new Regex(@"#define.+_(?<Type>MAJOR|MINOR|PATCH)_VERSION\s+(?<Value>.+)", RegexOptions.ExplicitCapture | RegexOptions.IgnoreCase);
			Dictionary<string, int> foundElements = new Dictionary<string, int>(3);
            foreach (var line in File.ReadLines(Filename))
            {
                try
                {
					Match match = regex.Match(line);
                    if (match.Success)
                    {
                        foundElements.Add(match.Groups["Type"].Value, int.Parse(match.Groups["Value"].Value));
                    }
                }
                catch (Exception ex)
                {
                    throw new AutomationException(ex, "Failed to parse line {0} in version file {1}", line, Filename);
                }
            }

            // must find all three parts to accept the version file.
            if (foundElements.Keys.Intersect(new[] { "MAJOR", "MINOR", "PATCH" }).Count() != 3)
            {
                throw new AutomationException("Failed to find MAJOR, MINOR, and PATCH fields from version file {0}", Filename);
            }
			Logger.LogInformation("Read {Arg0}.{Arg1}.{Arg2} from {Filename}", foundElements["MAJOR"], foundElements["MINOR"], foundElements["PATCH"], Filename);
            return new Version(foundElements["MAJOR"], foundElements["MINOR"], foundElements["PATCH"]);
        }

        /// <summary>
        /// Ctor that takes a pre-determined Version. Gets the Changelist and BranchName from the current <see cref="CommandUtils.P4Env"/>.
        /// </summary>
		/// <param name="InVersion">Predetermined version.</param>
		/// <param name="InChangelist">Predetermined changelist (optional)</param>
		/// <param name="InBranchName">Predetermined branch name (optional)</param>
		public FEngineVersionSupport(Version InVersion, int InChangelist = -1, string InBranchName = null)
        {
			Version = InVersion;
			if (InChangelist <= 0)
			{
				Changelist = CommandUtils.P4Enabled ? CommandUtils.P4Env.Changelist : 0;
			}
			else
			{
				Changelist = InChangelist;
			}
			if (String.IsNullOrEmpty(InBranchName))
			{
				BranchName = CommandUtils.P4Enabled ? CommandUtils.EscapePath(CommandUtils.P4Env.Branch) : "UnknownBranch";
			}
			else
			{
				BranchName = InBranchName;
			}
        }

        /// <summary>
        /// Gets a version string compatible with FEngineVersion's native parsing code.
        /// </summary>
        /// <remarks>
        /// The format looks like: Major.Minor.Build-Changelist+BranchName.
        /// </remarks>
        /// <returns></returns>
        public override string ToString()
        {
			return String.Format("{0}.{1}.{2}-{3}+{4}",
				Version.Major,
				Version.Minor,
				Version.Build,
                Changelist.ToString("0000000"),
                BranchName);
        }

		public override bool Equals(object obj)
		{
			FEngineVersionSupport that = obj as FEngineVersionSupport;
			if (that == null)
				return false;
			return this.ToString() == that.ToString();
		}

		public override int GetHashCode()
		{
			return this.ToString().GetHashCode();
		}

        /// <summary>
        /// Ctor initializes with the values from the supplied Version file. The BranchName and CL are also taken from the current <see cref="CommandUtils.P4Env"/>.
        /// </summary>
        /// <param name="Filename">Full path to the file with the version info.</param>
		/// <param name="InChangelist">Predetermined changelist (optional)</param>
		/// <param name="InBranchName">Predetermined branch name (optional)</param>
        public static FEngineVersionSupport FromVersionFile(string Filename, int InChangelist = -1, string InBranchName = null)
        {
			return new FEngineVersionSupport(ReadVersionFromFile(Filename), InChangelist, InBranchName);
        }

		/// <summary>
		/// Creates a <see cref="FEngineVersionSupport"/> from a string that matches the format given in <see cref="ToString"/>.
		/// </summary>
		/// <param name="versionString">Version string that should match the FEngineVersion::ToString() format.</param>
		/// <param name="bAllowNoVersion"></param>
		/// <returns>a new instance with fields initialized to the match those given in the string.</returns>
		public static FEngineVersionSupport FromString(string versionString, bool bAllowNoVersion = false)
        {
            try
            {
				if (bAllowNoVersion && versionString.StartsWith("++"))
				{
					// This form of version is used when a product has no major.minor.patch version
					// E.g. ++depot+UE4-ProdName-CL-12345678
					var clSplit = versionString.Split(new string[] { "-CL-" }, 2, StringSplitOptions.None);
					var dashSplit = clSplit[1].Split(new[] { '-' }, 2);
					var changelist = int.Parse(dashSplit[0]);
					var branchName = clSplit[0];
					return new FEngineVersionSupport(new Version(0, 0, 0), changelist, branchName);
				}
				else
				{
					// This is the standard Launcher versioning scheme, e.g. "4.5.0-12345678+++depot+UE4"
					var dotSplit = versionString.Split(new[] { '.' }, 3);
					var dashSplit = dotSplit[2].Split(new[] { '-' }, 2);
					var plusSplit = dashSplit[1].Split(new[] { '+' }, 2);
					var major = int.Parse(dotSplit[0]);
					var minor = int.Parse(dotSplit[1]);
					var patch = int.Parse(dashSplit[0]);
					var changelist = int.Parse(plusSplit[0]);
					var branchName = plusSplit[1];
					return new FEngineVersionSupport(new Version(major, minor, patch), changelist, branchName);
				}
            }
            catch (Exception ex)
            {
                throw new AutomationException(ex, "Failed to parse {0} as an FEngineVersion compatible string", versionString);
            }
        }
    }

    /// <summary>
	/// VersionFileUpdater.
	/// </summary>
	public class VersionFileUpdater
	{
		/// <summary>
		/// Constructor
		/// </summary>
		public VersionFileUpdater(FileReference Filename)
		{
			MyFile = new FileInfo(Filename.FullName);
			OriginalLines = new List<string>(InternalUtils.SafeReadAllLines(Filename.FullName));
			Lines = new List<string>(OriginalLines);

            if (CommandUtils.IsNullOrEmpty(Lines))
            {
                throw new AutomationException("Version file {0} was empty or not found!", Filename);
            }
		}

		/// <summary>
		/// Finds the line containing the same tokens as the given prefix, and change the suffix to be the given string
		/// </summary>
		/// <param name="Prefix">Prefix to search for</param>
		/// <param name="Suffix">Suffix to replace the rest of the line with</param>
		/// <param name="OccurenceIdx">Index of the matching occurrence to replace</param>
		public void ReplaceLine(string Prefix, string Suffix, int OccurenceIdx = 0)
		{
			int MatchIdx = 0;
			for(int LineIdx = 0; LineIdx < Lines.Count; LineIdx++)
			{
				int LineOffset = 0;
				if(MatchCppTokens(Lines[LineIdx], ref LineOffset, Prefix))
				{
					if(OccurenceIdx == MatchIdx)
					{
						while(LineOffset < Lines[LineIdx].Length && Char.IsWhiteSpace(Lines[LineIdx][LineOffset]))
						{
							LineOffset++;
						}
						Lines[LineIdx] = Lines[LineIdx].Substring(0, LineOffset) + Suffix;
						return;
					}
					MatchIdx++;
				}
			}
            throw new AutomationException("Unable to find line {0} in {1}", Prefix, MyFile.FullName);
		}

		/// <summary>
		/// Tries to read a sequence of tokens from the given line of text, updating a position within it.
		/// </summary>
		/// <param name="Line">Line to read tokens from</param>
		/// <param name="LineOffset">Position within </param>
		/// <param name="Other">The tokens to attempt to match</param>
		/// <returns>True if the tokens are matched</returns>
		static bool MatchCppTokens(string Line, ref int LineOffset, string Other)
		{
			int OtherOffset = 0;
			for(;;)
			{
				// Read the next token from the prefix text. If we've reached the end, return that we've found a match.
				string OtherToken = ReadCppToken(Other, ref OtherOffset);
				if(OtherToken == null)
				{
					return true;
				}

				// Otherwise read the next token from the line and check it matches
				string LineToken = ReadCppToken(Line, ref LineOffset);
				if(LineToken != OtherToken)
				{
					return false;
				}
			}
		}

		/// <summary>
		/// Reads a single token (using a very rough approximation of a C++ token) from the given line
		/// </summary>
		/// <param name="Line">The line to read from</param>
		/// <param name="Offset">Position within the line to start reading. Initial whitespace will be skipped.</param>
		/// <returns>The token that was read, or null if the end of the string</returns>
		static string ReadCppToken(string Line, ref int Offset)
		{
			// Skip any leading whitespace
			while(Offset < Line.Length && Char.IsWhiteSpace(Line[Offset]))
			{
				Offset++;
			}
			if(Offset == Line.Length)
			{
				return null;
			}

			// Find the length of the token
			int StartOffset = Offset++;
			if(Char.IsLetterOrDigit(Line[StartOffset]))
			{
				while(Offset < Line.Length)
				{
					char Character = Line[Offset];
					if(!Char.IsLetterOrDigit(Character) && Character != '_')
					{
						break;
					}
					Offset++;
				}
			}
			return Line.Substring(StartOffset, Offset - StartOffset);
		}

		/// <summary>
		/// Doc
		/// </summary>
		public void Commit()
		{
			bool bDifferent = Lines.Count != OriginalLines.Count;

			if (!bDifferent)
			{
				for (int i = 0; i < Lines.Count; ++i)
				{
					if (Lines[i] != OriginalLines[i])
					{
						bDifferent = true;
						break;
					}
				}
			}

			if (bDifferent)
			{
				MakeFileWriteable(MyFile.FullName);
				if (!InternalUtils.SafeWriteAllLines(MyFile.FullName, Lines.ToArray()))
				{
					throw new AutomationException("Unable to update version info in {0}", MyFile.FullName);
				}
				OriginalLines = Lines;
			}
		}

		/// <summary>
		/// Makes a version file writeable. If P4 is enabled, syncs it to revision 0 to prevent conflicts with the P4 have table rather than checking it out, since we never intend to 
		/// submit it to Perforce. For existing writeable files, P4V prompts the user to overwrite the next time they sync.
		/// </summary>
		/// <param name="FileName">The file to make writeable</param>
		/// <param name="bForce">Whether to force the writeable flag if P4 cannot clear it</param>
		public static void MakeFileWriteable(string FileName, bool bForce = false)
		{
			if(CommandUtils.IsReadOnly(FileName))
			{
				if(CommandUtils.P4Enabled && !CommandUtils.IsBuildMachine)
				{
					CommandUtils.P4.Sync(String.Format("\"{0}#0\"", FileName), false, false);
				}
				
				if(CommandUtils.FileExists_NoExceptions(FileName) && CommandUtils.IsReadOnly(FileName))
				{
					if(bForce)
					{
						CommandUtils.SetFileAttributes(FileName, ReadOnly: false);
					}
					else
					{
						throw new AutomationException("Cannot write to {0}; file is read-only", FileName);	
					}
				}
			}
		}

		/// <summary>
		/// Doc
		/// </summary>
		protected FileInfo MyFile;
		/// <summary>
		/// Doc
		/// </summary>
		protected List<string> Lines;
		/// <summary>
		/// Doc
		/// </summary>
		protected List<string> OriginalLines;
	}
}
