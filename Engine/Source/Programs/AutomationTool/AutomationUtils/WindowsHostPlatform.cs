// Copyright Epic Games, Inc. All Rights Reserved.

using EpicGames.Core;
using Microsoft.Extensions.Logging;
using System;
using System.Diagnostics;
using System.IO;
using System.Runtime.Versioning;
using UnrealBuildTool;

namespace AutomationTool
{
	class WindowsHostPlatform : HostPlatform
	{
		static string CachedFrameworkMsbuildTool = "";

		[SupportedOSPlatform("windows")]
		public override string GetFrameworkMsbuildExe()
		{
			if (string.IsNullOrEmpty(CachedFrameworkMsbuildTool))
			{
				try
				{
					// Look for visual studio msbuild
					FileReference msbuild = new(WindowsExports.GetMSBuildToolPath());
					if (msbuild != null && FileReference.Exists(msbuild))
					{
						CachedFrameworkMsbuildTool = msbuild.FullName;
						Logger.LogInformation("Using {MsBuild}", CachedFrameworkMsbuildTool);
						return CachedFrameworkMsbuildTool;
					}
				}
				catch (BuildException)
				{
				}

				FileReference dotnet = FileReference.FromString(CommandUtils.WhichApp("dotnet"));
				if (dotnet != null && FileReference.Exists(dotnet))
				{
					Logger.LogInformation("Using {DotNet}", dotnet.FullName);
					CachedFrameworkMsbuildTool = "dotnet msbuild";
				}
				else
				{
					throw new BuildException("Unable to find installation of MSBuild.");
				}
			}

			return CachedFrameworkMsbuildTool;
		}

		public override string RelativeBinariesFolder
		{
			get { return @"Engine/Binaries/Win64/"; }
		}

		public override string GetUnrealExePath(string UnrealExe)
		{
			if(Path.IsPathRooted(UnrealExe))
			{
				return CommandUtils.CombinePaths(UnrealExe);
			}
			else
			{
				return CommandUtils.CombinePaths(CommandUtils.CmdEnv.LocalRoot, RelativeBinariesFolder, UnrealExe);
			}
		}

		public override string LocalBuildsLogFolder
		{
			get { return CommandUtils.CombinePaths(PathSeparator.Slash, Environment.GetFolderPath(Environment.SpecialFolder.ApplicationData), "Unreal Engine", "AutomationTool", "Logs"); }
		}

		public override string P4Exe
		{
			get { return "p4.exe"; }
		}

		public override Process CreateProcess(string AppName)
		{
			Process NewProcess = new Process();
			return NewProcess;
		}

		public override void SetupOptionsForRun(ref string AppName, ref CommandUtils.ERunOptions Options, ref string CommandLine)
		{
		}

		public override void SetConsoleCtrlHandler(ProcessManager.CtrlHandlerDelegate Handler)
		{
			ProcessManager.SetConsoleCtrlHandler(Handler, true);
		}

		public override UnrealTargetPlatform HostEditorPlatform
		{
			get { return UnrealTargetPlatform.Win64; }
		}

		public override string PdbExtension
		{
			get { return ".pdb"; }
		}

		static string[] SystemServices = new string[]
		{
			"winlogon",
			"system idle process",
			"taskmgr",
			"spoolsv",
			"csrss",
			"smss",
			"svchost",
			"services",
			"lsass",
            "conhost",
            "oobe",
            "mmc"
		};
		public override string[] DontKillProcessList
		{
			get 
			{
				return SystemServices;
			}
		}
	}
}
