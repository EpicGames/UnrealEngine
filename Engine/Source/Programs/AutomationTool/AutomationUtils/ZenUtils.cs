// Copyright Epic Games, Inc. All Rights Reserved.

using AutomationTool;
using EpicGames.Core;
using EpicGames.Horde.Jobs;
using EpicGames.ProjectStore;
using EpicGames.Serialization;
using IdentityModel.OidcClient;
using Microsoft.Extensions.Logging;
using System;
using System.Collections.Concurrent;
using System.Collections.Generic;
using System.Diagnostics;
using System.IO;
using System.Linq;
using System.Net.Http;
using System.Net.Sockets;
using System.Net.WebSockets;
using System.Runtime.InteropServices;
using System.Text;
using System.Text.Json;
using System.Text.Json.Nodes;
using System.Threading.Tasks;
using UnrealBuildBase;
using UnrealBuildTool;

namespace AutomationUtils
{
	public class ZenWorkspace
	{
		public string id { get; set; }
		public string root_path { get; set; }
		public bool allow_share_creation_from_http { get; set; }
		public List<ZenWorkspaceShare> shares { get; set; }
	}

	public class ZenWorkspaceShare
	{
		public string id { get; set; }
		public string share_path { get; set; }
	}

	public class ZenRunContext
	{
		public string Executable { get; set; }
		public string CommandLineArguments { get; set; }
		public string DataPath { get; set; }

		public string HostName { get; private set; } = string.Empty;
		public int HostPort { get; private set; } = 0;
		/// <summary>
		/// Return the "HostName:HostPort" string that should be used as the "servername" when creating http URLs, e.g.
		/// $"http:///{HostName}/prj/{Project}/oplog/{Oplog}"
		/// When using unixsockets we use a valid debugname string rather than than the "servername" used to open the
		/// socket for the HttpClient, which is instead a path to the .sock file on disk.
		/// @see GetRobustHostNameAndPortStrings.
		/// </summary>
		public string GetHttpHostNameAndPortString()
		{
			return UsingUnixSocket ? "unixsocket" : $"{HostName}:{HostPort}";
		}
		public bool IsValid { get; private set; } = false;
		public bool UsingUnixSocket { get; private set; } = false;

		public static ILogger Logger => Log.Logger;
		public static int DefaultPort { get { return 8558; } }

		public void InitializeFromHostNameAndPort(string SocketHostNameAndPort)
		{
			SplitHostNameAndPort(SocketHostNameAndPort, out string LocalSocketHostName, out int LocalHostPort);
			InitializeHostNameInternal(LocalSocketHostName, LocalHostPort);
		}

		public void InitializeFromHostNameAndPort(string InSocketHostName, int InHostPort)
		{
			InitializeHostNameInternal(InSocketHostName, InHostPort);
		}

		public void InitializeFromHostName(string InSocketHostName)
		{
			InitializeHostNameInternal(InSocketHostName, DefaultPort);
		}

		public static ZenRunContext ReadFromContextFile(FileReference ContextFile)
		{
			if (!FileReference.Exists(ContextFile))
			{
				return null;
			}

			string ContextData = FileReference.ReadAllText(ContextFile);
			JsonSerializerOptions JsonOptions = ZenUtils.GetDefaultJsonSerializerOptions();
			ZenRunContext Ret = JsonSerializer.Deserialize<ZenRunContext>(ContextData, JsonOptions);
			if (Ret != null)
			{
				Ret.InitializeInstanceData();
			}
			return Ret;
		}

		public static ZenRunContext DiscoverServerContext()
		{
			FileReference ContextFile = new (ZenUtils.GetZenRunContextFile());
			if (!FileReference.Exists(ContextFile))
			{
				return null;
			}

			return ReadFromContextFile(ContextFile);
		}

		private void InitializeInstanceData()
		{
			string LockFilePath = Path.Combine(DataPath, ".lock");
			FileReference LockFile = new(LockFilePath);
			if (!FileReference.Exists(LockFile))
			{
				return;
			}

			byte[] CbObjectData;
			try
			{
				using(FileStream Stream = File.Open(LockFilePath, FileMode.Open, FileAccess.Read, FileShare.ReadWrite | FileShare.Delete))
				{
					CbObjectData = new byte[Stream.Length];
					Stream.ReadExactly(CbObjectData);
				}
			}
			catch(IOException)
			{
				return;
			}

			if (CbObjectData == null)
			{
				return;
			}

			CbObject LockObject = new CbObject(CbObjectData);
			int PID = LockObject["pid"].AsInt32();
			int ServerPort = LockObject["port"].AsInt32();

			bool ServerIsValid = PID > 0 && (ServerPort > 0 && ServerPort <= 0xffff);
			if (ServerIsValid)
			{
				string ServerHostName = "localhost"; // for now only support localhost
				InitializeHostNameInternal(ServerHostName, ServerPort);
			}
			else
			{
				HostName = string.Empty;
				HostPort = 0;
				IsValid = false;
			}
		}

		private void InitializeHostNameInternal(string InHostName, int InHostPort)
		{
			HostName = InHostName;
			HostPort = InHostPort;
			IsValid = true;
			UsingUnixSocket = IsUnixSocketPath(HostName);
			if (UsingUnixSocket)
			{
				if (HostPort > 0 && HostPort != DefaultPort)
				{
					Logger.LogWarning($"HostPort={HostPort} was unexpectedly provided when using a UnixSocket. UnixSockets communicate with a file and do not need a port. Ignoring the port.");
				}
				HostPort = 0;
			}
		}

		/// <summary>
		/// Return an HttpClient that can be used to communicate with the ZenServer. If a UnixSocket is being used,
		/// the SocketsHttpHandler of the HttpClient if set to read from the socket file.
		/// </summary>
		/// <returns>A new HttpClient, with SocketsHttpHandler set if necessary.</returns>
		private HttpClient CreateHttpClientInternal()
		{
			if (!UsingUnixSocket)
			{
				return new HttpClient();
			}
			return CreateUnixSocketHttpClient(HostName);
		}

		/// <summary>
		/// Return an HttpClient that communicates with a ZenServer at the given HostName. If the hostname specifies
		/// a UnixSocket, the SocketsHttpHandler of the HttpClient if set to read from the socket file.
		/// </summary>
		/// <returns>A new HttpClient, with SocketsHttpHandler set.</returns>
		public static HttpClient CreateHttpClient(string InSocketHostName)
		{
			if (!IsUnixSocketPath(InSocketHostName))
			{
				return new HttpClient();
			}
			return CreateUnixSocketHttpClient(InSocketHostName);
		}

		/// <summary>
		/// Return an HttpClient that communicates with a ZenServer through a UnixSocketPath. Sets the SocketsHttpHandler
		/// of the HttpClient to read from the socket file.
		/// </summary>
		/// <returns>A new HttpClient, with SocketsHttpHandler set.</returns>
		public static HttpClient CreateUnixSocketHttpClient(string UnixSocketPath)
		{
			SocketsHttpHandler SocketsHandler = new SocketsHttpHandler
			{
				ConnectCallback = async (Context, Token) => {
					Socket Socket = new Socket(AddressFamily.Unix, SocketType.Stream, ProtocolType.IP);
					UnixDomainSocketEndPoint EndPoint = new UnixDomainSocketEndPoint(UnixSocketPath);
					await Socket.ConnectAsync(EndPoint);
					return new NetworkStream(Socket, ownsSocket: true);
				}
			};
			return new HttpClient(SocketsHandler);
		}

		/// <summary>
		/// Split a string "Hostname:HostPort" into the two parts. Silently handle edge cases such as a missing
		/// HostPort and colons in the Hostname. Leading and trailing spaces are trimmed, but spaces before or
		/// after the colon are not trimmed and are treated as part of the hostname.
		/// Valid strings:
		/// "hostname", "1.2.3.4", "[::1]"
		/// "hostname:1234", "1.2.3.4:1234", "[::1]:1234"
		/// </summary>
		public static void SplitHostNameAndPort(string HostNameAndPort, out string OutHostName, out int OutHostPort)
		{
			HostNameAndPort = HostNameAndPort.Trim();
			string[] Parts = HostNameAndPort.Split(':');

			if (Parts.Length >= 2)
			{
				int PortIndex = Parts.Length - 1;
				if (!int.TryParse(Parts[PortIndex], out OutHostPort))
				{
					OutHostName = HostNameAndPort;
					OutHostPort = DefaultPort;
				}
				else
				{
					OutHostName = string.Join(':', Parts.Take(PortIndex));
				}
			}
			else
			{
				OutHostName = HostNameAndPort;
				OutHostPort = DefaultPort;
			}
		}

		/// <summary>
		/// Returns the hostname and port connection information as a single string, e.g. "localhost:8558".
		/// When not using unixsockets, OutSocketHostName and OutHttpHostName are identical, but when using unixsockets
		/// they can be different.
		/// 
		/// When using unix sockets, OutSocketHostNameAndPort is the unix filepath of the .sock file and is used in
		/// an HttpClient's HttpMessageHandler. OutHttpHostNameAndPort is used in the URI passed to HttpRequestMessage.
		/// It is used only for display to the user because the HttpMessageHandler bypasses it, but it needs to be a
		/// valid hostname string and can have no path separators. When the hostname specifies a unix socket we
		/// therefore return e.g. ("/usr/path/zenserver.sock", "unixsocket").
		/// </summary>
		public static void GetRobustHostNameAndPortStrings(string InHostName, int InHostPort,
			out string SocketHostNameAndPort, out string HttpHostNameAndPort)
		{
			if (IsUnixSocketPath(InHostName))
			{
				if (InHostPort > 0 && InHostPort != DefaultPort)
				{
					Logger.LogWarning($"HostPort={InHostPort} was unexpectedly provided when using a UnixSocket HostName=\"{InHostName}\". " +
						"UnixSockets communicate with a file and do not need a port. Ignoring the port.");
				}
				SocketHostNameAndPort = InHostName;
				HttpHostNameAndPort = "unixsocket";
			}
			else
			{
				SocketHostNameAndPort = InHostName;
				if (InHostPort > 0)
				{
					SocketHostNameAndPort += ":" + InHostPort;
				}
				HttpHostNameAndPort = SocketHostNameAndPort;
			}
		}

		/// <summary>
		/// Returns whether a given HostName (string must not include a :HostPort suffix) is a unix socket path,
		/// which requires communicating through the socket file rather than an http uri.
		/// </summary>
		public static bool IsUnixSocketPath(string InHostName)
		{
			return InHostName.EndsWith(".sock");
		}

		/// <summary>
		/// Queries the server's health status to check if the the service is running
		/// </summary>
		/// <returns></returns>
		public bool IsServiceRunning()
		{
			if (!IsValid)
			{
				return false;
			}

			string Uri = string.Format("http://{0}/health/ready", GetHttpHostNameAndPortString());
			using HttpRequestMessage Request = new HttpRequestMessage(HttpMethod.Get, Uri);

			HttpResponseMessage HttpGetResult = null;
			HttpClient Client = CreateHttpClientInternal();

			try
			{
				HttpGetResult = Client.Send(Request);
			}
			catch
			{
				return false;
			}

			if (HttpGetResult.IsSuccessStatusCode)
			{
				return true;
			}

			return false;
		}

		/// <summary>
		/// Checks if the server was run with <c>--workspaces-enabled</c>
		/// </summary>
		/// <returns></returns>
		public bool DoesServerSupportWorkspaces()
		{
			if (CommandLineArguments.Contains("--workspaces-enabled"))
			{
				return true;
			}

			return false;
		}

		/// <summary>
		/// If there doesn't exists a workspace pointing to BaseDir (or above in the subtree) will create a new zen workspace pointing to BaseDir.
		/// If a workspace already exists it won't change its Dynamic setting.
		/// </summary>
		/// <param name="BaseDir"></param>
		/// <param name="Dynamic">If the newly created workspace should allow connecting clients to create workspace shares on demand</param>
		/// <returns>ZenWorkspace if the workspace was created or one already exists. null on failure</returns>
		public ZenWorkspace GetOrCreateWorkspace(DirectoryReference BaseDir, bool Dynamic = false)
		{
			if (!DoesServerSupportWorkspaces())
			{
				return null;
			}

			if (!DirectoryReference.Exists(BaseDir))
			{
				Logger.LogError("Can't create zen workspace in {0}. Directory doesn't exist", BaseDir.FullName);
			}

			ZenWorkspace Workspace = FindWorkspaceForDir(BaseDir);
			if (Workspace != null)
			{
				return Workspace;
			}

			string Command = String.Format("workspace create {0}", BaseDir.FullName);
			if (Dynamic)
			{
				Command += " --allow-share-create-from-http";
			}

			if (!ZenUtils.RunZenExe(Command))
			{
				return null;
			}

			Workspace = FindWorkspaceForDir(BaseDir);
			return Workspace;
		}

		/// <summary>
		/// If there doesn't exists a workspace share pointing to ShareDir, will create a new zen workspace share pointing to it.
		/// </summary>
		/// <param name="ShareDir"></param>
		/// <returns></returns>
		public ZenWorkspaceShare GetOrCreateShare(DirectoryReference ShareDir)
		{
			if (!DoesServerSupportWorkspaces())
			{
				return null;
			}

			if (!DirectoryReference.Exists(ShareDir))
			{
				Logger.LogError("Can't create zen share in {0}. Directory doesn't exist", ShareDir);
				return null;
			}

			ZenWorkspaceShare Result = FindShareForDir(ShareDir);
			if (Result != null)
			{
				return Result;
			}

			ZenWorkspace Workspace = FindWorkspaceForDir(ShareDir);
			if (Workspace == null)
			{
				Logger.LogError("Can't create zen share in {0}. No zen workspace exists for this directory", ShareDir.FullName);
				return null;
			}

			DirectoryReference WorkspaceRef = new(Workspace.root_path);
			// don't want to have a share pointing to a workspace
			if (WorkspaceRef == ShareDir)
			{
				Logger.LogError("Can't create zen share in {0}. There already exists a workspace pointing to this directory.", ShareDir.FullName);
				return null;
			}

			string ShareRelPath = ShareDir.MakeRelativeTo(WorkspaceRef);
			string Command = String.Format("workspace-share create {0} \"{1}\"", Workspace.id, ShareRelPath);
			if (!ZenUtils.RunZenExe(Command))
			{
				return null;
			}

			Result = FindShareForDir(ShareDir);
			return Result;
		}

		/// <summary>
		/// Queries the server for a full list of existing workspaces
		/// </summary>
		/// <returns></returns>
		private List<ZenWorkspace> GetWorkspaceList()
		{
			if (!IsServiceRunning() || !DoesServerSupportWorkspaces())
			{
				return new List<ZenWorkspace>();
			}

			HttpResponseMessage HttpGetResult = null;
			HttpClient Client = CreateHttpClientInternal();
			string Uri = string.Format("http://{0}/ws", GetHttpHostNameAndPortString());
			using HttpRequestMessage Request = new HttpRequestMessage(HttpMethod.Get, Uri);
			Request.Headers.Add("Accept", "application/json");

			try
			{
				HttpGetResult = Client.Send(Request);
			}
			catch
			{
				return new List<ZenWorkspace>();
			}

			if (!HttpGetResult.IsSuccessStatusCode)
			{
				return new List<ZenWorkspace>();
			}

			Task<string> ResultContent = HttpGetResult.Content.ReadAsStringAsync();
			ResultContent.Wait();

			JsonSerializerOptions JsonOptions = new();
			JsonOptions.AllowTrailingCommas = true;
			JsonOptions.PropertyNameCaseInsensitive = true;
			JsonOptions.ReadCommentHandling = JsonCommentHandling.Skip;
			JsonNode WorkspaceJsonObject = JsonNode.Parse(ResultContent.Result);
			JsonArray WorkspaceArray = WorkspaceJsonObject["workspaces"]!.AsArray();
			List<ZenWorkspace> Result = JsonSerializer.Deserialize<List<ZenWorkspace>>(WorkspaceArray, JsonOptions);

			return Result;
		}

		/// <summary>
		/// Finds a zen workspace for the specified directory. Includes workspaces that are above in the directory tree
		/// </summary>
		/// <param name="BaseDir"></param>
		/// <returns>ZenWorkspace on success. null if there's no workspace for in this directory tree</returns>
		public ZenWorkspace FindWorkspaceForDir(DirectoryReference BaseDir)
		{
			List<ZenWorkspace> Workspaces = GetWorkspaceList();
			if (Workspaces == null)
			{
				return null;
			}

			foreach (ZenWorkspace Workspace in Workspaces)
			{
				if (BaseDir.IsUnderDirectory(new DirectoryReference(Workspace.root_path)))
				{
					return Workspace;
				}
			}

			return null;
		}

		/// <summary>
		/// Finds a zen workspace share for the specified directory.
		/// </summary>
		/// <param name="ShareDir"></param>
		/// <returns>ZenWorkspaceShare if the share exists. null either if there's no share or there exists no workspace in this dir subtree.</returns>
		public ZenWorkspaceShare FindShareForDir(DirectoryReference ShareDir)
		{
			ZenWorkspace Workspace = FindWorkspaceForDir(ShareDir);
			if (Workspace == null || Workspace.shares == null)
			{
				return null;
			}

			string RelShare = ShareDir.MakeRelativeTo(new DirectoryReference(Workspace.root_path));
			foreach (ZenWorkspaceShare ZenShare in Workspace.shares)
			{
				if (ZenShare.share_path == RelShare)
				{
					return ZenShare;
				}
			}

			return null;
		}

		/// <summary>
		/// Creates a zen workspace share pointing to ShareDir. If necessary it creates a zen workspace in the parent directory.
		/// ShareDir must not be a root directory, e.g. d:\
		/// </summary>
		/// <param name="ShareDir"></param>
		/// <returns>
		/// ZenWorkspace share on success.
		/// null if the directory doesn't exist or zen server failed to create the workspace
		/// </returns>
		public ZenWorkspaceShare CreateWorkspaceAndShare(DirectoryReference ShareDir)
		{
			if (!DoesServerSupportWorkspaces())
			{
				return null;
			}

			if (!DirectoryReference.Exists(ShareDir))
			{
				Logger.LogError("Can't create workspace and share in {0}. Directory doesn't exist", ShareDir.FullName);
				return null;
			}

			if (ShareDir.IsRootDirectory())
			{
				Logger.LogError("Can't create share in {0}. Share can't be a root directory", ShareDir);
				return null;
			}

			ZenWorkspaceShare Share = FindShareForDir(ShareDir);
			if (Share != null)
			{
				return Share;
			}

			ZenWorkspace Workspace = GetOrCreateWorkspace(ShareDir.ParentDirectory);
			if (Workspace == null)
			{
				Logger.LogError("Can't create workspace in share's parent directory {0}", ShareDir.ParentDirectory);
				return null;
			}

			Share = GetOrCreateShare(ShareDir);
			if (Share == null)
			{
				Logger.LogError("Create zen share in {0} failed", ShareDir);
				return null;
			}

			return Share;
		}
	}

	public static class ZenUtils
	{
		private static string GetServerExecutableName()
		{
			string ExecutableName = "zenserver";

			if (RuntimeInformation.IsOSPlatform(OSPlatform.Windows))
			{
				ExecutableName += ".exe";
			}

			return ExecutableName;
		}

		public static string GetZenInstallPath()
		{
			const string EpicProductIdentifier = "UnrealEngine";
			string LocalAppData = Environment.GetFolderPath(Environment.SpecialFolder.LocalApplicationData);
			return Path.Combine(LocalAppData, EpicProductIdentifier, "Common", "Zen", "Install");
		}

		public static string GetZenRunContextFile()
		{
			return Path.Combine(GetZenInstallPath(), "zenserver.runcontext");
		}

		public static FileReference GetZenExeLocation(string ExecutableName)
		{
			string PathString = String.Format("{0}/Binaries/{1}/{2}{3}", Unreal.EngineDirectory, HostPlatform.Current.HostEditorPlatform.ToString(), ExecutableName,
				RuntimeInformation.IsOSPlatform(OSPlatform.Windows) ? ".exe" : "");

			if (Path.IsPathRooted(PathString))
			{
				return new FileReference(PathString);
			}
			else
			{
				return new FileReference(Path.Combine(CommandUtils.CmdEnv.LocalRoot, PathString));
			}
		}

		public static bool IsLocalZenServerRunning()
		{
			FileReference ContextFileRef = new(GetZenRunContextFile());
			if (!FileReference.Exists(ContextFileRef))
			{
				return false;
			}

			ZenRunContext Context = ZenRunContext.ReadFromContextFile(ContextFileRef);
			if (Context == null)
			{
				return false;
			}

			return Context.IsServiceRunning();
		}

		public static bool IsZenServerRunning(string SocketHostNameAndPort)
		{
			ZenRunContext ZenServerContext = new ZenRunContext();
			ZenServerContext.InitializeFromHostNameAndPort(SocketHostNameAndPort);
			return ZenServerContext.IsServiceRunning();
		}

		public static bool IsZenServerRunning(string SocketHostName, int HostPort)
		{
			ZenRunContext ZenServerContext = new ZenRunContext();
			ZenServerContext.InitializeFromHostNameAndPort(SocketHostName, HostPort);
			return ZenServerContext.IsServiceRunning();
		}

		public static HttpClient CreateHttpClient(string SocketHostNameAndPort)
		{
			ZenRunContext.SplitHostNameAndPort(SocketHostNameAndPort, out string HostName, out int HostPort);
			return ZenRunContext.CreateHttpClient(HostName);
		}

		public static JsonSerializerOptions GetDefaultJsonSerializerOptions()
		{
			JsonSerializerOptions Options = new();
			Options.AllowTrailingCommas = true;
			Options.PropertyNameCaseInsensitive = true;
			Options.ReadCommentHandling = JsonCommentHandling.Skip;
			return Options;
		}

		public static void ZenLaunch(FileReference SponsorFile)
		{
			FileReference ZenLaunchExe = GetZenExeLocation("ZenLaunch");

			string LaunchArguments = String.Format("{0} -SponsorProcessID={1}", CommandUtils.MakePathSafeToUseWithCommandLine(SponsorFile.FullName), Environment.ProcessId);

			CommandUtils.RunAndLog(CommandUtils.CmdEnv, ZenLaunchExe.FullName, LaunchArguments, Options: CommandUtils.ERunOptions.Default);
		}

		public static bool RunZenExe(string Command)
		{
			FileReference ZenExe = GetZenExeLocation("zen");
			try
			{
				CommandUtils.RunAndLog(ZenExe.FullName, Command);
			}
			catch
			{
				Log.Logger.LogError("{0} failed to run", ZenExe.FullName);
				return false;
			}

			return true;
		}

		public static bool ShouldSetupPakStreaming(ProjectParams Params, DeploymentContext SC)
		{
			if (CommandUtils.IsBuildMachine)
			{
				return false;
			}

			if (!Params.UsePak(SC.StageTargetPlatform))
			{
				return false;
			}

			ConfigHierarchy EngineConfig = ConfigCache.ReadHierarchy(ConfigHierarchyType.Game, DirectoryReference.FromFile(Params.RawProjectPath), BuildHostPlatform.Current.Platform, SC.CustomConfig);
			EngineConfig.GetBool("/Script/UnrealEd.ProjectPackagingSettings", "bUseZenStore", out bool UseZenStore);
			EngineConfig.GetBool("/Script/UnrealEd.ProjectPackagingSettings", "bEnablePakStreaming", out bool EnablePakStreaming);

			return UseZenStore && EnablePakStreaming;
		}
	}

	public static class ZenUATProcessUtil
	{
		private static readonly ConcurrentDictionary<ManagedProcess, ZenUatProcessStub> ProcessStubs = new ConcurrentDictionary<ManagedProcess, ZenUatProcessStub>();
		public static IProcess GetUATProcessStub(ManagedProcess process)
		{
			return ProcessStubs.GetOrAdd(process, p => new ZenUatProcessStub(p));
		}
	}

	public class ZenUatProcessStub(ManagedProcess process) : IProcess
	{
		public void StopProcess(bool killDescendants = true)
		{
			process.Kill(killDescendants: killDescendants);
		}

		public bool HasExited => process.HasExited;

		public string GetProcessName()
		{
			try
			{
				return Process.GetProcessById(process.Id).ProcessName;
			}
			catch (ArgumentException)
			{
				// process is not running anymore
				return null;
			}
		}
	}
}
