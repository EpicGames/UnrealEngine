// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.IO;
using System.Linq;
using System.Threading.Tasks;
using EpicGames.Core;

#nullable enable

namespace AutomationTool
{
	/// <summary>
	/// Exception class thrown due to type and syntax errors in condition expressions
	/// </summary>
	public class BgConditionException : Exception
	{
		/// <summary>
		/// Constructor; formats the exception message with the given String.Format() style parameters.
		/// </summary>
		/// <param name="format">Formatting string, in String.Format syntax</param>
		/// <param name="args">Optional arguments for the string</param>
		public BgConditionException(string format, params object[] args) : base(String.Format(format, args))
		{
		}
	}

	/// <summary>
	/// Context for evaluating BuildGraph conditions
	/// </summary>
	/// <param name="RootDir">Root directory for resolving paths</param>
	public record class BgConditionContext(DirectoryReference RootDir);

	/// <summary>
	/// Class to evaluate condition expressions in build scripts, following this grammar:
	/// 
	///		or-expression   ::= and-expression
	///		                  | or-expression "Or" and-expression;
	///		    
	///		and-expression  ::= comparison
	///		                  | and-expression "And" comparison;
	///		                  
	///		comparison      ::= scalar
	///		                  | scalar "==" scalar
	///		                  | scalar "!=" scalar
	///		                  | scalar "&lt;" scalar
	///		                  | scalar "&lt;=" scalar;
	///		                  | scalar "&gt;" scalar
	///		                  | scalar "&gt;=" scalar;
	///		                  
	///     scalar          ::= "(" or-expression ")"
	///                       | "!" scalar
	///                       | "Exists" "(" scalar ")"
	///                       | "HasTrailingSlash" "(" scalar ")"
	///                       | string
	///                       | identifier;
	///                       
	///     string          ::= any sequence of characters terminated by single quotes (') or double quotes ("). Not escaped.
	///     identifier      ::= any sequence of letters, digits, or underscore characters.
	///     
	/// The type of each subexpression is always a scalar, which are converted to expression-specific types (eg. booleans, integers) as required.
	/// Scalar values are case-insensitive strings. The identifier 'true' and the strings "true" and "True" are all identical scalars.
	/// </summary>
	public class BgCondition
	{
		/// <summary>
		/// Sentinel added to the end of a sequence of tokens.
		/// </summary>
		const string EndToken = "<EOF>";

		/// <summary>
		/// Tokens for the condition
		/// </summary>
		readonly List<string> _tokens = new List<string>();

		/// <summary>
		/// The current token index
		/// </summary>
		int _idx;

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="text">The condition text</param>
		private BgCondition(string text)
		{
			Tokenize(text, _tokens);
		}

		/// <summary>
		/// Evaluates the given string as a condition. Throws a ConditionException on a type or syntax error.
		/// </summary>
		/// <param name="text"></param>
		/// <param name="context">Context for evaluating the condition</param>
		/// <returns>The result of evaluating the condition</returns>
		public static ValueTask<bool> EvaluateAsync(string text, BgConditionContext context)
		{
			return new BgCondition(text).EvaluateAsync(context);
		}

		/// <summary>
		/// Evaluates the given string as a condition. Throws a ConditionException on a type or syntax error.
		/// </summary>
		/// <param name="context">Context for evaluating the condition</param>
		/// <returns>The result of evaluating the condition</returns>
		async ValueTask<bool> EvaluateAsync(BgConditionContext context)
		{
			bool result = true;
			if (_tokens.Count > 1)
			{
				_idx = 0;

				string value = await EvaluateOrAsync(context);
				if (_tokens[_idx] != EndToken)
				{
					throw new BgConditionException("Garbage after expression: {0}", String.Join("", _tokens.Skip(_idx)));
				}
				
				result = CoerceToBool(value);
			}
			return result;
		}

		/// <summary>
		/// Evaluates an "or-expression" production.
		/// </summary>
		/// <param name="context">Context for evaluating the expression</param>
		/// <returns>A scalar representing the result of evaluating the expression.</returns>
		async ValueTask<string> EvaluateOrAsync(BgConditionContext context)
		{
			// <Condition> Or <Condition> Or...
			string result = await EvaluateAndAsync(context);
			while (String.Equals(_tokens[_idx], "Or", StringComparison.OrdinalIgnoreCase))
			{
				// Evaluate this condition. We use a binary OR here, because we want to parse everything rather than short-circuit it.
				_idx++;
				string lhs = result;
				string rhs = await EvaluateAndAsync(context);
				result = (CoerceToBool(lhs) | CoerceToBool(rhs)) ? "true" : "false";
			}
			return result;
		}

		/// <summary>
		/// Evaluates an "and-expression" production.
		/// </summary>
		/// <param name="context">Context for evaluating the expression</param>
		/// <returns>A scalar representing the result of evaluating the expression.</returns>
		async ValueTask<string> EvaluateAndAsync(BgConditionContext context)
		{
			// <Condition> And <Condition> And...
			string result = await EvaluateComparisonAsync(context);
			while (String.Equals(_tokens[_idx], "And", StringComparison.OrdinalIgnoreCase))
			{
				// Evaluate this condition. We use a binary AND here, because we want to parse everything rather than short-circuit it.
				_idx++;
				string lhs = result;
				string rhs = await EvaluateComparisonAsync(context);
				result = (CoerceToBool(lhs) & CoerceToBool(rhs)) ? "true" : "false";
			}
			return result;
		}

		/// <summary>
		/// Evaluates a "comparison" production.
		/// </summary>
		/// <param name="context">Context for evaluating the expression</param>
		/// <returns>The result of evaluating the expression</returns>
		async ValueTask<string> EvaluateComparisonAsync(BgConditionContext context)
		{
			// scalar
			// scalar == scalar
			// scalar != scalar
			// scalar < scalar
			// scalar <= scalar
			// scalar > scalar
			// scalar >= scalar

			string result = await EvaluateScalarAsync(context);
			if (_tokens[_idx] == "==")
			{
				// Compare two scalars for equality
				_idx++;
				string lhs = result;
				string rhs = await EvaluateScalarAsync(context);
				result = String.Equals(lhs, rhs, StringComparison.OrdinalIgnoreCase) ? "true" : "false";
			}
			else if (_tokens[_idx] == "!=")
			{
				// Compare two scalars for inequality
				_idx++;
				string lhs = result;
				string rhs = await EvaluateScalarAsync(context);
				result = String.Equals(lhs, rhs, StringComparison.OrdinalIgnoreCase) ? "false" : "true";
			}
			else if (_tokens[_idx] == "<")
			{
				// Compares whether the first integer is less than the second
				_idx++;
				int lhs = CoerceToInteger(result);
				int rhs = CoerceToInteger(await EvaluateScalarAsync(context));
				result = (lhs < rhs) ? "true" : "false";
			}
			else if (_tokens[_idx] == "<=")
			{
				// Compares whether the first integer is less than the second
				_idx++;
				int lhs = CoerceToInteger(result);
				int rhs = CoerceToInteger(await EvaluateScalarAsync(context));
				result = (lhs <= rhs) ? "true" : "false";
			}
			else if (_tokens[_idx] == ">")
			{
				// Compares whether the first integer is less than the second
				_idx++;
				int lhs = CoerceToInteger(result);
				int rhs = CoerceToInteger(await EvaluateScalarAsync(context));
				result = (lhs > rhs) ? "true" : "false";
			}
			else if (_tokens[_idx] == ">=")
			{
				// Compares whether the first integer is less than the second
				_idx++;
				int lhs = CoerceToInteger(result);
				int rhs = CoerceToInteger(await EvaluateScalarAsync(context));
				result = (lhs >= rhs) ? "true" : "false";
			}
			return result;
		}

		/// <summary>
		/// Evaluates arguments from a token string. Arguments are all comma-separated tokens until a closing ) is encountered
		/// </summary>
		/// <returns>The result of evaluating the expression</returns>
		IEnumerable<string> EvaluateArguments()
		{
			List<string> arguments = new List<string>();

			// skip opening bracket
			if (_tokens[_idx++] != "(")
			{
				throw new BgConditionException("Expected '('");
			}

			bool didCloseBracket = false;

			while (_idx < _tokens.Count)
			{
				string nextToken = _tokens.ElementAt(_idx++);

				if (nextToken == EndToken)
				{
					// ran out of items
					break;
				}
				else if (nextToken == ")")
				{
					didCloseBracket = true;
					break;
				}
				else if (nextToken != ",")
				{
					if (nextToken.First() == '\'' && nextToken.Last() == '\'')
					{
						nextToken = nextToken.Substring(1, nextToken.Length - 2);
					}
					arguments.Add(nextToken);
				}
			}

			if (!didCloseBracket)
			{
				throw new BgConditionException("Expected ')'");
			}

			return arguments;
		}

		/// <summary>
		/// Evaluates a "scalar" production.
		/// </summary>
		/// <param name="context">Context for evaluating the expression</param>
		/// <returns>The result of evaluating the expression</returns>
		async ValueTask<string> EvaluateScalarAsync(BgConditionContext context)
		{
			string result;
			if (_tokens[_idx] == "(")
			{
				// Subexpression
				_idx++;
				result = await EvaluateOrAsync(context);
				if (_tokens[_idx] != ")")
				{
					throw new BgConditionException("Expected ')'");
				}
				_idx++;
			}
			else if (_tokens[_idx] == "!")
			{
				// Logical not
				_idx++;
				string rhs = await EvaluateScalarAsync(context);
				result = CoerceToBool(rhs) ? "false" : "true";
			}
			else if (String.Equals(_tokens[_idx], "Exists", StringComparison.OrdinalIgnoreCase) && _tokens[_idx + 1] == "(")
			{
				// Check whether file or directory exists. Evaluate the argument as a subexpression.
				_idx++;
				string argument = await EvaluateScalarAsync(context);
				result = Exists(argument, context) ? "true" : "false";
			}
			else if (String.Equals(_tokens[_idx], "HasTrailingSlash", StringComparison.OrdinalIgnoreCase) && _tokens[_idx + 1] == "(")
			{
				// Check whether the given string ends with a slash
				_idx++;
				string argument = await EvaluateScalarAsync(context);
				result = (argument.Length > 0 && (argument[^1] == Path.DirectorySeparatorChar || argument[^1] == Path.AltDirectorySeparatorChar)) ? "true" : "false";
			}
			else if (String.Equals(_tokens[_idx], "Contains", StringComparison.OrdinalIgnoreCase) && _tokens[_idx + 1] == "(")
			{
				// Check a string contains a substring. If a separator is supplied the string is first split
				_idx++;
				IEnumerable<string> arguments = EvaluateArguments();

				if (arguments.Count() != 2)
				{
					throw new BgConditionException("Invalid argument count for 'Contains'. Expected (Haystack,Needle)");
				}

				result = Contains(arguments.ElementAt(0), arguments.ElementAt(1)) ? "true" : "false";
			}
			else if (String.Equals(_tokens[_idx], "ContainsItem", StringComparison.OrdinalIgnoreCase) && _tokens[_idx + 1] == "(")
			{
				// Check a string contains a substring. If a separator is supplied the string is first split
				_idx++;
				IEnumerable<string> arguments = EvaluateArguments();

				if (arguments.Count() != 3)
				{
					throw new BgConditionException("Invalid argument count for 'ContainsItem'. Expected (Haystack,Needle,HaystackSeparator)");
				}

				result = ContainsItem(arguments.ElementAt(0), arguments.ElementAt(1), arguments.ElementAt(2)) ? "true" : "false";
			}
			else if (String.Equals(_tokens[_idx], "Length", StringComparison.OrdinalIgnoreCase) && _tokens[_idx + 1] == "(")
			{
				// returns the length of the string
				_idx++;
				string argument = await EvaluateScalarAsync(context);
				result = argument.Length.ToString();
			}
			else
			{
				// Raw scalar. Remove quotes from strings, and allow literals and simple identifiers to pass through directly.
				string token = _tokens[_idx];
				if (token.Length >= 2 && (token[0] == '\'' || token[0] == '\"') && token[^1] == token[0])
				{
					result = token.Substring(1, token.Length - 2);
					_idx++;
				}
				else if (Char.IsLetterOrDigit(token[0]) || token[0] == '_')
				{
					result = token;
					_idx++;
				}
				else
				{
					throw new BgConditionException("Token '{0}' is not a valid scalar", token);
				}
			}
			return result;
		}

		/// <summary>
		/// Determine if a path exists
		/// </summary>
		static bool Exists(string path, BgConditionContext context)
		{
			try
			{
				return FileReference.Exists(FileReference.Combine(context.RootDir, path)) || DirectoryReference.Exists(DirectoryReference.Combine(context.RootDir, path));
			}
			catch
			{
				return false;
			}
		}

		/// <summary>
		/// Checks whether Haystack contains "Needle". 
		/// </summary>
		/// <param name="haystack">The string to search</param>
		/// <param name="needle">The string to search for</param>
		/// <returns>True if the path exists, false otherwise.</returns>
		static bool Contains(string haystack, string needle)
		{
			try
			{
				return haystack.Contains(needle, StringComparison.CurrentCultureIgnoreCase);
			}
			catch
			{
				return false;
			}
		}

		/// <summary>
		/// Checks whether HaystackItems contains "Needle". 
		/// </summary>
		/// <param name="haystack">The separated list of items to check</param>
		/// <param name="needle">The item to check for</param>
		/// <param name="haystackSeparator">The separator used in Haystack</param>
		/// <returns>True if the path exists, false otherwise.</returns>
		static bool ContainsItem(string haystack, string needle, string haystackSeparator)
		{
			try
			{
				IEnumerable<string> haystackItems = haystack.Split(new string[] { haystackSeparator }, StringSplitOptions.RemoveEmptyEntries);
				return haystackItems.Any(i => i.ToLower() == needle.ToLower());
			}
			catch
			{
				return false;
			}
		}

		/// <summary>
		/// Converts a scalar to a boolean value.
		/// </summary>
		/// <param name="scalar">The scalar to convert</param>
		/// <returns>The scalar converted to a boolean value.</returns>
		static bool CoerceToBool(string scalar)
		{
			bool result;
			if (String.Equals(scalar, "true", StringComparison.OrdinalIgnoreCase))
			{
				result = true;
			}
			else if (String.Equals(scalar, "false", StringComparison.OrdinalIgnoreCase))
			{
				result = false;
			}
			else
			{
				throw new BgConditionException("Token '{0}' cannot be coerced to a bool", scalar);
			}
			return result;
		}

		/// <summary>
		/// Converts a scalar to a boolean value.
		/// </summary>
		/// <param name="scalar">The scalar to convert</param>
		/// <returns>The scalar converted to an integer value.</returns>
		static int CoerceToInteger(string scalar)
		{
			int value;
			if (!Int32.TryParse(scalar, out value))
			{
				throw new BgConditionException("Token '{0}' cannot be coerced to an integer", scalar);
			}
			return value;
		}

		/// <summary>
		/// Splits an input string up into expression tokens.
		/// </summary>
		/// <param name="text">Text to be converted into tokens</param>
		/// <param name="tokens">List to receive a list of tokens</param>
		static void Tokenize(string text, List<string> tokens)
		{
			int idx = 0;
			while (idx < text.Length)
			{
				int endIdx = idx + 1;
				if (!Char.IsWhiteSpace(text[idx]))
				{
					// Scan to the end of the current token
					if (Char.IsNumber(text[idx]))
					{
						// Number
						while (endIdx < text.Length && Char.IsNumber(text[endIdx]))
						{
							endIdx++;
						}
					}
					else if (Char.IsLetter(text[idx]) || text[idx] == '_')
					{
						// Identifier
						while (endIdx < text.Length && (Char.IsLetterOrDigit(text[endIdx]) || text[endIdx] == '_'))
						{
							endIdx++;
						}
					}
					else if (text[idx] == '!' || text[idx] == '<' || text[idx] == '>' || text[idx] == '=')
					{
						// Operator that can be followed by an equals character
						if (endIdx < text.Length && text[endIdx] == '=')
						{
							endIdx++;
						}
					}
					else if (text[idx] == '\'' || text[idx] == '\"')
					{
						// String
						if (endIdx < text.Length)
						{
							endIdx++;
							while (endIdx < text.Length && text[endIdx - 1] != text[idx])
							{
								endIdx++;
							}
						}
					}
					tokens.Add(text.Substring(idx, endIdx - idx));
				}
				idx = endIdx;
			}
			tokens.Add(EndToken);
		}

		/// <summary>
		/// Test cases for conditions.
		/// </summary>
		public static async Task TestConditionsAsync()
		{
			await TestConditionAsync("1 == 2", false);
			await TestConditionAsync("1 == 1", true);
			await TestConditionAsync("1 != 2", true);
			await TestConditionAsync("1 != 1", false);
			await TestConditionAsync("'hello' == 'hello'", true);
			await TestConditionAsync("'hello' == ('hello')", true);
			await TestConditionAsync("'hello' == 'world'", false);
			await TestConditionAsync("'hello' != ('world')", true);
			await TestConditionAsync("true == ('true')", true);
			await TestConditionAsync("true == ('True')", true);
			await TestConditionAsync("true == ('false')", false);
			await TestConditionAsync("true == !('False')", true);
			await TestConditionAsync("true == 'true' and 'false' == 'False'", true);
			await TestConditionAsync("true == 'true' and 'false' == 'true'", false);
			await TestConditionAsync("true == 'false' or 'false' == 'false'", true);
			await TestConditionAsync("true == 'false' or 'false' == 'true'", true);
		}

		/// <summary>
		/// Helper method to evaluate a condition and check it's the expected result
		/// </summary>
		/// <param name="condition">Condition to evaluate</param>
		/// <param name="expectedResult">The expected result</param>
		static async Task TestConditionAsync(string condition, bool expectedResult)
		{
			bool result = await new BgCondition(condition).EvaluateAsync(new BgConditionContext(DirectoryReference.GetCurrentDirectory()));
			Console.WriteLine("{0}: {1} = {2}", (result == expectedResult) ? "PASS" : "FAIL", condition, result);
		}
	}
}
