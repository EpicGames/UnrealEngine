// Copyright Epic Games, Inc. All Rights Reserved.

using EpicGames.BuildGraph.Expressions;
using EpicGames.Core;
using Microsoft.Extensions.Logging;

#nullable enable

namespace AutomationTool
{
	/// <summary>
	/// Base class for any user defined graphs
	/// </summary>
	public abstract class BgGraphBuilder
	{
		/// <summary>
		/// Accessor for default logger instance
		/// </summary>
		protected static ILogger Logger => Log.Logger;

		/// <summary>
		/// Callback used to instantiate the graph
		/// </summary>
		public abstract BgGraph CreateGraph();
	}
}
