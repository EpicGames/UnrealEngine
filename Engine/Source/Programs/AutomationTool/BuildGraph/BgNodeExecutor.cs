// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.ComponentModel;
using System.Linq;
using System.Reflection;
using System.Runtime.CompilerServices;
using System.Threading.Tasks;
using EpicGames.BuildGraph;
using EpicGames.BuildGraph.Expressions;
using EpicGames.Core;
using Microsoft.Extensions.Logging;
using OpenTracing;
using OpenTracing.Util;
using UnrealBuildBase;
using UnrealBuildTool;

#nullable enable

namespace AutomationTool
{
	/// <summary>
	/// Base class for binding and executing nodes
	/// </summary>
	abstract class BgNodeExecutor
	{
		public abstract Task<bool> Execute(JobContext job, Dictionary<string, HashSet<FileReference>> tagNameToFileSet);
	}

	class BgBytecodeNodeExecutor : BgNodeExecutor
	{
		class BgContextImpl : BgContext
		{
			public BgContextImpl(JobContext jobContext, Dictionary<string, HashSet<FileReference>> tagNameToFileSet)
				: base(tagNameToFileSet.ToDictionary(x => x.Key, x => FileSet.FromFiles(Unreal.RootDirectory, x.Value)))
			{
				_ = jobContext;
			}

			public override string Stream => CommandUtils.P4Enabled ? CommandUtils.P4Env.Branch : "";

			public override int Change => CommandUtils.P4Enabled ? CommandUtils.P4Env.Changelist : 0;

			public override int CodeChange => CommandUtils.P4Enabled ? CommandUtils.P4Env.CodeChangelist : 0;

			public override (int Major, int Minor, int Patch) EngineVersion
			{
				get
				{
					ReadOnlyBuildVersion current = ReadOnlyBuildVersion.Current;
					return (current.MajorVersion, current.MinorVersion, current.PatchVersion);
				}
			}

			public override bool IsBuildMachine => CommandUtils.IsBuildMachine;
		}

		readonly BgNodeDef _node;

		public BgBytecodeNodeExecutor(BgNodeDef node)
		{
			_node = node;
		}

		public static bool Bind(ILogger logger)
		{
			_ = logger;
			return true;
		}

		/// <summary>
		/// ExecuteAsync the method given in the 
		/// </summary>
		/// <param name="job"></param>
		/// <param name="tagNameToFileSet"></param>
		/// <returns></returns>
		public override async Task<bool> Execute(JobContext job, Dictionary<string, HashSet<FileReference>> tagNameToFileSet)
		{
			BgThunkDef thunk = _node.Thunk!;
			MethodInfo method = thunk.Method;

			HashSet<FileReference> buildProducts = tagNameToFileSet[_node.DefaultOutput.TagName];

			BgContextImpl context = new BgContextImpl(job, tagNameToFileSet);

			ParameterInfo[] parameters = method.GetParameters();

			object?[] arguments = new object[parameters.Length];
			for (int idx = 0; idx < parameters.Length; idx++)
			{
				Type parameterType = parameters[idx].ParameterType;
				if (parameterType == typeof(BgContext))
				{
					arguments[idx] = context;
				}
				else
				{
					arguments[idx] = thunk.Arguments[idx];
				}
			}

			Task task = (Task)method.Invoke(null, arguments)!;
			await task;

			if (_node.Outputs.Count > 0)
			{
				object? result = null;
				if (method.ReturnType.IsGenericType && method.ReturnType.GetGenericTypeDefinition() == typeof(Task<>))
				{
					Type taskType = task.GetType();
#pragma warning disable CA1849 // Task.Result synchronously blocks
					PropertyInfo property = taskType.GetProperty(nameof(Task<int>.Result))!;
#pragma warning restore CA1849
					result = property!.GetValue(task);
				}

				object?[] outputValues;
				if (result is ITuple tuple)
				{
					outputValues = Enumerable.Range(0, tuple.Length).Select(x => tuple[x]).ToArray();
				}
				else
				{
					outputValues = new[] { result };
				}

				for (int idx = 0; idx < outputValues.Length; idx++)
				{
					if (outputValues[idx] is BgFileSetOutputExpr fileSet)
					{
						string tagName = _node.Outputs[idx + 1].TagName;
						tagNameToFileSet[tagName] = new HashSet<FileReference>(fileSet.Value.Flatten().Values);
						buildProducts.UnionWith(tagNameToFileSet[tagName]);
					}
				}
			}

			return true;
		}
	}

	/// <summary>
	/// Implementation of <see cref="BgNodeDef"/> for graphs defined through XML syntax
	/// </summary>
	class BgScriptNodeExecutor : BgNodeExecutor
	{
		/// <summary>
		/// The script node
		/// </summary>
		public BgScriptNode Node { get; }

		/// <summary>
		/// List of bound task implementations
		/// </summary>
		readonly List<BgTaskImpl> _boundTasks = new List<BgTaskImpl>();

		/// <summary>
		/// Constructor
		/// </summary>
		public BgScriptNodeExecutor(BgScriptNode node)
		{
			Node = node;
		}

		public async ValueTask<bool> BindAsync(Dictionary<string, ScriptTaskBinding> nameToTask, Dictionary<string, BgNodeOutput> tagNameToNodeOutput, ILogger logger)
		{
			bool result = true;
			foreach (BgTask taskInfo in Node.Tasks)
			{
				BgTaskImpl? boundTask = await BindTaskAsync(taskInfo, nameToTask, tagNameToNodeOutput, logger);
				if (boundTask == null)
				{
					result = false;
				}
				else
				{
					_boundTasks.Add(boundTask);
				}
			}
			return result;
		}

		async ValueTask<BgTaskImpl?> BindTaskAsync(BgTask taskInfo, Dictionary<string, ScriptTaskBinding> nameToTask, IReadOnlyDictionary<string, BgNodeOutput> tagNameToNodeOutput, ILogger logger)
		{
			// Get the reflection info for this element
			ScriptTaskBinding? task;
			if (!nameToTask.TryGetValue(taskInfo.Name, out task))
			{
				logger.LogScriptError(taskInfo.Location, "Unknown task '{TaskName}'", taskInfo.Name);
				return null;
			}

			// Check all the required parameters are present
			bool hasRequiredAttributes = true;
			foreach (ScriptTaskParameterBinding parameter in task.NameToParameter.Values)
			{
				if (!parameter.Optional && !taskInfo.Arguments.ContainsKey(parameter.Name))
				{
					logger.LogScriptError(taskInfo.Location, "Missing required attribute - {AttrName}", parameter.Name);
					hasRequiredAttributes = false;
				}
			}

			// Create a context for evaluating conditions
			BgConditionContext conditionContext = new BgConditionContext(Unreal.RootDirectory);

			// Read all the attributes into a parameters object for this task
			object parametersObject = Activator.CreateInstance(task.ParametersClass)!;
			foreach ((string name, string value) in taskInfo.Arguments)
			{
				// Get the field that this attribute should be written to in the parameters object
				ScriptTaskParameterBinding? parameter;
				if (!task.NameToParameter.TryGetValue(name, out parameter))
				{
					logger.LogScriptError(taskInfo.Location, "Unknown attribute '{AttrName}'", name);
					continue;
				}

				// If it's a collection type, split it into separate values
				try
				{
					if (parameter.CollectionType == null)
					{
						// Parse it and assign it to the parameters object
						object? fieldValue = await ParseValueAsync(value, parameter.ValueType, conditionContext);
						if (fieldValue != null)
						{
							parameter.SetValue(parametersObject, fieldValue);
						}
						else if (!parameter.Optional)
						{
							logger.LogScriptError(taskInfo.Location, "Empty value for parameter '{AttrName}' is not allowed.", name);
						}
					}
					else
					{
						// Get the collection, or create one if necessary
						object? collectionValue = parameter.GetValue(parametersObject);
						if (collectionValue == null)
						{
							collectionValue = Activator.CreateInstance(parameter.ParameterType)!;
							parameter.SetValue(parametersObject, collectionValue);
						}

						// Parse the values and add them to the collection
						List<string> valueStrings = BgTaskImpl.SplitDelimitedList(value);
						foreach (string valueString in valueStrings)
						{
							object? elementValue = await ParseValueAsync(valueString, parameter.ValueType, conditionContext);
							if (elementValue != null)
							{
								parameter.CollectionType.InvokeMember("Add", BindingFlags.InvokeMethod | BindingFlags.Instance | BindingFlags.Public, null, collectionValue, new object[] { elementValue });
							}
						}
					}
				}
				catch (Exception ex)
				{
					logger.LogScriptError(taskInfo.Location, "Unable to parse argument {Name} from {Value}", name, value);
					logger.LogDebug(ex, "Exception while parsing argument {Name}", name);
				}
			}

			// Construct the task
			if (!hasRequiredAttributes)
			{
				return null;
			}

			// Add it to the list
			BgTaskImpl newTask = (BgTaskImpl)Activator.CreateInstance(task.TaskClass, parametersObject)!;

			// Set up the source location for diagnostics
			newTask.SourceLocation = taskInfo.Location;

			// Make sure all the read tags are local or listed as a dependency
			foreach (string readTagName in newTask.FindConsumedTagNames())
			{
				BgNodeOutput? output;
				if (tagNameToNodeOutput.TryGetValue(readTagName, out output))
				{
					if (output != null && output.ProducingNode != Node && !(Node.Inputs.Contains(output) || Node.OptionalInputs.Contains(output)))
					{
						logger.LogScriptError(taskInfo.Location, "The tag '{TagName}' is not a dependency of node '{Node}'", readTagName, Node.Name);
					}
				}
			}

			// Make sure all the written tags are local or listed as an output
			foreach (string modifiedTagName in newTask.FindProducedTagNames())
			{
				BgNodeOutput? output;
				if (tagNameToNodeOutput.TryGetValue(modifiedTagName, out output))
				{
					if (output != null && !Node.Outputs.Contains(output))
					{
						logger.LogScriptError(taskInfo.Location, "The tag '{TagName}' is created by '{Node}', and cannot be modified downstream", output.TagName, output.ProducingNode.Name);
					}
				}
			}
			return newTask;
		}

		/// <summary>
		/// Parse a value of the given type
		/// </summary>
		/// <param name="valueText">The text to parse</param>
		/// <param name="valueType">Type of the value to parse</param>
		/// <param name="context">Context for evaluating boolean expressions</param>
		/// <returns>Value that was parsed</returns>
		static async ValueTask<object?> ParseValueAsync(string valueText, Type valueType, BgConditionContext context)
		{
			// Parse it and assign it to the parameters object
			if (valueType.IsEnum)
			{
				return Enum.Parse(valueType, valueText);
			}
			else if (valueType == typeof(bool))
			{
				return await BgCondition.EvaluateAsync(valueText, context);
			}
			else if (valueType == typeof(FileReference))
			{
				if (String.IsNullOrEmpty(valueText))
				{
					return null;
				}
				else
				{
					return BgTaskImpl.ResolveFile(valueText);
				}
			}
			else if (valueType == typeof(DirectoryReference))
			{
				if (String.IsNullOrEmpty(valueText))
				{
					return null;
				}
				else
				{
					return BgTaskImpl.ResolveDirectory(valueText);
				}
			}

			TypeConverter converter = TypeDescriptor.GetConverter(valueType);
			if (converter.CanConvertFrom(typeof(string)))
			{
				return converter.ConvertFromString(valueText);
			}
			else
			{
				return Convert.ChangeType(valueText, valueType);
			}
		}

		/// <summary>
		/// Build all the tasks for this node
		/// </summary>
		/// <param name="job">Information about the current job</param>
		/// <param name="tagNameToFileSet">Mapping from tag names to the set of files they include. Should be set to contain the node inputs on entry.</param>
		/// <returns>Whether the task succeeded or not. Exiting with an exception will be caught and treated as a failure.</returns>
		public override async Task<bool> Execute(JobContext job, Dictionary<string, HashSet<FileReference>> tagNameToFileSet)
		{
			// Run each of the tasks in order
			HashSet<FileReference> buildProducts = tagNameToFileSet[Node.DefaultOutput.TagName];
			for (int idx = 0; idx < _boundTasks.Count; idx++)
			{
				using (IScope scope = GlobalTracer.Instance.BuildSpan("Task").WithTag("resource", _boundTasks[idx].GetTraceName()).StartActive())
				{
					ITaskExecutor? executor = _boundTasks[idx].GetExecutor();
					if (executor == null)
					{
						// ExecuteAsync this task directly
						try
						{
							_boundTasks[idx].GetTraceMetadata(scope.Span, "");
							await _boundTasks[idx].ExecuteAsync(job, buildProducts, tagNameToFileSet);
						}
						catch (Exception ex)
						{
							ExceptionUtils.AddContext(ex, "while executing task {0}", _boundTasks[idx].GetTraceString());

							BgScriptLocation? sourceLocation = _boundTasks[idx].SourceLocation;
							if (sourceLocation != null)
							{
								ExceptionUtils.AddContext(ex, "at {0}({1})", sourceLocation.File, sourceLocation.LineNumber);
							}

							throw;
						}
					}
					else
					{
						_boundTasks[idx].GetTraceMetadata(scope.Span, "1.");

						// The task has a custom executor, which may be able to execute several tasks simultaneously. Try to add the following tasks.
						int firstIdx = idx;
						while (idx + 1 < Node.Tasks.Count && executor.Add(_boundTasks[idx + 1]))
						{
							idx++;
							_boundTasks[idx].GetTraceMetadata(scope.Span, String.Format("{0}.", 1 + idx - firstIdx));
						}
						try
						{
							await executor.ExecuteAsync(job, buildProducts, tagNameToFileSet);
						}
						catch (Exception ex)
						{
							for (int taskIdx = firstIdx; taskIdx <= idx; taskIdx++)
							{
								ExceptionUtils.AddContext(ex, "while executing {0}", _boundTasks[taskIdx].GetTraceString());
							}

							BgScriptLocation? sourceLocation = _boundTasks[firstIdx].SourceLocation;
							if (sourceLocation != null)
							{
								ExceptionUtils.AddContext(ex, "at {0}({1})", sourceLocation.File, sourceLocation.LineNumber);
							}

							throw;
						}
					}
				}
			}

			// Remove anything that doesn't exist, since these files weren't explicitly tagged
			buildProducts.RemoveWhere(x => !FileReference.Exists(x));
			return true;
		}
	}
}