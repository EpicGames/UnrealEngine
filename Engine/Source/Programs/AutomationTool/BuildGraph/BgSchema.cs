// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Diagnostics;
using System.Linq;
using System.Xml;
using System.Xml.Schema;
using EpicGames.BuildGraph;
using EpicGames.Core;

#nullable enable

namespace AutomationTool
{
	/// <summary>
	/// Specifies validation that should be performed on a task parameter.
	/// </summary>
	public enum TaskParameterValidationType
	{
		/// <summary>
		/// Allow any valid values for the field type.
		/// </summary>
		Default,

		/// <summary>
		/// A list of tag names separated by semicolons
		/// </summary>
		TagList,

		/// <summary>
		/// A file specification, which may contain tags and wildcards.
		/// </summary>
		FileSpec,
	}

	/// <summary>
	/// Information about a parameter to a task
	/// </summary>
	[DebuggerDisplay("{Name}")]
	public class BgScriptTaskParameter
	{
		/// <summary>
		/// Name of this parameter
		/// </summary>
		public string Name { get; }

		/// <summary>
		/// The type for values assigned to this field
		/// </summary>
		public Type ValueType { get; }

		/// <summary>
		/// The ICollection interface for this type
		/// </summary>
		public Type? CollectionType { get; }

		/// <summary>
		/// Validation type for this field
		/// </summary>
		public TaskParameterValidationType ValidationType { get; }

		/// <summary>
		/// Whether this parameter is optional
		/// </summary>
		public bool Optional { get; }

		/// <summary>
		/// Constructor
		/// </summary>
		public BgScriptTaskParameter(string name, Type valueType, TaskParameterValidationType validationType, bool optional)
		{
			Name = name;
			ValueType = valueType;
			ValidationType = validationType;
			Optional = optional;

			if (ValueType.IsGenericType && ValueType.GetGenericTypeDefinition() == typeof(Nullable<>))
			{
				ValueType = ValueType.GetGenericArguments()[0];
				Optional = true;
			}

			if (ValueType.IsClass)
			{
				foreach (Type interfaceType in ValueType.GetInterfaces())
				{
					if (interfaceType.IsGenericType)
					{
						Type genericInterfaceType = interfaceType.GetGenericTypeDefinition();
						if (genericInterfaceType == typeof(ICollection<>))
						{
							CollectionType = interfaceType;
							ValueType = interfaceType.GetGenericArguments()[0];
						}
					}
				}
			}
		}
	}

	/// <summary>
	/// Helper class to serialize a task from an xml element
	/// </summary>
	[DebuggerDisplay("{Name}")]
	public class BgScriptTask
	{
		/// <summary>
		/// Name of this task
		/// </summary>
		public string Name { get; }

		/// <summary>
		/// Parameters for this task
		/// </summary>
		public List<BgScriptTaskParameter> Parameters { get; }

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="name">Name of the task</param>
		/// <param name="parameters">Parameters for the task</param>
		public BgScriptTask(string name, List<BgScriptTaskParameter> parameters)
		{
			Name = name;
			Parameters = new List<BgScriptTaskParameter>(parameters);
		}
	}

#pragma warning disable CS1591
	/// <summary>
	/// Enumeration of standard types used in the schema. Avoids hard-coding names.
	/// </summary>
	public enum ScriptSchemaStandardType
	{
		Graph,
		Agent,
		AgentBody,
		Node,
		NodeBody,
		Aggregate,
		Artifact,
		Report,
		Badge,
		Label,
		Notify,
		Include,
		Option,
		EnvVar,
		Property,
		Regex,
		StringOp,
		Macro,
		MacroBody,
		Extend,
		Expand,
		Annotate,
		Trace,
		Warning,
		Error,
		Name,
		NameList,
		Tag,
		TagList,
		NameOrTag,
		NameOrTagList,
		QualifiedName,
		BalancedString,
		Boolean,
		Integer,
		LabelChange
	}
#pragma warning restore CS1591

	/// <summary>
	/// Schema for build graph definitions. Stores information about the supported tasks, and allows validating an XML document.
	/// </summary>
	public class BgScriptSchema
	{
		/// <summary>
		/// Name of the root element
		/// </summary>
		public const string RootElementName = "BuildGraph";

		/// <summary>
		/// Namespace for the schema
		/// </summary>
		public const string NamespaceUri = "http://www.epicgames.com/BuildGraph";

		/// <summary>
		/// Qualified name for the string type
		/// </summary>
		static readonly XmlQualifiedName s_stringTypeName = new XmlQualifiedName("string", "http://www.w3.org/2001/XMLSchema");

		/// <summary>
		/// The inner xml schema
		/// </summary>
		public XmlSchema CompiledSchema { get; }

		/// <summary>
		/// Characters which are not permitted in names.
		/// </summary>
		public const string IllegalNameCharacters = "^<>:\"/\\|?*";

		/// <summary>
		/// Pattern which matches any name; alphanumeric characters, with single embedded spaces.
		/// </summary>
		const string NamePattern = "[^ " + IllegalNameCharacters + "]+( [^ " + IllegalNameCharacters + "]+)*";

		/// <summary>
		/// Pattern which matches a list of names, separated by semicolons.
		/// </summary>
		const string NameListPattern = NamePattern + "(;" + NamePattern + ")*";

		/// <summary>
		/// Pattern which matches any tag name; a name with a leading '#' character
		/// </summary>
		const string TagPattern = "#" + NamePattern;

		/// <summary>
		/// Pattern which matches a list of tag names, separated by semicolons;
		/// </summary>
		const string TagListPattern = TagPattern + "(;" + TagPattern + ")*";

		/// <summary>
		/// Pattern which matches any name or tag name; a name with a leading '#' character
		/// </summary>
		const string NameOrTagPattern = "#?" + NamePattern;

		/// <summary>
		/// Pattern which matches a list of names or tag names, separated by semicolons;
		/// </summary>
		const string NameOrTagListPattern = NameOrTagPattern + "(;" + NameOrTagPattern + ")*";

		/// <summary>
		/// Pattern which matches a qualified name.
		/// </summary>
		const string QualifiedNamePattern = NamePattern + "(\\." + NamePattern + ")*";

		/// <summary>
		/// Pattern which matches a property name
		/// </summary>
		const string PropertyPattern = "\\$\\(" + NamePattern + "\\)";

		/// <summary>
		/// Pattern which matches balanced parentheses in a string
		/// </summary>
		const string StringWithPropertiesPattern = "[^\\$]*" + "(" + "(" + PropertyPattern + "|" + "\\$[^\\(]" + ")" + "[^\\$]*" + ")+" + "\\$?";

		/// <summary>
		/// Pattern which matches balanced parentheses in a string
		/// </summary>
		const string BalancedStringPattern = "[^\\$]*" + "(" + "(" + PropertyPattern + "|" + "\\$[^\\(]" + ")" + "[^\\$]*" + ")*" + "\\$?";

		private BgScriptSchema(XmlSchema schema)
		{
			CompiledSchema = schema;
		}

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="tasks">Set of known tasks</param>
		/// <param name="primitiveTypes">Mapping of task name to information about how to construct it</param>
		public BgScriptSchema(IEnumerable<BgScriptTask> tasks, List<(Type, ScriptSchemaStandardType)> primitiveTypes)
		{
			// Create a lookup from standard types to their qualified names
			Dictionary<Type, XmlQualifiedName> typeToSchemaTypeName = new Dictionary<Type, XmlQualifiedName>();
			typeToSchemaTypeName.Add(typeof(string), GetQualifiedTypeName(ScriptSchemaStandardType.BalancedString));
			typeToSchemaTypeName.Add(typeof(bool), GetQualifiedTypeName(ScriptSchemaStandardType.Boolean));
			typeToSchemaTypeName.Add(typeof(int), GetQualifiedTypeName(ScriptSchemaStandardType.Integer));
			foreach ((Type type, ScriptSchemaStandardType schemaType) in primitiveTypes)
			{
				typeToSchemaTypeName.Add(type, GetQualifiedTypeName(schemaType));
			}

			// Create all the custom user types, and add them to the qualified name lookup
			List<XmlSchemaType> userTypes = new List<XmlSchemaType>();
			foreach (Type type in tasks.SelectMany(x => x.Parameters).Select(x => x.ValueType))
			{
				if (!typeToSchemaTypeName.ContainsKey(type))
				{
					if (type.IsClass && type.GetInterfaces().Any(x => x.GetGenericTypeDefinition() == typeof(ICollection<>)))
					{
						typeToSchemaTypeName.Add(type, GetQualifiedTypeName(ScriptSchemaStandardType.BalancedString));
					}
					else
					{
						string name = type.Name + "UserType";
						XmlSchemaType schemaType = CreateUserType(name, type);
						userTypes.Add(schemaType);
						typeToSchemaTypeName.Add(type, new XmlQualifiedName(name, NamespaceUri));
					}
				}
			}

			// Create all the task types
			Dictionary<string, XmlSchemaComplexType> taskNameToType = new Dictionary<string, XmlSchemaComplexType>();
			foreach (BgScriptTask task in tasks)
			{
				XmlSchemaComplexType taskType = new XmlSchemaComplexType();
				taskType.Name = task.Name + "TaskType";
				foreach (BgScriptTaskParameter parameter in task.Parameters)
				{
					XmlQualifiedName? schemaTypeName = GetQualifiedTypeName(parameter.ValidationType);
					if (schemaTypeName == null)
					{
						schemaTypeName = typeToSchemaTypeName[parameter.ValueType];
					}
					taskType.Attributes.Add(CreateSchemaAttribute(parameter.Name, schemaTypeName, parameter.Optional ? XmlSchemaUse.Optional : XmlSchemaUse.Required));
				}
				taskType.Attributes.Add(CreateSchemaAttribute("If", ScriptSchemaStandardType.BalancedString, XmlSchemaUse.Optional));
				taskNameToType.Add(task.Name, taskType);
			}

			// Create the schema object
			XmlSchema newSchema = new XmlSchema();
			newSchema.TargetNamespace = NamespaceUri;
			newSchema.ElementFormDefault = XmlSchemaForm.Qualified;
			newSchema.Items.Add(CreateSchemaElement(RootElementName, ScriptSchemaStandardType.Graph));
			newSchema.Items.Add(CreateGraphType());
			newSchema.Items.Add(CreateAgentType());
			newSchema.Items.Add(CreateAgentBodyType());
			newSchema.Items.Add(CreateNodeType());
			newSchema.Items.Add(CreateNodeBodyType(taskNameToType));
			newSchema.Items.Add(CreateAggregateType());
			newSchema.Items.Add(CreateArtifactType());
			newSchema.Items.Add(CreateReportType());
			newSchema.Items.Add(CreateBadgeType());
			newSchema.Items.Add(CreateLabelType());
			newSchema.Items.Add(CreateEnumType(GetTypeName(ScriptSchemaStandardType.LabelChange), typeof(BgLabelChange)));
			newSchema.Items.Add(CreateNotifyType());
			newSchema.Items.Add(CreateIncludeType());
			newSchema.Items.Add(CreateOptionType());
			newSchema.Items.Add(CreateEnvVarType());
			newSchema.Items.Add(CreatePropertyType());
			newSchema.Items.Add(CreateRegexType());
			newSchema.Items.Add(CreateStringOpType());
			newSchema.Items.Add(CreateMacroType());
			newSchema.Items.Add(CreateMacroBodyType(taskNameToType));
			newSchema.Items.Add(CreateExtendType());
			newSchema.Items.Add(CreateExpandType());
			newSchema.Items.Add(CreateAnnotateType());
			newSchema.Items.Add(CreateDiagnosticType(ScriptSchemaStandardType.Trace));
			newSchema.Items.Add(CreateDiagnosticType(ScriptSchemaStandardType.Warning));
			newSchema.Items.Add(CreateDiagnosticType(ScriptSchemaStandardType.Error));
			newSchema.Items.Add(CreateSimpleTypeFromRegex(GetTypeName(ScriptSchemaStandardType.Name), "(" + NamePattern + "|" + StringWithPropertiesPattern + ")"));
			newSchema.Items.Add(CreateSimpleTypeFromRegex(GetTypeName(ScriptSchemaStandardType.NameList), "(" + NameListPattern + "|" + StringWithPropertiesPattern + ")"));
			newSchema.Items.Add(CreateSimpleTypeFromRegex(GetTypeName(ScriptSchemaStandardType.Tag), "(" + TagPattern + "|" + StringWithPropertiesPattern + ")"));
			newSchema.Items.Add(CreateSimpleTypeFromRegex(GetTypeName(ScriptSchemaStandardType.TagList), "(" + TagListPattern + "|" + StringWithPropertiesPattern + ")"));
			newSchema.Items.Add(CreateSimpleTypeFromRegex(GetTypeName(ScriptSchemaStandardType.NameOrTag), "(" + NameOrTagPattern + "|" + StringWithPropertiesPattern + ")"));
			newSchema.Items.Add(CreateSimpleTypeFromRegex(GetTypeName(ScriptSchemaStandardType.NameOrTagList), "(" + NameOrTagListPattern + "|" + StringWithPropertiesPattern + ")"));
			newSchema.Items.Add(CreateSimpleTypeFromRegex(GetTypeName(ScriptSchemaStandardType.QualifiedName), "(" + QualifiedNamePattern + "|" + StringWithPropertiesPattern + ")"));
			newSchema.Items.Add(CreateSimpleTypeFromRegex(GetTypeName(ScriptSchemaStandardType.BalancedString), BalancedStringPattern));
			newSchema.Items.Add(CreateSimpleTypeFromRegex(GetTypeName(ScriptSchemaStandardType.Boolean), "(true|True|false|False|" + StringWithPropertiesPattern + ")"));
			newSchema.Items.Add(CreateSimpleTypeFromRegex(GetTypeName(ScriptSchemaStandardType.Integer), "(" + "(-?[1-9][0-9]*|0)" + "|" + StringWithPropertiesPattern + ")"));

			foreach (XmlSchemaComplexType type in taskNameToType.Values)
			{
				newSchema.Items.Add(type);
			}
			foreach (XmlSchemaSimpleType type in userTypes)
			{
				newSchema.Items.Add(type);
			}

			// Now that we've finished, compile it and store it to the class
			XmlSchemaSet newSchemaSet = new XmlSchemaSet();
			newSchemaSet.Add(newSchema);
			newSchemaSet.Compile();

			XmlSchema? schema = null;
			foreach (XmlSchema? newCompiledSchema in newSchemaSet.Schemas())
			{
				schema = newCompiledSchema!;
			}
			CompiledSchema = schema!;
		}

		/// <summary>
		/// Export the schema to a file
		/// </summary>
		/// <param name="file"></param>
		public void Export(FileReference file)
		{
			DirectoryReference.CreateDirectory(file.Directory);

			XmlWriterSettings settings = new XmlWriterSettings();
			settings.Indent = true;
			settings.IndentChars = "  ";
			settings.NewLineChars = "\n";

			DirectoryReference.CreateDirectory(file.Directory);
			using (XmlWriter writer = XmlWriter.Create(file.FullName, settings))
			{
				CompiledSchema.Write(writer);
			}
		}

		/// <summary>
		/// Imports a schema from a file
		/// </summary>
		/// <param name="file">The XML file to import</param>
		/// <returns>A <see cref="BgScriptSchema"/> deserialized from the XML file, or null if file doesn't exist</returns>
		public static BgScriptSchema? Import(FileReference file)
		{
			if (!FileReference.Exists(file))
			{
				return null;
			}

			using (XmlReader schemaFile = XmlReader.Create(file.FullName))
			{
				BgScriptSchema importedSchema = new BgScriptSchema(XmlSchema.Read(schemaFile, ValidationCallback)!);
				return importedSchema;
			}
		}

		static void ValidationCallback(object? sender, ValidationEventArgs eventArgs)
		{
			if (eventArgs.Severity == XmlSeverityType.Warning)
			{
				Log.WriteLine(LogEventType.Warning, "WARNING: {0}", eventArgs.Message);
			}
			else if (eventArgs.Severity == XmlSeverityType.Error)
			{
				Log.WriteLine(LogEventType.Error, "ERROR: {0}", eventArgs.Message);
			}
		}

		/// <summary>
		/// Gets the bare name for the given script type
		/// </summary>
		/// <param name="type">Script type to find the name of</param>
		/// <returns>Name of the schema type that matches the given script type</returns>
		static string GetTypeName(ScriptSchemaStandardType type)
		{
			return type.ToString() + "Type";
		}

		/// <summary>
		/// Gets the qualified name for the given script type
		/// </summary>
		/// <param name="type">Script type to find the qualified name for</param>
		/// <returns>Qualified name of the schema type that matches the given script type</returns>
		static XmlQualifiedName GetQualifiedTypeName(ScriptSchemaStandardType type)
		{
			return new XmlQualifiedName(GetTypeName(type), NamespaceUri);
		}

		/// <summary>
		/// Gets the qualified name of the schema type for the given type of validation
		/// </summary>
		/// <returns>Qualified name for the corresponding schema type</returns>
		static XmlQualifiedName? GetQualifiedTypeName(TaskParameterValidationType type)
		{
			switch (type)
			{
				case TaskParameterValidationType.TagList:
					return GetQualifiedTypeName(ScriptSchemaStandardType.TagList);
			}
			return null;
		}

		/// <summary>
		/// Creates the schema type representing the graph type
		/// </summary>
		/// <returns>Type definition for a graph</returns>
		static XmlSchemaType CreateGraphType()
		{
			XmlSchemaChoice graphChoice = new XmlSchemaChoice();
			graphChoice.MinOccurs = 0;
			graphChoice.MaxOccursString = "unbounded";
			graphChoice.Items.Add(CreateSchemaElement("Include", ScriptSchemaStandardType.Include));
			graphChoice.Items.Add(CreateSchemaElement("Option", ScriptSchemaStandardType.Option));
			graphChoice.Items.Add(CreateSchemaElement("EnvVar", ScriptSchemaStandardType.EnvVar));
			graphChoice.Items.Add(CreateSchemaElement("Property", ScriptSchemaStandardType.Property));
			graphChoice.Items.Add(CreateSchemaElement("Regex", ScriptSchemaStandardType.Regex));
			graphChoice.Items.Add(CreateSchemaElement("StringOp", ScriptSchemaStandardType.StringOp));
			graphChoice.Items.Add(CreateSchemaElement("Macro", ScriptSchemaStandardType.Macro));
			graphChoice.Items.Add(CreateSchemaElement("Extend", ScriptSchemaStandardType.Extend));
			graphChoice.Items.Add(CreateSchemaElement("Agent", ScriptSchemaStandardType.Agent));
			graphChoice.Items.Add(CreateSchemaElement("Aggregate", ScriptSchemaStandardType.Aggregate));
			graphChoice.Items.Add(CreateSchemaElement("Artifact", ScriptSchemaStandardType.Artifact));
			graphChoice.Items.Add(CreateSchemaElement("Report", ScriptSchemaStandardType.Report));
			graphChoice.Items.Add(CreateSchemaElement("Badge", ScriptSchemaStandardType.Badge));
			graphChoice.Items.Add(CreateSchemaElement("Label", ScriptSchemaStandardType.Label));
			graphChoice.Items.Add(CreateSchemaElement("Notify", ScriptSchemaStandardType.Notify));
			graphChoice.Items.Add(CreateSchemaElement("Annotate", ScriptSchemaStandardType.Annotate));
			graphChoice.Items.Add(CreateSchemaElement("Trace", ScriptSchemaStandardType.Trace));
			graphChoice.Items.Add(CreateSchemaElement("Warning", ScriptSchemaStandardType.Warning));
			graphChoice.Items.Add(CreateSchemaElement("Error", ScriptSchemaStandardType.Error));
			graphChoice.Items.Add(CreateSchemaElement("Expand", ScriptSchemaStandardType.Expand));
			graphChoice.Items.Add(CreateDoElement(ScriptSchemaStandardType.Graph));
			graphChoice.Items.Add(CreateSwitchElement(ScriptSchemaStandardType.Graph));
			graphChoice.Items.Add(CreateForEachElement(ScriptSchemaStandardType.Graph));

			XmlSchemaComplexType graphType = new XmlSchemaComplexType();
			graphType.Name = GetTypeName(ScriptSchemaStandardType.Graph);
			graphType.Particle = graphChoice;
			return graphType;
		}

		/// <summary>
		/// Creates the schema type representing the agent type
		/// </summary>
		/// <returns>Type definition for an agent</returns>
		static XmlSchemaType CreateAgentType()
		{
			XmlSchemaComplexContentExtension extension = new XmlSchemaComplexContentExtension();
			extension.BaseTypeName = GetQualifiedTypeName(ScriptSchemaStandardType.AgentBody);
			extension.Attributes.Add(CreateSchemaAttribute("Name", s_stringTypeName, XmlSchemaUse.Required));
			extension.Attributes.Add(CreateSchemaAttribute("Type", ScriptSchemaStandardType.NameList, XmlSchemaUse.Optional));
			extension.Attributes.Add(CreateSchemaAttribute("If", ScriptSchemaStandardType.BalancedString, XmlSchemaUse.Optional));

			XmlSchemaComplexContent contentModel = new XmlSchemaComplexContent();
			contentModel.Content = extension;

			XmlSchemaComplexType complexType = new XmlSchemaComplexType();
			complexType.Name = GetTypeName(ScriptSchemaStandardType.Agent);
			complexType.ContentModel = contentModel;
			return complexType;
		}

		/// <summary>
		/// Creates the schema type representing the contents of agent type
		/// </summary>
		/// <returns>Type definition for an agent</returns>
		static XmlSchemaType CreateAgentBodyType()
		{
			XmlSchemaChoice agentChoice = new XmlSchemaChoice();
			agentChoice.MinOccurs = 0;
			agentChoice.MaxOccursString = "unbounded";
			agentChoice.Items.Add(CreateSchemaElement("Property", ScriptSchemaStandardType.Property));
			agentChoice.Items.Add(CreateSchemaElement("Regex", ScriptSchemaStandardType.Regex));
			agentChoice.Items.Add(CreateSchemaElement("StringOp", ScriptSchemaStandardType.StringOp));
			agentChoice.Items.Add(CreateSchemaElement("EnvVar", ScriptSchemaStandardType.EnvVar));
			agentChoice.Items.Add(CreateSchemaElement("Node", ScriptSchemaStandardType.Node));
			agentChoice.Items.Add(CreateSchemaElement("Trace", ScriptSchemaStandardType.Trace));
			agentChoice.Items.Add(CreateSchemaElement("Label", ScriptSchemaStandardType.Label));
			agentChoice.Items.Add(CreateSchemaElement("Annotate", ScriptSchemaStandardType.Annotate));
			agentChoice.Items.Add(CreateSchemaElement("Artifact", ScriptSchemaStandardType.Artifact));
			agentChoice.Items.Add(CreateSchemaElement("Warning", ScriptSchemaStandardType.Warning));
			agentChoice.Items.Add(CreateSchemaElement("Error", ScriptSchemaStandardType.Error));
			agentChoice.Items.Add(CreateSchemaElement("Expand", ScriptSchemaStandardType.Expand));
			agentChoice.Items.Add(CreateDoElement(ScriptSchemaStandardType.AgentBody));
			agentChoice.Items.Add(CreateSwitchElement(ScriptSchemaStandardType.AgentBody));
			agentChoice.Items.Add(CreateForEachElement(ScriptSchemaStandardType.AgentBody));

			XmlSchemaComplexType agentType = new XmlSchemaComplexType();
			agentType.Name = GetTypeName(ScriptSchemaStandardType.AgentBody);
			agentType.Particle = agentChoice;
			return agentType;
		}

		/// <summary>
		/// Creates the schema type representing the node type
		/// </summary>
		/// <returns>Type definition for a node</returns>
		static XmlSchemaType CreateNodeType()
		{
			XmlSchemaComplexContentExtension extension = new XmlSchemaComplexContentExtension();
			extension.BaseTypeName = GetQualifiedTypeName(ScriptSchemaStandardType.NodeBody);
			extension.Attributes.Add(CreateSchemaAttribute("Name", ScriptSchemaStandardType.Name, XmlSchemaUse.Required));
			extension.Attributes.Add(CreateSchemaAttribute("Requires", ScriptSchemaStandardType.NameOrTagList, XmlSchemaUse.Optional));
			extension.Attributes.Add(CreateSchemaAttribute("OptionalRequires", ScriptSchemaStandardType.NameOrTagList, XmlSchemaUse.Optional));
			extension.Attributes.Add(CreateSchemaAttribute("Produces", ScriptSchemaStandardType.TagList, XmlSchemaUse.Optional));
			extension.Attributes.Add(CreateSchemaAttribute("After", ScriptSchemaStandardType.NameOrTagList, XmlSchemaUse.Optional));
			extension.Attributes.Add(CreateSchemaAttribute("Token", ScriptSchemaStandardType.BalancedString, XmlSchemaUse.Optional));
			extension.Attributes.Add(CreateSchemaAttribute("If", ScriptSchemaStandardType.BalancedString, XmlSchemaUse.Optional));
			extension.Attributes.Add(CreateSchemaAttribute("RunEarly", ScriptSchemaStandardType.Boolean, XmlSchemaUse.Optional));
			extension.Attributes.Add(CreateSchemaAttribute("NotifyOnWarnings", ScriptSchemaStandardType.Boolean, XmlSchemaUse.Optional));
			extension.Attributes.Add(CreateSchemaAttribute("Annotations", ScriptSchemaStandardType.BalancedString, XmlSchemaUse.Optional));
			extension.Attributes.Add(CreateSchemaAttribute("IgnoreModified", ScriptSchemaStandardType.BalancedString, XmlSchemaUse.Optional));
			extension.Attributes.Add(CreateSchemaAttribute("AllowRetry", ScriptSchemaStandardType.Boolean, XmlSchemaUse.Optional));

			XmlSchemaComplexContent contentModel = new XmlSchemaComplexContent();
			contentModel.Content = extension;

			XmlSchemaComplexType complexType = new XmlSchemaComplexType();
			complexType.Name = GetTypeName(ScriptSchemaStandardType.Node);
			complexType.ContentModel = contentModel;
			return complexType;
		}

		/// <summary>
		/// Creates the schema type representing the body of the node type
		/// </summary>
		/// <returns>Type definition for a node</returns>
		static XmlSchemaType CreateNodeBodyType(Dictionary<string, XmlSchemaComplexType> taskNameToType)
		{
			XmlSchemaChoice nodeChoice = new XmlSchemaChoice();
			nodeChoice.MinOccurs = 0;
			nodeChoice.MaxOccursString = "unbounded";
			nodeChoice.Items.Add(CreateSchemaElement("Property", ScriptSchemaStandardType.Property));
			nodeChoice.Items.Add(CreateSchemaElement("Regex", ScriptSchemaStandardType.Regex));
			nodeChoice.Items.Add(CreateSchemaElement("StringOp", ScriptSchemaStandardType.StringOp));
			nodeChoice.Items.Add(CreateSchemaElement("EnvVar", ScriptSchemaStandardType.EnvVar));
			nodeChoice.Items.Add(CreateSchemaElement("Trace", ScriptSchemaStandardType.Trace));
			nodeChoice.Items.Add(CreateSchemaElement("Warning", ScriptSchemaStandardType.Warning));
			nodeChoice.Items.Add(CreateSchemaElement("Error", ScriptSchemaStandardType.Error));
			nodeChoice.Items.Add(CreateSchemaElement("Expand", ScriptSchemaStandardType.Expand));
			nodeChoice.Items.Add(CreateDoElement(ScriptSchemaStandardType.NodeBody));
			nodeChoice.Items.Add(CreateSwitchElement(ScriptSchemaStandardType.NodeBody));
			nodeChoice.Items.Add(CreateForEachElement(ScriptSchemaStandardType.NodeBody));

			if (taskNameToType == null)
			{
				nodeChoice.Items.Add(new XmlSchemaAny());
			}
			else
			{
				foreach (KeyValuePair<string, XmlSchemaComplexType> pair in taskNameToType.OrderBy(x => x.Key))
				{
					nodeChoice.Items.Add(CreateSchemaElement(pair.Key, new XmlQualifiedName(pair.Value.Name, NamespaceUri)));
				}
			}

			XmlSchemaComplexType nodeType = new XmlSchemaComplexType();
			nodeType.Name = GetTypeName(ScriptSchemaStandardType.NodeBody);
			nodeType.Particle = nodeChoice;
			return nodeType;
		}

		/// <summary>
		/// Creates the schema type representing the aggregate type
		/// </summary>
		/// <returns>Type definition for an aggregate</returns>
		static XmlSchemaType CreateAggregateType()
		{
			XmlSchemaComplexType aggregateType = new XmlSchemaComplexType();
			aggregateType.Name = GetTypeName(ScriptSchemaStandardType.Aggregate);
			aggregateType.Attributes.Add(CreateSchemaAttribute("Name", ScriptSchemaStandardType.Name, XmlSchemaUse.Required));
			aggregateType.Attributes.Add(CreateSchemaAttribute("Label", ScriptSchemaStandardType.BalancedString, XmlSchemaUse.Optional));
			aggregateType.Attributes.Add(CreateSchemaAttribute("Requires", ScriptSchemaStandardType.NameOrTagList, XmlSchemaUse.Required));
			aggregateType.Attributes.Add(CreateSchemaAttribute("Include", ScriptSchemaStandardType.NameOrTagList, XmlSchemaUse.Optional));
			aggregateType.Attributes.Add(CreateSchemaAttribute("Exclude", ScriptSchemaStandardType.NameOrTagList, XmlSchemaUse.Optional));
			aggregateType.Attributes.Add(CreateSchemaAttribute("If", ScriptSchemaStandardType.BalancedString, XmlSchemaUse.Optional));
			return aggregateType;
		}

		/// <summary>
		/// Creates the schema type representing the artifact type
		/// </summary>
		/// <returns>Type definition for an artifact</returns>
		static XmlSchemaType CreateArtifactType()
		{
			XmlSchemaComplexType artifactType = new XmlSchemaComplexType();
			artifactType.Name = GetTypeName(ScriptSchemaStandardType.Artifact);
			artifactType.Attributes.Add(CreateSchemaAttribute("Name", ScriptSchemaStandardType.Name, XmlSchemaUse.Required));
			artifactType.Attributes.Add(CreateSchemaAttribute("Type", ScriptSchemaStandardType.Name, XmlSchemaUse.Required));
			artifactType.Attributes.Add(CreateSchemaAttribute("Description", ScriptSchemaStandardType.BalancedString, XmlSchemaUse.Optional));
			artifactType.Attributes.Add(CreateSchemaAttribute("BasePath", ScriptSchemaStandardType.BalancedString, XmlSchemaUse.Optional));
			artifactType.Attributes.Add(CreateSchemaAttribute("Tag", ScriptSchemaStandardType.Tag, XmlSchemaUse.Optional));
			artifactType.Attributes.Add(CreateSchemaAttribute("Keys", ScriptSchemaStandardType.BalancedString, XmlSchemaUse.Optional));
			artifactType.Attributes.Add(CreateSchemaAttribute("Metadata", ScriptSchemaStandardType.BalancedString, XmlSchemaUse.Optional));
			artifactType.Attributes.Add(CreateSchemaAttribute("If", ScriptSchemaStandardType.BalancedString, XmlSchemaUse.Optional));
			return artifactType;
		}

		/// <summary>
		/// Creates the schema type representing the report type
		/// </summary>
		/// <returns>Type definition for a report</returns>
		static XmlSchemaType CreateReportType()
		{
			XmlSchemaComplexType reportType = new XmlSchemaComplexType();
			reportType.Name = GetTypeName(ScriptSchemaStandardType.Report);
			reportType.Attributes.Add(CreateSchemaAttribute("Name", ScriptSchemaStandardType.Name, XmlSchemaUse.Required));
			reportType.Attributes.Add(CreateSchemaAttribute("Requires", ScriptSchemaStandardType.NameOrTagList, XmlSchemaUse.Required));
			reportType.Attributes.Add(CreateSchemaAttribute("If", ScriptSchemaStandardType.BalancedString, XmlSchemaUse.Optional));
			return reportType;
		}

		/// <summary>
		/// Creates the schema type representing the badge type
		/// </summary>
		/// <returns>Type definition for a badge</returns>
		static XmlSchemaType CreateBadgeType()
		{
			XmlSchemaComplexType badgeType = new XmlSchemaComplexType();
			badgeType.Name = GetTypeName(ScriptSchemaStandardType.Badge);
			badgeType.Attributes.Add(CreateSchemaAttribute("Name", ScriptSchemaStandardType.Name, XmlSchemaUse.Required));
			badgeType.Attributes.Add(CreateSchemaAttribute("Requires", ScriptSchemaStandardType.NameOrTagList, XmlSchemaUse.Optional));
			badgeType.Attributes.Add(CreateSchemaAttribute("Targets", ScriptSchemaStandardType.NameOrTagList, XmlSchemaUse.Optional));
			badgeType.Attributes.Add(CreateSchemaAttribute("Project", ScriptSchemaStandardType.BalancedString, XmlSchemaUse.Required));
			badgeType.Attributes.Add(CreateSchemaAttribute("Change", ScriptSchemaStandardType.BalancedString, XmlSchemaUse.Optional));
			badgeType.Attributes.Add(CreateSchemaAttribute("If", ScriptSchemaStandardType.BalancedString, XmlSchemaUse.Optional));
			return badgeType;
		}

		/// <summary>
		/// Creates the schema type representing the label type
		/// </summary>
		/// <returns>Type definition for a label</returns>
		static XmlSchemaType CreateLabelType()
		{
			XmlSchemaComplexType labelType = new XmlSchemaComplexType();
			labelType.Name = GetTypeName(ScriptSchemaStandardType.Label);
			labelType.Attributes.Add(CreateSchemaAttribute("Name", ScriptSchemaStandardType.Name, XmlSchemaUse.Optional));
			labelType.Attributes.Add(CreateSchemaAttribute("Category", ScriptSchemaStandardType.Name, XmlSchemaUse.Optional));
			labelType.Attributes.Add(CreateSchemaAttribute("UgsBadge", ScriptSchemaStandardType.Name, XmlSchemaUse.Optional));
			labelType.Attributes.Add(CreateSchemaAttribute("UgsProject", ScriptSchemaStandardType.BalancedString, XmlSchemaUse.Optional));
			labelType.Attributes.Add(CreateSchemaAttribute("Change", ScriptSchemaStandardType.LabelChange, XmlSchemaUse.Optional));
			labelType.Attributes.Add(CreateSchemaAttribute("Requires", ScriptSchemaStandardType.NameOrTagList, XmlSchemaUse.Required));
			labelType.Attributes.Add(CreateSchemaAttribute("Include", ScriptSchemaStandardType.NameOrTagList, XmlSchemaUse.Optional));
			labelType.Attributes.Add(CreateSchemaAttribute("Exclude", ScriptSchemaStandardType.NameOrTagList, XmlSchemaUse.Optional));
			labelType.Attributes.Add(CreateSchemaAttribute("If", ScriptSchemaStandardType.BalancedString, XmlSchemaUse.Optional));
			return labelType;
		}

		/// <summary>
		/// Creates the schema type representing a notifier
		/// </summary>
		/// <returns>Type definition for a notifier</returns>
		static XmlSchemaType CreateNotifyType()
		{
			XmlSchemaComplexType aggregateType = new XmlSchemaComplexType();
			aggregateType.Name = GetTypeName(ScriptSchemaStandardType.Notify);
			aggregateType.Attributes.Add(CreateSchemaAttribute("Targets", ScriptSchemaStandardType.NameOrTagList, XmlSchemaUse.Optional));
			aggregateType.Attributes.Add(CreateSchemaAttribute("Except", ScriptSchemaStandardType.NameOrTagList, XmlSchemaUse.Optional));
			aggregateType.Attributes.Add(CreateSchemaAttribute("Nodes", ScriptSchemaStandardType.NameOrTagList, XmlSchemaUse.Optional));
			aggregateType.Attributes.Add(CreateSchemaAttribute("Reports", ScriptSchemaStandardType.NameList, XmlSchemaUse.Optional));
			aggregateType.Attributes.Add(CreateSchemaAttribute("Users", ScriptSchemaStandardType.BalancedString, XmlSchemaUse.Optional));
			aggregateType.Attributes.Add(CreateSchemaAttribute("Submitters", ScriptSchemaStandardType.BalancedString, XmlSchemaUse.Optional));
			aggregateType.Attributes.Add(CreateSchemaAttribute("Warnings", ScriptSchemaStandardType.Boolean, XmlSchemaUse.Optional));
			aggregateType.Attributes.Add(CreateSchemaAttribute("If", ScriptSchemaStandardType.BalancedString, XmlSchemaUse.Optional));
			aggregateType.Attributes.Add(CreateSchemaAttribute("Absolute", ScriptSchemaStandardType.BalancedString, XmlSchemaUse.Optional));
			return aggregateType;
		}

		/// <summary>
		/// Creates the schema type representing an annotation
		/// </summary>
		/// <returns>Type definition for a notifier</returns>
		static XmlSchemaType CreateAnnotateType()
		{
			XmlSchemaComplexType aggregateType = new XmlSchemaComplexType();
			aggregateType.Name = GetTypeName(ScriptSchemaStandardType.Annotate);
			aggregateType.Attributes.Add(CreateSchemaAttribute("Targets", ScriptSchemaStandardType.NameOrTagList, XmlSchemaUse.Optional));
			aggregateType.Attributes.Add(CreateSchemaAttribute("Except", ScriptSchemaStandardType.NameOrTagList, XmlSchemaUse.Optional));
			aggregateType.Attributes.Add(CreateSchemaAttribute("Nodes", ScriptSchemaStandardType.NameOrTagList, XmlSchemaUse.Optional));
			aggregateType.Attributes.Add(CreateSchemaAttribute("Values", ScriptSchemaStandardType.BalancedString, XmlSchemaUse.Optional));
			aggregateType.Attributes.Add(CreateSchemaAttribute("If", ScriptSchemaStandardType.BalancedString, XmlSchemaUse.Optional));
			return aggregateType;
		}

		/// <summary>
		/// Creates the schema type representing an include type
		/// </summary>
		/// <returns>Type definition for an include directive</returns>
		static XmlSchemaType CreateIncludeType()
		{
			XmlSchemaComplexType propertyType = new XmlSchemaComplexType();
			propertyType.Name = GetTypeName(ScriptSchemaStandardType.Include);
			propertyType.Attributes.Add(CreateSchemaAttribute("Script", ScriptSchemaStandardType.BalancedString, XmlSchemaUse.Required));
			propertyType.Attributes.Add(CreateSchemaAttribute("If", ScriptSchemaStandardType.BalancedString, XmlSchemaUse.Optional));
			propertyType.Attributes.Add(CreateSchemaAttribute("IgnoreDuplicateIncludes", ScriptSchemaStandardType.BalancedString, XmlSchemaUse.Optional));
			return propertyType;
		}

		/// <summary>
		/// Creates the schema type representing a parameter type
		/// </summary>
		/// <returns>Type definition for a parameter</returns>
		static XmlSchemaType CreateOptionType()
		{
			XmlSchemaComplexType optionType = new XmlSchemaComplexType();
			optionType.Name = GetTypeName(ScriptSchemaStandardType.Option);
			optionType.Attributes.Add(CreateSchemaAttribute("Name", ScriptSchemaStandardType.Name, XmlSchemaUse.Required));
			optionType.Attributes.Add(CreateSchemaAttribute("Restrict", s_stringTypeName, XmlSchemaUse.Optional));
			optionType.Attributes.Add(CreateSchemaAttribute("DefaultValue", s_stringTypeName, XmlSchemaUse.Required));
			optionType.Attributes.Add(CreateSchemaAttribute("Description", s_stringTypeName, XmlSchemaUse.Required));
			optionType.Attributes.Add(CreateSchemaAttribute("If", ScriptSchemaStandardType.BalancedString, XmlSchemaUse.Optional));
			return optionType;
		}

		/// <summary>
		/// Creates the schema type representing a environment variable type
		/// </summary>
		/// <returns>Type definition for an environment variable property</returns>
		static XmlSchemaType CreateEnvVarType()
		{
			XmlSchemaComplexType envVarType = new XmlSchemaComplexType();
			envVarType.Name = GetTypeName(ScriptSchemaStandardType.EnvVar);
			envVarType.Attributes.Add(CreateSchemaAttribute("Name", ScriptSchemaStandardType.Name, XmlSchemaUse.Required));
			envVarType.Attributes.Add(CreateSchemaAttribute("If", ScriptSchemaStandardType.BalancedString, XmlSchemaUse.Optional));
			return envVarType;
		}

		/// <summary>
		/// Creates the schema type representing a property type
		/// </summary>
		/// <returns>Type definition for a property</returns>
		static XmlSchemaType CreatePropertyType()
		{
			XmlSchemaSimpleContentExtension extension = new XmlSchemaSimpleContentExtension();
			extension.BaseTypeName = s_stringTypeName;
			extension.Attributes.Add(CreateSchemaAttribute("Name", ScriptSchemaStandardType.Name, XmlSchemaUse.Required));
			extension.Attributes.Add(CreateSchemaAttribute("Value", s_stringTypeName, XmlSchemaUse.Optional));
			extension.Attributes.Add(CreateSchemaAttribute("Separator", s_stringTypeName, XmlSchemaUse.Optional));
			extension.Attributes.Add(CreateSchemaAttribute("Multiline", s_stringTypeName, XmlSchemaUse.Optional));
			extension.Attributes.Add(CreateSchemaAttribute("If", ScriptSchemaStandardType.BalancedString, XmlSchemaUse.Optional));
			extension.Attributes.Add(CreateSchemaAttribute("CreateInParentScope", ScriptSchemaStandardType.Boolean, XmlSchemaUse.Optional));

			XmlSchemaSimpleContent contentModel = new XmlSchemaSimpleContent();
			contentModel.Content = extension;

			XmlSchemaComplexType propertyType = new XmlSchemaComplexType();
			propertyType.Name = GetTypeName(ScriptSchemaStandardType.Property);
			propertyType.ContentModel = contentModel;
			return propertyType;
		}

		/// <summary>
		/// Creates the schema type representing a regex type
		/// </summary>
		/// <returns>Type definition for a regex</returns>
		static XmlSchemaType CreateRegexType()
		{
			XmlSchemaSimpleContentExtension extension = new XmlSchemaSimpleContentExtension();
			extension.BaseTypeName = s_stringTypeName;

			extension.Attributes.Add(CreateSchemaAttribute("Input", s_stringTypeName, XmlSchemaUse.Required));
			extension.Attributes.Add(CreateSchemaAttribute("Pattern", s_stringTypeName, XmlSchemaUse.Required));
			extension.Attributes.Add(CreateSchemaAttribute("Capture", ScriptSchemaStandardType.NameList, XmlSchemaUse.Required));
			extension.Attributes.Add(CreateSchemaAttribute("Optional", ScriptSchemaStandardType.Boolean, XmlSchemaUse.Optional));
			extension.Attributes.Add(CreateSchemaAttribute("If", ScriptSchemaStandardType.BalancedString, XmlSchemaUse.Optional));

			XmlSchemaSimpleContent contentModel = new XmlSchemaSimpleContent();
			contentModel.Content = extension;

			XmlSchemaComplexType regexType = new XmlSchemaComplexType();
			regexType.Name = GetTypeName(ScriptSchemaStandardType.Regex);
			regexType.ContentModel = contentModel;
			return regexType;
		}

		/// <summary>
		/// Creates the schema type representing a stringop type
		/// </summary>
		/// <returns>Type definition for a stringop element</returns>
		static XmlSchemaType CreateStringOpType()
		{
			XmlSchemaSimpleContentExtension extension = new XmlSchemaSimpleContentExtension();
			extension.BaseTypeName = s_stringTypeName;

			extension.Attributes.Add(CreateSchemaAttribute("Input", s_stringTypeName, XmlSchemaUse.Required));
			extension.Attributes.Add(CreateSchemaAttribute("Output", s_stringTypeName, XmlSchemaUse.Required));
			extension.Attributes.Add(CreateSchemaAttribute("Method", s_stringTypeName, XmlSchemaUse.Required));
			extension.Attributes.Add(CreateSchemaAttribute("Arguments", s_stringTypeName, XmlSchemaUse.Optional));
			extension.Attributes.Add(CreateSchemaAttribute("If", ScriptSchemaStandardType.BalancedString, XmlSchemaUse.Optional));

			XmlSchemaSimpleContent contentModel = new XmlSchemaSimpleContent();
			contentModel.Content = extension;

			XmlSchemaComplexType stringOpType = new XmlSchemaComplexType();
			stringOpType.Name = GetTypeName(ScriptSchemaStandardType.StringOp);
			stringOpType.ContentModel = contentModel;
			return stringOpType;
		}

		/// <summary>
		/// Creates the schema type representing the macro type
		/// </summary>
		/// <returns>Type definition for a node</returns>
		static XmlSchemaType CreateMacroType()
		{
			XmlSchemaComplexContentExtension extension = new XmlSchemaComplexContentExtension();
			extension.BaseTypeName = GetQualifiedTypeName(ScriptSchemaStandardType.MacroBody);
			extension.Attributes.Add(CreateSchemaAttribute("Name", ScriptSchemaStandardType.Name, XmlSchemaUse.Required));
			extension.Attributes.Add(CreateSchemaAttribute("Arguments", ScriptSchemaStandardType.BalancedString, XmlSchemaUse.Optional));
			extension.Attributes.Add(CreateSchemaAttribute("OptionalArguments", ScriptSchemaStandardType.BalancedString, XmlSchemaUse.Optional));

			XmlSchemaComplexContent contentModel = new XmlSchemaComplexContent();
			contentModel.Content = extension;

			XmlSchemaComplexType complexType = new XmlSchemaComplexType();
			complexType.Name = GetTypeName(ScriptSchemaStandardType.Macro);
			complexType.ContentModel = contentModel;
			return complexType;
		}

		/// <summary>
		/// Creates the schema type representing the macro type
		/// </summary>
		/// <returns>Type definition for a node</returns>
		static XmlSchemaType CreateMacroBodyType(Dictionary<string, XmlSchemaComplexType> taskNameToType)
		{
			XmlSchemaChoice macroChoice = new XmlSchemaChoice();
			macroChoice.MinOccurs = 0;
			macroChoice.MaxOccursString = "unbounded";

			// Graph scope
			macroChoice.Items.Add(CreateSchemaElement("Include", ScriptSchemaStandardType.Include));
			macroChoice.Items.Add(CreateSchemaElement("Option", ScriptSchemaStandardType.Option));
			macroChoice.Items.Add(CreateSchemaElement("EnvVar", ScriptSchemaStandardType.EnvVar));
			macroChoice.Items.Add(CreateSchemaElement("Property", ScriptSchemaStandardType.Property));
			macroChoice.Items.Add(CreateSchemaElement("Regex", ScriptSchemaStandardType.Regex));
			macroChoice.Items.Add(CreateSchemaElement("StringOp", ScriptSchemaStandardType.StringOp));
			macroChoice.Items.Add(CreateSchemaElement("Macro", ScriptSchemaStandardType.Macro));
			macroChoice.Items.Add(CreateSchemaElement("Agent", ScriptSchemaStandardType.Agent));
			macroChoice.Items.Add(CreateSchemaElement("Aggregate", ScriptSchemaStandardType.Aggregate));
			macroChoice.Items.Add(CreateSchemaElement("Artifact", ScriptSchemaStandardType.Artifact));
			macroChoice.Items.Add(CreateSchemaElement("Report", ScriptSchemaStandardType.Report));
			macroChoice.Items.Add(CreateSchemaElement("Badge", ScriptSchemaStandardType.Badge));
			macroChoice.Items.Add(CreateSchemaElement("Notify", ScriptSchemaStandardType.Notify));
			macroChoice.Items.Add(CreateSchemaElement("Trace", ScriptSchemaStandardType.Trace));
			macroChoice.Items.Add(CreateSchemaElement("Warning", ScriptSchemaStandardType.Warning));
			macroChoice.Items.Add(CreateSchemaElement("Error", ScriptSchemaStandardType.Error));
			macroChoice.Items.Add(CreateSchemaElement("Expand", ScriptSchemaStandardType.Expand));
			macroChoice.Items.Add(CreateSchemaElement("Label", ScriptSchemaStandardType.Label));

			// Agent scope
			macroChoice.Items.Add(CreateSchemaElement("Node", ScriptSchemaStandardType.Node));

			// Node scope
			macroChoice.Items.Add(CreateDoElement(ScriptSchemaStandardType.NodeBody));
			macroChoice.Items.Add(CreateSwitchElement(ScriptSchemaStandardType.NodeBody));
			macroChoice.Items.Add(CreateForEachElement(ScriptSchemaStandardType.NodeBody));
			foreach (KeyValuePair<string, XmlSchemaComplexType> pair in taskNameToType.OrderBy(x => x.Key))
			{
				macroChoice.Items.Add(CreateSchemaElement(pair.Key, new XmlQualifiedName(pair.Value.Name, NamespaceUri)));
			}

			XmlSchemaComplexType nodeType = new XmlSchemaComplexType();
			nodeType.Name = GetTypeName(ScriptSchemaStandardType.MacroBody);
			nodeType.Particle = macroChoice;
			return nodeType;
		}

		/// <summary>
		/// Creates the schema type representing the macro type
		/// </summary>
		/// <returns>Type definition for a node</returns>
		static XmlSchemaType CreateExtendType()
		{
			XmlSchemaComplexContentExtension extension = new XmlSchemaComplexContentExtension();
			extension.BaseTypeName = GetQualifiedTypeName(ScriptSchemaStandardType.MacroBody);
			extension.Attributes.Add(CreateSchemaAttribute("Name", ScriptSchemaStandardType.Name, XmlSchemaUse.Required));

			XmlSchemaComplexContent contentModel = new XmlSchemaComplexContent();
			contentModel.Content = extension;

			XmlSchemaComplexType complexType = new XmlSchemaComplexType();
			complexType.Name = GetTypeName(ScriptSchemaStandardType.Extend);
			complexType.ContentModel = contentModel;
			return complexType;
		}

		/// <summary>
		/// Creates the schema type representing a macro expansion
		/// </summary>
		/// <returns>Type definition for expanding a macro</returns>
		static XmlSchemaType CreateExpandType()
		{
			XmlSchemaAnyAttribute anyAttribute = new XmlSchemaAnyAttribute();
			anyAttribute.ProcessContents = XmlSchemaContentProcessing.Skip;

			XmlSchemaComplexType propertyType = new XmlSchemaComplexType();
			propertyType.Name = GetTypeName(ScriptSchemaStandardType.Expand);
			propertyType.Attributes.Add(CreateSchemaAttribute("Name", ScriptSchemaStandardType.BalancedString, XmlSchemaUse.Required));
			propertyType.Attributes.Add(CreateSchemaAttribute("If", ScriptSchemaStandardType.BalancedString, XmlSchemaUse.Optional));
			propertyType.AnyAttribute = anyAttribute;
			return propertyType;
		}

		/// <summary>
		/// Creates the schema type representing a warning or error type
		/// </summary>
		/// <returns>Type definition for a warning</returns>
		static XmlSchemaType CreateDiagnosticType(ScriptSchemaStandardType standardType)
		{
			XmlSchemaComplexType propertyType = new XmlSchemaComplexType();
			propertyType.Name = GetTypeName(standardType);
			propertyType.Attributes.Add(CreateSchemaAttribute("Message", ScriptSchemaStandardType.BalancedString, XmlSchemaUse.Required));
			propertyType.Attributes.Add(CreateSchemaAttribute("If", ScriptSchemaStandardType.BalancedString, XmlSchemaUse.Optional));
			propertyType.Attributes.Add(CreateSchemaAttribute("ReportOnExecution", ScriptSchemaStandardType.Boolean, XmlSchemaUse.Optional));
			propertyType.Attributes.Add(CreateSchemaAttribute("ReportOnNodes", ScriptSchemaStandardType.NameList, XmlSchemaUse.Optional));
			return propertyType;
		}

		/// <summary>
		/// Creates an element representing a conditional "Do" block, which recursively contains another type
		/// </summary>
		/// <param name="innerType">The base type for the do block to contain</param>
		/// <returns>New schema element for the block</returns>
		static XmlSchemaElement CreateDoElement(ScriptSchemaStandardType innerType)
		{
			XmlSchemaComplexContentExtension extension = new XmlSchemaComplexContentExtension();
			extension.BaseTypeName = GetQualifiedTypeName(innerType);
			extension.Attributes.Add(CreateSchemaAttribute("If", ScriptSchemaStandardType.BalancedString, XmlSchemaUse.Optional));

			XmlSchemaComplexContent contentModel = new XmlSchemaComplexContent();
			contentModel.Content = extension;

			XmlSchemaComplexType schemaType = new XmlSchemaComplexType();
			schemaType.ContentModel = contentModel;

			XmlSchemaElement element = new XmlSchemaElement();
			element.Name = "Do";
			element.SchemaType = schemaType;
			return element;
		}

		/// <summary>
		/// Creates an element representing a conditional "Switch" block, which recursively contains another type
		/// </summary>
		/// <param name="innerType">The base type for the do block to contain</param>
		/// <returns>New schema element for the block</returns>
		static XmlSchemaElement CreateSwitchElement(ScriptSchemaStandardType innerType)
		{
			// Create the "Option" element
			XmlSchemaComplexContentExtension caseExtension = new XmlSchemaComplexContentExtension();
			caseExtension.BaseTypeName = GetQualifiedTypeName(innerType);
			caseExtension.Attributes.Add(CreateSchemaAttribute("If", ScriptSchemaStandardType.BalancedString, XmlSchemaUse.Required));

			XmlSchemaComplexContent caseContentModel = new XmlSchemaComplexContent();
			caseContentModel.Content = caseExtension;

			XmlSchemaComplexType caseSchemaType = new XmlSchemaComplexType();
			caseSchemaType.ContentModel = caseContentModel;

			XmlSchemaElement caseElement = new XmlSchemaElement();
			caseElement.Name = "Case";
			caseElement.SchemaType = caseSchemaType;
			caseElement.MinOccurs = 0;
			caseElement.MaxOccursString = "unbounded";

			// Create the "Otherwise" element
			XmlSchemaElement otherwiseElement = new XmlSchemaElement();
			otherwiseElement.Name = "Default";
			otherwiseElement.SchemaTypeName = GetQualifiedTypeName(innerType);
			otherwiseElement.MinOccurs = 0;
			otherwiseElement.MaxOccurs = 1;

			// Create the "Switch" element
			XmlSchemaSequence switchSequence = new XmlSchemaSequence();
			switchSequence.Items.Add(caseElement);
			switchSequence.Items.Add(otherwiseElement);

			XmlSchemaComplexType switchSchemaType = new XmlSchemaComplexType();
			switchSchemaType.Particle = switchSequence;

			XmlSchemaElement switchElement = new XmlSchemaElement();
			switchElement.Name = "Switch";
			switchElement.SchemaType = switchSchemaType;
			return switchElement;
		}

		/// <summary>
		/// Creates an element representing a conditional "ForEach" block, which recursively contains another type
		/// </summary>
		/// <param name="innerType">The base type for the foreach block to contain</param>
		/// <returns>New schema element for the block</returns>
		static XmlSchemaElement CreateForEachElement(ScriptSchemaStandardType innerType)
		{
			XmlSchemaComplexContentExtension extension = new XmlSchemaComplexContentExtension();
			extension.BaseTypeName = GetQualifiedTypeName(innerType);
			extension.Attributes.Add(CreateSchemaAttribute("Name", ScriptSchemaStandardType.BalancedString, XmlSchemaUse.Required));
			extension.Attributes.Add(CreateSchemaAttribute("Values", ScriptSchemaStandardType.BalancedString, XmlSchemaUse.Required));
			extension.Attributes.Add(CreateSchemaAttribute("Separator", ScriptSchemaStandardType.BalancedString, XmlSchemaUse.Optional));
			extension.Attributes.Add(CreateSchemaAttribute("If", ScriptSchemaStandardType.BalancedString, XmlSchemaUse.Optional));

			XmlSchemaComplexContent contentModel = new XmlSchemaComplexContent();
			contentModel.Content = extension;

			XmlSchemaComplexType schemaType = new XmlSchemaComplexType();
			schemaType.ContentModel = contentModel;

			XmlSchemaElement element = new XmlSchemaElement();
			element.Name = "ForEach";
			element.SchemaType = schemaType;
			return element;
		}

		/// <summary>
		/// Constructs an XmlSchemaElement and initializes it with the given parameters
		/// </summary>
		/// <param name="name">Element name</param>
		/// <param name="schemaType">Type enumeration for the attribute</param>
		/// <returns>A new XmlSchemaElement object</returns>
		static XmlSchemaElement CreateSchemaElement(string name, ScriptSchemaStandardType schemaType)
		{
			return CreateSchemaElement(name, GetQualifiedTypeName(schemaType));
		}

		/// <summary>
		/// Constructs an XmlSchemaElement and initializes it with the given parameters
		/// </summary>
		/// <param name="name">Element name</param>
		/// <param name="schemaTypeName">Qualified name of the type for this element</param>
		/// <returns>A new XmlSchemaElement object</returns>
		static XmlSchemaElement CreateSchemaElement(string name, XmlQualifiedName schemaTypeName)
		{
			XmlSchemaElement element = new XmlSchemaElement();
			element.Name = name;
			element.SchemaTypeName = schemaTypeName;
			return element;
		}

		/// <summary>
		/// Constructs an XmlSchemaAttribute and initialize it with the given parameters
		/// </summary>
		/// <param name="name">The attribute name</param>
		/// <param name="schemaType">Type enumeration for the attribute</param>
		/// <param name="use">Whether the attribute is required or optional</param>
		/// <returns>A new XmlSchemaAttribute object</returns>
		static XmlSchemaAttribute CreateSchemaAttribute(string name, ScriptSchemaStandardType schemaType, XmlSchemaUse use)
		{
			return CreateSchemaAttribute(name, GetQualifiedTypeName(schemaType), use);
		}

		/// <summary>
		/// Constructs an XmlSchemaAttribute and initialize it with the given parameters
		/// </summary>
		/// <param name="name">The attribute name</param>
		/// <param name="schemaTypeName">Qualified name of the type for this attribute</param>
		/// <param name="use">Whether the attribute is required or optional</param>
		/// <returns>The new attribute</returns>
		static XmlSchemaAttribute CreateSchemaAttribute(string name, XmlQualifiedName schemaTypeName, XmlSchemaUse use)
		{
			XmlSchemaAttribute attribute = new XmlSchemaAttribute();
			attribute.Name = name;
			attribute.SchemaTypeName = schemaTypeName;
			attribute.Use = use;
			return attribute;
		}

		/// <summary>
		/// Creates a simple type that is the union of two other types
		/// </summary>
		/// <param name="name">The name of the type</param>
		/// <param name="validTypes">List of valid types for the union</param>
		/// <returns>A simple type which will match the given pattern</returns>
		static XmlSchemaSimpleType CreateSimpleTypeFromUnion(string name, params XmlSchemaType[] validTypes)
		{
			XmlSchemaSimpleTypeUnion union = new XmlSchemaSimpleTypeUnion();
			foreach (XmlSchemaType validType in validTypes)
			{
				union.BaseTypes.Add(validType);
			}

			XmlSchemaSimpleType unionType = new XmlSchemaSimpleType();
			unionType.Name = name;
			unionType.Content = union;
			return unionType;
		}

		/// <summary>
		/// Creates a simple type that matches a regex
		/// </summary>
		/// <param name="name">Name of the new type</param>
		/// <param name="pattern">Regex pattern to match</param>
		/// <returns>A simple type which will match the given pattern</returns>
		static XmlSchemaSimpleType CreateSimpleTypeFromRegex(string? name, string pattern)
		{
			XmlSchemaPatternFacet patternFacet = new XmlSchemaPatternFacet();
			patternFacet.Value = pattern;

			XmlSchemaSimpleTypeRestriction restriction = new XmlSchemaSimpleTypeRestriction();
			restriction.BaseTypeName = s_stringTypeName;
			restriction.Facets.Add(patternFacet);

			XmlSchemaSimpleType simpleType = new XmlSchemaSimpleType();
			simpleType.Name = name;
			simpleType.Content = restriction;
			return simpleType;
		}

		/// <summary>
		/// Create a schema type for the given user type. Currently only handles enumerations.
		/// </summary>
		/// <param name="name">Name for the new type</param>
		/// <param name="type">CLR type information to create a schema type for</param>
		static XmlSchemaType CreateUserType(string name, Type type)
		{
			if (type.IsEnum)
			{
				return CreateSimpleTypeFromUnion(name, CreateEnumType(null, type), CreateSimpleTypeFromRegex(null, StringWithPropertiesPattern));
			}
			else
			{
				throw new Exception($"Cannot create custom type in schema for '{type.Name}'");
			}
		}

		/// <summary>
		/// Create a schema type for the given enum.
		/// </summary>
		/// <param name="name">Name for the new type</param>
		/// <param name="type">CLR type information to create a schema type for</param>
		static XmlSchemaType CreateEnumType(string? name, Type type)
		{
			XmlSchemaSimpleTypeRestriction restriction = new XmlSchemaSimpleTypeRestriction();
			restriction.BaseTypeName = s_stringTypeName;

			foreach (string enumName in Enum.GetNames(type))
			{
				XmlSchemaEnumerationFacet facet = new XmlSchemaEnumerationFacet();
				facet.Value = enumName;
				restriction.Facets.Add(facet);
			}

			XmlSchemaSimpleType schemaType = new XmlSchemaSimpleType();
			schemaType.Name = name;
			schemaType.Content = restriction;
			return schemaType;
		}
	}
}