// Copyright Epic Games, Inc. All Rights Reserved.

#nullable enable

namespace AutomationTool
{
#if false
	/// <summary>
	/// Base class for binding and executing nodes
	/// </summary>
	abstract class BgNodeExecutor
	{
		public abstract Task<bool> ExecuteAsync(JobContext Job, Dictionary<string, HashSet<FileReference>> TagNameToFileSet);
	}

	class BgExpressionNodeExecutor : BgNodeExecutor
	{
		readonly BgExpressionNode Node;
		MethodInfo? method;

		public BgExpressionNodeExecutor(BgExpressionNode node)
		{
			Node = node;
		}

	/// <summary>
	/// Implementation of <see cref="BgNode"/> for graphs defined through XML syntax
	/// </summary>
	class BgScriptNode : BgNode
	{
		/// <summary>
		/// List of tasks to execute
		/// </summary>
		public List<BgTask> Tasks { get; } = new List<BgTask>();

		/// <summary>
		/// List of bound task implementations
		/// </summary>
		List<BgTaskImpl> _boundTasks = new List<BgTaskImpl>();

		/// <summary>
		/// Constructor
		/// </summary>
		public BgScriptNodeExecutor(string name, IReadOnlyList<BgNodeOutput> inputs, IReadOnlyList<string> outputNames, IReadOnlyList<BgNode> inputDependencies, IReadOnlyList<BgNode> orderDependencies, IReadOnlyList<FileReference> requiredTokens)
			: base(name, inputs, outputNames, inputDependencies, orderDependencies, requiredTokens)
		{
		}

		public bool Bind(Dictionary<string, ScriptTaskBinding> nameToTask, Dictionary<string, BgNodeOutput> tagNameToNodeOutput, IBgScriptReaderContext context, ILogger logger)
		{
			bool bResult = true;
			foreach (BgTask TaskInfo in Node.Tasks)
			{
				BgTaskImpl? boundTask = BindTask(TaskInfo, nameToTask, tagNameToNodeOutput, context, logger);
				if (boundTask == null)
				{
					bResult = false;
				}
				else
				{
					_boundTasks.Add(boundTask);
				}
			}
			return bResult;
		}

		BgTaskImpl? BindTask(BgTask TaskInfo, Dictionary<string, ScriptTaskBinding> NameToTask, IReadOnlyDictionary<string, BgNodeOutput> TagNameToNodeOutput, IBgScriptReaderContext Context, ILogger Logger)
		{
			// Get the reflection info for this element
			ScriptTaskBinding? Task;
			if (!NameToTask.TryGetValue(TaskInfo.Name, out Task))
			{
				Logger.LogScriptError(TaskInfo.Location, "Unknown task '{TaskName}'", TaskInfo.Name);
				return null;
			}

			// Check all the required parameters are present
			bool bHasRequiredAttributes = true;
			foreach (ScriptTaskParameterBinding Parameter in Task.NameToParameter.Values)
			{
				if (!Parameter.Optional && !TaskInfo.Arguments.ContainsKey(Parameter.Name))
				{
					Logger.LogScriptError(TaskInfo.Location, "Missing required attribute - {AttrName}", Parameter.Name);
					bHasRequiredAttributes = false;
				}
			}

			// Read all the attributes into a parameters object for this task
			object ParametersObject = Activator.CreateInstance(Task.ParametersClass)!;
			foreach ((string Name, string Value) in TaskInfo.Arguments)
			{
				// Get the field that this attribute should be written to in the parameters object
				ScriptTaskParameterBinding? Parameter;
				if (!Task.NameToParameter.TryGetValue(Name, out Parameter))
				{
					Logger.LogScriptError(TaskInfo.Location, "Unknown attribute '{AttrName}'", Name);
					continue;
				}

				// If it's a collection type, split it into separate values
				if (Parameter.CollectionType == null)
				{
					// Parse it and assign it to the parameters object
					object? FieldValue = ParseValue(Value, Parameter.ValueType, Context);
					Parameter.FieldInfo.SetValue(ParametersObject, FieldValue);
				}
				else
				{
					// Get the collection, or create one if necessary
					object? CollectionValue = Parameter.FieldInfo.GetValue(ParametersObject);
					if (CollectionValue == null)
					{
						CollectionValue = Activator.CreateInstance(Parameter.FieldInfo.FieldType)!;
						Parameter.FieldInfo.SetValue(ParametersObject, CollectionValue);
					}

					// Parse the values and add them to the collection
					List<string> ValueStrings = BgTaskImpl.SplitDelimitedList(Value);
					foreach (string ValueString in ValueStrings)
					{
						object ElementValue = ParseValue(ValueString, Parameter.ValueType, Context)!;
						Parameter.CollectionType.InvokeMember("Add", BindingFlags.InvokeMethod | BindingFlags.Instance | BindingFlags.Public, null, CollectionValue, new object[] { ElementValue });
					}
				}
			}

			// Construct the task
			if (!bHasRequiredAttributes)
			{
				return null;
			}

			// Add it to the list
			BgTaskImpl NewTask = (BgTaskImpl)Activator.CreateInstance(Task.TaskClass, ParametersObject)!;

			// Set up the source location for diagnostics
			NewTask.SourceLocation = TaskInfo.Location;

			// Make sure all the read tags are local or listed as a dependency
			foreach (string ReadTagName in NewTask.FindConsumedTagNames())
			{
				BgNodeOutput? Output;
				if (TagNameToNodeOutput.TryGetValue(ReadTagName, out Output))
				{
					if (Output != null && Output.ProducingNode != Node && !Node.Inputs.Contains(Output))
					{
						Logger.LogScriptError(TaskInfo.Location, "The tag '{TagName}' is not a dependency of node '{Node}'", ReadTagName, Node.Name);
					}
				}
			}

			// Make sure all the written tags are local or listed as an output
			foreach (string ModifiedTagName in NewTask.FindProducedTagNames())
			{
				BgNodeOutput? Output;
				if (TagNameToNodeOutput.TryGetValue(ModifiedTagName, out Output))
				{
					if (Output != null && !Node.Outputs.Contains(Output))
					{
						Logger.LogScriptError(TaskInfo.Location, "The tag '{TagName}' is created by '{Node}', and cannot be modified downstream", Output.TagName, Output.ProducingNode.Name);
					}
				}
			}
			return NewTask;
		}

		/// <summary>
		/// Parse a value of the given type
		/// </summary>
		/// <param name="ValueText">The text to parse</param>
		/// <param name="ValueType">Type of the value to parse</param>
		/// <param name="Context">Context for the script reader</param>
		/// <returns>Value that was parsed</returns>
		static object? ParseValue(string ValueText, Type ValueType, IBgScriptReaderContext Context)
		{
			// Parse it and assign it to the parameters object
			if (ValueType.IsEnum)
			{
				return Enum.Parse(ValueType, ValueText);
			}
			else if (ValueType == typeof(Boolean))
			{
				return BgCondition.EvaluateAsync(ValueText, Context).Result;
			}
			else if (ValueType == typeof(FileReference))
			{
				return BgTaskImpl.ResolveFile(ValueText);
			}
			else if (ValueType == typeof(DirectoryReference))
			{
				return BgTaskImpl.ResolveDirectory(ValueText);
			}

			TypeConverter Converter = TypeDescriptor.GetConverter(ValueType);
			if (Converter.CanConvertFrom(typeof(string)))
			{
				return Converter.ConvertFromString(ValueText);
			}
			else
			{
				return Convert.ChangeType(ValueText, ValueType);
			}
		}

		/// <summary>
		/// Build all the tasks for this node
		/// </summary>
		/// <param name="Job">Information about the current job</param>
		/// <param name="TagNameToFileSet">Mapping from tag names to the set of files they include. Should be set to contain the node inputs on entry.</param>
		/// <returns>Whether the task succeeded or not. Exiting with an exception will be caught and treated as a failure.</returns>
		public override async Task<bool> ExecuteAsync(JobContext Job, Dictionary<string, HashSet<FileReference>> TagNameToFileSet)
		{
			// Run each of the tasks in order
			HashSet<FileReference> BuildProducts = TagNameToFileSet[Node.DefaultOutput.TagName];
			for (int Idx = 0; Idx < _boundTasks.Count; Idx++)
			{
				using (IScope Scope = GlobalTracer.Instance.BuildSpan("Task").WithTag("resource", _boundTasks[Idx].GetTraceName()).StartActive())
				{
					ITaskExecutor Executor = _boundTasks[Idx].GetExecutor();
					if (Executor == null)
					{
						// Execute this task directly
						try
						{
							_boundTasks[Idx].GetTraceMetadata(Scope.Span, "");
							await _boundTasks[Idx].ExecuteAsync(Job, BuildProducts, TagNameToFileSet);
						}
						catch (Exception Ex)
						{
							ExceptionUtils.AddContext(Ex, "while executing task {0}", _boundTasks[Idx].GetTraceString());
							if (_boundTasks[Idx].SourceLocation != null)
							{
								ExceptionUtils.AddContext(Ex, "at {0}({1})", _boundTasks[Idx].SourceLocation.File, _boundTasks[Idx].SourceLocation.LineNumber);
							}
							throw;
						}
					}
					else
					{
						_boundTasks[Idx].GetTraceMetadata(Scope.Span, "1.");

						// The task has a custom executor, which may be able to execute several tasks simultaneously. Try to add the following tasks.
						int FirstIdx = Idx;
						while (Idx + 1 < Node.Tasks.Count && Executor.Add(_boundTasks[Idx + 1]))
						{
							Idx++;
							_boundTasks[Idx].GetTraceMetadata(Scope.Span, string.Format("{0}.", 1 + Idx - FirstIdx));
						}
						try
						{
							await Executor.ExecuteAsync(Job, BuildProducts, TagNameToFileSet);
						}
						catch (Exception Ex)
						{
							for (int TaskIdx = FirstIdx; TaskIdx <= Idx; TaskIdx++)
							{
								ExceptionUtils.AddContext(Ex, "while executing {0}", _boundTasks[TaskIdx].GetTraceString());
							}
							if (_boundTasks[FirstIdx].SourceLocation != null)
							{
								ExceptionUtils.AddContext(Ex, "at {0}({1})", _boundTasks[FirstIdx].SourceLocation.File, _boundTasks[FirstIdx].SourceLocation.LineNumber);
							}
							throw;
						}
					}
				}
			}

			// Remove anything that doesn't exist, since these files weren't explicitly tagged
			BuildProducts.RemoveWhere(x => !FileReference.Exists(x));
			return true;
		}
	}
#endif
}
