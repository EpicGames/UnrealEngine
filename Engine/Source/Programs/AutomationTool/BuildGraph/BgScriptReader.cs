// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Diagnostics.CodeAnalysis;
using System.IO;
using System.Linq;
using System.Text;
using System.Text.RegularExpressions;
using System.Threading.Tasks;
using System.Xml;
using System.Xml.Schema;
using AutomationTool.Tasks;
using EpicGames.BuildGraph;
using EpicGames.Core;
using Microsoft.Extensions.Logging;

#nullable enable

namespace AutomationTool
{
	/// <summary>
	/// Location of an element within a file
	/// </summary>
	public class BgScriptLocation
	{
		/// <summary>
		/// The file containing this element
		/// </summary>
		public FileReference File { get; }

		/// <summary>
		/// The line number containing this element
		/// </summary>
		public int LineNumber { get; }

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="file"></param>
		/// <param name="lineNumber"></param>
		public BgScriptLocation(FileReference file, int lineNumber)
		{
			File = file;
			LineNumber = lineNumber;
		}
	}

	/// <summary>
	/// Implementation of XmlDocument which preserves line numbers for its elements
	/// </summary>
	public class BgScriptDocument : XmlDocument
	{
		/// <summary>
		/// The file being read
		/// </summary>
		FileReference File { get; }

		/// <summary>
		/// Interface to the LineInfo on the active XmlReader
		/// </summary>
		IXmlLineInfo? _lineInfo;

		/// <summary>
		/// Set to true if the reader encounters an error
		/// </summary>
		bool _hasErrors;

		/// <summary>
		/// Logger for validation errors
		/// </summary>
		ILogger Logger { get; }

		/// <summary>
		/// Private constructor. Use ScriptDocument.Load to read an XML document.
		/// </summary>
		BgScriptDocument(FileReference file, ILogger logger)
		{
			File = file;
			Logger = logger;
		}

		/// <summary>
		/// Overrides XmlDocument.CreateElement() to construct ScriptElements rather than XmlElements
		/// </summary>
		public override XmlElement CreateElement(string? prefix, string? localName, string? namespaceUri)
		{
			BgScriptLocation location = new BgScriptLocation(File, _lineInfo!.LineNumber);
			return new BgScriptElement(location, prefix!, localName!, namespaceUri!, this);
		}

		/// <summary>
		/// Loads a script document from the given file
		/// </summary>
		/// <param name="file">The file to load</param>
		/// <param name="data"></param>
		/// <param name="schema">The schema to validate against</param>
		/// <param name="logger">Logger for output messages</param>
		/// <param name="outDocument">If successful, the document that was read</param>
		/// <returns>True if the document could be read, false otherwise</returns>
		public static bool TryRead(FileReference file, byte[] data, BgScriptSchema? schema, ILogger logger, [NotNullWhen(true)] out BgScriptDocument? outDocument)
		{
			BgScriptDocument document = new BgScriptDocument(file, logger);

			XmlReaderSettings settings = new XmlReaderSettings();
			if (schema != null)
			{
				settings.Schemas.Add(schema.CompiledSchema);
				settings.ValidationType = ValidationType.Schema;
				settings.ValidationEventHandler += document.ValidationEvent;
			}

			using (MemoryStream stream = new MemoryStream(data))
			using (XmlReader reader = XmlReader.Create(stream, settings))
			{
				// Read the document
				document._lineInfo = (IXmlLineInfo)reader;
				try
				{
					document.Load(reader);
				}
				catch (XmlException ex)
				{
					if (!document._hasErrors)
					{
						BgScriptLocation location = new BgScriptLocation(file, ex.LineNumber);
						logger.LogScriptError(location, "{Message}", ex.Message);
						document._hasErrors = true;
					}
				}

				// If we hit any errors while parsing
				if (document._hasErrors)
				{
					outDocument = null;
					return false;
				}

				// Check that the root element is valid. If not, we didn't actually validate against the schema.
				if (document.DocumentElement!.Name != BgScriptSchema.RootElementName)
				{
					BgScriptLocation location = new BgScriptLocation(file, 1);
					logger.LogScriptError(location, "Script does not have a root element called '{ElementName}'", BgScriptSchema.RootElementName);
					outDocument = null;
					return false;
				}
				if (document.DocumentElement.NamespaceURI != BgScriptSchema.NamespaceUri)
				{
					BgScriptLocation location = new BgScriptLocation(file, 1);
					logger.LogScriptError(location, "Script root element is not in the '{Namespace}' namespace (add the xmlns=\"{NewNamespace}\" attribute)", BgScriptSchema.NamespaceUri, BgScriptSchema.NamespaceUri);
					outDocument = null;
					return false;
				}
			}

			outDocument = document;
			return true;
		}

		/// <summary>
		/// Callback for validation errors in the document
		/// </summary>
		/// <param name="sender">Standard argument for ValidationEventHandler</param>
		/// <param name="args">Standard argument for ValidationEventHandler</param>
		void ValidationEvent(object? sender, ValidationEventArgs args)
		{
			BgScriptLocation location = new BgScriptLocation(File, args.Exception.LineNumber);
			if (args.Severity == XmlSeverityType.Warning)
			{
				Logger.LogScriptWarning(location, "{Message}", args.Message);
			}
			else
			{
				Logger.LogScriptError(location, "{Message}", args.Message);
				_hasErrors = true;
			}
		}
	}

	/// <summary>
	/// Implementation of XmlElement which preserves line numbers
	/// </summary>
	public class BgScriptElement : XmlElement
	{
		/// <summary>
		/// Location of the element within the file
		/// </summary>
		public BgScriptLocation Location { get; }

		/// <summary>
		/// Constructor
		/// </summary>
		public BgScriptElement(BgScriptLocation location, string prefix, string localName, string namespaceUri, BgScriptDocument document)
			: base(prefix, localName, namespaceUri, document)
		{
			Location = location;
		}
	}

	/// <summary>
	/// Stores information about a script function that has been declared
	/// </summary>
	class BgScriptMacro
	{
		/// <summary>
		/// Name of the function
		/// </summary>
		public string Name { get; }

		/// <summary>
		/// Element where the function was declared
		/// </summary>
		public List<BgScriptElement> Elements { get; } = new List<BgScriptElement>();

		/// <summary>
		/// The total number of arguments
		/// </summary>
		public int NumArguments { get; }

		/// <summary>
		/// Number of arguments that are required
		/// </summary>
		public int NumRequiredArguments { get; }

		/// <summary>
		/// Maps an argument name to its type
		/// </summary>
		public IReadOnlyDictionary<string, int> ArgumentNameToIndex { get; }

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="name">Name of the function</param>
		/// <param name="element">Element containing the function definition</param>
		/// <param name="argumentNameToIndex">Map of argument name to index</param>
		/// <param name="numRequiredArguments">Number of arguments that are required. Indices 0 to NumRequiredArguments - 1 are required.</param>
		public BgScriptMacro(string name, BgScriptElement element, Dictionary<string, int> argumentNameToIndex, int numRequiredArguments)
		{
			Name = name;
			Elements.Add(element);
			NumArguments = argumentNameToIndex.Count;
			NumRequiredArguments = numRequiredArguments;
			ArgumentNameToIndex = argumentNameToIndex;
		}
	}

	/// <summary>
	/// Extension methods for writing script error messages
	/// </summary>
	public static class BgScriptExtensions
	{
		/// <summary>
		/// Utility method to log a script error at a particular location
		/// </summary>
		public static void LogScriptError(this ILogger logger, BgScriptLocation location, string format, params object[] args)
		{
			object[] allArgs = new object[args.Length + 2];
			allArgs[0] = location.File;
			allArgs[1] = location.LineNumber;
			args.CopyTo(allArgs, 2);

#pragma warning disable CA2254 // Message templates should be constant
			logger.LogError(KnownLogEvents.AutomationTool_BuildGraphScript, $"{{Script}}({{Line}}): error: {format}", allArgs);
#pragma warning restore CA2254
		}

		/// <summary>
		/// Utility method to log a script warning at a particular location
		/// </summary>
		public static void LogScriptWarning(this ILogger logger, BgScriptLocation location, string format, params object[] args)
		{
			object[] allArgs = new object[args.Length + 2];
			allArgs[0] = location.File;
			allArgs[1] = location.LineNumber;
			args.CopyTo(allArgs, 2);

#pragma warning disable CA2254 // Message templates should be constant
			logger.LogWarning(KnownLogEvents.AutomationTool_BuildGraphScript, $"{{Script}}({{Line}}): warning: {format}", allArgs);
#pragma warning restore CA2254
		}
	}

	/// <summary>
	/// Overridden version of <see cref="BgNodeDef"/> which contains a list of tasks
	/// </summary>
	class BgScriptNode : BgNodeDef
	{
		/// <summary>
		/// List of tasks to execute
		/// </summary>
		public List<BgTask> Tasks { get; } = new List<BgTask>();

		/// <summary>
		/// Constructor
		/// </summary>
		public BgScriptNode(string name, IReadOnlyList<BgNodeOutput> inputs, IReadOnlyList<BgNodeOutput> optionalInputs, IReadOnlyList<string> outputNames, IReadOnlyList<BgNodeDef> inputDependencies, IReadOnlyList<BgNodeDef> optionalInputDependencies, IReadOnlyList<BgNodeDef> orderDependencies, IReadOnlyList<FileReference> requiredTokens, IReadOnlyList<string> ignoreModified)
			: base(name, inputs, optionalInputs, outputNames, inputDependencies, optionalInputDependencies, orderDependencies, requiredTokens, ignoreModified)
		{
		}
	}

	/// <summary>
	/// Reader for build graph definitions. Instanced to contain temporary state; public interface is through ScriptReader.TryRead().
	/// </summary>
	public class BgScriptReader
	{
		/// <summary>
		/// Root directory to resolve relative paths against
		/// </summary>
		readonly DirectoryReference _rootDir;

		/// <summary>
		/// List of property name to value lookups. Modifications to properties are scoped to nodes and agents. EnterScope() pushes an empty dictionary onto the end of this list, and LeaveScope() removes one. 
		/// ExpandProperties() searches from last to first lookup when trying to resolve a property name, and takes the first it finds.
		/// </summary>
		protected List<Dictionary<string, string?>> ScopedProperties { get; } = new List<Dictionary<string, string?>>();

		/// <summary>
		/// When declaring a property in a nested scope, we enter its name into a set for each parent scope which prevents redeclaration in an OUTER scope later. Subsequent NESTED scopes can redeclare it.
		/// The former is likely a coding error, since it implies that the scope of the variable was meant to be further out, whereas the latter is common for temporary and loop variables.
		/// </summary>
		readonly List<HashSet<string>> _shadowProperties = new List<HashSet<string>>();

		/// <summary>
		/// Maps from a function name to its definition
		/// </summary>
		readonly Dictionary<string, BgScriptMacro> _macroNameToDefinition = new Dictionary<string, BgScriptMacro>();

		/// <summary>
		/// Set of scripts that have been previously read via an include element
		/// This is used to prevent duplicate reads of a script if it has been guarded against
		/// </summary>
		readonly HashSet<FileReference> _includedScripts = new HashSet<FileReference>();

		/// <summary>
		/// The current graph
		/// </summary>
		readonly BgGraphDef _graph = new BgGraphDef();

		/// <summary>
		/// Arguments for evaluating the graph
		/// </summary>
		readonly Dictionary<string, string> _arguments;

		/// <summary>
		/// The name of the node if only a single node is going to be built, otherwise null.
		/// </summary>
		readonly string? _singleNodeName;

		/// <summary>
		/// Schema for the script
		/// </summary>
		BgScriptSchema? Schema { get; }

		/// <summary>
		/// Logger for diagnostic messages
		/// </summary>
		protected ILogger Logger { get; }

		/// <summary>
		/// The number of errors encountered during processing so far
		/// </summary>
		public int NumErrors { get; private set; }

		BgAgentDef? _enclosingAgent;
		BgScriptNode? _enclosingNode;

		/// <summary>
		/// Private constructor. Use ScriptReader.TryRead() to read a script file.
		/// </summary>
		/// <param name="rootDir">Root directory to resolve relative paths against</param>
		/// <param name="defaultProperties">Default properties available to the script</param>
		/// <param name="arguments">Arguments passed in to the graph on the command line</param>
		/// <param name="singleNodeName">If a single node will be processed, the name of that node.</param>
		/// <param name="schema">Schema for the script</param>
		/// <param name="logger">Logger for diagnostic messages</param>
		protected BgScriptReader(DirectoryReference rootDir, IDictionary<string, string?> defaultProperties, IReadOnlyDictionary<string, string> arguments, string? singleNodeName, BgScriptSchema? schema, ILogger logger)
		{
			_rootDir = rootDir;

			Schema = schema;
			Logger = logger;

			EnterScope();

			_arguments = new Dictionary<string, string>(arguments, StringComparer.OrdinalIgnoreCase);
			_singleNodeName = singleNodeName;

			foreach (KeyValuePair<string, string?> pair in defaultProperties)
			{
				SetPropertyValue(null!, pair.Key, pair.Value);
			}
		}

		/// <summary>
		/// Try to read a script file from the given file.
		/// </summary>
		/// <param name="file">File to read from</param>
		/// <param name="rootDir">Root directory to resolve files to</param>
		/// <param name="arguments">Arguments passed in to the graph on the command line</param>
		/// <param name="defaultProperties">Default properties available to the script</param>
		/// <param name="schema">Schema for the script</param>
		/// <param name="logger">Logger for output messages</param>
		/// <param name="singleNodeName">If a single node will be processed, the name of that node.</param>
		/// <returns>True if the graph was read, false if there were errors</returns>
		public static async Task<BgGraphDef?> ReadAsync(FileReference file, DirectoryReference rootDir, Dictionary<string, string> arguments, Dictionary<string, string?> defaultProperties, BgScriptSchema? schema, ILogger logger, string? singleNodeName = null)
		{
			// Read the file and build the graph
			BgScriptReader reader = new BgScriptReader(rootDir, defaultProperties, arguments, singleNodeName, schema, logger);
			if (!await reader.TryReadAsync(file) || reader.NumErrors > 0)
			{
				return null;
			}

			// Make sure all the arguments were valid
			HashSet<string> validArgumentNames = new HashSet<string>(reader._graph.Options.Select(x => x.Name), StringComparer.OrdinalIgnoreCase);
			validArgumentNames.Add("PreflightChange");

			// All default properties are valid arguments too
			foreach (string defaultPropertyKey in defaultProperties.Keys)
			{
				validArgumentNames.Add(defaultPropertyKey);
			}

			bool hasInvalidArguments = false;
			foreach (string argumentName in arguments.Keys)
			{
				if (!validArgumentNames.Contains(argumentName))
				{
					hasInvalidArguments = true;
					logger.LogWarning(KnownLogEvents.AutomationTool_BuildGraphScript, "Unknown argument '{ArgumentName}' for '{Script}'", argumentName, file);
				}
			}

			if (hasInvalidArguments)
			{
				logger.LogInformation("Valid arguments for '{Script}': {Arguments}", file, String.Join(", ", validArgumentNames.OrderBy(x => x, StringComparer.OrdinalIgnoreCase)));
			}

			// Return the constructed graph
			return reader._graph;
		}

		/// <summary>
		/// Read the script from the given file
		/// </summary>
		/// <param name="file">File to read from</param>
		protected async Task<bool> TryReadAsync(FileReference file)
		{
			// Get the data for this file
			byte[]? data = await FileReference.ReadAllBytesAsync(file);
			if (data == null)
			{
				Logger.LogError("Unable to open file {File}", file);
				NumErrors++;
				return false;
			}

			// Read the document and validate it against the schema
			BgScriptDocument? document;
			if (!BgScriptDocument.TryRead(file, data, Schema, Logger, out document))
			{
				NumErrors++;
				return false;
			}

			_includedScripts.Add(file);

			// Read the root BuildGraph element
			await ReadGraphBodyAsync(document.DocumentElement!);
			return true;
		}

		/// <summary>
		/// Reads the contents of a graph
		/// </summary>
		/// <param name="element">The parent element to read from</param>
		async Task ReadGraphBodyAsync(XmlElement element)
		{
			foreach (BgScriptElement childElement in element.ChildNodes.OfType<BgScriptElement>())
			{
				switch (childElement.Name)
				{
					case "Include":
						await ReadIncludeAsync(childElement);
						break;
					case "Option":
						await ReadOptionAsync(childElement);
						break;
					case "Property":
						await ReadPropertyAsync(childElement);
						break;
					case "Regex":
						await ReadRegexAsync(childElement);
						break;
					case "StringOp":
						await ReadStringOpAsync(childElement);
						break;
					case "EnvVar":
						await ReadEnvVarAsync(childElement);
						break;
					case "Macro":
						ReadMacro(childElement);
						break;
					case "Extend":
						await ReadExtendAsync(childElement);
						break;
					case "Agent":
						await ReadAgentAsync(childElement);
						break;
					case "Aggregate":
						await ReadAggregateAsync(childElement);
						break;
					case "Artifact":
						await ReadArtifactAsync(childElement);
						break;
					case "Report":
						await ReadReportAsync(childElement);
						break;
					case "Badge":
						await ReadBadgeAsync(childElement);
						break;
					case "Label":
						await ReadLabelAsync(childElement);
						break;
					case "Notify":
						await ReadNotifierAsync(childElement);
						break;
					case "Trace":
						await ReadDiagnosticAsync(childElement, LogLevel.Information);
						break;
					case "Warning":
						await ReadDiagnosticAsync(childElement, LogLevel.Warning);
						break;
					case "Error":
						await ReadDiagnosticAsync(childElement, LogLevel.Error);
						break;
					case "Do":
						await ReadBlockAsync(childElement, ReadGraphBodyAsync);
						break;
					case "Switch":
						await ReadSwitchAsync(childElement, ReadGraphBodyAsync);
						break;
					case "ForEach":
						await ReadForEachAsync(childElement, ReadGraphBodyAsync);
						break;
					case "Expand":
						await ReadExpandAsync(childElement, ReadGraphBodyAsync);
						break;
					case "Annotate":
						await ReadAnnotationAsync(childElement);
						break;
					default:
						LogError(childElement, "Invalid element '{ElementName}'", childElement.Name);
						break;
				}
			}
		}

		/// <summary>
		/// Push a new property scope onto the stack
		/// </summary>
		protected void EnterScope()
		{
			ScopedProperties.Add(new Dictionary<string, string?>(StringComparer.OrdinalIgnoreCase));
			_shadowProperties.Add(new HashSet<string>(StringComparer.OrdinalIgnoreCase));
		}

		/// <summary>
		/// Pop a property scope from the stack
		/// </summary>
		protected void LeaveScope()
		{
			ScopedProperties.RemoveAt(ScopedProperties.Count - 1);
			_shadowProperties.RemoveAt(_shadowProperties.Count - 1);
		}

		/// <summary>
		/// Sets a property value in the current scope
		/// </summary>
		/// <param name="element">Element containing the property assignment. Used for error messages if the property is shadowed in another scope.</param>
		/// <param name="name">Name of the property</param>
		/// <param name="value">Value for the property</param>
		/// <param name="createInParentScope">If true, this property should be added to the parent scope and not the current scope. Cannot be used if the parent scope already contains a parameter with this name or if there is no parent scope</param>
		protected void SetPropertyValue(BgScriptElement element, string name, string? value, bool createInParentScope = false)
		{
			// Find the scope containing this property, defaulting to the current scope
			int scopeIdx = 0;
			while (scopeIdx < ScopedProperties.Count - 1 && !ScopedProperties[scopeIdx].ContainsKey(name))
			{
				scopeIdx++;
			}

			if (createInParentScope)
			{
				if (scopeIdx != ScopedProperties.Count - 1)
				{
					LogError(element, "Property '{PropertyName}' was already used in a parent scope but has CreateInParentScope=\"true\". Rename the property to avoid the conflict or disable CreateInParentScope.", name);
					return;
				}
				else if ((scopeIdx - 1) < 0)
				{
					LogError(element, "Property '{Propertyname}' has CreateInParentScope=\"true\" but has no parent scope.", name);
					return;
				}
				else
				{
					scopeIdx--;
				}
			}

			if (_shadowProperties[scopeIdx].Contains(name))
			{
				// Make sure this property name was not already used in a child scope; it likely indicates an error.
				LogError(element, "Property '{PropertyName}' was already used in a child scope. Move this definition before the previous usage if they are intended to share scope, or use a different name.", name);
			}
			else
			{
				// Make sure it's added to the shadow property list for every parent scope
				for (int idx = 0; idx < scopeIdx; idx++)
				{
					_shadowProperties[idx].Add(name);
				}
				ScopedProperties[scopeIdx][name] = value;
			}
		}

		/// <summary>
		/// Tries to get the value of a property
		/// </summary>
		/// <param name="name">Name of the property</param>
		/// <param name="value">On success, contains the value of the property. Set to null otherwise.</param>
		/// <returns>True if the property was found, false otherwise</returns>
		protected bool TryGetPropertyValue(string name, out string? value)
		{
			int valueLength = 0;
			if (name.Contains(':', StringComparison.Ordinal))
			{
				string[] tokens = name.Split(':');
				name = tokens[0];
				valueLength = Int32.Parse(tokens[1]);
			}

			// Check each scope for the property
			for (int scopeIdx = ScopedProperties.Count - 1; scopeIdx >= 0; scopeIdx--)
			{
				string? scopeValue;
				if (ScopedProperties[scopeIdx].TryGetValue(name, out scopeValue))
				{
					value = scopeValue;

					// It's valid for a property to exist but have a null value. It won't be expanded
					// Handle $(PropName:-6) where PropName might be "Foo"
					if (value != null && value.Length > Math.Abs(valueLength))
					{
						if (valueLength > 0)
						{
							value = value.Substring(0, valueLength);
						}
						if (valueLength < 0)
						{
							value = value.Substring(value.Length + valueLength, -valueLength);
						}
					}
					return true;
				}
			}

			// If we didn't find it, return false.
			value = null;
			return false;
		}

		/// <summary>
		/// Read an include directive, and the contents of the target file
		/// </summary>
		/// <param name="element">Xml element to read the definition from</param>
		async Task ReadIncludeAsync(BgScriptElement element)
		{
			if (await EvaluateConditionAsync(element))
			{
				string basePath = element.Location.File.MakeRelativeTo(_rootDir).Replace(Path.DirectorySeparatorChar, '/');

				HashSet<FileReference> files = new HashSet<FileReference>();
				foreach (string script in ReadListAttribute(element, "Script"))
				{
					string? includePath = CombinePaths(basePath, script);
					if (includePath == null)
					{
						LogError(element, $"Path '{script}' cannot be combined with '{basePath}'");
					}
					else if (Regex.IsMatch(includePath, @"\*|\?|\.\.\."))
					{
						files.UnionWith(FindMatchingFiles(_rootDir, includePath));
					}
					else
					{
						files.Add(new FileReference(includePath));
					}
				}

				foreach (FileReference file in files.OrderBy(x => x.FullName, StringComparer.OrdinalIgnoreCase))
				{
					// Check if duplicate includes should be guarded against
					bool ignoreDuplicateIncludes = false;
					const string AttributeName = "IgnoreDuplicateIncludes";
					if (element.HasAttribute(AttributeName))
					{
						try
						{
							string text = ExpandProperties(element, element.GetAttribute(AttributeName));
							if (await BgCondition.EvaluateAsync(text, new BgConditionContext(_rootDir)))
							{
								ignoreDuplicateIncludes = true;
							}
						}
						catch (BgConditionException ex)
						{
							LogError(element, "Error in condition: {Message}", ex.Message);
							return;
						}
					}

					if(!ignoreDuplicateIncludes || !_includedScripts.Contains(file))
					{
						Logger.LogDebug("Including file {File}", file);
						await TryReadAsync(file);
					}	
				}
			}
		}

		/// <summary>
		/// Combine two paths without validating the result
		/// </summary>
		static string? CombinePaths(string basePath, string nextPath)
		{
			if (Path.IsPathRooted(nextPath))
			{
				return nextPath;
			}

			List<string> fragments = new List<string>(basePath.Split('/'));
			fragments.RemoveAt(fragments.Count - 1);

			foreach (string appendFragment in nextPath.Split('/'))
			{
				if (appendFragment.Equals(".", StringComparison.Ordinal))
				{
					continue;
				}
				else if (appendFragment.Equals("..", StringComparison.Ordinal))
				{
					if (fragments.Count > 0)
					{
						fragments.RemoveAt(fragments.Count - 1);
					}
					else
					{
						return null;
					}
				}
				else
				{
					fragments.Add(appendFragment);
				}
			}
			return String.Join('/', fragments);
		}

		/// <summary>
		/// Find files matching a pattern
		/// </summary>
		static IEnumerable<FileReference> FindMatchingFiles(DirectoryReference rootDir, string pattern)
		{
			FileFilter filter = new FileFilter();
			filter.AddRule(pattern, FileFilterType.Include);

			return filter.ApplyToDirectory(rootDir, true);
		}

		/// <summary>
		/// Reads the definition of a graph option; a parameter which can be set by the user on the command-line or via an environment variable.
		/// </summary>
		/// <param name="element">Xml element to read the definition from</param>
		async Task ReadOptionAsync(BgScriptElement element)
		{
			if (await EvaluateConditionAsync(element))
			{
				string name = ReadAttribute(element, "Name");
				if (ValidateName(element, name))
				{
					// Make sure we're at global scope
					if (ScopedProperties.Count > 1)
					{
						LogError(element, "Incorrect scope depth for reading option settings");
						return;
					}

					// Check if the property already exists. If it does, we don't need to register it as an option.
					string? existingValue;
					if (TryGetPropertyValue(name, out existingValue) && existingValue != null)
					{
						// If there's a restriction on this definition, check it matches
						string restrict = ReadAttribute(element, "Restrict");
						if (!String.IsNullOrEmpty(restrict) && !Regex.IsMatch(existingValue, "^" + restrict + "$", RegexOptions.IgnoreCase))
						{
							LogError(element, "'{Name} is already set to '{ExistingValue}', which does not match the given restriction ('{Restrict}')", name, existingValue, restrict);
						}
					}
					else
					{
						// Create a new option object to store the settings
						BgStringOptionDef option = new BgStringOptionDef(name);
						option.Description = ReadAttribute(element, "Description");
						option.DefaultValue = ReadAttribute(element, "DefaultValue");
						_graph.Options.Add(option);

						// Get the value of this property
						string? value;
						if (!_arguments.TryGetValue(name, out value))
						{
							value = option.DefaultValue;
						}
						SetPropertyValue(element, name, value);

						// If there's a restriction on it, check it's valid
						string restrict = ReadAttribute(element, "Restrict");
						if (!String.IsNullOrEmpty(restrict))
						{
							string pattern = "^(" + restrict + ")$";
							if (!Regex.IsMatch(value, pattern, RegexOptions.IgnoreCase))
							{
								LogError(element, "'{Value}' is not a valid value for '{Name}' (required: '{Restrict}')", value, name, restrict);
							}
							if (option.DefaultValue != value && !Regex.IsMatch(option.DefaultValue, pattern, RegexOptions.IgnoreCase))
							{
								LogError(element, "Default value '{DefaultValue}' is not valid for '{Name}' (required: '{Restrict}')", option.DefaultValue, name, restrict);
							}
						}
					}
				}
			}
		}

		/// <summary>
		/// Reads a property assignment.
		/// </summary>
		/// <param name="element">Xml element to read the definition from</param>
		async Task ReadPropertyAsync(BgScriptElement element)
		{
			if (await EvaluateConditionAsync(element))
			{
				string name = ReadAttribute(element, "Name");
				if (ValidateName(element, name))
				{
					string value = ReadAttribute(element, "Value");
					if (element.HasChildNodes)
					{
						StringBuilder builder = new StringBuilder(value);
						if (ReadBooleanAttribute(element, "Multiline", false))
						{
							// Get a hanging indent from the first line, and remove that whitespace from subsequent lines
							int prefixLen = -1;

							List<string> lines = new List<string>(element.InnerText.Split('\n'));
							while (lines.Count > 0 && String.IsNullOrWhiteSpace(lines[0]))
							{
								lines.RemoveAt(0);
							}
							while (lines.Count > 0 && String.IsNullOrWhiteSpace(lines[^1]))
							{
								lines.RemoveAt(lines.Count - 1);
							}

							foreach (string line in lines)
							{
								int whitespaceLen = 0;
								while (whitespaceLen < line.Length && Char.IsWhiteSpace(line[whitespaceLen]))
								{
									whitespaceLen++;
								}

								if (prefixLen == -1)
								{
									prefixLen = whitespaceLen;
								}

								whitespaceLen = Math.Min(whitespaceLen, prefixLen);
								builder.AppendLine(line.Substring(whitespaceLen));
							}
						}
						else
						{
							// Get the separator character
							string separator = ";";
							if (element.HasAttribute("Separator"))
							{
								separator = ReadAttribute(element, "Separator");
							}

							// Read the element content, and append each line to the value as a semicolon delimited list
							foreach (string line in element.InnerText.Split('\n'))
							{
								string trimLine = ExpandProperties(element, line.Trim());
								if (trimLine.Length > 0)
								{
									if (builder.Length > 0)
									{
										builder.Append(separator);
									}
									builder.Append(trimLine);
								}
							}
						}
						value = builder.ToString();
					}
					SetPropertyValue(element, name, value, ReadBooleanAttribute(element, "CreateInParentScope", false));
				}
			}
		}

		/// <summary>
		/// Reads a Regex assignment.
		/// </summary>
		/// <param name="element">Xml element to read the definition from</param>
		async Task ReadRegexAsync(BgScriptElement element)
		{
			if (await EvaluateConditionAsync(element))
			{
				// Get the pattern
				string regexString = ReadAttribute(element, "Pattern");

				// Make sure its a valid regex.
				Regex? regexValue = ParseRegex(element, regexString);
				if (regexValue != null)
				{
					// read the names in 
					string[] captureNames = ReadListAttribute(element, "Capture");

					// get number of groups we passed in
					int[] groupNumbers = regexValue.GetGroupNumbers();

					// make sure the number of property names is the same as the number of match groups
					// this includes the entire string match group as [0], so don't count that one.
					if (captureNames.Length != groupNumbers.Length - 1)
					{
						LogError(element, "MatchGroup count: {Count} does not match the number of names specified: {NameCount}", groupNumbers.Length - 1, captureNames.Length);
					}
					else
					{
						// apply the regex to the value
						string input = ReadAttribute(element, "Input");
						Match match = regexValue.Match(input);

						bool optional = await BgCondition.EvaluateAsync(ReadAttribute(element, "Optional"), new BgConditionContext(_rootDir));
						if (!match.Success)
						{
							if (!optional)
							{
								LogError(element, "Regex {Regex} did not find a match against input string {Input}", regexString, input);
							}
						}
						else
						{
							// assign each property to the group it matches, skip over [0]
							for (int matchIdx = 1; matchIdx < groupNumbers.Length; matchIdx++)
							{
								SetPropertyValue(element, captureNames[matchIdx - 1], match.Groups[matchIdx].Value);
							}
						}
					}
				}
			}
		}

		/// <summary>
		/// Reads a StringOp element and applies string method.
		/// </summary>
		/// <param name="element">Xml element to read the definition from</param>
		async Task ReadStringOpAsync(BgScriptElement element)
		{
			if (await EvaluateConditionAsync(element))
			{
				string input = ReadAttribute(element, "Input");
				string method = ReadAttribute(element, "Method");
				string output = ReadAttribute(element, "Output");

				string[] arguments = Array.Empty<string>();

				const string ArgumentsName = "Arguments";

				if (element.HasAttribute(ArgumentsName))
				{
					arguments = ReadAttribute(element, ArgumentsName).Split(';');
				}

				// Supply more string operations here
				string operationResult;
				switch (method)
				{
					case "ToLower":
						operationResult = input.ToLower();
						break;
					case "ToUpper":
						operationResult = input.ToUpper();
						break;
					case "Replace":
						if (arguments.Length != 2)
						{
							LogError(element, $"String operation 'Replace' requires exactly 2 arguments.");
							return;
						}

						operationResult = input.Replace(arguments[0], arguments[1], StringComparison.Ordinal);
						break;
					case "SplitFirst":
						if (arguments.Length != 1)
						{
							LogError(element, $"String operation 'SplitFirst' requires exactly 1 argument.");
							return;
						}

						operationResult = input.Split(arguments[0]).First();
						break;
					case "SplitLast":
						if (arguments.Length != 1)
						{
							LogError(element, $"String operation 'SplitLast' requires exactly 1 argument.");
							return;
						}

						operationResult = input.Split(arguments[0]).Last();
						break;
					default:
						LogError(element, $"String operation '{method}' not available.");
						return;
				}
				SetPropertyValue(element, output, operationResult);
			}
		}

		/// <summary>
		/// Reads a property assignment from an environment variable.
		/// </summary>
		/// <param name="element">Xml element to read the definition from</param>
		async Task ReadEnvVarAsync(BgScriptElement element)
		{
			if (await EvaluateConditionAsync(element))
			{
				string name = ReadAttribute(element, "Name");
				if (ValidateName(element, name))
				{
					string envVarName = name;
					if (!OperatingSystem.IsWindows())
					{
						// Non-windows platforms don't allow dashes in variable names. The engine platform layer substitutes underscores for them.
						envVarName = envVarName.Replace("-", "_", StringComparison.Ordinal);
					}

					string value = Environment.GetEnvironmentVariable(envVarName) ?? "";
					SetPropertyValue(element, name, value);
				}
			}
		}

		/// <summary>
		/// Reads a macro definition
		/// </summary>
		/// <param name="element">Xml element to read the definition from</param>
		void ReadMacro(BgScriptElement element)
		{
			string name = element.GetAttribute("Name");
			if (ValidateName(element, name))
			{
				BgScriptMacro? originalDefinition;
				if (_macroNameToDefinition.TryGetValue(name, out originalDefinition))
				{
					BgScriptLocation location = originalDefinition.Elements[0].Location;
					LogError(element, "Macro '{Name}' has already been declared (see {File} line {Line})", name, location.File, location.LineNumber);
				}
				else
				{
					Dictionary<string, int> argumentNameToIndex = new Dictionary<string, int>();
					ReadMacroArguments(element, "Arguments", argumentNameToIndex);

					int numRequiredArguments = argumentNameToIndex.Count;
					ReadMacroArguments(element, "OptionalArguments", argumentNameToIndex);

					_macroNameToDefinition.Add(name, new BgScriptMacro(name, element, argumentNameToIndex, numRequiredArguments));
				}
			}
		}

		/// <summary>
		/// Reads a list of macro arguments from an attribute
		/// </summary>
		/// <param name="element">The element containing the attributes</param>
		/// <param name="attributeName">Name of the attribute containing the arguments</param>
		/// <param name="argumentNameToIndex">List of arguments to add to</param>
		void ReadMacroArguments(BgScriptElement element, string attributeName, Dictionary<string, int> argumentNameToIndex)
		{
			string attributeValue = ReadAttribute(element, attributeName);
			if (attributeValue != null)
			{
				foreach (string argumentName in attributeValue.Split(new char[] { ';' }, StringSplitOptions.RemoveEmptyEntries))
				{
					if (argumentNameToIndex.ContainsKey(argumentName))
					{
						LogWarning(element, "Argument '{Name}' is listed multiple times", argumentName);
					}
					else
					{
						argumentNameToIndex.Add(argumentName, argumentNameToIndex.Count);
					}
				}
			}
		}

		/// <summary>
		/// Reads a macro definition
		/// </summary>
		/// <param name="element">Xml element to read the definition from</param>
		async Task ReadExtendAsync(BgScriptElement element)
		{
			if (await EvaluateConditionAsync(element))
			{
				string name = ReadAttribute(element, "Name");

				BgScriptMacro? originalDefinition;
				if (_macroNameToDefinition.TryGetValue(name, out originalDefinition))
				{
					originalDefinition.Elements.Add(element);
				}
				else
				{
					LogError(element, "Macro '{Name}' has not been declared", name);
				}
			}
		}

		/// <summary>
		/// Reads the definition for an agent.
		/// </summary>
		/// <param name="element">Xml element to read the definition from</param>
		async Task ReadAgentAsync(BgScriptElement element)
		{
			string? name;
			if (await EvaluateConditionAsync(element) && TryReadObjectName(element, out name))
			{
				// Read the valid agent types. This may be omitted if we're continuing an existing agent.
				string[] types = ReadListAttribute(element, "Type");

				// Create the agent object, or continue an existing one
				BgAgentDef? agent;
				if (_graph.NameToAgent.TryGetValue(name, out agent))
				{
					if (types.Length > 0 && agent.PossibleTypes.Count > 0)
					{
						if (types.Length != agent.PossibleTypes.Count || !types.SequenceEqual(agent.PossibleTypes, StringComparer.InvariantCultureIgnoreCase))
						{
							LogError(element, "Agent types ({Types}) were different than previous agent definition with types ({PossibleTypes}). Must either be empty or match exactly.", String.Join(",", types), String.Join(",", agent.PossibleTypes));
						}
					}
				}
				else
				{
					if (types.Length == 0)
					{
						LogError(element, "Missing type for agent '{Name}'", name);
					}
					agent = new BgAgentDef(name);
					agent.PossibleTypes.AddRange(types);
					_graph.NameToAgent.Add(name, agent);
					_graph.Agents.Add(agent);
				}

				// Process all the child elements.
				_enclosingAgent = agent;
				await ReadAgentBodyAsync(element);
				_enclosingAgent = null;
			}
		}

		/// <summary>
		/// Read the contents of an agent definition
		/// </summary>
		/// <param name="element">Xml element to read the definition from</param>
		protected async Task ReadAgentBodyAsync(BgScriptElement element)
		{
			EnterScope();
			foreach (BgScriptElement childElement in element.ChildNodes.OfType<BgScriptElement>())
			{
				switch (childElement.Name)
				{
					case "Property":
						await ReadPropertyAsync(childElement);
						break;
					case "Regex":
						await ReadRegexAsync(childElement);
						break;
					case "StringOp":
						await ReadStringOpAsync(childElement);
						break;
					case "Node":
						await ReadNodeAsync(childElement);
						break;
					case "Aggregate":
						await ReadAggregateAsync(childElement);
						break;
					case "Artifact":
						await ReadArtifactAsync(childElement);
						break;
					case "Trace":
						await ReadDiagnosticAsync(childElement, LogLevel.Information);
						break;
					case "Warning":
						await ReadDiagnosticAsync(childElement, LogLevel.Warning);
						break;
					case "Error":
						await ReadDiagnosticAsync(childElement, LogLevel.Error);
						break;
					case "Label":
						await ReadLabelAsync(childElement);
						break;
					case "Do":
						await ReadBlockAsync(childElement, ReadAgentBodyAsync);
						break;
					case "Switch":
						await ReadSwitchAsync(childElement, ReadAgentBodyAsync);
						break;
					case "ForEach":
						await ReadForEachAsync(childElement, ReadAgentBodyAsync);
						break;
					case "Expand":
						await ReadExpandAsync(childElement, ReadAgentBodyAsync);
						break;
					case "Annotate":
						await ReadAnnotationAsync(childElement);
						break;
					default:
						LogError(childElement, "Unexpected element type '{ElementName}'", childElement.Name);
						break;
				}
			}
			LeaveScope();
		}

		/// <summary>
		/// Reads the definition for an aggregate
		/// </summary>
		/// <param name="element">Xml element to read the definition from</param>
		async Task ReadAggregateAsync(BgScriptElement element)
		{
			string? name;
			if (await EvaluateConditionAsync(element) && TryReadObjectName(element, out name) && CheckNameIsUnique(element, name))
			{
				string[] requiredNames = ReadListAttribute(element, "Requires");

				BgAggregateDef newAggregate = new BgAggregateDef(name);
				foreach (BgNodeDef referencedNode in ResolveReferences(element, requiredNames))
				{
					newAggregate.RequiredNodes.Add(referencedNode);
				}
				_graph.NameToAggregate[name] = newAggregate;

				string labelCategoryName = ReadAttribute(element, "Label");
				if (!String.IsNullOrEmpty(labelCategoryName))
				{
					BgLabelDef label;

					// Create the label
					int slashIdx = labelCategoryName.IndexOf('/', StringComparison.Ordinal);
					if (slashIdx != -1)
					{
						label = new BgLabelDef(labelCategoryName.Substring(slashIdx + 1), labelCategoryName.Substring(0, slashIdx), null, null, BgLabelChange.Current);
					}
					else
					{
						label = new BgLabelDef(labelCategoryName, "Other", null, null, BgLabelChange.Current);
					}

					// Find all the included nodes
					foreach (BgNodeDef requiredNode in newAggregate.RequiredNodes)
					{
						label.RequiredNodes.Add(requiredNode);
						label.IncludedNodes.Add(requiredNode);
						label.IncludedNodes.UnionWith(requiredNode.OrderDependencies);
					}

					string[] includedNames = ReadListAttribute(element, "Include");
					foreach (BgNodeDef includedNode in ResolveReferences(element, includedNames))
					{
						label.IncludedNodes.Add(includedNode);
						label.IncludedNodes.UnionWith(includedNode.OrderDependencies);
					}

					string[] excludedNames = ReadListAttribute(element, "Exclude");
					foreach (BgNodeDef excludedNode in ResolveReferences(element, excludedNames))
					{
						label.IncludedNodes.Remove(excludedNode);
						label.IncludedNodes.ExceptWith(excludedNode.OrderDependencies);
					}

					_graph.Labels.Add(label);
				}
			}
		}

		/// <summary>
		/// Reads the definition for an artifact
		/// </summary>
		/// <param name="element">Xml element to read the definition from</param>
		async Task ReadArtifactAsync(BgScriptElement element)
		{
			string? name;
			if (await EvaluateConditionAsync(element) && TryReadObjectName(element, out name))
			{
				string? type = ReadAttribute(element, "Type");
				if (String.IsNullOrEmpty(type))
				{
					type = null;
				}

				string? description = ReadAttribute(element, "Description");
				if (String.IsNullOrEmpty(description))
				{
					description = null;
				}

				string? basePath = ReadAttribute(element, "BasePath");
				if (String.IsNullOrEmpty(basePath))
				{
					basePath = null;
				}

				string tag = ReadAttribute(element, "Tag");
				if (String.IsNullOrEmpty(tag))
				{
					tag = $"#{name}";
				}
				if (!_graph.TagNameToNodeOutput.TryGetValue(tag, out _))
				{
					LogError(element, "Artifact '{Name}' references non-existent tag '{Tag}'", name, tag);
				}

				string[] keys = ReadListAttribute(element, "Keys");
				string[] metadata = ReadListAttribute(element, "Metadata");

				BgArtifactDef newArtifact = new BgArtifactDef(name, type, description, basePath, null, tag, keys, metadata);
				_graph.Artifacts.Add(newArtifact);
			}
		}

		/// <summary>
		/// Reads the definition for a report
		/// </summary>
		/// <param name="element">Xml element to read the definition from</param>
		async Task ReadReportAsync(BgScriptElement element)
		{
			string? name;
			if (await EvaluateConditionAsync(element) && TryReadObjectName(element, out name) && CheckNameIsUnique(element, name))
			{
				string[] requiredNames = ReadListAttribute(element, "Requires");

				BgReport newReport = new BgReport(name);
				foreach (BgNodeDef referencedNode in ResolveReferences(element, requiredNames))
				{
					newReport.Nodes.Add(referencedNode);
					newReport.Nodes.UnionWith(referencedNode.OrderDependencies);
				}
				_graph.NameToReport.Add(name, newReport);
			}
		}

		/// <summary>
		/// Reads the definition for a badge
		/// </summary>
		/// <param name="element">Xml element to read the definition from</param>
		async Task ReadBadgeAsync(BgScriptElement element)
		{
			string? name;
			if (await EvaluateConditionAsync(element) && TryReadObjectName(element, out name))
			{
				string[] requiredNames = ReadListAttribute(element, "Requires");
				string[] targetNames = ReadListAttribute(element, "Targets");
				string project = ReadAttribute(element, "Project");
				int change = ReadIntegerAttribute(element, "Change", 0);

				BgBadgeDef newBadge = new BgBadgeDef(name, project, change);
				foreach (BgNodeDef referencedNode in ResolveReferences(element, requiredNames))
				{
					newBadge.Nodes.Add(referencedNode);
				}
				foreach (BgNodeDef referencedNode in ResolveReferences(element, targetNames))
				{
					newBadge.Nodes.Add(referencedNode);
					newBadge.Nodes.UnionWith(referencedNode.OrderDependencies);
				}
				_graph.Badges.Add(newBadge);
			}
		}

		/// <summary>
		/// Reads the definition for a label
		/// </summary>
		/// <param name="element">Xml element to read the definition from</param>
		async Task ReadLabelAsync(BgScriptElement element)
		{
			if (await EvaluateConditionAsync(element))
			{
				string name = ReadAttribute(element, "Name");
				if (!String.IsNullOrEmpty(name))
				{
					ValidateName(element, name);
				}

				string category = ReadAttribute(element, "Category");

				string[] requiredNames = ReadListAttribute(element, "Requires");
				string[] includedNames = ReadListAttribute(element, "Include");
				string[] excludedNames = ReadListAttribute(element, "Exclude");

				string ugsBadge = ReadAttribute(element, "UgsBadge");
				string ugsProject = ReadAttribute(element, "UgsProject");

				BgLabelChange change = ReadEnumAttribute<BgLabelChange>(element, "Change", BgLabelChange.Current);

				BgLabelDef newLabel = new BgLabelDef(name, category, ugsBadge, ugsProject, change);
				foreach (BgNodeDef referencedNode in ResolveReferences(element, requiredNames))
				{
					newLabel.RequiredNodes.Add(referencedNode);
					newLabel.IncludedNodes.Add(referencedNode);
					newLabel.IncludedNodes.UnionWith(referencedNode.OrderDependencies);
				}
				foreach (BgNodeDef includedNode in ResolveReferences(element, includedNames))
				{
					newLabel.IncludedNodes.Add(includedNode);
					newLabel.IncludedNodes.UnionWith(includedNode.OrderDependencies);
				}
				foreach (BgNodeDef excludedNode in ResolveReferences(element, excludedNames))
				{
					newLabel.IncludedNodes.Remove(excludedNode);
					newLabel.IncludedNodes.ExceptWith(excludedNode.OrderDependencies);
				}
				_graph.Labels.Add(newLabel);
			}
		}

		/// <summary>
		/// Reads the definition for a node, and adds it to the given agent
		/// </summary>
		/// <param name="element">Xml element to read the definition from</param>
		async Task ReadNodeAsync(BgScriptElement element)
		{
			string? name;
			if (await EvaluateConditionAsync(element) && TryReadObjectName(element, out name))
			{
				string[] requiresNames = ReadListAttribute(element, "Requires");
				string[] optionalRequiresNames = ReadListAttribute(element, "OptionalRequires");
				string[] producesNames = ReadListAttribute(element, "Produces");
				string[] afterNames = ReadListAttribute(element, "After");
				string[] tokenFileNames = ReadListAttribute(element, "Token");
				bool runEarly = ReadBooleanAttribute(element, "RunEarly", false);
				bool allowRetry = ReadBooleanAttribute(element, "AllowRetry", true);
				bool notifyOnWarnings = ReadBooleanAttribute(element, "NotifyOnWarnings", true);
				Dictionary<string, string> annotations = ReadAnnotationsAttribute(element, "Annotations");
				string[] ignoreModified = ReadListAttribute(element, "IgnoreModified");

				// Resolve all the inputs we depend on
				HashSet<BgNodeOutput> inputs = ResolveInputReferences(element, requiresNames);

				// Gather up all the input dependencies, and check they're all upstream of the current node
				HashSet<BgNodeDef> inputDependencies = [];
				foreach (BgNodeDef inputDependency in inputs.Select(x => x.ProducingNode).Distinct())
				{
					inputDependencies.Add(inputDependency);
				}

				// Resolve all the optional inputs we depend on
				HashSet<BgNodeOutput> optionalInputs = ResolveInputReferences(element, optionalRequiresNames);

				// If an input is both required and optional the required taking precedence
				HashSet<BgNodeOutput> duplicateInputs = new(inputs);
				duplicateInputs.IntersectWith(optionalInputs);
				foreach (BgNodeOutput duplicateInput in duplicateInputs)
				{
					LogWarning(element, "Optional input '{Tag}' is also a required input. The required input takes precedence", duplicateInput.TagName);
				}
				optionalInputs.ExceptWith(duplicateInputs);

				// Gather all the optional input dependencies, and check they're all upstream of the current node
				HashSet<BgNodeDef> optionalInputDependencies = [];
				foreach (BgNodeDef optionalInputDependency in optionalInputs.Select(x => x.ProducingNode).Distinct())
				{
					optionalInputDependencies.Add(optionalInputDependency);
				}

				// Remove all the lock names from the list of required names
				HashSet<FileReference> requiredTokens = new HashSet<FileReference>(tokenFileNames.Select(x => new FileReference(x)));

				// Recursively include all their dependencies too
				// Intentionally only traverses InputDependencies (required), not OptionalInputDependencies.
				// A required dependency's own optional deps don't need to succeed for it to succeed, so they
				// don't belong in the required chain. Execution ordering for optional dependents is handled
				// separately via OrderDependencies below.
				foreach (BgNodeDef inputDependency in inputDependencies.ToArray())
				{
					requiredTokens.UnionWith(inputDependency.RequiredTokens);
					inputDependencies.UnionWith(inputDependency.InputDependencies);
				}

				// Recursively include all their optional dependencies too
				foreach (BgNodeDef optionalInputDependency in optionalInputDependencies.ToArray())
				{
					// A node that optionally depends on another must still respect its resource locks
					requiredTokens.UnionWith(optionalInputDependency.RequiredTokens);
					optionalInputDependencies.UnionWith(optionalInputDependency.InputDependencies);
					optionalInputDependencies.UnionWith(optionalInputDependency.OptionalInputDependencies);
				}

				// Validate all the outputs
				List<string> validOutputNames = new List<string>();
				foreach (string producesName in producesNames)
				{
					BgNodeOutput? existingOutput;
					if (_graph.TagNameToNodeOutput.TryGetValue(producesName, out existingOutput))
					{
						LogError(element, "Output tag '{Tag}' is already generated by node '{Name}'", producesName, existingOutput.ProducingNode.Name);
					}
					else if (!producesName.StartsWith("#", StringComparison.Ordinal))
					{
						LogError(element, "Output tag names must begin with a '#' character ('{Name}')", producesName);
					}
					else
					{
						validOutputNames.Add(producesName);
					}
				}

				// Gather up all the order dependencies
				HashSet<BgNodeDef> orderDependencies = new HashSet<BgNodeDef>(inputDependencies);
				orderDependencies.UnionWith(optionalInputDependencies);
				orderDependencies.UnionWith(ResolveReferences(element, afterNames));

				// Recursively include all their order dependencies too
				foreach (BgNodeDef orderDependency in orderDependencies.ToArray())
				{
					orderDependencies.UnionWith(orderDependency.OrderDependencies);
				}

				// Check that we're not dependent on anything completing that is declared after the initial declaration of this agent.
				int agentIdx = _graph.Agents.IndexOf(_enclosingAgent!);
				for (int idx = agentIdx + 1; idx < _graph.Agents.Count; idx++)
				{
					foreach (BgNodeDef node in _graph.Agents[idx].Nodes.Where(x => orderDependencies.Contains(x)))
					{
						LogError(element, "Node '{Name}' has a dependency on '{OtherName}', which was declared after the initial definition of '{AgentName}'.", name, node.Name, _enclosingAgent!.Name);
					}
				}

				// Construct and register the node
				if (CheckNameIsUnique(element, name))
				{
					// Add it to the node lookup
					BgScriptNode newNode = new(name, inputs.ToArray(), optionalInputs.ToArray(), validOutputNames.ToArray(), inputDependencies.ToArray(), optionalInputDependencies.ToArray(), orderDependencies.ToArray(), requiredTokens.ToArray(), ignoreModified.ToArray())
					{
						RunEarly = runEarly,
						AllowRetry = allowRetry,
						NotifyOnWarnings = notifyOnWarnings
					};
					foreach ((string key, string value) in annotations)
					{
						BgAnnotationDef? annotationDef = newNode.Annotations.FirstOrDefault(a => a.Name.Equals(key, StringComparison.OrdinalIgnoreCase));
						if (annotationDef == null)
						{
							annotationDef = new BgAnnotationDef(key, value);
							newNode.Annotations.Add(annotationDef);
						}
						else
						{
							annotationDef.Value = value;
						}
					}
					_graph.NameToNode.Add(name, newNode);

					// Register all the output tags in the global name table.
					foreach (BgNodeOutput output in newNode.Outputs)
					{
						BgNodeOutput? existingOutput;
						if (_graph.TagNameToNodeOutput.TryGetValue(output.TagName, out existingOutput))
						{
							LogError(element, "Node '{NodeName}' already has an output called '{TagName}'", existingOutput.ProducingNode.Name, output.TagName);
						}
						else
						{
							_graph.TagNameToNodeOutput.Add(output.TagName, output);
						}
					}

					// Add all the tasks
					_enclosingNode = newNode;
					await ReadNodeBodyAsync(element);
					_enclosingNode = null;

					// Add it to the current agent
					_enclosingAgent!.Nodes.Add(newNode);
				}
			}
		}

		/// <summary>
		/// Reads the contents of a node element
		/// </summary>
		/// <param name="element">Xml element to read the definition from</param>
		protected async Task ReadNodeBodyAsync(XmlElement element)
		{
			EnterScope();
			foreach (BgScriptElement childElement in element.ChildNodes.OfType<BgScriptElement>())
			{
				switch (childElement.Name)
				{
					case "Property":
						await ReadPropertyAsync(childElement);
						break;
					case "Regex":
						await ReadRegexAsync(childElement);
						break;
					case "StringOp":
						await ReadStringOpAsync(childElement);
						break;
					case "Trace":
						await ReadDiagnosticAsync(childElement, LogLevel.Information);
						break;
					case "Warning":
						await ReadDiagnosticAsync(childElement, LogLevel.Warning);
						break;
					case "Error":
						await ReadDiagnosticAsync(childElement, LogLevel.Error);
						break;
					case "Do":
						await ReadBlockAsync(childElement, ReadNodeBodyAsync);
						break;
					case "Switch":
						await ReadSwitchAsync(childElement, ReadNodeBodyAsync);
						break;
					case "ForEach":
						await ReadForEachAsync(childElement, ReadNodeBodyAsync);
						break;
					case "Expand":
						await ReadExpandAsync(childElement, ReadNodeBodyAsync);
						break;
					default:
						await ReadTaskAsync(childElement);
						break;
				}
			}
			LeaveScope();
		}

		/// <summary>
		/// Reads a block element
		/// </summary>
		/// <param name="element">Xml element to read the definition from</param>
		/// <param name="readContentsAsync">Delegate to read the contents of the element, if the condition evaluates to true</param>
		async Task ReadBlockAsync(BgScriptElement element, Func<BgScriptElement, Task> readContentsAsync)
		{
			if (await EvaluateConditionAsync(element))
			{
				await readContentsAsync(element);
			}
		}

		/// <summary>
		/// Reads a "Switch" element 
		/// </summary>
		/// <param name="element">Xml element to read the definition from</param>
		/// <param name="readContentsAsync">Delegate to read the contents of the element, if the condition evaluates to true</param>
		async Task ReadSwitchAsync(BgScriptElement element, Func<BgScriptElement, Task> readContentsAsync)
		{
			foreach (BgScriptElement childElement in element.ChildNodes.OfType<BgScriptElement>())
			{
				if (childElement.Name == "Default" || await EvaluateConditionAsync(childElement))
				{
					await readContentsAsync(childElement);
					break;
				}
			}
		}

		/// <summary>
		/// Reads a "ForEach" element 
		/// </summary>
		/// <param name="element">Xml element to read the definition from</param>
		/// <param name="readContentsAsync">Delegate to read the contents of the element, if the condition evaluates to true</param>
		async Task ReadForEachAsync(BgScriptElement element, Func<BgScriptElement, Task> readContentsAsync)
		{
			EnterScope();
			if (await EvaluateConditionAsync(element))
			{
				string name = ReadAttribute(element, "Name");
				string separator = ReadAttribute(element, "Separator");
				if (separator.Length > 1)
				{
					LogWarning(element, "Node {Name}'s Separator attribute is more than one character ({Separator}). Defaulting to ;", name, separator);
					separator = ";";
				}
				if (String.IsNullOrEmpty(separator))
				{
					separator = ";";
				}
				if (ValidateName(element, name))
				{
					if (ScopedProperties.Any(x => x.ContainsKey(name)))
					{
						LogError(element, "Loop variable '{Name}' already exists as a local property in an outer scope", name);
					}
					else
					{
						// Loop through all the values
						string[] values = ReadListAttribute(element, "Values", Convert.ToChar(separator));
						foreach (string value in values)
						{
							ScopedProperties[^1][name] = value;
							await readContentsAsync(element);
						}
					}
				}
			}
			LeaveScope();
		}

		/// <summary>
		/// Reads an "Expand" element 
		/// </summary>
		/// <param name="element">Xml element to read the definition from</param>
		/// <param name="readContentsAsync">Delegate to read the contents of the element, if the condition evaluates to true</param>
		async Task ReadExpandAsync(BgScriptElement element, Func<BgScriptElement, Task> readContentsAsync)
		{
			if (await EvaluateConditionAsync(element))
			{
				string name = ReadAttribute(element, "Name");
				if (ValidateName(element, name))
				{
					BgScriptMacro? macro;
					if (!_macroNameToDefinition.TryGetValue(name, out macro))
					{
						LogError(element, "Macro '{Name}' does not exist", name);
					}
					else
					{
						// Parse the argument list
						string[] arguments = new string[macro.ArgumentNameToIndex.Count];
						foreach (XmlAttribute? attribute in element.Attributes)
						{
							if (attribute != null && attribute.Name != "Name" && attribute.Name != "If")
							{
								int index;
								if (macro.ArgumentNameToIndex.TryGetValue(attribute.Name, out index))
								{
									arguments[index] = ExpandProperties(element, attribute.Value);
								}
								else
								{
									LogWarning(element, "Macro '{Name}' does not take an argument '{ArgName}'", name, attribute.Name);
								}
							}
						}

						// Make sure none of the required arguments are missing
						bool hasMissingArguments = false;
						for (int idx = 0; idx < macro.NumRequiredArguments; idx++)
						{
							if (arguments[idx] == null)
							{
								LogWarning(element, "Macro '{Name}' is missing argument '{ArgName}'", macro.Name, macro.ArgumentNameToIndex.First(x => x.Value == idx).Key);
								hasMissingArguments = true;
							}
						}

						// Expand the function
						if (!hasMissingArguments)
						{
							EnterScope();
							foreach (KeyValuePair<string, int> pair in macro.ArgumentNameToIndex)
							{
								ScopedProperties[^1][pair.Key] = arguments[pair.Value] ?? "";
							}
							foreach (BgScriptElement macroElement in macro.Elements)
							{
								await readContentsAsync(macroElement);
							}
							LeaveScope();
						}
					}
				}
			}
		}

		/// <summary>
		/// Reads a task definition from the given element, and add it to the given list
		/// </summary>
		/// <param name="element">Xml element to read the definition from</param>
		async Task ReadTaskAsync(BgScriptElement element)
		{
			// If we're running a single node and this element's parent isn't the single node to run, ignore the error and return.
			if (!String.IsNullOrWhiteSpace(_singleNodeName) && _enclosingNode!.Name != _singleNodeName)
			{
				return;
			}

			if (await EvaluateConditionAsync(element))
			{
				BgTask info = new BgTask(element.Location, element.Name);
				foreach (XmlAttribute? attribute in element.Attributes)
				{
					if (!String.Equals(attribute!.Name, "If", StringComparison.OrdinalIgnoreCase))
					{
						string expandedValue = ExpandProperties(element, attribute.Value);
						info.Arguments.Add(attribute.Name, expandedValue);
					}
				}
				_enclosingNode!.Tasks.Add(info);

				if (info.Name.Equals("CreateArtifact", StringComparison.OrdinalIgnoreCase))
				{
					AddArtifactFromTask(element, info.Arguments);
				}
				if (info.Name.Equals("CreateCloudArtifact", StringComparison.OrdinalIgnoreCase))
				{
					AddCloudArtifactFromTask(element, info.Arguments);
				}
			}
		}

		void AddArtifactFromTask(BgScriptElement element, Dictionary<string, string> arguments)
		{
			arguments.TryGetValue(nameof(CreateArtifactTaskParameters.Name), out string? name);
			arguments.TryGetValue(nameof(CreateArtifactTaskParameters.Type), out string? type);
			arguments.TryGetValue(nameof(CreateArtifactTaskParameters.Description), out string? description);
			arguments.TryGetValue(nameof(CreateArtifactTaskParameters.Keys), out string? keys);
			arguments.TryGetValue(nameof(CreateArtifactTaskParameters.Metadata), out string? metadata);

			if (String.IsNullOrEmpty(name))
			{
				LogError(element, "Missing artifact name");
				return;
			}
			if (String.IsNullOrEmpty(type))
			{
				LogError(element, "Missing artifact type");
				return;
			}

			string[] keysArray = (keys ?? String.Empty).Split(';', StringSplitOptions.RemoveEmptyEntries | StringSplitOptions.TrimEntries);
			string[] metadataArray = (metadata ?? String.Empty).Split(';', StringSplitOptions.RemoveEmptyEntries | StringSplitOptions.TrimEntries);

			BgArtifactDef artifact = new BgArtifactDef(name, type, description, null, _enclosingNode!.Name, null, keysArray, metadataArray);
			_graph.Artifacts.Add(artifact);
		}

		void AddCloudArtifactFromTask(BgScriptElement element, Dictionary<string, string> arguments)
		{
			arguments.TryGetValue(nameof(CreateCloudArtifactTaskParameters.Name), out string? name);
			arguments.TryGetValue(nameof(CreateCloudArtifactTaskParameters.HordeArtifactID), out string? hordeArtifactID);
			arguments.TryGetValue(nameof(CreateCloudArtifactTaskParameters.Type), out string? type);
			arguments.TryGetValue(nameof(CreateCloudArtifactTaskParameters.Description), out string? description);
			arguments.TryGetValue(nameof(CreateCloudArtifactTaskParameters.Metadata), out string? metadata);

			string? artifactName = String.IsNullOrEmpty(hordeArtifactID) ? name : hordeArtifactID;
			if (String.IsNullOrEmpty(artifactName))
			{
				LogError(element, "Missing artifact name");
				return;
			}
			if (String.IsNullOrEmpty(type))
			{
				LogError(element, "Missing artifact type");
				return;
			}

			string[] metadataArray = (metadata ?? String.Empty).Split(';', StringSplitOptions.RemoveEmptyEntries | StringSplitOptions.TrimEntries);
			List<string> metadataList =
			[
				..metadataArray,
				"Backend=Zen"
			];

			BgArtifactDef artifact = new BgArtifactDef(artifactName, type, description, null, _enclosingNode!.Name, null, Array.Empty<string>(), metadataList.ToArray());
			_graph.Artifacts.Add(artifact);
		}

		/// <summary>
		/// Reads the definition for an email notifier
		/// </summary>
		/// <param name="element">Xml element to read the definition from</param>
		async Task ReadNotifierAsync(BgScriptElement element)
		{
			if (await EvaluateConditionAsync(element))
			{
				string[] targetNames = ReadListAttribute(element, "Targets");
				string[] exceptNames = ReadListAttribute(element, "Except");
				string[] individualNodeNames = ReadListAttribute(element, "Nodes");
				string[] reportNames = ReadListAttribute(element, "Reports");
				string[] users = ReadListAttribute(element, "Users");
				string[] submitters = ReadListAttribute(element, "Submitters");
				bool? warnings = element.HasAttribute("Warnings") ? (bool?)ReadBooleanAttribute(element, "Warnings", true) : null;
				bool absolute = element.HasAttribute("Absolute") && ReadBooleanAttribute(element, "Absolute", true);

				// Find the list of targets which are included, and recurse through all their dependencies
				HashSet<BgNodeDef> nodes = new HashSet<BgNodeDef>();
				if (targetNames != null)
				{
					HashSet<BgNodeDef> targetNodes = ResolveReferences(element, targetNames);
					foreach (BgNodeDef node in targetNodes)
					{
						nodes.Add(node);
						nodes.UnionWith(node.InputDependencies);
					}
				}

				// Add all the individually referenced nodes
				if (individualNodeNames != null)
				{
					HashSet<BgNodeDef> individualNodes = ResolveReferences(element, individualNodeNames);
					nodes.UnionWith(individualNodes);
				}

				// Exclude all the exceptions
				if (exceptNames != null)
				{
					HashSet<BgNodeDef> exceptNodes = ResolveReferences(element, exceptNames);
					nodes.ExceptWith(exceptNodes);
				}

				// Update all the referenced nodes with the settings
				foreach (BgNodeDef node in nodes)
				{
					if (users != null)
					{
						if (absolute)
						{
							node.NotifyUsers = new HashSet<string>(users);
						}
						else
						{
							node.NotifyUsers.UnionWith(users);
						}
					}
					if (submitters != null)
					{
						if (absolute)
						{
							node.NotifySubmitters = new HashSet<string>(submitters);
						}
						else
						{
							node.NotifySubmitters.UnionWith(submitters);
						}
					}
					if (warnings.HasValue)
					{
						node.NotifyOnWarnings = warnings.Value;
					}
				}

				// Add the users to the list of reports
				if (reportNames != null && users != null)
				{
					foreach (string reportName in reportNames)
					{
						BgReport? report;
						if (_graph.NameToReport.TryGetValue(reportName, out report))
						{
							report.NotifyUsers.UnionWith(users);
						}
						else
						{
							LogError(element, "Report '{ReportName}' has not been defined", reportName);
						}
					}
				}
			}
		}

		/// <summary>
		/// Reads a graph annotation
		/// </summary>
		/// <param name="element">Xml element to read the definition from</param>
		async Task ReadAnnotationAsync(BgScriptElement element)
		{
			if (await EvaluateConditionAsync(element))
			{
				string[] targetNames = ReadListAttribute(element, "Targets");
				string[] exceptNames = ReadListAttribute(element, "Except");
				string[] individualNodeNames = ReadListAttribute(element, "Nodes");
				Dictionary<string, string> annotations = ReadAnnotationsAttribute(element, "Values");

				// Find the list of targets which are included, and recurse through all their dependencies
				HashSet<BgNodeDef> nodes = new HashSet<BgNodeDef>();
				if (targetNames != null)
				{
					HashSet<BgNodeDef> targetNodes = ResolveReferences(element, targetNames);
					foreach (BgNodeDef node in targetNodes)
					{
						nodes.Add(node);
						nodes.UnionWith(node.InputDependencies);
					}
				}

				// Add all the individually referenced nodes
				if (individualNodeNames != null)
				{
					HashSet<BgNodeDef> individualNodes = ResolveReferences(element, individualNodeNames);
					nodes.UnionWith(individualNodes);
				}

				// Exclude all the exceptions
				if (exceptNames != null)
				{
					HashSet<BgNodeDef> exceptNodes = ResolveReferences(element, exceptNames);
					nodes.ExceptWith(exceptNodes);
				}

				// Update all the referenced nodes with the settings
				foreach (BgNodeDef node in nodes)
				{
					foreach ((string key, string value) in annotations)
					{
						BgAnnotationDef? annotationDef = node.Annotations.FirstOrDefault(a => a.Name.Equals(key, StringComparison.OrdinalIgnoreCase));
						if (annotationDef == null)
						{
							annotationDef = new BgAnnotationDef(key, value);
							node.Annotations.Add(annotationDef);
						}
						else
						{
							annotationDef.Value = value;
						}
					}
				}
			}
		}

		/// <summary>
		/// Reads a warning from the given element, evaluates the condition on it, and writes it to the log if the condition passes.
		/// </summary>
		/// <param name="element">Xml element to read the definition from</param>
		/// <param name="level">The diagnostic event type</param>
		async Task ReadDiagnosticAsync(BgScriptElement element, LogLevel level)
		{
			if (await EvaluateConditionAsync(element))
			{
				string message = ReadAttribute(element, "Message");
				bool reportOnExecution = ReadBooleanAttribute(element, "ReportOnExecution", false);
				string[] reportOnNodes = ReadListAttribute(element, "ReportOnNodes");

				BgDiagnosticDef diagnostic = new(element.Location.File.FullName, element.Location.LineNumber, level, message, reportOnExecution, reportOnNodes);
				if (_enclosingNode != null)
				{
					_enclosingNode.Diagnostics.Add(diagnostic);
				}
				else if (_enclosingAgent != null)
				{
					_enclosingAgent.Diagnostics.Add(diagnostic);
				}
				else
				{
					_graph.Diagnostics.Add(diagnostic);
				}
			}
		}

		/// <summary>
		/// Reads an object name from its defining element. Outputs an error if the name is missing.
		/// </summary>
		/// <param name="element">Element to read the name for</param>
		/// <param name="name">Output variable to receive the name of the object</param>
		/// <returns>True if the object had a valid name (assigned to the Name variable), false if the name was invalid or missing.</returns>
		protected bool TryReadObjectName(BgScriptElement element, [NotNullWhen(true)] out string? name)
		{
			// Check the name attribute is present
			if (!element.HasAttribute("Name"))
			{
				LogError(element, "Missing 'Name' attribute");
				name = null;
				return false;
			}

			// Get the value of it, strip any leading or trailing whitespace, and make sure it's not empty
			string value = ReadAttribute(element, "Name");
			if (!ValidateName(element, value))
			{
				name = null;
				return false;
			}

			// Return it
			name = value;
			return true;
		}

		/// <summary>
		/// Checks that the given name does not already used to refer to a node, and print an error if it is.
		/// </summary>
		/// <param name="element">Xml element to read from</param>
		/// <param name="name">Name of the alias</param>
		/// <returns>True if the name was registered correctly, false otherwise.</returns>
		bool CheckNameIsUnique(BgScriptElement element, string name)
		{
			// Get the nodes that it maps to
			if (_graph.ContainsName(name))
			{
				LogError(element, "'{Name}' is already defined; cannot add a second time", name);
				return false;
			}
			return true;
		}

		/// <summary>
		/// Resolve a list of references to a set of nodes
		/// </summary>
		/// <param name="element">Element used to locate any errors</param>
		/// <param name="referenceNames">Sequence of names to look up</param>
		/// <returns>Hashset of all the nodes included by the given names</returns>
		HashSet<BgNodeDef> ResolveReferences(BgScriptElement element, IEnumerable<string> referenceNames)
		{
			HashSet<BgNodeDef> nodes = new HashSet<BgNodeDef>();
			foreach (string referenceName in referenceNames)
			{
				BgNodeDef[]? otherNodes;
				if (_graph.TryResolveReference(referenceName, out otherNodes))
				{
					nodes.UnionWith(otherNodes);
				}
				else if (!referenceName.StartsWith("#", StringComparison.Ordinal) && _graph.TagNameToNodeOutput.ContainsKey("#" + referenceName))
				{
					LogError(element, "Reference to '{Name}' cannot be resolved; did you mean '{PossibleName}'?", referenceName, $"#{referenceName}");
				}
				else
				{
					LogError(element, "Reference to '{Name}' cannot be resolved; check it has been defined.", referenceName);
				}
			}
			return nodes;
		}

		/// <summary>
		/// Resolve a list of references to a set of nodes
		/// </summary>
		/// <param name="element">Element used to locate any errors</param>
		/// <param name="referenceNames">Sequence of names to look up</param>
		/// <returns>Set of all the nodes included by the given names</returns>
		HashSet<BgNodeOutput> ResolveInputReferences(BgScriptElement element, IEnumerable<string> referenceNames)
		{
			HashSet<BgNodeOutput> inputs = new HashSet<BgNodeOutput>();
			foreach (string referenceName in referenceNames)
			{
				BgNodeOutput[]? referenceInputs;
				if (_graph.TryResolveInputReference(referenceName, out referenceInputs))
				{
					inputs.UnionWith(referenceInputs);
				}
				else if (!referenceName.StartsWith("#", StringComparison.Ordinal) && _graph.TagNameToNodeOutput.ContainsKey("#" + referenceName))
				{
					LogError(element, "Reference to '{Name}' cannot be resolved; did you mean '{PossibleName}'?", referenceName, $"#{referenceName}");
				}
				else
				{
					LogError(element, "Reference to '{Name}' cannot be resolved; check it has been defined.", referenceName);
				}
			}
			return inputs;
		}

		/// <summary>
		/// Checks that the given name is valid syntax
		/// </summary>
		/// <param name="element">The element that contains the name</param>
		/// <param name="name">The name to check</param>
		/// <returns>True if the name is valid</returns>
		protected bool ValidateName(BgScriptElement element, string name)
		{
			// Check it's not empty
			if (name.Length == 0)
			{
				LogError(element, "Name is empty");
				return false;
			}

			// Check there are no invalid characters
			for (int idx = 0; idx < name.Length; idx++)
			{
				if (idx > 0 && name[idx] == ' ' && name[idx - 1] == ' ')
				{
					LogError(element, "Consecutive spaces in object name '{Name}'", name);
					return false;
				}
				if (Char.IsControl(name[idx]) || BgScriptSchema.IllegalNameCharacters.Contains(name[idx], StringComparison.Ordinal))
				{
					LogError(element, "Invalid character in object name '{Name}': '{Character}'", name, name[idx]);
					return false;
				}
			}
			return true;
		}

		/// <summary>
		/// Constructs a regex from a regex string and returns it
		/// </summary>
		/// <param name="element">The element that contains the regex</param>
		/// <param name="regex">The pattern to construct</param>
		/// <returns>The regex if is valid, otherwise null</returns>
		protected Regex? ParseRegex(BgScriptElement element, string regex)
		{
			if (regex.Length == 0)
			{
				LogError(element, "Regex is empty");
				return null;
			}
			try
			{
				return new Regex(regex);
			}
			catch (ArgumentException invalidRegex)
			{
				LogError(element, "Could not construct the Regex, reason: {Reason}", invalidRegex.Message);
				return null;
			}
		}

		/// <summary>
		/// Expands any properties and reads an attribute.
		/// </summary>
		/// <param name="element">Element to read the attribute from</param>
		/// <param name="name">Name of the attribute</param>
		/// <returns>Array of names, with all leading and trailing whitespace removed</returns>
		protected string ReadAttribute(BgScriptElement element, string name)
		{
			return ExpandProperties(element, element.GetAttribute(name));
		}

		/// <summary>
		/// Expands any properties and reads a list of strings from an attribute, separated by semi-colon characters
		/// </summary>
		/// <param name="element"></param>
		/// <param name="name"></param>
		/// <param name="separator"></param>
		/// <returns>Array of names, with all leading and trailing whitespace removed</returns>
		protected string[] ReadListAttribute(BgScriptElement element, string name, char separator = ';')
		{
			string value = ReadAttribute(element, name);

			// The main assumption here is that if there are separators within quotes
			// in the string, we want to ignore those and only split with the separators
			// outside quotes. 
			return SplitIgnoreSeparatorInQuotes(value, separator);
		}

		/// <summary>
		/// Parse a map of annotations
		/// </summary>
		/// <param name="element"></param>
		/// <param name="name"></param>
		/// <returns></returns>
		protected Dictionary<string, string> ReadAnnotationsAttribute(BgScriptElement element, string name)
		{
			string[] pairs = ReadListAttribute(element, name);
			return ParseAnnotations(element, pairs);
		}

		/// <summary>
		/// Split string with given separator into a list of strings while ignoring any 
		/// separators within quotes
		/// </summary>
		/// <param name="value"></param>
		/// <param name="separator"></param>
		/// <returns>
		/// An array of strings split by given separator while ignoring
		/// separators within quotes (if present).
		/// For example, if <br /> 
		/// <code> separator='+' </code> and 
		/// <code> value = "Path=\"D:\\Path+A\\To\\Dir\"+Path=\\"D:\\Path+B\\To\\Dir\"" </code> 
		/// This will return the following list of strings <br />
		/// <code>
		/// [ 
		/// "Path=\"D:\\Path+A\\To\\Dir\"",
		/// "Path=\"D:\\Path+B\\To\\Dir\"" 
		/// ] 
		/// </code>
		/// While ensuring the strings are not split by the separator chars within the quotes. 
		/// </returns>
		private static string[] SplitIgnoreSeparatorInQuotes(string value, char separator = ';')
		{
			const char EscapeValue = (char)127; 
			StringBuilder stringBuilder = new StringBuilder(value);

			// Find quote strings within given string and replace separator chars if present with escape value.
			// The final resulting string can be easily split using the given separator.
			for (int idx = value.IndexOf('\"', StringComparison.Ordinal); idx != -1; idx = value.IndexOf('\"', idx + 1))
			{
				int endIdx = value.ToString().IndexOf('\"', idx + 1);
				if (endIdx == -1)
				{
					break;
				}

				stringBuilder = stringBuilder.Replace(separator, EscapeValue, idx + 1, endIdx - (idx + 1)); 
				idx = endIdx;
			}

			value = stringBuilder.ToString();

			// Once we have split the string using the separator, ensure we replace 
			// the escape values.
			return value.Split(new char[] { separator })
				.Select(x => x.Replace(EscapeValue, separator).Trim())
				.Where(x => x.Length > 0)
				.ToArray();
		}

		/// <summary>
		/// Parse a map of annotations
		/// </summary>
		/// <param name="element"></param>
		/// <param name="pairs"></param>
		/// <returns></returns>
		private Dictionary<string, string> ParseAnnotations(BgScriptElement element, string[] pairs)
		{
			// Find the annotations to apply
			Dictionary<string, string> pairMap = new Dictionary<string, string>(StringComparer.OrdinalIgnoreCase);
			foreach (string pair in pairs)
			{
				if (!String.IsNullOrWhiteSpace(pair))
				{
					int idx = pair.IndexOf('=', StringComparison.Ordinal);
					if (idx < 0)
					{
						LogError(element, "Invalid annotation '{Pair}'", pair);
						continue;
					}

					string key = pair.Substring(0, idx).Trim();
					if (!Regex.IsMatch(key, @"[a-zA-Z0-9_\.]+"))
					{
						LogError(element, "Invalid annotation key '{Pair}'", pair);
						continue;
					}
					if (pairMap.ContainsKey(key))
					{
						LogError(element, "Annotation key '{Key}' was specified twice", key);
						continue;
					}

					pairMap.Add(key, pair.Substring(idx + 1).Trim());
				}
			}
			return pairMap;
		}

		/// <summary>
		/// Reads an attribute from the given XML element, expands any properties in it, and parses it as a boolean.
		/// </summary>
		/// <param name="element">Element to read the attribute from</param>
		/// <param name="name">Name of the attribute</param>
		/// <param name="defaultValue">Default value if the attribute is missing</param>
		/// <returns>The value of the attribute field</returns>
		protected bool ReadBooleanAttribute(BgScriptElement element, string name, bool defaultValue)
		{
			bool result = defaultValue;
			if (element.HasAttribute(name))
			{
				string value = ReadAttribute(element, name).Trim();
				if (value.Equals("true", StringComparison.OrdinalIgnoreCase))
				{
					result = true;
				}
				else if (value.Equals("false", StringComparison.OrdinalIgnoreCase))
				{
					result = false;
				}
				else
				{
					LogError(element, "Invalid boolean value '{0}' - expected 'true' or 'false'", value);
				}
			}
			return result;
		}

		/// <summary>
		/// Reads an attribute from the given XML element, expands any properties in it, and parses it as an integer.
		/// </summary>
		/// <param name="element">Element to read the attribute from</param>
		/// <param name="name">Name of the attribute</param>
		/// <param name="defaultValue">Default value for the integer, if the attribute is missing</param>
		/// <returns>The value of the attribute field</returns>
		protected int ReadIntegerAttribute(BgScriptElement element, string name, int defaultValue)
		{
			int result = defaultValue;
			if (element.HasAttribute(name))
			{
				string value = ReadAttribute(element, name).Trim();

				int intValue;
				if (Int32.TryParse(value, out intValue))
				{
					result = intValue;
				}
				else
				{
					LogError(element, "Invalid integer value '{Value}'", value);
				}
			}
			return result;
		}

		/// <summary>
		/// Reads an attribute from the given XML element, expands any properties in it, and parses it as an enum of the given type.
		/// </summary>
		/// <typeparam name="T">The enum type to parse the attribute as</typeparam>
		/// <param name="element">Element to read the attribute from</param>
		/// <param name="name">Name of the attribute</param>
		/// <param name="defaultValue">Default value for the enum, if the attribute is missing</param>
		/// <returns>The value of the attribute field</returns>
		protected T ReadEnumAttribute<T>(BgScriptElement element, string name, T defaultValue) where T : struct
		{
			T result = defaultValue;
			if (element.HasAttribute(name))
			{
				string value = ReadAttribute(element, name).Trim();

				T enumValue;
				if (Enum.TryParse(value, true, out enumValue))
				{
					result = enumValue;
				}
				else
				{
					LogError(element, "Invalid value '{Value}' - expected {PossibleValues}", value, String.Join("/", Enum.GetNames(typeof(T))));
				}
			}
			return result;
		}

		/// <summary>
		/// Outputs an error message to the log and increments the number of errors, referencing the file and line number of the element that caused it.
		/// </summary>
		/// <param name="element">The script element causing the error</param>
		/// <param name="format">Standard String.Format()-style format string</param>
		/// <param name="args">Optional arguments</param>
		protected void LogError(BgScriptElement element, string format, params object[] args)
		{
			Logger.LogScriptError(element.Location, format, args);
			NumErrors++;
		}

		/// <summary>
		/// Outputs a warning message to the log, referencing the file and line number of the element that caused it.
		/// </summary>
		/// <param name="element">The script element causing the error</param>
		/// <param name="format">Standard String.Format()-style format string</param>
		/// <param name="args">Optional arguments</param>
		protected void LogWarning(BgScriptElement element, string format, params object[] args)
		{
			Logger.LogScriptWarning(element.Location, format, args);
		}

		/// <summary>
		/// Evaluates the (optional) conditional expression on a given XML element via the If="..." attribute, and returns true if the element is enabled.
		/// </summary>
		/// <param name="element">The element to check</param>
		/// <returns>True if the element's condition evaluates to true (or doesn't have a conditional expression), false otherwise</returns>
		protected async Task<bool> EvaluateConditionAsync(BgScriptElement element)
		{
			// Check if the element has a conditional attribute
			const string AttributeName = "If";
			if (!element.HasAttribute(AttributeName))
			{
				return true;
			}

			// If it does, try to evaluate it.
			try
			{
				string text = ExpandProperties(element, element.GetAttribute("If"));
				return await BgCondition.EvaluateAsync(text, new BgConditionContext(_rootDir));
			}
			catch (BgConditionException ex)
			{
				LogError(element, "Error in condition: {Message}", ex.Message);
				return false;
			}
		}

		/// <summary>
		/// Expand all the property references (of the form $(PropertyName)) in a string.
		/// </summary>
		/// <param name="element">The element containing the string. Used for diagnostic messages.</param>
		/// <param name="text">The input string to expand properties in</param>
		/// <returns>The expanded string</returns>
		protected string ExpandProperties(BgScriptElement element, string text)
		{
			string result = text;
			// Iterate in reverse order to handle cases where there are nested expansions like $(Outer$(Inner))
			for (int idx = result.LastIndexOf("$(", StringComparison.Ordinal); idx != -1; idx = result.LastIndexOf("$(", idx, idx + 1, StringComparison.Ordinal))
			{
				// Find the end of the variable name
				int endIdx = result.IndexOf(')', idx + 2);
				if (endIdx == -1)
				{
					break;
				}

				// Extract the variable name from the string
				string name = result.Substring(idx + 2, endIdx - (idx + 2));

				// Find the value for it, either from the dictionary or the environment block
				string? value;
				if (!TryGetPropertyValue(name, out value))
				{
					LogWarning(element, "Property '{Name}' is not defined", name);
					value = "";
				}

				// Check if we've got a value for this variable
				if (value != null)
				{
					// Replace the variable, or skip past it
					result = result.Substring(0, idx) + value + result.Substring(endIdx + 1);
				}
			}
			return result;
		}

		/// <inheritdoc/>
		public static object GetNativePath(DirectoryReference rootDir, string path)
		{
			return FileReference.Combine(rootDir, path).FullName;
		}
	}
}