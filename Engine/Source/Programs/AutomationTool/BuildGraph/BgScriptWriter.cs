// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Linq;
using System.Xml;
using EpicGames.BuildGraph;
using EpicGames.Core;

#nullable enable

namespace AutomationTool
{
	/// <summary>
	/// Implementation of XmlDocument which preserves line numbers for its elements
	/// </summary>
	public static class BgScriptWriter
	{
		/// <summary>
		/// Writes a preprocessed build graph to a script file
		/// </summary>
		/// <param name="graph">Graph to output</param>
		/// <param name="file">The file to load</param>
		/// <param name="schemaFile">Schema file for validation</param>
		public static void Write(this BgGraphDef graph, FileReference file, FileReference? schemaFile)
		{
			XmlWriterSettings settings = new XmlWriterSettings();
			settings.Indent = true;
			settings.IndentChars = "\t";

			using (XmlWriter writer = XmlWriter.Create(file.FullName, settings))
			{
				writer.WriteStartElement("BuildGraph", "http://www.epicgames.com/BuildGraph");

				if (schemaFile != null)
				{
					writer.WriteAttributeString("schemaLocation", "http://www.w3.org/2001/XMLSchema-instance", "http://www.epicgames.com/BuildGraph " + schemaFile.MakeRelativeTo(file.Directory));
				}

				foreach (BgAgentDef agent in graph.Agents)
				{
					agent.Write(writer);
				}

				foreach (BgAggregateDef aggregate in graph.NameToAggregate.Values)
				{
					// If the aggregate has no required elements, skip it.
					if (aggregate.RequiredNodes.Count == 0)
					{
						continue;
					}

					writer.WriteStartElement("Aggregate");
					writer.WriteAttributeString("Name", aggregate.Name);
					writer.WriteAttributeString("Requires", String.Join(";", aggregate.RequiredNodes.Select(x => x.Name)));
					writer.WriteEndElement();
				}

				foreach (BgLabelDef label in graph.Labels)
				{
					writer.WriteStartElement("Label");
					if (label.DashboardCategory != null)
					{
						writer.WriteAttributeString("Category", label.DashboardCategory);
					}
					writer.WriteAttributeString("Name", label.DashboardName);
					writer.WriteAttributeString("Requires", String.Join(";", label.RequiredNodes.Select(x => x.Name)));

					HashSet<BgNodeDef> includedNodes = new HashSet<BgNodeDef>(label.IncludedNodes);
					includedNodes.ExceptWith(label.IncludedNodes.SelectMany(x => x.InputDependencies));
					includedNodes.ExceptWith(label.IncludedNodes.SelectMany(x => x.OptionalInputDependencies));
					includedNodes.ExceptWith(label.RequiredNodes);
					if (includedNodes.Count > 0)
					{
						writer.WriteAttributeString("Include", String.Join(";", includedNodes.Select(x => x.Name)));
					}

					HashSet<BgNodeDef> excludedNodes = new HashSet<BgNodeDef>(label.IncludedNodes);
					excludedNodes.UnionWith(label.IncludedNodes.SelectMany(x => x.InputDependencies));
					excludedNodes.UnionWith(label.IncludedNodes.SelectMany(x => x.OptionalInputDependencies));
					excludedNodes.ExceptWith(label.IncludedNodes);
					excludedNodes.ExceptWith(excludedNodes.ToArray().SelectMany(x => x.InputDependencies));
					excludedNodes.ExceptWith(excludedNodes.ToArray().SelectMany(x => x.OptionalInputDependencies));
					if (excludedNodes.Count > 0)
					{
						writer.WriteAttributeString("Exclude", String.Join(";", excludedNodes.Select(x => x.Name)));
					}
					writer.WriteEndElement();
				}

				foreach (BgReport report in graph.NameToReport.Values)
				{
					writer.WriteStartElement("Report");
					writer.WriteAttributeString("Name", report.Name);
					writer.WriteAttributeString("Requires", String.Join(";", report.Nodes.Select(x => x.Name)));
					writer.WriteEndElement();
				}

				foreach (BgBadgeDef badge in graph.Badges)
				{
					writer.WriteStartElement("Badge");
					writer.WriteAttributeString("Name", badge.Name);
					if (badge.Project != null)
					{
						writer.WriteAttributeString("Project", badge.Project);
					}
					if (badge.Change != 0)
					{
						writer.WriteAttributeString("Change", badge.Change.ToString());
					}
					writer.WriteAttributeString("Requires", String.Join(";", badge.Nodes.Select(x => x.Name)));
					writer.WriteEndElement();
				}

				writer.WriteEndElement();
			}
		}

		/// <summary>
		/// Writes this agent group out to a file, filtering nodes by a controlling trigger
		/// </summary>
		/// <param name="agent">Agent to output</param>
		/// <param name="writer">The XML writer to output to</param>
		public static void Write(this BgAgentDef agent, XmlWriter writer)
		{
			writer.WriteStartElement("Agent");
			writer.WriteAttributeString("Name", agent.Name);
			writer.WriteAttributeString("Type", String.Join(";", agent.PossibleTypes));
			foreach (BgNodeDef node in agent.Nodes)
			{
				node.Write(writer);
			}
			writer.WriteEndElement();
		}

		/// <summary>
		/// Write this node to an XML writer
		/// </summary>
		/// <param name="node">Node to output</param>
		/// <param name="writer">The writer to output the node to</param>
		public static void Write(this BgNodeDef node, XmlWriter writer)
		{
			writer.WriteStartElement("Node");
			writer.WriteAttributeString("Name", node.Name);

			string[] requireNames = node.Inputs.Select(x => x.TagName).ToArray();
			if (requireNames.Length > 0)
			{
				writer.WriteAttributeString("Requires", String.Join(";", requireNames));
			}

			string[] optionalRequireNames = node.OptionalInputs.Select(x => x.TagName).ToArray();
			if (optionalRequireNames.Length > 0)
			{
				writer.WriteAttributeString("OptionalRequires", String.Join(";", optionalRequireNames));
			}

			string[] producesNames = node.Outputs.Where(x => x != node.DefaultOutput).Select(x => x.TagName).ToArray();
			if (producesNames.Length > 0)
			{
				writer.WriteAttributeString("Produces", String.Join(";", producesNames));
			}

			string[] afterNames = node.GetDirectOrderDependencies().Except(node.InputDependencies).Except(node.OptionalInputDependencies).Select(x => x.Name).ToArray();
			if (afterNames.Length > 0)
			{
				writer.WriteAttributeString("After", String.Join(";", afterNames));
			}

			if (!node.NotifyOnWarnings)
			{
				writer.WriteAttributeString("NotifyOnWarnings", node.NotifyOnWarnings.ToString());
			}

			if (node.RunEarly)
			{
				writer.WriteAttributeString("RunEarly", node.RunEarly.ToString());
			}

			if (!node.AllowRetry)
			{
				writer.WriteAttributeString("AllowRetry", node.AllowRetry.ToString());
			}

			if (node.IgnoreModified.Count > 0)
			{
				writer.WriteAttributeString("IgnoreModified", String.Join(";", node.IgnoreModified));
			}

			BgScriptNode scriptNode = (BgScriptNode)node;
			foreach (BgTask task in scriptNode.Tasks)
			{
				task.Write(writer);
			}

			writer.WriteEndElement();
		}
	}
}