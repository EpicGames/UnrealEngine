// Copyright Epic Games, Inc. All Rights Reserved.

using System.Collections.Generic;
using System.Xml;

#nullable enable

namespace AutomationTool
{
	/// <summary>
	/// A task invocation
	/// </summary>
	public class BgTask
	{
		/// <summary>
		/// Line number in a source file that this task was declared. Optional; used for log messages.
		/// </summary>
		public BgScriptLocation Location { get; }

		/// <summary>
		/// Name of the task
		/// </summary>
		public string Name { get; set; }

		/// <summary>
		/// Arguments for the task
		/// </summary>
		public Dictionary<string, string> Arguments { get; } = new Dictionary<string, string>();

		/// <summary>
		/// Constructor
		/// </summary>
		public BgTask(BgScriptLocation location, string name)
		{
			Location = location;
			Name = name;
		}

		/// <summary>
		/// Write to an xml file
		/// </summary>
		/// <param name="writer"></param>
		public void Write(XmlWriter writer)
		{
			writer.WriteStartElement(Name);
			foreach (KeyValuePair<string, string> argument in Arguments)
			{
				writer.WriteAttributeString(argument.Key, argument.Value);
			}
			writer.WriteEndElement();
		}
	}
}
