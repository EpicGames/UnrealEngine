<Project Sdk="Microsoft.NET.Sdk">
  <Import Project="..\..\Shared\UnrealEngine.csproj.props" />

  <PropertyGroup>
    <OutputType>Library</OutputType>
    <GenerateAssemblyInfo>false</GenerateAssemblyInfo>
    <GenerateTargetFrameworkAttribute>false</GenerateTargetFrameworkAttribute>
    <RootNamespace>AutomationTool</RootNamespace>
    <AssemblyName>BuildGraph.Automation</AssemblyName>
    <WarningsNotAsErrors>612,618</WarningsNotAsErrors>
    <OutputPath Condition="'$(ArtifactsPath)' == ''">..\..\..\..\Binaries\DotNET\AutomationTool\AutomationScripts\BuildGraph</OutputPath>
    <TreatWarningsAsErrors>true</TreatWarningsAsErrors>
    <GenerateDocumentationFile>True</GenerateDocumentationFile>
  </PropertyGroup>
  <ItemGroup>
    <Reference Include="fastJSON">
      <HintPath>..\..\..\..\Binaries\ThirdParty\fastJSON\netstandard2.0\fastJSON.dll</HintPath>
    </Reference>
    <Reference Include="Ionic.Zip.Reduced">
      <HintPath>..\..\..\..\Binaries\DotNET\Ionic.Zip.Reduced.dll</HintPath>
      <Private>True</Private>
    </Reference>
  </ItemGroup>
  <ItemGroup>
    <Compile Include="..\..\Shared\MetaData.cs">
      <Link>Properties\MetaData.cs</Link>
    </Compile>
  </ItemGroup>
  <ItemGroup>
    <ProjectReference Include="..\..\Shared\EpicGames.BuildGraph\EpicGames.BuildGraph.csproj" />
    <ProjectReference Include="..\..\Shared\EpicGames.Core\EpicGames.Core.csproj" PrivateAssets="All"><Private>false</Private></ProjectReference>
    <ProjectReference Include="..\..\Shared\EpicGames.ProjectStore\EpicGames.ProjectStore.csproj" PrivateAssets="All" />
    <ProjectReference Include="..\..\UnrealBuildTool\UnrealBuildTool.csproj" PrivateAssets="All"><Private>false</Private></ProjectReference>
    <ProjectReference Include="..\AutomationUtils\AutomationUtils.Automation.csproj" PrivateAssets="All"><Private>false</Private></ProjectReference>
  </ItemGroup>
</Project>