// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.IO;
using System.Linq;
using System.Reflection;
using System.Runtime.InteropServices;
using System.Text;
using System.Threading.Tasks;
using System.Xml;
using EpicGames.BuildGraph;
using EpicGames.BuildGraph.Expressions;
using EpicGames.Core;
using EpicGames.MCP.Automation;
using Microsoft.Extensions.Logging;
using OpenTracing;
using OpenTracing.Util;
using UnrealBuildBase;
using UnrealBuildTool;

#nullable enable
#pragma warning disable CA1724

namespace AutomationTool
{
	/// <summary>
	/// Implementation of ScriptTaskParameter corresponding to a field in a parameter class
	/// </summary>
	abstract class ScriptTaskParameterBinding : BgScriptTaskParameter
	{
		public abstract Type ParameterType { get; }

		public ScriptTaskParameterBinding(string name, Type valueType, TaskParameterValidationType validationType, bool optional)
			: base(name, valueType, validationType, optional)
		{
		}

		public abstract string GetXmlDocName();
		public abstract object? GetValue(object sourceObject);
		public abstract void SetValue(object targetObject, object? value);
	}

	class ScriptTaskParameterPropertyBinding : ScriptTaskParameterBinding
	{
		readonly PropertyInfo _propertyInfo;

		public ScriptTaskParameterPropertyBinding(string name, PropertyInfo propertyInfo, TaskParameterValidationType validationType, bool optional)
			: base(name, propertyInfo.PropertyType, validationType, optional)
		{
			_propertyInfo = propertyInfo;
		}

		public override Type ParameterType
			=> _propertyInfo.PropertyType;

		public override string GetXmlDocName()
			=> "P:" + _propertyInfo.DeclaringType!.FullName + "." + Name;

		public override object? GetValue(object sourceObject)
			=> _propertyInfo.GetValue(sourceObject);

		public override void SetValue(object targetObject, object? value)
			=> _propertyInfo.SetValue(targetObject, value);
	}

	class ScriptTaskParameterFieldBinding : ScriptTaskParameterBinding
	{
		readonly FieldInfo _fieldInfo;

		public ScriptTaskParameterFieldBinding(string name, FieldInfo fieldInfo, TaskParameterValidationType validationType, bool optional)
			: base(name, fieldInfo.FieldType, validationType, optional)
		{
			_fieldInfo = fieldInfo;
		}

		public override Type ParameterType
			=> _fieldInfo.FieldType;

		public override string GetXmlDocName()
			=> "F:" + _fieldInfo.DeclaringType!.FullName + "." + Name;

		public override object? GetValue(object sourceObject)
			=> _fieldInfo.GetValue(sourceObject);

		public override void SetValue(object targetObject, object? value)
			=> _fieldInfo.SetValue(targetObject, value);
	}

	/// <summary>
	/// Binding of a ScriptTask to a Script
	/// </summary>
	class ScriptTaskBinding : BgScriptTask
	{
		/// <summary>
		/// Type of the task to construct with this info
		/// </summary>
		public Type TaskClass { get; }

		/// <summary>
		/// Type to construct with the parsed parameters
		/// </summary>
		public Type ParametersClass { get; }

		/// <summary>
		/// Map from name to parameter
		/// </summary>
		public IReadOnlyDictionary<string, ScriptTaskParameterBinding> NameToParameter { get; }

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="name">Name of the task</param>
		/// <param name="taskClass">Task class to create</param>
		/// <param name="parametersClass">Class type of an object to be constructed and passed as an argument to the task class constructor</param>
		public ScriptTaskBinding(string name, Type taskClass, Type parametersClass)
			: this(name, taskClass, parametersClass, CreateParameters(parametersClass))
		{
		}

		/// <summary>
		/// Private constructor
		/// </summary>
		private ScriptTaskBinding(string name, Type taskClass, Type parametersClass, List<ScriptTaskParameterBinding> parameters)
			: base(name, parameters.ConvertAll<BgScriptTaskParameter>(x => x))
		{
			TaskClass = taskClass;
			ParametersClass = parametersClass;
			NameToParameter = parameters.ToDictionary(x => x.Name, x => x);
		}

		static List<ScriptTaskParameterBinding> CreateParameters(Type parametersClass)
		{
			List<ScriptTaskParameterBinding> scriptTaskParameters = new List<ScriptTaskParameterBinding>();
			foreach (FieldInfo field in parametersClass.GetFields())
			{
				if (field.MemberType == MemberTypes.Field)
				{
					TaskParameterAttribute? parameterAttribute = field.GetCustomAttribute<TaskParameterAttribute>();
					if (parameterAttribute != null)
					{
						scriptTaskParameters.Add(new ScriptTaskParameterFieldBinding(field.Name, field, parameterAttribute.ValidationType, parameterAttribute.Optional));
					}
				}
			}
			foreach (PropertyInfo property in parametersClass.GetProperties(BindingFlags.Public | BindingFlags.Instance))
			{
				TaskParameterAttribute? parameterAttribute = property.GetCustomAttribute<TaskParameterAttribute>();
				if (parameterAttribute != null)
				{
					scriptTaskParameters.Add(new ScriptTaskParameterPropertyBinding(property.Name, property, parameterAttribute.ValidationType, parameterAttribute.Optional));
				}
			}
			return scriptTaskParameters;
		}
	}

	/// <summary>
	/// Tool to execute build automation scripts for UE projects, which can be run locally or in parallel across a build farm (assuming synchronization and resource allocation implemented by a separate system).
	///
	/// Build graphs are declared using an XML script using syntax similar to MSBuild, ANT or NAnt, and consist of the following components:
	///
	/// - Tasks:        Building blocks which can be executed as part of the build process. Many predefined tasks are provided ('Cook', 'Compile', 'Copy', 'Stage', 'Log', 'PakFile', etc...), and additional tasks may be 
	///                 added be declaring classes derived from AutomationTool.BuildTask in other UAT modules. 
	/// - Nodes:        A named sequence of tasks which are executed in order to produce outputs. Nodes may have dependencies on other nodes for their outputs before they can be executed. Declared with the 'Node' element.
	/// - Agents:		A machine which can execute a sequence of nodes, if running as part of a build system. Has no effect when building locally. Declared with the 'Agent' element.
	/// - Triggers:     Container for agents which should only be executed when explicitly triggered (using the -Trigger=... or -SkipTriggers command line argument). Declared with the 'Trigger' element.
	/// - Notifiers:    Specifies email recipients for failures in one or more nodes, whether they should receive notifications on warnings, and so on.
	/// 
	/// Scripts may set properties with the &lt;Property Name="Foo" Value="Bar"/&gt; syntax. Properties referenced with the $(Property Name) notation are valid within all strings, and will be expanded as macros when the 
	/// script is read. If a property name is not set explicitly, it defaults to the contents of an environment variable with the same name. Properties may be sourced from environment variables or the command line using
	/// the &lt;EnvVar&gt; and &lt;Option&gt; elements respectively.
	///
	/// Any elements can be conditionally defined via the "If" attribute. A full grammar for conditions is written up in Condition.cs.
	/// 
	/// File manipulation is done using wildcards and tags. Any attribute that accepts a list of files may consist of: a Perforce-style wildcard (matching any number of "...", "*" and "?" patterns in any location), a 
	/// full path name, or a reference to a tagged collection of files, denoted by prefixing with a '#' character. Files may be added to a tag set using the &lt;Tag&gt; Task, which also allows performing set union/difference 
	/// style operations. Each node can declare multiple outputs in the form of a list of named tags, which other nodes can then depend on.
	/// 
	/// Build graphs may be executed in parallel as part build system. To do so, the initial graph configuration is generated by running with the -Export=... argument (producing a JSON file listing the nodes 
	/// and dependencies to execute). Each participating agent should be synced to the same changelist, and UAT should be re-run with the appropriate -Node=... argument. Outputs from different nodes are transferred between 
	/// agents via shared storage, typically a network share, the path to which can be specified on the command line using the -SharedStorageDir=... argument. Note that the allocation of machines, and coordination between 
	/// them, is assumed to be managed by an external system based on the contents of the script generated by -Export=....
	/// 
	/// A schema for the known set of tasks can be generated by running UAT with the -Schema=... option. Generating a schema and referencing it from a BuildGraph script allows Visual Studio to validate and auto-complete 
	/// elements as you type.
	/// </summary>
	[Help("Tool for creating extensible build processes in UE which can be run locally or in parallel across a build farm.")]
	[ParamHelp("Script=<FileName>", "Path to the script describing the graph", ParamType = typeof(File), Required = true)]
	[ParamHelp("Target=<Name>", "Name of the node or output tag to be built")]
	[ParamHelp("Schema", "Generates a schema to the default location", ParamType = typeof(bool))]
	[ParamHelp("Schema=<FileName>", "Generate a schema describing valid script documents, including all the known tasks", ParamType = typeof(string))]
	[ParamHelp("ImportSchema=<FileName>", "Imports a schema from an existing schema file", ParamType = typeof(File))]
	[ParamHelp("Set:<Property>=<Value>", "Sets a named property to the given value", Action = ParamHelpAttribute.ParamAction.Append, Flag = "-Set:")]
	[ParamHelp("Branch=<Value>", "Overrides the auto-detection of the current branch")]
	[ParamHelp("Clean", "Cleans all cached state of completed build nodes before running", ParamType = typeof(bool), Deprecated = true)]
	[ParamHelp("CleanNode=<Name>[+<Name>...]", "Cleans just the given nodes before running", MultiSelectSeparator = "+")]
	[ParamHelp("Resume", "Resumes a local build from the last node that completed successfully", ParamType = typeof(bool))]
	[ParamHelp("ListOnly", "Shows the contents of the preprocessed graph, but does not execute it", ParamType = typeof(bool), DefaultValue = false)]
	[ParamHelp("ShowDiagnostics", "When running with -ListOnly, causes diagnostic messages entered when parsing the graph to be shown", ParamType = typeof(bool))]
	[ParamHelp("ShowDeps", "Show node dependencies in the graph output", ParamType = typeof(bool))]
	[ParamHelp("ShowNotifications", "Show notifications that will be sent for each node in the output", ParamType = typeof(bool))]
	[ParamHelp("Trigger=<Name>", "Executes only nodes behind the given trigger")]
	[ParamHelp("SkipTrigger=<Name>[+<Name>...]", "Skips the given triggers, including all the nodes behind them in the graph", MultiSelectSeparator = "+")]
	[ParamHelp("SkipTriggers", "Skips all triggers", ParamType = typeof(bool))]
	[ParamHelp("TokenSignature=<Name>", "Specifies the signature identifying the current job, to be written to tokens for nodes that require them. Tokens are ignored if this parameter is not specified.")]
	[ParamHelp("SkipTargetsWithoutTokens", "Excludes targets which we can't acquire tokens for, rather than failing", ParamType = typeof(bool))]
	[ParamHelp("Preprocess=<FileName>", "Writes the preprocessed graph to the given file", ParamType = typeof(File))]
	[ParamHelp("Export=<FileName>", "Exports a JSON file containing the preprocessed build graph, for use as part of a build system", ParamType = typeof(File))]
	[ParamHelp("HordeExport=<FileName>", "Exports a JSON file containing the full build graph for use by Horde.", ParamType = typeof(File))]
	[ParamHelp("PublicTasksOnly", "Only include built-in tasks in the schema, excluding any other UAT modules", ParamType = typeof(bool))]
	[ParamHelp("SharedStorageDir=<DirName>", "Sets the directory to use to transfer build products between agents in a build farm", ParamType = typeof(Directory))]
	[ParamHelp("SingleNode=<Name>", "Run only the given node. Intended for use on a build system after running with -Export.")]
	[ParamHelp("WriteToSharedStorage", "Allow writing to shared storage. If not set, but -SharedStorageDir is specified, build products will read but not written", ParamType = typeof(bool))]
	[ParamHelp("IgnoreOptionalNodeFailures", "Continue execution when nodes that are only optional dependencies fail", ParamType = typeof(bool))]
	public class BuildGraph : BuildCommand
	{
		/// <summary>
		/// Main entry point for the BuildGraph command
		/// </summary>
		public override async Task<ExitCode> ExecuteAsync()
		{
			// Parse the command line parameters
			string? className = ParseParamValue("Class", null);
			string? scriptFileName = ParseParamValue("Script", null);
			string[] targetNames = ParseParamValues("Target").SelectMany(x => x.Split(';', '+').Select(y => y.Trim()).Where(y => y.Length > 0)).ToArray();
			string? documentationFileName = ParseParamValue("Documentation", null);
			string? schemaFileName = ParseParamValue("Schema", null);
			string? importSchemaFileName = ParseParamValue("ImportSchema", null);
			string? exportFileName = ParseParamValue("Export", null);
			string? hordeExportFileName = ParseParamValue("HordeExport", null);
			string? preprocessedFileName = ParseParamValue("Preprocess", null);
			string? sharedStorageDir = ParseParamValue("SharedStorageDir", null);
			string? singleNodeName = ParseParamValue("SingleNode", null);
			string? triggerName = ParseParamValue("Trigger", null);
			string? tokenSignature = ParseParamValue("TokenSignature", null);
			bool skipTargetsWithoutTokens = ParseParam("SkipTargetsWithoutTokens");
			bool resume = singleNodeName != null || ParseParam("Resume");
			bool listOnly = ParseParam("ListOnly");
			bool showDiagnostics = ParseParam("ShowDiagnostics");
			bool writeToSharedStorage = ParseParam("WriteToSharedStorage") || CommandUtils.IsBuildMachine;
			bool publicTasksOnly = ParseParam("PublicTasksOnly");
			bool skipValidation = ParseParam("SkipValidation");
			bool ignoreOptionalNodeFailures = ParseParam("IgnoreOptionalNodeFailures");
			string? reportName = ParseParamValue("ReportName", null);
			string? branchOverride = ParseParamValue("Branch", null);

			GraphPrintOptions printOptions = GraphPrintOptions.ShowCommandLineOptions;
			if (ParseParam("ShowDeps"))
			{
				printOptions |= GraphPrintOptions.ShowDependencies;
			}
			if (ParseParam("ShowNotifications"))
			{
				printOptions |= GraphPrintOptions.ShowNotifications;
			}

			if (schemaFileName == null && ParseParam("Schema"))
			{
				schemaFileName = FileReference.Combine(Unreal.EngineDirectory, "Build", "Graph", "Schema.xsd").FullName;
			}

			// Parse any specific nodes to clean
			List<string> cleanNodes = new List<string>();
			foreach (string nodeList in ParseParamValues("CleanNode"))
			{
				foreach (string nodeName in nodeList.Split('+', ';'))
				{
					cleanNodes.Add(nodeName);
				}
			}

			// Get the standard P4 properties, defaulting to the environment variables if not set. This allows setting p4-like properties without having a P4 connection.
			string? branch = P4Enabled ? P4Env.Branch : GetEnvVarOrNull(EnvVarNames.BuildRootP4);
			int? change = P4Enabled ? P4Env.Changelist : GetEnvVarIntOrNull(EnvVarNames.Changelist);
			int? codeChange = P4Enabled ? P4Env.CodeChangelist : GetEnvVarIntOrNull(EnvVarNames.CodeChangelist);

			// Set up the standard properties which build scripts might need
			Dictionary<string, string?> defaultProperties = new Dictionary<string, string?>(StringComparer.InvariantCultureIgnoreCase);
			defaultProperties["Branch"] = branch ?? "Unknown";
			defaultProperties["Depot"] = (branch != null && branch.StartsWith("//", StringComparison.Ordinal)) ? branch.Substring(2).Split('/').First() : "Unknown";
			defaultProperties["EscapedBranch"] = String.IsNullOrEmpty(branch) ? "Unknown" : CommandUtils.EscapePath(branch);
			defaultProperties["Change"] = (change ?? 0).ToString();
			defaultProperties["CodeChange"] = (codeChange ?? 0).ToString();
			defaultProperties["IsBuildMachine"] = IsBuildMachine ? "true" : "false";
			defaultProperties["HostPlatform"] = HostPlatform.Current.HostEditorPlatform.ToString();
			defaultProperties["HostArchitecture"] = RuntimeInformation.ProcessArchitecture.ToString();
			defaultProperties["RestrictedFolderNames"] = String.Join(";", RestrictedFolder.GetNames());
			defaultProperties["RestrictedFolderFilter"] = String.Join(";", RestrictedFolder.GetNames().Select(x => String.Format(".../{0}/...", x)));
			defaultProperties["DataDrivenPlatforms"] = String.Join(";", DataDrivenPlatformInfo.GetAllPlatformInfos().Keys);

			// Look for overrides
			if (!String.IsNullOrEmpty(branchOverride))
			{
				Logger.LogInformation("Overriding default branch '{Branch}' with '{BranchOverride}'", defaultProperties["Branch"], branchOverride);
				defaultProperties["Branch"] = branchOverride;
				defaultProperties["EscapedBranch"] = CommandUtils.EscapePath(defaultProperties["Branch"]);
			}

			// Prevent expansion of the root directory if we're just preprocessing the output. They may vary by machine.
			if (preprocessedFileName == null)
			{
				defaultProperties["RootDir"] = Unreal.RootDirectory.FullName;
			}
			else
			{
				defaultProperties["RootDir"] = null;
			}

			// Attempt to read existing Build Version information
			BuildVersion? version;
			if (BuildVersion.TryRead(BuildVersion.GetDefaultFileName(), out version))
			{
				defaultProperties["EngineMajorVersion"] = version.MajorVersion.ToString();
				defaultProperties["EngineMinorVersion"] = version.MinorVersion.ToString();
				defaultProperties["EnginePatchVersion"] = version.PatchVersion.ToString();
				defaultProperties["EngineCompatibleChange"] = version.CompatibleChangelist.ToString();
			}

			// If the -project flag is given, pass useful information into the graph
			ParseProjectIdParams(out FileReference? projectFile, out string? projectName,
				out DirectoryReference? projectDir);
			if (projectFile != null)
			{
				defaultProperties["ProjectFile"] = projectFile.FullName;
			}
			if (projectName != null)
			{
				defaultProperties["ProjectName"] = projectName;
			}
			if (projectDir != null)
			{ 
				defaultProperties["ProjectDir"] = projectDir.FullName;
			}

			// Add any additional custom arguments from the command line (of the form -Set:X=Y)
			Dictionary<string, string> arguments = new Dictionary<string, string>(StringComparer.InvariantCultureIgnoreCase);
			foreach (string param in Params)
			{
				const string SetPrefix = "set:";
				if (param.StartsWith(SetPrefix, StringComparison.InvariantCultureIgnoreCase))
				{
					int equalsIdx = param.IndexOf('=', StringComparison.Ordinal);
					if (equalsIdx >= 0)
					{
						arguments[param.Substring(SetPrefix.Length, equalsIdx - SetPrefix.Length)] = param.Substring(equalsIdx + 1);
					}
					else
					{
						Logger.LogWarning("Missing value for '{Arg0}'", param.Substring(SetPrefix.Length));
					}
				}

				const string AppendPrefix = "append:";
				if (param.StartsWith(AppendPrefix, StringComparison.InvariantCultureIgnoreCase))
				{
					int equalsIdx = param.IndexOf('=', StringComparison.Ordinal);
					if (equalsIdx >= 0)
					{
						string property = param.Substring(AppendPrefix.Length, equalsIdx - AppendPrefix.Length);
						string value = param.Substring(equalsIdx + 1);
						if (arguments.ContainsKey(property))
						{
							arguments[property] = arguments[property] + ";" + value;
						}
						else
						{
							arguments[property] = value;
						}
					}
					else
					{
						Logger.LogWarning("Missing value for '{Arg0}'", param.Substring(AppendPrefix.Length));
					}
				}
			}

			// Find all the tasks from the loaded assemblies
			Dictionary<string, ScriptTaskBinding> nameToTask = new Dictionary<string, ScriptTaskBinding>();
			if (!FindAvailableTasks(nameToTask, publicTasksOnly))
			{
				return ExitCode.Error_Unknown;
			}

			// Generate documentation
			if (documentationFileName != null)
			{
				WriteDocumentation(nameToTask, new FileReference(documentationFileName));
				return ExitCode.Success;
			}

			// Create the graph
			BgGraphDef? graph;
			if (className != null)
			{
				// Find all the graph builders
				Dictionary<string, Type> nameToType = new Dictionary<string, Type>();
				FindAvailableGraphs(nameToType);

				Type? builderType;
				if (!nameToType.TryGetValue(className, out builderType))
				{
					Logger.LogError("Unable to find graph '{GraphName}'", className);
					Logger.LogInformation("");
					Logger.LogInformation("Available graphs:");
					foreach (string name in nameToType.Keys.OrderBy(x => x))
					{
						Logger.LogInformation("  {GraphName}", name);
					}
					return ExitCode.Error_Unknown;
				}

				BgGraphBuilder builder = (BgGraphBuilder)Activator.CreateInstance(builderType)!;
				BgGraph graphSpec = builder.CreateGraph();

				(byte[] data, BgThunkDef[] methods) = BgCompiler.Compile(graphSpec);

				BgInterpreter interpreter = new BgInterpreter(data, methods, arguments);
				// interpreter.Disassemble(Logger);
				graph = ((BgObjectDef)interpreter.Evaluate()).Deserialize<BgGraphExpressionDef>().ToGraphDef();
			}
			else
			{
				// Import schema if one is passed in
				BgScriptSchema? schema;
				if (!String.IsNullOrWhiteSpace(importSchemaFileName))
				{
					schema = BgScriptSchema.Import(new(importSchemaFileName));
				}
				else
				{
					// Add any primitive types
					List<(Type, ScriptSchemaStandardType)> primitiveTypes = new List<(Type, ScriptSchemaStandardType)>();
					primitiveTypes.Add((typeof(FileReference), ScriptSchemaStandardType.BalancedString));
					primitiveTypes.Add((typeof(DirectoryReference), ScriptSchemaStandardType.BalancedString));
					primitiveTypes.Add((typeof(UnrealTargetPlatform), ScriptSchemaStandardType.BalancedString));
					primitiveTypes.Add((typeof(MCPPlatform), ScriptSchemaStandardType.BalancedString));

					// Create a schema for the given tasks
					schema = new BgScriptSchema(nameToTask.Values, primitiveTypes);
					if (schemaFileName != null)
					{
						FileReference fullSchemaFileName = new FileReference(schemaFileName);
						Logger.LogInformation("Writing schema to {Arg0}...", fullSchemaFileName.FullName);
						schema.Export(fullSchemaFileName);
						if (scriptFileName == null)
						{
							return ExitCode.Success;
						}
					}
				}

				// Check there was a script specified
				if (scriptFileName == null)
				{
					Logger.LogError("Missing -Script= parameter for BuildGraph");
					return ExitCode.Error_Unknown;
				}

				// Normalize the script filename
				FileReference fullScriptFile = FileReference.Combine(Unreal.RootDirectory, scriptFileName);

				// Read the script from disk
				graph = await BgScriptReader.ReadAsync(fullScriptFile, Unreal.RootDirectory, arguments, defaultProperties, schema, Logger, singleNodeName);
				if (graph == null)
				{
					return ExitCode.Error_Unknown;
				}
			}

			// Get the temp storage manifest directory. When spawning buildgraph through a UAT child process, be careful not to
			// overwrite any manifests from the parent. These may be required by the managing build system.
			DirectoryReference rootDir = new DirectoryReference(CommandUtils.CmdEnv.LocalRoot);
			DirectoryReference manifestDir = DirectoryReference.Combine(rootDir, "Engine", "Saved", CmdEnv.IsChildInstance? "BuildGraphChildInstance" : "BuildGraph");

			// Create the temp storage handler
			TempStorage storage = new TempStorage(rootDir, manifestDir, (sharedStorageDir == null) ? null : new DirectoryReference(sharedStorageDir), writeToSharedStorage);
			if (!resume)
			{
				storage.CleanLocal();
			}
			foreach (string cleanNode in cleanNodes)
			{
				storage.CleanLocalNode(cleanNode);
			}

			// Convert the supplied target references into nodes 
			HashSet<BgNodeDef> targetNodes = new HashSet<BgNodeDef>();
			if (targetNames.Length == 0)
			{
				if (!listOnly && singleNodeName == null)
				{
					Logger.LogError("Missing -Target= parameter for BuildGraph");
					return ExitCode.Error_Unknown;
				}
				targetNodes.UnionWith(graph.Agents.SelectMany(x => x.Nodes));
			}
			else
			{
				IEnumerable<string>? nodesToResolve = null;

				// If we're only building a single node and using a preprocessed reference we only need to try to resolve the references
				// for that node.
				if (singleNodeName != null && preprocessedFileName != null)
				{
					nodesToResolve = new List<string> { singleNodeName };
				}
				else
				{
					nodesToResolve = targetNames;
				}

				foreach (string targetName in nodesToResolve)
				{
					BgNodeDef[]? nodes;
					if (!graph.TryResolveReference(targetName, out nodes))
					{
						Logger.LogError("Target '{TargetName}' is not in graph", targetName);
						return ExitCode.Error_Unknown;
					}
					targetNodes.UnionWith(nodes);
				}
			}

			// Try to acquire tokens for all the target nodes we want to build
			if (tokenSignature != null)
			{
				// Find all the lock files
				HashSet<FileReference> requiredTokens = new HashSet<FileReference>(targetNodes.SelectMany(x => x.RequiredTokens));

				// List out all the required tokens
				if (singleNodeName == null)
				{
					Logger.LogInformation("Required tokens:");
					foreach (BgNodeDef node in targetNodes)
					{
						foreach (FileReference requiredToken in node.RequiredTokens)
						{
							Logger.LogInformation("  '{Node}' requires {RequiredToken}", node, requiredToken);
						}
					}
				}

				// Try to create all the lock files
				List<FileReference> createdTokens = new List<FileReference>();
				if (!listOnly)
				{
					createdTokens.AddRange(requiredTokens.Where(x => WriteTokenFile(x, tokenSignature)));
				}

				// Find all the tokens that we don't have
				Dictionary<FileReference, string> missingTokens = new Dictionary<FileReference, string>();
				foreach (FileReference requiredToken in requiredTokens)
				{
					string? currentOwner = ReadTokenFile(requiredToken);
					if (currentOwner != null && currentOwner != tokenSignature)
					{
						missingTokens.Add(requiredToken, currentOwner);
					}
				}

				// If we want to skip all the nodes with missing locks, adjust the target nodes to account for it
				if (missingTokens.Count > 0)
				{
					if (skipTargetsWithoutTokens)
					{
						// Find all the nodes we're going to skip
						HashSet<BgNodeDef> skipNodes = new HashSet<BgNodeDef>();
						foreach (IGrouping<string, FileReference> missingTokensForBuild in missingTokens.GroupBy(x => x.Value, x => x.Key))
						{
							Logger.LogInformation("Skipping the following nodes due to {Arg0}:", missingTokensForBuild.Key);
							foreach (FileReference missingToken in missingTokensForBuild)
							{
								foreach (BgNodeDef skipNode in targetNodes.Where(x => x.RequiredTokens.Contains(missingToken) && skipNodes.Add(x)))
								{
									Logger.LogInformation("    {SkipNode}", skipNode);
								}
							}
						}

						// Write a list of everything left over
						if (skipNodes.Count > 0)
						{
							targetNodes.ExceptWith(skipNodes);
							Logger.LogInformation("Remaining target nodes:");
							foreach (BgNodeDef targetNode in targetNodes)
							{
								Logger.LogInformation("    {TargetNode}", targetNode);
							}
							if (targetNodes.Count == 0)
							{
								Logger.LogInformation("    None.");
							}
						}
					}
					else
					{
						foreach (KeyValuePair<FileReference, string> pair in missingTokens)
						{
							List<BgNodeDef> skipNodes = targetNodes.Where(x => x.RequiredTokens.Contains(pair.Key)).ToList();
							Logger.LogError("Cannot run {Arg0} due to previous build: {Arg1}", String.Join(", ", skipNodes), pair.Value);
						}
						foreach (FileReference createdToken in createdTokens)
						{
							FileReference.Delete(createdToken);
						}
						return ExitCode.Error_Unknown;
					}
				}
			}

			// Cull the graph to include only those nodes
			graph.Select(targetNodes);

			// If a report for the whole build was requested, insert it into the graph
			if (reportName != null)
			{
				BgReport newReport = new BgReport(reportName);
				newReport.Nodes.UnionWith(graph.Agents.SelectMany(x => x.Nodes));
				graph.NameToReport.Add(reportName, newReport);
			}

			// Export the graph for Horde
			if (hordeExportFileName != null)
			{
				graph.ExportForHorde(new FileReference(hordeExportFileName));
			}

			// Write out the preprocessed script
			if (preprocessedFileName != null)
			{
				FileReference preprocessedFileLocation = new FileReference(preprocessedFileName);
				Logger.LogInformation("Writing {PreprocessedFileLocation}...", preprocessedFileLocation);
				graph.Write(preprocessedFileLocation, (schemaFileName != null) ? new FileReference(schemaFileName) : null);
				listOnly = true;
			}

			// If we're just building a single node, find it 
			BgNodeDef? singleNode = null;
			if (singleNodeName != null && !graph.NameToNode.TryGetValue(singleNodeName, out singleNode))
			{
				Logger.LogError("Node '{SingleNodeName}' is not in the trimmed graph", singleNodeName);
				return ExitCode.Error_Unknown;
			}

			// If we just want to show the contents of the graph, do so and exit.
			if (listOnly)
			{
				HashSet<BgNodeDef> completedNodes = FindCompletedNodes(graph, storage);
				graph.Print(completedNodes, printOptions, Log.Logger);
			}

			// Print out all the diagnostic messages which still apply, unless we're running a step as part of a build system or just listing the contents of the file. 
			if (singleNode == null && (!listOnly || showDiagnostics))
			{
				List<BgDiagnosticDef> diagnostics = graph.GetAllDiagnostics();
				foreach (BgDiagnosticDef diagnostic in diagnostics)
				{
					if (diagnostic.Level == LogLevel.Information)
					{
						Logger.LogInformation("{Arg0}({Arg1}): {Arg2}", diagnostic.File, diagnostic.Line, diagnostic.Message);
					}
					else if (diagnostic.Level == LogLevel.Warning)
					{
						Logger.LogWarning("{Arg0}({Arg1}): warning: {Arg2}", diagnostic.File, diagnostic.Line, diagnostic.Message);
					}
					else
					{
						Logger.LogError("{Arg0}({Arg1}): error: {Arg2}", diagnostic.File, diagnostic.Line, diagnostic.Message);
					}
				}
				if (diagnostics.Any(x => x.Level == LogLevel.Error))
				{
					return ExitCode.Error_Unknown;
				}
			}
			else if (singleNode != null || listOnly)
			{
				List<BgDiagnosticDef> diagnostics = [];
				foreach (BgDiagnosticDef diagnostic in graph.GetAllDiagnostics())
				{
					if (diagnostic.ReportOnExecution)
					{
						if (singleNode != null && diagnostic.ReportOnNodes.Length > 0)
						{
							if (diagnostic.ReportOnNodes.Contains(singleNode.Name, StringComparer.OrdinalIgnoreCase))
							{
								diagnostics.Add(diagnostic);
							}
						}
						else
						{
							diagnostics.Add(diagnostic);
						}
					}
				}
				foreach (BgDiagnosticDef diagnostic in diagnostics)
				{
					if (diagnostic.Level == LogLevel.Information)
					{
						Logger.LogInformation("{File}({LineNumber}): {Message}", diagnostic.File, diagnostic.Line, diagnostic.Message);
					}
					else if (diagnostic.Level == LogLevel.Warning)
					{
						Logger.LogWarning("{File}({LineNumber}): warning: {Message}", diagnostic.File, diagnostic.Line, diagnostic.Message);
					}
					else
					{
						Logger.LogError("{File}({LineNumber}): error: {Message}", diagnostic.File, diagnostic.Line, diagnostic.Message);
					}
				}
				if (diagnostics.Any(x => x.Level == LogLevel.Error))
				{
					return ExitCode.Error_Unknown;
				}
			}

			// Export the graph to a file
			if (exportFileName != null)
			{
				HashSet<BgNodeDef> completedNodes = FindCompletedNodes(graph, storage);
				graph.Print(completedNodes, printOptions, Log.Logger);
				graph.Export(new FileReference(exportFileName), completedNodes);
				return ExitCode.Success;
			}

			// Create tasks for the entire graph
			Dictionary<BgNodeDef, BgNodeExecutor> nodeToExecutor = new Dictionary<BgNodeDef, BgNodeExecutor>();
			if (skipValidation && singleNode != null)
			{
				if (!await BindNodesAsync(singleNode, nameToTask, graph.TagNameToNodeOutput, nodeToExecutor))
				{
					return ExitCode.Error_Unknown;
				}
			}
			else
			{
				if (!await BindNodesAsync(graph, nameToTask, nodeToExecutor))
				{
					return ExitCode.Error_Unknown;
				}
			}

			// Execute the command
			if (!listOnly)
			{
				if (singleNode != null)
				{
					if (!await BuildNodeAsync(graph, singleNode, nodeToExecutor, storage, withBanner: true))
					{
						return ExitCode.Error_Unknown;
					}
				}
				else
				{
					if (!await BuildAllNodesAsync(graph, nodeToExecutor, storage, ignoreOptionalNodeFailures))
					{
						return ExitCode.Error_Unknown;
					}
				}
			}
			return ExitCode.Success;
		}

		static async ValueTask<bool> BindNodesAsync(BgGraphDef graph, Dictionary<string, ScriptTaskBinding> nameToTask, Dictionary<BgNodeDef, BgNodeExecutor> nodeToExecutor)
		{
			bool result = true;
			foreach (BgAgentDef agent in graph.Agents)
			{
				foreach (BgNodeDef node in agent.Nodes)
				{
					result &= await BindNodesAsync(node, nameToTask, graph.TagNameToNodeOutput, nodeToExecutor);
				}
			}
			return result;
		}

		static async ValueTask<bool> BindNodesAsync(BgNodeDef node, Dictionary<string, ScriptTaskBinding> nameToTask, Dictionary<string, BgNodeOutput> tagNameToNodeOutput, Dictionary<BgNodeDef, BgNodeExecutor> nodeToExecutor)
		{
			if (node is BgScriptNode scriptNode)
			{
				BgScriptNodeExecutor executor = new BgScriptNodeExecutor(scriptNode);
				nodeToExecutor[node] = executor;
				return await executor.BindAsync(nameToTask, tagNameToNodeOutput, Logger);
			}
			else if (node is BgNodeExpressionDef bytecodeNode)
			{
				BgBytecodeNodeExecutor executor = new BgBytecodeNodeExecutor(bytecodeNode);
				nodeToExecutor[node] = executor;
				return BgBytecodeNodeExecutor.Bind(Logger);
			}
			else
			{
				throw new NotImplementedException();
			}
		}

		static void FindAvailableGraphs(Dictionary<string, Type> nameToType)
		{
			foreach (Assembly loadedAssembly in ScriptManager.AllScriptAssemblies)
			{
				Type[] types;
				try
				{
					types = loadedAssembly.GetTypes();
				}
				catch (ReflectionTypeLoadException ex)
				{
					Logger.LogWarning(ex, "Exception {Ex} while trying to get types from assembly {LoadedAssembly}. LoaderExceptions: {Arg2}", ex, loadedAssembly, String.Join("\n", ex.LoaderExceptions.Select(x => x?.Message)));
					continue;
				}

				foreach (Type type in types)
				{
					if (type.IsSubclassOf(typeof(BgGraphBuilder)))
					{
						nameToType.Add(type.Name, type);
					}
				}
			}
		}

		/// <summary>
		/// Find all the tasks which are available from the loaded assemblies
		/// </summary>
		/// <param name="nameToTask">Mapping from task name to information about how to serialize it</param>
		/// <param name="publicTasksOnly">Whether to include just public tasks, or all the tasks in any loaded assemblies</param>
		static bool FindAvailableTasks(Dictionary<string, ScriptTaskBinding> nameToTask, bool publicTasksOnly)
		{
			IEnumerable<Assembly> loadedScriptAssemblies = ScriptManager.AllScriptAssemblies;

			if (publicTasksOnly)
			{
				loadedScriptAssemblies = loadedScriptAssemblies.Where(x => IsPublicAssembly(new FileReference(x.Location)));
			}
			foreach (Assembly loadedAssembly in loadedScriptAssemblies)
			{
				Type[] types;
				try
				{
					types = loadedAssembly.GetTypes();
				}
				catch (ReflectionTypeLoadException ex)
				{
					Logger.LogWarning(ex, "Exception {Ex} while trying to get types from assembly {LoadedAssembly}. LoaderExceptions: {Arg2}", ex, loadedAssembly, String.Join("\n", ex.LoaderExceptions.Select(x => x?.Message)));
					continue;
				}

				foreach (Type type in types)
				{
					foreach (TaskElementAttribute elementAttribute in type.GetCustomAttributes<TaskElementAttribute>())
					{
						if (!type.IsSubclassOf(typeof(BgTaskImpl)))
						{
							Logger.LogError("Class '{Arg0}' has TaskElementAttribute, but is not derived from 'BgTaskImpl'", type.Name);
							return false;
						}
						if (nameToTask.ContainsKey(elementAttribute.Name))
						{
							Logger.LogError("Found multiple handlers for task elements called '{Arg0}'", elementAttribute.Name);
							return false;
						}
						nameToTask.Add(elementAttribute.Name, new ScriptTaskBinding(elementAttribute.Name, type, elementAttribute.ParametersType));
					}
				}
			}
			return true;
		}

		/// <summary>
		/// Reads the contents of the given token
		/// </summary>
		/// <returns>Contents of the token, or null if it does not exist</returns>
		public static string? ReadTokenFile(FileReference location)
		{
			return FileReference.Exists(location) ? File.ReadAllText(location.FullName) : null;
		}

		/// <summary>
		/// Attempts to write an owner to a token file transactionally
		/// </summary>
		/// <returns>True if the lock was acquired, false otherwise</returns>
		public static bool WriteTokenFile(FileReference location, string signature)
		{
			// Check it doesn't already exist
			if (FileReference.Exists(location))
			{
				return false;
			}

			// Make sure the directory exists
			try
			{
				DirectoryReference.CreateDirectory(location.Directory);
			}
			catch (Exception ex)
			{
				throw new AutomationException(ex, "Unable to create '{0}'", location.Directory);
			}

			// Create a temp file containing the owner name
			string tempFileName;
			for (int idx = 0; ; idx++)
			{
				tempFileName = String.Format("{0}.{1}.tmp", location.FullName, idx);
				try
				{
					byte[] bytes = Encoding.UTF8.GetBytes(signature);
					using (FileStream stream = File.Open(tempFileName, FileMode.CreateNew, FileAccess.Write, FileShare.None))
					{
						stream.Write(bytes, 0, bytes.Length);
					}
					break;
				}
				catch (IOException)
				{
					if (!File.Exists(tempFileName))
					{
						throw;
					}
				}
			}

			// Try to move the temporary file into place. 
			try
			{
				File.Move(tempFileName, location.FullName);
				return true;
			}
			catch
			{
				if (!File.Exists(tempFileName))
				{
					throw;
				}
				return false;
			}
		}

		/// <summary>
		/// Checks whether the given assembly is a publically distributed engine assembly.
		/// </summary>
		/// <param name="file">Assembly location</param>
		/// <returns>True if the assembly is distributed publically</returns>
		static bool IsPublicAssembly(FileReference file)
		{
			DirectoryReference engineDirectory = Unreal.EngineDirectory;
			if (file.IsUnderDirectory(engineDirectory))
			{
				string[] pathFragments = file.MakeRelativeTo(engineDirectory).Split(Path.DirectorySeparatorChar, Path.AltDirectorySeparatorChar);
				if (pathFragments.All(x => !x.Equals("NotForLicensees", StringComparison.OrdinalIgnoreCase) && !x.Equals("NoRedist", StringComparison.OrdinalIgnoreCase) && !x.Equals("LimitedAccess", StringComparison.OrdinalIgnoreCase)))
				{
					return true;
				}
			}
			return false;
		}

		/// <summary>
		/// Find all the nodes in the graph which are already completed
		/// </summary>
		/// <param name="graph">The graph instance</param>
		/// <param name="storage">The temp storage backend which stores the shared state</param>
		static HashSet<BgNodeDef> FindCompletedNodes(BgGraphDef graph, TempStorage storage)
		{
			HashSet<BgNodeDef> completedNodes = new HashSet<BgNodeDef>();
			foreach (BgNodeDef node in graph.Agents.SelectMany(x => x.Nodes))
			{
				if (storage.IsComplete(node.Name))
				{
					completedNodes.Add(node);
				}
			}
			return completedNodes;
		}

		/// <summary>
		/// Builds all the nodes in the graph
		/// </summary>
		/// <param name="graph">The graph instance</param>
		/// <param name="nodeToExecutor">Map from node to executor</param>
		/// <param name="storage">The temp storage backend which stores the shared state</param>
		/// <param name="ignoreOptionalNodeFailures">Continue execution when nodes that are only optional dependencies fail</param>
		/// <returns>True if everything built successfully</returns>
		async Task<bool> BuildAllNodesAsync(BgGraphDef graph, Dictionary<BgNodeDef, BgNodeExecutor> nodeToExecutor, TempStorage storage, bool ignoreOptionalNodeFailures)
		{
			// Build a flat list of nodes to execute, in order
			BgNodeDef[] nodesToExecute = graph.Agents.SelectMany(x => x.Nodes).ToArray();

			// Check the integrity of any local nodes that have been completed. It's common to run formal builds locally between regular development builds, so we may have 
			// stale local state. Rather than failing later, detect and clean them up now.
			HashSet<BgNodeDef> cleanedNodes = new HashSet<BgNodeDef>();
			foreach (BgNodeDef nodeToExecute in nodesToExecute)
			{
				FileFilter ignoreModifiedFilter = new FileFilter(nodeToExecute.IgnoreModified);
				if (nodeToExecute.InputDependencies.Any(x => cleanedNodes.Contains(x)) ||
				    nodeToExecute.OptionalInputDependencies.Any(x => cleanedNodes.Contains(x)) ||
				    !storage.CheckLocalIntegrity(nodeToExecute.Name, nodeToExecute.Outputs.Select(x => x.TagName), ignoreModifiedFilter))
				{
					storage.CleanLocalNode(nodeToExecute.Name);
					cleanedNodes.Add(nodeToExecute);
				}
			}

			// Nodes that are only optional dependencies, which can fail without stopping the build
			HashSet<BgNodeDef> nodesThatCanFail = [];
			foreach (BgNodeDef node in nodesToExecute)
			{
				nodesThatCanFail.UnionWith(node.OptionalInputDependencies);
			}

			// Remove nodes that are also required or order-only dependencies; these must complete successfully
			foreach (BgNodeDef node in nodesToExecute)
			{
				nodesThatCanFail.ExceptWith(node.InputDependencies);

				// Exclude After dependencies
				nodesThatCanFail.ExceptWith(node.GetDirectOrderDependencies().Except(node.InputDependencies).Except(node.OptionalInputDependencies));
			}

			bool hasErrors = false;

			// ExecuteAsync them in order
			int nodeIdx = 0;
			foreach (BgNodeDef nodeToExecute in nodesToExecute)
			{
				Logger.LogInformation("****** [{Arg0}/{Arg1}] {Arg2}", ++nodeIdx, nodesToExecute.Length, nodeToExecute.Name);
				if (!storage.IsComplete(nodeToExecute.Name))
				{
					Logger.LogInformation("");

					try
					{
						// Node executors signal failure via AutomationException. Other exception types 
						// or returning false indicate unexpected errors and should always fail the build
						if (!await BuildNodeAsync(graph, nodeToExecute, nodeToExecutor, storage, withBanner: false))
						{
							return false;
						}
					}
					catch (AutomationException ex) when (ignoreOptionalNodeFailures && nodesThatCanFail.Contains(nodeToExecute))
					{
						Logger.LogError(ex, "Node '{Node}' failed {Message}. Continuing because node is only an optional dependency", nodeToExecute, ex.Message);
						hasErrors = true;
					}

					Logger.LogInformation("");
				}
			}

			return !hasErrors;
		}

		/// <summary>
		/// Helper class to execute a cleanup script on termination
		/// </summary>
		class CleanupScriptRunner : IDisposable
		{
			readonly ILogger _logger;
			readonly FileReference? _scriptFile;

			public CleanupScriptRunner(ILogger logger)
			{
				_logger = logger;

				string? cleanupScriptEnvVar = Environment.GetEnvironmentVariable(CustomTask.CleanupScriptEnvVarName);
				if (String.IsNullOrEmpty(cleanupScriptEnvVar))
				{
					string extension = RuntimeInformation.IsOSPlatform(OSPlatform.Windows) ? "bat" : "sh";
					_scriptFile = FileReference.Combine(Unreal.EngineDirectory, "Intermediate", $"OnExit.{extension}");
					FileReference.Delete(_scriptFile);
					Environment.SetEnvironmentVariable(CustomTask.CleanupScriptEnvVarName, _scriptFile.FullName);
				}
			}

			public void Dispose()
			{
				if (_scriptFile != null)
				{
					if (FileReference.Exists(_scriptFile))
					{
						_logger.LogInformation("Executing cleanup commands from {ScriptFile}", _scriptFile);
						if (RuntimeInformation.IsOSPlatform(OSPlatform.Windows))
						{
							CommandUtils.Run(BuildHostPlatform.Current.Shell.FullName, $"/C {CommandLineArguments.Quote(_scriptFile.FullName)}");
						}
						else
						{
							CommandUtils.Run(BuildHostPlatform.Current.Shell.FullName, CommandLineArguments.Quote(_scriptFile.FullName));
						}
					}
					Environment.SetEnvironmentVariable(CustomTask.CleanupScriptEnvVarName, null);
				}
			}
		}

		/// <summary>
		/// Build a node
		/// </summary>
		/// <param name="graph">The graph to which the node belongs. Used to determine which outputs need to be transferred to temp storage.</param>
		/// <param name="node">The node to build</param>
		/// <param name="nodeToExecutor">Map from node to executor</param>
		/// <param name="storage">The temp storage backend which stores the shared state</param>
		/// <param name="withBanner">Whether to write a banner before and after this node's log output</param>
		/// <returns>True if the node built successfully, false otherwise.</returns>
		async Task<bool> BuildNodeAsync(BgGraphDef graph, BgNodeDef node, Dictionary<BgNodeDef, BgNodeExecutor> nodeToExecutor, TempStorage storage, bool withBanner)
		{
			DirectoryReference rootDir = new DirectoryReference(CommandUtils.CmdEnv.LocalRoot);

			// Register something to execute cleanup commands
			using CleanupScriptRunner cleanupRunner = new CleanupScriptRunner(Logger);

			// Create a filter for modified files that should be ignored
			FileFilter ignoreModifiedFilter = new FileFilter(node.IgnoreModified);

			// Create the mapping of tag names to file sets
			Dictionary<string, HashSet<FileReference>> tagNameToFileSet = new Dictionary<string, HashSet<FileReference>>();

			// Read all the input tags for this node, and build a list of referenced input storage blocks
			HashSet<TempStorageBlockRef> inputStorageBlocks = new HashSet<TempStorageBlockRef>();
			foreach (BgNodeOutput input in node.Inputs)
			{
				TempStorageTagManifest fileList = storage.ReadTagFileList(input.ProducingNode.Name, input.TagName) ?? throw new InvalidOperationException();
				tagNameToFileSet[input.TagName] = fileList.ToFileSet(rootDir);
				inputStorageBlocks.UnionWith(fileList.Blocks);
			}

			// Read all the optional input tags for this node, and build a list of referenced input storage blocks
			foreach (BgNodeOutput optionalInput in node.OptionalInputs)
			{
				try
				{
					TempStorageTagManifest? fileList = storage.ReadTagFileList(optionalInput.ProducingNode.Name, optionalInput.TagName);
					if (fileList != null)
					{
						tagNameToFileSet[optionalInput.TagName] = fileList.ToFileSet(rootDir);
						inputStorageBlocks.UnionWith(fileList.Blocks);
					}
				}
				catch (AutomationException ex)
				{
					Logger.LogWarning(ex, "Failed to read optional tagged input {Input} from temp storage: {Message}. The producing step may have failed. Continuing without it", optionalInput, ex.Message);
				}
			}

			// Read the manifests for all the input storage blocks
			Dictionary<TempStorageBlockRef, TempStorageBlockManifest> inputManifests = new Dictionary<TempStorageBlockRef, TempStorageBlockManifest>();
			using (IScope scope = GlobalTracer.Instance.BuildSpan("TempStorage").WithTag("resource", "read").StartActive())
			{
				scope.Span.SetTag("blocks", inputStorageBlocks.Count);
				foreach (TempStorageBlockRef inputStorageBlock in inputStorageBlocks)
				{
					TempStorageBlockManifest manifest = storage.Retrieve(inputStorageBlock.NodeName, inputStorageBlock.OutputName, ignoreModifiedFilter);
					inputManifests[inputStorageBlock] = manifest;
				}
				scope.Span.SetTag("size", inputManifests.Sum(x => x.Value.GetTotalSize()));
			}

			// Read all the input storage blocks, keeping track of which block each file came from
			Dictionary<string, (TempStorageFile, TempStorageBlockRef)> inputFiles = new Dictionary<string, (TempStorageFile, TempStorageBlockRef)>(FileReference.Comparer);
			foreach (KeyValuePair<TempStorageBlockRef, TempStorageBlockManifest> pair in inputManifests)
			{
				foreach (TempStorageFile newFile in pair.Value.Files)
				{
					(TempStorageFile File, TempStorageBlockRef Block) existingItem;
					if (inputFiles.TryGetValue(newFile.RelativePath, out existingItem)
						&& !ignoreModifiedFilter.Matches(newFile.ToFileReference(rootDir).FullName)
						&& !TempStorage.IsDuplicateBuildProduct(newFile.ToFileReference(rootDir)))
					{
						if (existingItem.File.LastWriteTimeUtcTicks != newFile.LastWriteTimeUtcTicks)
						{
							Logger.LogError("File '{File}' was produced by {InputStorageBlock} and {CurrentStorageBlock}", newFile.RelativePath, existingItem.Block, pair.Key);
						}
					}
					inputFiles[newFile.RelativePath] = (newFile, pair.Key);
				}
			}

			// Add placeholder outputs for the current node
			foreach (BgNodeOutput output in node.Outputs)
			{
				tagNameToFileSet.Add(output.TagName, new HashSet<FileReference>());
			}

			// ExecuteAsync the node
			if (withBanner)
			{
				Console.WriteLine();
				Logger.LogInformation("========== Starting: {Arg0} ==========", node.Name);
			}
			if (!await nodeToExecutor[node].Execute(new JobContext(node.Name, this), tagNameToFileSet))
			{
				return false;
			}
			if (withBanner)
			{
				Logger.LogInformation("========== Finished: {Arg0} ==========", node.Name);
				Console.WriteLine();
			}

			// Check that none of the inputs have been clobbered
			Dictionary<string, string> modifiedFiles = new Dictionary<string, string>(StringComparer.InvariantCultureIgnoreCase);
			foreach (TempStorageFile file in inputManifests.Values.SelectMany(x => x.Files))
			{
				string? message;
				if (!modifiedFiles.ContainsKey(file.RelativePath) && !ignoreModifiedFilter.Matches(file.ToFileReference(Unreal.RootDirectory).FullName) && !file.Compare(Unreal.RootDirectory, out message))
				{
					// look up the previous nodes to help with error diagnosis
					List<string> previousNodeNames = inputManifests.Where(x => x.Value.Files.Contains(file))
						.Select(x => x.Key.NodeName)
						.ToList();
					if (previousNodeNames.Count > 0)
					{
						message += $" (previous {(previousNodeNames.Count == 1 ? "step" : "steps")}: {String.Join(" + ", previousNodeNames)})";
					}

					modifiedFiles.Add(file.RelativePath, message);
				}
			}
			if (modifiedFiles.Count > 0)
			{
				string modifiedFileList = "";
				if (modifiedFiles.Count < 100)
				{
					modifiedFileList = String.Join("\n", modifiedFiles.Select(x => $"  {x.Value}"));
				}
				else
				{
					modifiedFileList = String.Join("\n", modifiedFiles.Take(100).Select(x => $"  {x.Value}"));
					modifiedFileList += $"\n  ...and {modifiedFiles.Count - 100} more.";
				}
				throw new AutomationException("Build {0} from a previous step have been modified:\n{1}\nOutput overlapping artifacts to a different location, or ignore them using the Node's IgnoreModified attribute.", (modifiedFiles.Count == 1) ? "product" : "products", modifiedFileList);
			}

			// Determine all the output files which are required to be copied to temp storage (because they're referenced by nodes in another agent)
			HashSet<FileReference> referencedOutputFiles = new HashSet<FileReference>();
			foreach (BgAgentDef agent in graph.Agents)
			{
				bool sameAgent = agent.Nodes.Contains(node);
				foreach (BgNodeDef otherNode in agent.Nodes)
				{
					if (!sameAgent)
					{
						foreach (BgNodeOutput input in otherNode.Inputs.Where(x => x.ProducingNode == node))
						{
							referencedOutputFiles.UnionWith(tagNameToFileSet[input.TagName]);
						}

						foreach (BgNodeOutput optionalInput in otherNode.OptionalInputs.Where(x => x.ProducingNode == node))
						{
							if (tagNameToFileSet.TryGetValue(optionalInput.TagName, out HashSet<FileReference>? fileSet))
							{
								referencedOutputFiles.UnionWith(fileSet);
							}
						}
					}
				}
			}

			// Find a block name for all new outputs
			Dictionary<FileReference, string> fileToOutputName = new Dictionary<FileReference, string>();
			foreach (BgNodeOutput output in node.Outputs)
			{
				HashSet<FileReference> files = tagNameToFileSet[output.TagName];
				foreach (FileReference file in files)
				{
					if (file.IsUnderDirectory(rootDir))
					{
						if (output == node.DefaultOutput)
						{
							if (!fileToOutputName.ContainsKey(file))
							{
								fileToOutputName[file] = "";
							}
						}
						else
						{
							string? outputName;
							if (fileToOutputName.TryGetValue(file, out outputName) && outputName.Length > 0)
							{
								fileToOutputName[file] = String.Format("{0}+{1}", outputName, output.TagName.Substring(1));
							}
							else
							{
								fileToOutputName[file] = output.TagName.Substring(1);
							}
						}
					}
				}
			}

			// Invert the dictionary to make a mapping of storage block to the files each contains
			Dictionary<string, HashSet<FileReference>> outputStorageBlockToFiles = new Dictionary<string, HashSet<FileReference>>();
			foreach (KeyValuePair<FileReference, string> pair in fileToOutputName)
			{
				HashSet<FileReference>? files;
				if (!outputStorageBlockToFiles.TryGetValue(pair.Value, out files))
				{
					files = new HashSet<FileReference>();
					outputStorageBlockToFiles.Add(pair.Value, files);
				}
				files.Add(pair.Key);
			}

			// Write all the storage blocks, and update the mapping from file to storage block
			using (GlobalTracer.Instance.BuildSpan("TempStorage").WithTag("resource", "Write").StartActive())
			{
				Dictionary<FileReference, TempStorageBlockRef> outputFileToStorageBlock = new Dictionary<FileReference, TempStorageBlockRef>();
				foreach (KeyValuePair<string, HashSet<FileReference>> pair in outputStorageBlockToFiles)
				{
					TempStorageBlockRef outputBlock = new TempStorageBlockRef(node.Name, pair.Key);
					foreach (FileReference file in pair.Value)
					{
						outputFileToStorageBlock.Add(file, outputBlock);
					}
					storage.Archive(node.Name, pair.Key, pair.Value.ToArray(), pair.Value.Any(x => referencedOutputFiles.Contains(x)));
				}

				// Find all the output tags that are published artifacts
				Dictionary<string, BgArtifactDef> outputNameToArtifact = new Dictionary<string, BgArtifactDef>(StringComparer.OrdinalIgnoreCase);
				foreach (BgArtifactDef artifact in graph.Artifacts)
				{
					if (artifact.TagName != null)
					{
						outputNameToArtifact.Add(artifact.TagName, artifact);
					}
				}

				// Publish all the output tags
				foreach (BgNodeOutput output in node.Outputs)
				{
					HashSet<FileReference> files = tagNameToFileSet[output.TagName];

					HashSet<TempStorageBlockRef> storageBlocks = new HashSet<TempStorageBlockRef>();
					foreach (FileReference file in files)
					{
						TempStorageBlockRef? storageBlock;
						if (outputFileToStorageBlock.TryGetValue(file, out storageBlock))
						{
							storageBlocks.Add(storageBlock);
						}
					}

					IEnumerable<string> keys = Enumerable.Empty<string>();
					if (outputNameToArtifact.TryGetValue(output.TagName, out BgArtifactDef? artifact))
					{
						keys = artifact.Keys;
					}

					storage.WriteFileList(node.Name, output.TagName, files, storageBlocks.ToArray(), keys);
				}
			}

			// Mark the node as succeeded
			storage.MarkAsComplete(node.Name);
			return true;
		}

		/// <summary>
		/// Gets an environment variable, returning null if it's not set or empty.
		/// </summary>
		static string? GetEnvVarOrNull(string name)
		{
			string? envVar = Environment.GetEnvironmentVariable(name);
			return String.IsNullOrEmpty(envVar) ? null : envVar;
		}

		/// <summary>
		/// Gets an environment variable as an integer, returning null if it's not set or empty.
		/// </summary>
		static int? GetEnvVarIntOrNull(string name)
		{
			string? envVar = GetEnvVarOrNull(name);
			if (envVar != null && Int32.TryParse(envVar, out int value))
			{
				return value;
			}
			return null;
		}

		/// <summary>
		/// Generate HTML documentation for all the tasks
		/// </summary>
		/// <param name="nameToTask">Map of task name to implementation</param>
		/// <param name="outputFile">Output file</param>
		static void WriteDocumentation(Dictionary<string, ScriptTaskBinding> nameToTask, FileReference outputFile)
		{
			// Find all the assemblies containing tasks
			Assembly[] taskAssemblies = nameToTask.Values.Select(x => x.ParametersClass.Assembly).Distinct().ToArray();

			// Read documentation for each of them
			Dictionary<string, XmlElement> memberNameToElement = new Dictionary<string, XmlElement>();
			foreach (Assembly taskAssembly in taskAssemblies)
			{
				string xmlFileName = Path.ChangeExtension(taskAssembly.Location, ".xml");
				if (File.Exists(xmlFileName))
				{
					// Read the document
					XmlDocument document = new XmlDocument();
					document.Load(xmlFileName);

					// Parse all the members, and add them to the map
					foreach (XmlElement element in document.SelectNodes("/doc/members/member")!)
					{
						string name = element.GetAttribute("name");
						memberNameToElement.Add(name, element);
					}
				}
			}

			// Create the output directory
			if (FileReference.Exists(outputFile))
			{
				FileReference.MakeWriteable(outputFile);
			}
			else
			{
				DirectoryReference.CreateDirectory(outputFile.Directory);
			}

			// Write the output file
			if (outputFile.HasExtension(".udn"))
			{
				WriteDocumentationUdn(nameToTask, memberNameToElement, outputFile);
			}
			else if (outputFile.HasExtension(".html"))
			{
				WriteDocumentationHtml(nameToTask, memberNameToElement, outputFile);
			}
			else
			{
				throw new BuildException("Unable to detect format from extension of output file ({0})", outputFile);
			}
		}

		/// <summary>
		/// Writes documentation to a UDN file
		/// </summary>
		/// <param name="nameToTask">Map of name to script task</param>
		/// <param name="memberNameToElement">Map of field name to XML documenation element</param>
		/// <param name="outputFile">The output file to write to</param>
		static void WriteDocumentationUdn(Dictionary<string, ScriptTaskBinding> nameToTask, Dictionary<string, XmlElement> memberNameToElement, FileReference outputFile)
		{
			using (StreamWriter writer = new StreamWriter(outputFile.FullName))
			{
				writer.WriteLine("Availability: NoPublish");
				writer.WriteLine("Title: BuildGraph Predefined Tasks");
				writer.WriteLine("Crumbs: %ROOT%, Programming, Programming/Development, Programming/Development/BuildGraph, Programming/Development/BuildGraph/BuildGraphScriptTasks");
				writer.WriteLine("Description: This is a procedurally generated markdown page.");
				writer.WriteLine("version: {0}.{1}", ReadOnlyBuildVersion.Current.MajorVersion, ReadOnlyBuildVersion.Current.MinorVersion);
				writer.WriteLine("parent:Programming/Development/BuildGraph/BuildGraphScriptTasks");
				writer.WriteLine();
				foreach (string taskName in nameToTask.Keys.OrderBy(x => x))
				{
					// Get the task object
					ScriptTaskBinding task = nameToTask[taskName];

					// Get the documentation for this task
					XmlElement? taskElement;
					if (memberNameToElement.TryGetValue("T:" + task.TaskClass.FullName, out taskElement))
					{
						// Write the task heading
						writer.WriteLine("### {0}", taskName);
						writer.WriteLine();
						writer.WriteLine(ConvertToMarkdown(taskElement.SelectSingleNode("summary")!));
						writer.WriteLine();

						// Document the parameters
						List<string[]> rows = new List<string[]>();
						foreach (string parameterName in task.NameToParameter.Keys)
						{
							// Get the parameter data
							ScriptTaskParameterBinding parameter = task.NameToParameter[parameterName];

							// Get the documentation for this parameter
							XmlElement? parameterElement;
							if (memberNameToElement.TryGetValue(parameter.GetXmlDocName(), out parameterElement))
							{
								Type fieldType = parameter.ParameterType;
								if (fieldType.IsGenericType && fieldType.GetGenericTypeDefinition() == typeof(Nullable<>))
								{
									fieldType = fieldType.GetGenericArguments()[0];
								}

								string typeName;
								if (parameter.ValidationType != TaskParameterValidationType.Default)
								{
									typeName = parameter.ValidationType.ToString();
								}
								else if (fieldType == typeof(int))
								{
									typeName = "Integer";
								}
								else if (fieldType == typeof(HashSet<DirectoryReference>))
								{
									typeName = "DirectoryList";
								}
								else
								{
									typeName = fieldType.Name;
								}

								string[] columns = new string[4];
								columns[0] = parameterName;
								columns[1] = typeName;
								columns[2] = parameter.Optional ? "Optional" : "Required";
								columns[3] = ConvertToMarkdown(parameterElement.SelectSingleNode("summary")!);
								rows.Add(columns);
							}
						}

						// Always include the "If" attribute
						string[] ifColumns = new string[4];
						ifColumns[0] = "If";
						ifColumns[1] = "Condition";
						ifColumns[2] = "Optional";
						ifColumns[3] = "Whether to execute this task. It is ignored if this condition evaluates to false.";
						rows.Add(ifColumns);

						// Get the width of each column
						int[] widths = new int[4];
						for (int idx = 0; idx < 4; idx++)
						{
							widths[idx] = rows.Max(x => x[idx].Length);
						}

						// Format the markdown table
						string format = String.Format("| {{0,-{0}}} | {{1,-{1}}} | {{2,-{2}}} | {{3,-{3}}} |", widths[0], widths[1], widths[2], widths[3]);
						writer.WriteLine(format, "", "", "", "");
						writer.WriteLine(format, new string('-', widths[0]), new string('-', widths[1]), new string('-', widths[2]), new string('-', widths[3]));
						for (int idx = 0; idx < rows.Count; idx++)
						{
							writer.WriteLine(format, rows[idx][0], rows[idx][1], rows[idx][2], rows[idx][3]);
						}

						// Blank line before next task
						writer.WriteLine();
					}
				}
			}
		}

		/// <summary>
		/// Writes documentation to an HTML file
		/// </summary>
		/// <param name="nameToTask">Map of name to script task</param>
		/// <param name="memberNameToElement">Map of field name to XML documenation element</param>
		/// <param name="outputFile">The output file to write to</param>
		static void WriteDocumentationHtml(Dictionary<string, ScriptTaskBinding> nameToTask, Dictionary<string, XmlElement> memberNameToElement, FileReference outputFile)
		{
			Logger.LogInformation("Writing {OutputFile}...", outputFile);
			using (StreamWriter writer = new StreamWriter(outputFile.FullName))
			{
				writer.WriteLine("<html>");
				writer.WriteLine("  <head>");
				writer.WriteLine("    <style>");
				writer.WriteLine("      table { border-collapse: collapse }");
				writer.WriteLine("      table, th, td { border: 1px solid black; }");
				writer.WriteLine("    </style>");
				writer.WriteLine("  </head>");
				writer.WriteLine("  <body>");
				writer.WriteLine("    <h1>BuildGraph Tasks</h1>");
				foreach (string taskName in nameToTask.Keys.OrderBy(x => x))
				{
					// Get the task object
					ScriptTaskBinding task = nameToTask[taskName];

					// Get the documentation for this task
					XmlElement? taskElement;
					if (memberNameToElement.TryGetValue("T:" + task.TaskClass.FullName, out taskElement))
					{
						// Write the task heading
						writer.WriteLine("    <h2>{0}</h2>", taskName);
						writer.WriteLine("    <p>{0}</p>", taskElement.SelectSingleNode("summary")!.InnerXml.Trim());

						// Start the parameter table
						writer.WriteLine("    <table>");
						writer.WriteLine("      <tr>");
						writer.WriteLine("        <th>Attribute</th>");
						writer.WriteLine("        <th>Type</th>");
						writer.WriteLine("        <th>Usage</th>");
						writer.WriteLine("        <th>Description</th>");
						writer.WriteLine("      </tr>");

						// Document the parameters
						foreach (string parameterName in task.NameToParameter.Keys)
						{
							// Get the parameter data
							ScriptTaskParameterBinding parameter = task.NameToParameter[parameterName];

							// Get the documentation for this parameter
							XmlElement? parameterElement;
							if (memberNameToElement.TryGetValue(parameter.GetXmlDocName(), out parameterElement))
							{
								string typeName = parameter.ParameterType.Name;
								if (parameter.ValidationType != TaskParameterValidationType.Default)
								{
									StringBuilder newTypeName = new StringBuilder(parameter.ValidationType.ToString());
									for (int idx = 1; idx < newTypeName.Length; idx++)
									{
										if (Char.IsLower(newTypeName[idx - 1]) && Char.IsUpper(newTypeName[idx]))
										{
											newTypeName.Insert(idx, ' ');
										}
									}
									typeName = newTypeName.ToString();
								}

								writer.WriteLine("      <tr>");
								writer.WriteLine("         <td>{0}</td>", parameterName);
								writer.WriteLine("         <td>{0}</td>", typeName);
								writer.WriteLine("         <td>{0}</td>", parameter.Optional ? "Optional" : "Required");
								writer.WriteLine("         <td>{0}</td>", parameterElement.SelectSingleNode("summary")!.InnerXml.Trim());
								writer.WriteLine("      </tr>");
							}
						}

						// Always include the "If" attribute
						writer.WriteLine("     <tr>");
						writer.WriteLine("       <td>If</td>");
						writer.WriteLine("       <td>Condition</td>");
						writer.WriteLine("       <td>Optional</td>");
						writer.WriteLine("       <td>Whether to execute this task. It is ignored if this condition evaluates to false.</td>");
						writer.WriteLine("     </tr>");

						// Close the table
						writer.WriteLine("    <table>");
					}
				}
				writer.WriteLine("  </body>");
				writer.WriteLine("</html>");
			}
		}

		/// <summary>
		/// Converts an XML documentation node to markdown
		/// </summary>
		/// <param name="node">The node to read</param>
		/// <returns>Text in markdown format</returns>
		static string ConvertToMarkdown(XmlNode node)
		{
			string text = node.InnerXml;

			StringBuilder result = new StringBuilder();
			for (int idx = 0; idx < text.Length; idx++)
			{
				if (Char.IsWhiteSpace(text[idx]))
				{
					result.Append(' ');
					while (idx + 1 < text.Length && Char.IsWhiteSpace(text[idx + 1]))
					{
						idx++;
					}
				}
				else
				{
					result.Append(text[idx]);
				}
			}
			return result.ToString().Trim();
		}
	}

	/// <summary>
	/// Legacy command name for compatibility.
	/// </summary>
	public class Build : BuildGraph
	{
	}
}

