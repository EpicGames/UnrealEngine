// Copyright Epic Games, Inc. All Rights Reserved.

#nullable enable

namespace AutomationTool
{
	/// <summary>
	/// Used to pass information to tasks about the currently running job.
	/// </summary>
	public class JobContext
	{
		/// <summary>
		/// The current node name
		/// </summary>
		public string CurrentNode { get; }

		/// <summary>
		/// The command that is running the current job.
		/// </summary>
		public BuildCommand OwnerCommand { get; }

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="inOwnerCommand">The command running the current job</param>
		public JobContext(BuildCommand inOwnerCommand) : this("Unknown", inOwnerCommand)
		{
		}

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="inCurrentNode">The current node being executed</param>
		/// <param name="inOwnerCommand">The command running the current job</param>
		public JobContext(string inCurrentNode, BuildCommand inOwnerCommand)
		{
			CurrentNode = inCurrentNode;
			OwnerCommand = inOwnerCommand;
		}
	}
}
