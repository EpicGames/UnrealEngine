// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Threading.Tasks;
using AutomationTool.Tasks;
using EpicGames.BuildGraph;
using EpicGames.BuildGraph.Expressions;
using EpicGames.Core;
using Microsoft.Extensions.Logging;
using UnrealBuildBase;
using UnrealBuildTool;
using static AutomationTool.Tasks.StandardTasks;

#nullable enable

namespace AutomationTool
{
	class InstalledBuild : BgGraphBuilder
	{
		static FileSet Workspace { get; } = FileSet.FromDirectory(Unreal.RootDirectory);
		static DirectoryReference RootDir { get; } = new DirectoryReference(CommandUtils.CmdEnv.LocalRoot);
		static DirectoryReference IntermediateDir => DirectoryReference.Combine(RootDir, "Engine", "Intermediate", "Installed");
		static DirectoryReference TempMiscDir => DirectoryReference.Combine(IntermediateDir, "General");
		static DirectoryReference TempCsToolsDir => DirectoryReference.Combine(IntermediateDir, "CsTools");
		static DirectoryReference TempDdcDir => DirectoryReference.Combine(IntermediateDir, "DDC");

		//		static string[] PluginsExceptions =
		//		{
		//			"Engine/Plugins/Enterprise/DatasmithCADImporter/...",
		//			"Engine/Plugins/Enterprise/DatasmithC4DImporter/...",
		//			"Engine/Plugins/Enterprise/AxFImporter/...",
		//			"Engine/Plugins/Enterprise/MDLImporter/..."
		//		};

		//		static string[] WinSignFilter =
		//		{
		//			"*.exe",
		//			"*.dll"
		//		};

		static readonly string[] s_winStripFilter =
		{
			"*.pdb",
			"-/Engine/Binaries/Win64/UnrealEditor*.pdb",
			"-/Engine/Plugins/.../Binaries/Win64/UnrealEditor*.pdb",
		};

		static List<string> GetDdcProjects(UnrealTargetPlatform platform)
		{
			List<string> projects = new List<string>();
			projects.Add("Templates/TP_AEC_ArchvisBP/TP_AEC_ArchvisBP.uproject");
			projects.Add("Templates/TP_AEC_BlankBP/TP_AEC_BlankBP.uproject");
			projects.Add("Templates/TP_AEC_CollabBP/TP_AEC_CollabBP.uproject");
			projects.Add("Templates/TP_AEC_ProdConfigBP/TP_AEC_ProdConfigBP.uproject");
			projects.Add("Templates/TP_FirstPersonBP/TP_FirstPersonBP.uproject");
			projects.Add("Templates/TP_HandheldARBP/TP_HandheldARBP.uproject");
			projects.Add("Templates/TP_AEC_HandheldARBP/TP_AEC_HandheldARBP.uproject");
			projects.Add("Templates/TP_MFG_HandheldARBP/TP_MFG_HandheldARBP.uproject");
			projects.Add("Templates/TP_MFG_CollabBP/TP_MFG_CollabBP.uproject");
			projects.Add("Templates/TP_MFG_ProdConfigBP/TP_MFG_ProdConfigBP.uproject");
			projects.Add("Templates/TP_PhotoStudioBP/TP_PhotoStudioBP.uproject");
			projects.Add("Templates/TP_PuzzleBP/TP_PuzzleBP.uproject");
			projects.Add("Templates/TP_ThirdPersonBP/TP_ThirdPersonBP.uproject");
			projects.Add("Templates/TP_TopDownBP/TP_TopDownBP.uproject");
			projects.Add("Templates/TP_VehicleAdvBP/TP_VehicleAdvBP.uproject");
			projects.Add("Templates/TP_VirtualRealityBP/TP_VirtualRealityBP.uproject");
			projects.Add("Templates/TP_DMXBP/TP_DMXBP.uproject");
			//			Projects.Add("Samples/StarterContent/StarterContent.uproject");

			if (platform == UnrealTargetPlatform.Win64)
			{
				projects.Add("Templates/TP_InCamVFXBP/TP_InCamVFXBP.uproject");
			}
			return projects;
		}

		public override BgGraph CreateGraph()
		{
			string rootDir = CommandUtils.CmdEnv.LocalRoot;

			BgBoolOption hostPlatformOnly = new BgBoolOption("HostPlatformOnly", "A helper option to make an installed build for your host platform only, so that you don't have to disable each platform individually", false);
			BgBoolOption hostPlatformEditorOnly = new BgBoolOption("HostPlatformEditorOnly", "A helper option to make an installed build for your host platform only, so that you don't have to disable each platform individually", false);
			BgBoolOption allPlatforms = new BgBoolOption("AllPlatforms", "Include all target platforms by default", false);
			BgBoolOption compileDatasmithPlugins = new BgBoolOption("CompileDatasmithPlugins", "If Datasmith plugins should be compiled on a separate node.", false);

			UnrealTargetPlatform currentHostPlatform = HostPlatform.Current.HostEditorPlatform;

			BgBool defaultWithWin64 = !(hostPlatformEditorOnly | (hostPlatformOnly & (currentHostPlatform != UnrealTargetPlatform.Win64)));
			BgBool defaultWithMac = !(hostPlatformEditorOnly | (hostPlatformOnly & (currentHostPlatform != UnrealTargetPlatform.Mac)));
			BgBool defaultWithLinux = !(hostPlatformEditorOnly | (hostPlatformOnly & (currentHostPlatform != UnrealTargetPlatform.Linux)));
			BgBool defaultWithLinuxArm64 = !(hostPlatformEditorOnly | (hostPlatformOnly & (currentHostPlatform != UnrealTargetPlatform.Linux)));
			BgBool defaultWithPlatform = !(hostPlatformEditorOnly | hostPlatformOnly);
			BgBool defaultWithIos = !((currentHostPlatform != UnrealTargetPlatform.Mac) & !allPlatforms);

			BgBoolOption withWin64 = new BgBoolOption("WithWin64", "Include the Win64 target platform", defaultWithWin64);
			BgBoolOption withMac = new BgBoolOption("WithMac", "Include the Mac target platform", defaultWithMac);
			BgBoolOption withAndroid = new BgBoolOption("WithAndroid", "Include the Android target platform", defaultWithPlatform);
			BgBoolOption withIos = new BgBoolOption("WithIOS", "Include the iOS target platform", defaultWithIos);
			BgBoolOption withTvos = new BgBoolOption("WithTVOS", "Include the tvOS target platform", defaultWithIos);
			BgBoolOption withLinux = new BgBoolOption("WithLinux", "Include the Linux target platform", defaultWithLinux);
			BgBoolOption withLinuxArm64 = new BgBoolOption("WithLinuxArm64", "Include the Linux AArch64 target platform", defaultWithLinuxArm64);

			BgBoolOption withClient = new BgBoolOption("WithClient", "Include precompiled client targets", false);
			BgBoolOption withServer = new BgBoolOption("WithServer", "Include precompiled server targets", false);
			BgBoolOption withDdc = new BgBoolOption("WithDDC", "Build a standalone derived-data cache for the engine content and templates", true);
			BgBoolOption hostPlatformDdcOnly = new BgBoolOption("HostPlatformDDCOnly", "Whether to include DDC for the host platform only", true);
			BgBoolOption signExecutables = new BgBoolOption("SignExecutables", "Sign the executables produced where signing is available", false);

			BgStringOption analyticsTypeOverride = new BgStringOption("AnalyticsTypeOverride", "Identifier for analytic events to send", "");
			BgBool embedSrcSrvInfo = new BgBoolOption("EmbedSrcSrvInfo", "Whether to add Source indexing to Windows game apps so they can be added to a symbol server", false);

			BgList<BgString> defaultGameConfigurations = BgList<BgString>.Create(nameof(UnrealTargetConfiguration.DebugGame), nameof(UnrealTargetConfiguration.Development), nameof(UnrealTargetConfiguration.Shipping));
			BgList<BgString> gameConfigurationStrings = new BgListOption("GameConfigurations", description: "Which game configurations to include for packaged applications", style: BgListOptionStyle.CheckList, values: defaultGameConfigurations);
			BgList<BgEnum<UnrealTargetConfiguration>> gameConfigurations = gameConfigurationStrings.Select(x => BgEnum<UnrealTargetConfiguration>.Parse(x));

			BgBoolOption withFullDebugInfo = new BgBoolOption("WithFullDebugInfo", "Generate full debug info for binary editor and packaged application builds", false);

			BgStringOption builtDirectory = new BgStringOption("BuiltDirectory", "Directory for outputting the built engine", rootDir + "/LocalBuilds/Engine");

			BgStringOption crashReporterApiurl = new BgStringOption("CrashReporterAPIURL", "The URL to use to talk to the CrashReporterClient API.", "");
			BgStringOption crashReporterApiKey = new BgStringOption("CrashReporterAPIKey", "The API key to use to talk to the CrashReporterClient API.", "");
			BgStringOption buildId = new BgStringOption("BuildId", "The unique build identifier to associate with this installed build", "");

			BgString crashReporterCompileArgs = "";
			crashReporterCompileArgs = crashReporterCompileArgs.If(crashReporterApiurl != "" & crashReporterApiKey != "", BgString.Format("-define:CRC_TELEMETRY_URL=\"{0}\" -define:CRC_TELEMETRY_KEY_DEV=\"{1}\" -define:CRC_TELEMETRY_KEY_RELEASE=\"{1}\" -OverrideBuildEnvironment", crashReporterApiurl, crashReporterApiKey));

			List<BgAggregate> aggregates = new List<BgAggregate>();

			/////// EDITORS ////////////////////////////////////////////////

			//// Windows ////
			BgAgent editorWin64 = new BgAgent("Editor Win64", "Win64_Licensee");

			BgNode versionFilesNode = editorWin64
				.AddNode(x => UpdateVersionFilesAsync(x));

			BgNode<BgFileSet> winEditorNode = editorWin64
				.AddNode(x => CompileUnrealEditorWin64Async(x, crashReporterCompileArgs, embedSrcSrvInfo, compileDatasmithPlugins, withFullDebugInfo, signExecutables))
				.Requires(versionFilesNode);

			aggregates.Add(new BgAggregate("Win64 Editor", winEditorNode, label: "Editors/Win64"));

			/////// TARGET PLATFORMS ////////////////////////////////////////////////

			//// Win64 ////

			BgAgent targetWin64 = new BgAgent("Target Win64", "Win64_Licensee");

			BgNode<BgFileSet> winGame = targetWin64
				.AddNode(x => CompileUnrealGameWin64Async(x, gameConfigurations, embedSrcSrvInfo, withFullDebugInfo, signExecutables))
				.Requires(versionFilesNode);

			aggregates.Add(new BgAggregate("TargetPlatforms_Win64", winGame));

			/////// TOOLS //////////////////////////////////////////////////////////

			//// Build Rules ////

			BgAgent buildRules = new BgAgent("BuildRules", "Win64_Licensee");

			BgNode rulesAssemblies = buildRules
				.AddNode(x => CompileRulesAssembliesAsync(x));

			//// Win Tools ////

			BgAgent toolsGroupWin64 = new BgAgent("Tools Group Win64", "Win64_Licensee");

			BgNode<BgFileSet> winTools = toolsGroupWin64
				.AddNode(x => BuildToolsWin64Async(x, crashReporterCompileArgs))
				.Requires(versionFilesNode);

			BgNode<BgFileSet> csTools = toolsGroupWin64
				.AddNode(x => BuildToolsCsAsync(x, signExecutables))
				.Requires(versionFilesNode);

			/////// DDC //////////////////////////////////////////////////////////

			BgAgent ddcGroupWin64 = new BgAgent("DDC Group Win64", "Win64_Licensee");

			BgList<BgString> ddcPlatformsWin64 = BgList<BgString>.Create("WindowsEditor");
			ddcPlatformsWin64 = ddcPlatformsWin64.If(withWin64, x => x.Add("Windows"));

			BgNode ddcNode = ddcGroupWin64
				.AddNode(x => BuildDdcWin64Async(x, ddcPlatformsWin64, BgList<BgFileSet>.Create(winEditorNode.Output, winTools.Output)))
				.Requires(winEditorNode, winTools);

			/////// STAGING ///////

			// Windows 
			BgAgent winStageAgent = new BgAgent("Installed Build Group Win64", "Win64_Licensee");

			BgList<BgFileSet> winInstalledFiles = BgList<BgFileSet>.Empty;
			winInstalledFiles = winInstalledFiles.Add(winEditorNode.Output);
			winInstalledFiles = winInstalledFiles.Add(winTools.Output);
			winInstalledFiles = winInstalledFiles.Add(csTools.Output);
			winInstalledFiles = winInstalledFiles.If(withWin64, x => x.Add(winGame.Output));

			BgList<BgString> winPlatforms = BgList<BgString>.Empty;
			winPlatforms = winPlatforms.If(withWin64, x => x.Add("Win64"));

			BgList<BgString> winContentOnlyPlatforms = BgList<BgString>.Empty;

			BgString winOutputDir = "LocalBuilds/Engine/Windows";
			BgString winFinalizeArgs = BgString.Format("-OutputDir=\"{0}\" -Platforms={1} -ContentOnlyPlatforms={2}", winOutputDir, BgString.Join(";", winPlatforms), BgString.Join(";", winContentOnlyPlatforms));

			BgNode winInstalledNode = winStageAgent
				.AddNode(x => MakeInstalledBuildWin64Async(x, winInstalledFiles, winFinalizeArgs, winOutputDir))
				.Requires(winInstalledFiles);

			aggregates.Add(new BgAggregate("HostPlatforms_Win64", winInstalledNode, label: "Builds/Win64"));

			return new BgGraph(BgList<BgNode>.Empty, aggregates);
		}

		/// <summary>
		/// Update the build version
		/// </summary>
		static async Task UpdateVersionFilesAsync(BgContext state)
		{
			if (state.IsBuildMachine)
			{
				await SetVersionAsync(state.Change, state.Stream.Replace('/', '+'));
			}
		}

		/// <summary>
		/// Builds the Windows editor
		/// </summary>
		[BgNodeName("Compile UnrealEditor Win64")]
		static async Task<BgFileSet> CompileUnrealEditorWin64Async(BgContext state, BgString crashReporterCompileArgs, BgBool embedSrcSrvInfo, BgBool compileDatasmithPlugins, BgBool withFullDebugInfo, BgBool signExecutables)
		{
			_ = compileDatasmithPlugins;

			FileSet outputFiles = FileSet.Empty;
			outputFiles += await CompileAsync("UnrealEditor", UnrealTargetPlatform.Win64, UnrealTargetConfiguration.DebugGame, arguments: $"-precompile -allmodules {state.Get(crashReporterCompileArgs)}");
			outputFiles += await CompileAsync("UnrealEditor", UnrealTargetPlatform.Win64, UnrealTargetConfiguration.Development, arguments: $"-precompile -allmodules {state.Get(crashReporterCompileArgs)}");

			if (state.Get(embedSrcSrvInfo))
			{
				// Embed source info into the PDB files. Should be done from this machine to ensure that paths are correct.
				Logger.LogInformation("Embedding source file information into PDB files...");
				FileSet sourceFiles = Workspace.Filter("Engine/Source/...;Engine/Plugins/...").Except("Engine/Source/ThirdParty/...").Filter("*.c;*.h;*.cpp;*.hpp;*.inl");
				//				State.SrcSrv(BinaryFiles: Full, SourceFiles: SourceFiles);
				_ = sourceFiles;
			}

			if (!state.Get(withFullDebugInfo))
			{
				FileSet unstrippedFiles = outputFiles.Filter(s_winStripFilter);
				outputFiles += await StripAsync(unstrippedFiles, UnrealTargetPlatform.Win64, baseDir: RootDir, outputDir: TempMiscDir);
				outputFiles -= unstrippedFiles;
			}

			if (state.Get(signExecutables))
			{
				//				FileSet UnsignedFiles = OutputFiles.Filter(WinSignFilter);
				//				FileSet FilesToCopy = UnsignedFiles.Except((new HashSet<FileReference>(UnsignedFiles.Where(x => !x.IsUnderDirectory(TempMiscDir)));
				//				UnsignedFiles += State.Copy(FilesToCopy.Flatten(TempMiscDir));
				//				UnsignedFiles.Sign();
				//				OutputFiles -= FilesToCopy;
				//				OutputFiles += UnsignedFiles;
			}

			return outputFiles;
		}

		/// <summary>
		/// Builds the game target
		/// </summary>
		[BgNodeName("Compile UnrealGame Win64")]
		static async Task<BgFileSet> CompileUnrealGameWin64Async(BgContext state, BgList<BgEnum<UnrealTargetConfiguration>> configurations, BgBool embedSrcSrvInfo, BgBool withFullDebugInfo, BgBool signExecutables)
		{
			_ = embedSrcSrvInfo;
			_ = withFullDebugInfo;
			_ = signExecutables;

			FileSet files = FileSet.Empty;

			List<UnrealTargetConfiguration> configurationsValue = state.Get(configurations);
			foreach (UnrealTargetConfiguration configuration in configurationsValue)
			{
				files += await CompileAsync("UnrealGame", UnrealTargetPlatform.Win64, configuration, arguments: "-precompile -allmodules -nolink");
				files += await CompileAsync("UnrealGame", UnrealTargetPlatform.Win64, configuration, arguments: "-precompile", clean: false);
			}

			return files;
		}

		static async Task<BgFileSet> CompileRulesAssembliesAsync(BgContext state)
		{
			_ = state;

			FileReference unrealBuildToolDll = FileReference.Combine(RootDir, "Engine/Binaries/DotNET/UnrealBuildTool/UnrealBuildTool.dll");

			await SpawnAsync(Unreal.DotnetPath.FullName, arguments: $"\"{unrealBuildToolDll}\" -Mode=QueryTargets");

			return Workspace.Filter("Engine/Intermediate/Build/BuildRules/...");
		}

		static async Task<BgFileSet> BuildToolsWin64Async(BgContext state, BgString crashReporterCompileArgs)
		{
			string crashReportClientArgs = state.Get(crashReporterCompileArgs);

			// State.Tag(Files: "#NotForLicensee Build Tools Win64", With: Files);

			FileSet files = FileSet.Empty;
			files += await CompileAsync("CrashReportClient", UnrealTargetPlatform.Win64, UnrealTargetConfiguration.Shipping, arguments: crashReportClientArgs);
			files += await CompileAsync("CrashReportClientEditor", UnrealTargetPlatform.Win64, UnrealTargetConfiguration.Shipping, arguments: crashReportClientArgs);
			files += await CompileAsync("ShaderCompileWorker", UnrealTargetPlatform.Win64, UnrealTargetConfiguration.Development);
			files += await CompileAsync("EpicWebHelper", UnrealTargetPlatform.Win64, UnrealTargetConfiguration.Development);
			files += await CompileAsync("UnrealInsights", UnrealTargetPlatform.Win64, UnrealTargetConfiguration.Development);
			files += await CompileAsync("UnrealFrontend", UnrealTargetPlatform.Win64, UnrealTargetConfiguration.Development);
			files += await CompileAsync("UnrealLightmass", UnrealTargetPlatform.Win64, UnrealTargetConfiguration.Development);
			files += await CompileAsync("InterchangeWorker", UnrealTargetPlatform.Win64, UnrealTargetConfiguration.Development);
			files += await CompileAsync("UnrealPak", UnrealTargetPlatform.Win64, UnrealTargetConfiguration.Development);
			files += await CompileAsync("UnrealMultiUserServer", UnrealTargetPlatform.Win64, UnrealTargetConfiguration.Development);
			files += await CompileAsync("UnrealRecoverySvc", UnrealTargetPlatform.Win64, UnrealTargetConfiguration.Development);
			files += await CompileAsync("LiveCodingConsole", UnrealTargetPlatform.Win64, UnrealTargetConfiguration.Development);
			files += await CompileAsync("BootstrapPackagedGame", UnrealTargetPlatform.Win64, UnrealTargetConfiguration.Shipping);
			files += await CompileAsync("BuildPatchTool", UnrealTargetPlatform.Win64, UnrealTargetConfiguration.Shipping);
			files += await CompileAsync("SwitchboardListener", UnrealTargetPlatform.Win64, UnrealTargetConfiguration.Development);
			files += FileSet.FromFile(RootDir, "Engine/Binaries/Win64/XGEControlWorker.exe");
			files += FileSet.FromFile(RootDir, "Engine/Saved/UnrealBuildTool/BuildConfiguration.Schema.xsd");

			return files;
		}

		[BgNodeName("Build Tools CS")]
		static async Task<BgFileSet> BuildToolsCsAsync(BgContext state, BgBool signExecutables)
		{
			FileUtils.ForceDeleteDirectory(TempCsToolsDir);

			// Copy Source and referenced libraries to a new location with Confidential folders removed
			FileSet uatProjects = Workspace.Filter("Engine/Source/Programs/AutomationTool/....csproj");
			_ = uatProjects;

			FileSet redistUatSource = Workspace.Filter("Engine/Binaries/DotNET/...;Engine/Binaries/ThirdParty/Newtonsoft/...;Engine/Binaries/ThirdParty/IOS/...;Engine/Binaries/ThirdParty/VisualStudio/...;Engine/Source/Programs/...;Engine/Platforms/*/Source/Programs/...;Engine/Source/Editor/SwarmInterface/...");
			await redistUatSource.CopyToAsync(Unreal.RootDirectory);

			// Compile all the tools
			CsCompileOutput output = CsCompileOutput.Empty;
			if (HostPlatform.Current.HostEditorPlatform == UnrealTargetPlatform.Win64)
			{
				output += await CsCompileAsync(FileReference.Combine(TempCsToolsDir, "Engine/Source/Programs/UnrealSwarm/SwarmCoordinator/SwarmCoordinator.csproj"), platform: "AnyCPU", configuration: "Development");
				output += await CsCompileAsync(FileReference.Combine(TempCsToolsDir, "Engine/Source/Programs/UnrealSwarm/Agent/Agent.csproj"), platform: "AnyCPU", configuration: "Development");
				output += await CsCompileAsync(FileReference.Combine(TempCsToolsDir, "Engine/Source/Editor/SwarmInterface/DotNET/SwarmInterface.csproj"), platform: "AnyCPU", configuration: "Development");
				output += await CsCompileAsync(FileReference.Combine(TempCsToolsDir, "Engine/Source/Programs/UnrealControls/UnrealControls.csproj"), platform: "AnyCPU", configuration: "Development");
				output += await CsCompileAsync(FileReference.Combine(TempCsToolsDir, "Engine/Source/Programs/IOS/iPhonePackager/iPhonePackager.csproj"), platform: "AnyCPU", configuration: "Development", arguments: "/verbosity:minimal /target:Rebuild");
				output += await CsCompileAsync(FileReference.Combine(TempCsToolsDir, "Engine/Source/Programs/NetworkProfiler/NetworkProfiler/NetworkProfiler.csproj"), platform: "AnyCPU", configuration: "Development");
			}

			FileSet binaries = output.Binaries + output.References;

			// Tag AutomationTool and UnrealBuildTool folders recursively as NET Core dependencies are not currently handled by CsCompile
			binaries += FileSet.FromDirectory(TempCsToolsDir).Filter("Engine/Binaries/DotNET/AutomationTool/...");
			binaries += FileSet.FromDirectory(TempCsToolsDir).Filter("Engine/Binaries/DotNET/UnrealBuildTool/...");

			// Tag AutomationTool Script module build records, so that prebuilt modules may be discovered in the absence of source code
			binaries += FileSet.FromDirectory(TempCsToolsDir).Filter("Engine/Intermediate/ScriptModules/...");

			if (state.Get(signExecutables))
			{
				//				Binaries.Sign();
			}

			return binaries;
		}

		/// <summary>
		/// Creates a DDC pack file for the supported platforms
		/// </summary>
		[BgNodeName("Build DDC Win64")]
		static async Task<BgFileSet> BuildDdcWin64Async(BgContext state, BgList<BgString> platforms, BgList<BgFileSet> dependencies)
		{
			// Build up a list of files needed to build DDC
			FileSet toCopy = state.Get(dependencies);
			toCopy += await CsCompileAsync(FileReference.Combine(RootDir, "Engine/Source/Programs/UnrealBuildTool/UnrealBuildTool.csproj"), platform: "AnyCPU", configuration: "Development", enumerateOnly: true).MergeAsync();
			toCopy += await toCopy.Filter("*.target").TagReceiptsAsync(runtimeDependencies: true);
			toCopy += FileSet.FromFile(RootDir, "Engine/Binaries/DotNET/Ionic.Zip.Reduced.dll");
			toCopy += FileSet.FromFile(RootDir, "Engine/Binaries/DotNET/OneSky.dll");
			toCopy += FileSet.FromDirectory(RootDir).Filter("Templates/TemplateResources/...");

			foreach (DirectoryReference extensionDir in Unreal.GetExtensionDirs(Unreal.EngineDirectory))
			{
				toCopy += FileSet.FromDirectory(extensionDir).Filter("Content/...").Except("....psd;....pdn;....fbx;....po");
				toCopy += FileSet.FromDirectory(extensionDir).Filter("Config/...").Except("....vdf");
				toCopy += FileSet.FromDirectory(extensionDir).Filter("Shaders/...");

				DirectoryReference extensionPluginsDir = DirectoryReference.Combine(extensionDir, "Plugins");
				toCopy += FileSet.FromDirectory(extensionPluginsDir).Filter("....uplugin;.../Config/...;.../Content/...;.../Resources/...;.../Shaders/...;.../Templates/...").Except(".../TwitchLiveStreaming/...");
			}

			// Filter out the files not needed to build DDC. Removing confidential folders can affect DDC keys, so we want to be sure that we're making DDC with a build that can use it.
			FileSet filteredCopyList = toCopy.Except(".../Source/...;.../Intermediate/...");

			Dictionary<FileReference, FileReference> targetFileToSourceFile = new Dictionary<FileReference, FileReference>();
			MapFilesToOutputDir(filteredCopyList, TempDdcDir, targetFileToSourceFile);

			FileUtils.ForceDeleteDirectoryContents(TempDdcDir);
			await filteredCopyList.CopyToAsync(TempDdcDir);

			// Run the DDC commandlet
			List<string> arguments = new List<string>();
			arguments.Add($"-TempDir=\"{TempDdcDir}\"");
			arguments.Add($"-FeaturePacks=\"{String.Join(";", GetDdcProjects(UnrealTargetPlatform.Win64))}\"");
			arguments.Add($"-TargetPlatforms={String.Join("+", state.Get(platforms))}");
			arguments.Add($"-HostPlatform=Win64");
			await state.CommandAsync("BuildDerivedDataCache", arguments: String.Join(" ", arguments));

			// Return a tag for the output file
			return FileSet.FromFile(TempDdcDir, "Engine/DerivedDataCache/Compressed.ddp");
		}

		/// <summary>
		/// Copy all the build artifacts to the output folder
		/// </summary>
		[BgNodeName("Make Installed Build Win64")]
		static async Task<BgFileSet> MakeInstalledBuildWin64Async(BgContext state, BgList<BgFileSet> inputFiles, BgString finalizeArgs, BgString outputDir)
		{
			// Find all the input files, and add any runtime dependencies from the receipts into the list
			FileSet sourceFiles = state.Get(inputFiles);
			sourceFiles += await sourceFiles.Filter("*.target").TagReceiptsAsync(runtimeDependencies: true);

			// Include any files referenced by dependency lists
			FileSet dependencyListFiles = sourceFiles.Filter(".../DependencyList.txt;.../DependencyList-AllModules.txt");
			foreach (FileReference dependencyListFile in dependencyListFiles)
			{
				string[] lines = await FileReference.ReadAllLinesAsync(dependencyListFile);
				foreach (string line in lines)
				{
					string trimLine = line.Trim();
					if (trimLine.Length > 0)
					{
						sourceFiles += FileSet.FromFile(Unreal.RootDirectory, trimLine);
					}
				}
			}
			sourceFiles -= dependencyListFiles;

			// Clear the output directory
			DirectoryReference outputDirRef = new DirectoryReference(state.Get(outputDir));
			FileUtils.ForceDeleteDirectoryContents(outputDirRef);
			FileSet installedFiles = await sourceFiles.CopyToAsync(outputDirRef);

			// Run the finalize command 
			await state.CommandAsync("FinalizeInstalledBuild", arguments: state.Get(finalizeArgs));

			// Get the final list of files
			installedFiles += FileSet.FromFile(outputDirRef, "Engine/Build/InstalledEngine.txt");

			// Sanitize any receipts in the output directory
			await installedFiles.Filter("*.target").SanitizeReceiptsAsync();

			return installedFiles;
		}

		static void MapFilesToOutputDir(IEnumerable<FileReference> sourceFiles, DirectoryReference targetDir, Dictionary<FileReference, FileReference> targetFileToSourceFile)
		{
			foreach (FileReference sourceFile in sourceFiles)
			{
				DirectoryReference baseDir;
				if (sourceFile.IsUnderDirectory(TempCsToolsDir))
				{
					baseDir = TempCsToolsDir;
				}
				else if (sourceFile.IsUnderDirectory(TempMiscDir))
				{
					baseDir = TempMiscDir;
				}
				else if (sourceFile.IsUnderDirectory(TempDdcDir))
				{
					baseDir = TempDdcDir;
				}
				else
				{
					baseDir = Unreal.RootDirectory;
				}

				FileReference targetFile = FileReference.Combine(targetDir, sourceFile.MakeRelativeTo(baseDir));
				targetFileToSourceFile[targetFile] = sourceFile;
			}
		}
	}
}
