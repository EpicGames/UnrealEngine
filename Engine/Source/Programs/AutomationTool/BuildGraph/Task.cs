// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.IO;
using System.Linq;
using System.Reflection;
using System.Text;
using System.Threading.Tasks;
using System.Xml;
using EpicGames.Core;
using Microsoft.Extensions.Logging;
using OpenTracing;
using UnrealBuildBase;

#nullable enable

namespace AutomationTool
{
	/// <summary>
	/// Attribute to mark parameters to a task, which should be read as XML attributes from the script file.
	/// </summary>
	[AttributeUsage(AttributeTargets.Field | AttributeTargets.Property)]
	public sealed class TaskParameterAttribute : Attribute
	{
		/// <summary>
		/// Whether the parameter can be omitted
		/// </summary>
		public bool Optional
		{
			get;
			set;
		}

		/// <summary>
		/// Sets additional restrictions on how this field is validated in the schema. Default is to allow any valid field type.
		/// </summary>
		public TaskParameterValidationType ValidationType
		{
			get;
			set;
		}
	}

	/// <summary>
	/// Attribute used to associate an XML element name with a parameter block that can be used to construct tasks
	/// </summary>
	[AttributeUsage(AttributeTargets.Class)]
	public sealed class TaskElementAttribute : Attribute
	{
		/// <summary>
		/// Name of the XML element that can be used to denote this class
		/// </summary>
		public string Name { get; }

		/// <summary>
		/// Type to be constructed from the deserialized element
		/// </summary>
		public Type ParametersType { get; }

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="name">Name of the XML element used to denote this object</param>
		/// <param name="parametersType">Type to be constructed from this object</param>
		public TaskElementAttribute(string name, Type parametersType)
		{
			Name = name;
			ParametersType = parametersType;
		}
	}

	/// <summary>
	/// Proxy to handle executing multiple tasks simultaneously (such as compile tasks). If a task supports simultaneous execution, it can return a separate
	/// executor an executor instance from GetExecutor() callback. If not, it must implement ExecuteAsync().
	/// </summary>
	public interface ITaskExecutor
	{
		/// <summary>
		/// Adds another task to this executor
		/// </summary>
		/// <param name="task">Task to add</param>
		/// <returns>True if the task could be added, false otherwise</returns>
		bool Add(BgTaskImpl task);

		/// <summary>
		/// ExecuteAsync all the tasks added to this executor.
		/// </summary>
		/// <param name="job">Information about the current job</param>
		/// <param name="buildProducts">Set of build products produced by this node.</param>
		/// <param name="tagNameToFileSet">Mapping from tag names to the set of files they include</param>
		/// <returns>Whether the task succeeded or not. Exiting with an exception will be caught and treated as a failure.</returns>
		Task ExecuteAsync(JobContext job, HashSet<FileReference> buildProducts, Dictionary<string, HashSet<FileReference>> tagNameToFileSet);
	}

	/// <summary>
	/// Base class for all custom build tasks
	/// </summary>
	public abstract class BgTaskImpl
	{
		/// <summary>
		/// Accessor for the default log interface
		/// </summary>
		protected static ILogger Logger => Log.Logger;

		/// <summary>
		/// Line number in a source file that this task was declared. Optional; used for log messages.
		/// </summary>
		public BgScriptLocation? SourceLocation { get; set; }

		/// <summary>
		/// ExecuteAsync this node.
		/// </summary>
		/// <param name="job">Information about the current job</param>
		/// <param name="buildProducts">Set of build products produced by this node.</param>
		/// <param name="tagNameToFileSet">Mapping from tag names to the set of files they include</param>
		/// <returns>Whether the task succeeded or not. Exiting with an exception will be caught and treated as a failure.</returns>
		public abstract Task ExecuteAsync(JobContext job, HashSet<FileReference> buildProducts, Dictionary<string, HashSet<FileReference>> tagNameToFileSet);

		/// <summary>
		/// Creates a proxy to execute this node.
		/// </summary>
		/// <returns>New proxy instance if one is available to execute this task, otherwise null.</returns>
		public virtual ITaskExecutor? GetExecutor()
		{
			return null;
		}

		/// <summary>
		/// Output this task out to an XML writer.
		/// </summary>
		public abstract void Write(XmlWriter writer);

		/// <summary>
		/// Writes this task to an XML writer, using the given parameters object.
		/// </summary>
		/// <param name="writer">Writer for the XML schema</param>
		/// <param name="parameters">Parameters object that this task is constructed with</param>
		protected void Write(XmlWriter writer, object parameters)
		{
			TaskElementAttribute element = GetType().GetCustomAttribute<TaskElementAttribute>() ?? throw new InvalidOperationException();
			writer.WriteStartElement(element.Name);

			foreach (FieldInfo field in parameters.GetType().GetFields())
			{
				if (field.MemberType == MemberTypes.Field)
				{
					TaskParameterAttribute? parameterAttribute = field.GetCustomAttribute<TaskParameterAttribute>();
					if (parameterAttribute != null)
					{
						object? value = field.GetValue(parameters);
						if (value != null)
						{
							writer.WriteAttributeString(field.Name, value.ToString());
						}
					}
				}
			}
			foreach (PropertyInfo property in parameters.GetType().GetProperties())
			{
				TaskParameterAttribute? parameterAttribute = property.GetCustomAttribute<TaskParameterAttribute>();
				if (parameterAttribute != null)
				{
					object? value = property.GetValue(parameters);
					if (value != null)
					{
						writer.WriteAttributeString(property.Name, value.ToString());
					}
				}
			}

			writer.WriteEndElement();
		}

		/// <summary>
		/// Returns a string used for trace messages
		/// </summary>
		public string GetTraceString()
		{
			StringBuilder builder = new StringBuilder();
			using (XmlWriter writer = XmlWriter.Create(new StringWriter(builder), new XmlWriterSettings() { OmitXmlDeclaration = true }))
			{
				Write(writer);
			}
			return builder.ToString();
		}

		/// <summary>
		/// Gets the name of this task for tracing
		/// </summary>
		/// <returns>The trace name</returns>
		public virtual string GetTraceName()
		{
			TaskElementAttribute? taskElement = GetType().GetCustomAttribute<TaskElementAttribute>();
			return (taskElement != null) ? taskElement.Name : "unknown";
		}

		/// <summary>
		/// Get properties to include in tracing info
		/// </summary>
		/// <param name="span">The scope to add properties to</param>
		/// <param name="prefix">Prefix for metadata entries</param>
		public virtual void GetTraceMetadata(ITraceSpan span, string prefix)
		{
			if (SourceLocation != null)
			{
				span.AddMetadata(prefix + "source.file", SourceLocation.File.FullName);
				span.AddMetadata(prefix + "source.line", SourceLocation.LineNumber.ToString());
			}
		}

		/// <summary>
		/// Get properties to include in tracing info
		/// </summary>
		/// <param name="span">The scope to add properties to</param>
		/// <param name="prefix">Prefix for metadata entries</param>
		public virtual void GetTraceMetadata(ISpan span, string prefix)
		{
			if (SourceLocation != null)
			{
				span.SetTag(prefix + "source.file", SourceLocation.File.FullName);
				span.SetTag(prefix + "source.line", SourceLocation.LineNumber);
			}
		}

		/// <summary>
		/// Find all the tags which are used as inputs to this task
		/// </summary>
		/// <returns>The tag names which are read by this task</returns>
		public abstract IEnumerable<string> FindConsumedTagNames();

		/// <summary>
		/// Find all the tags which are modified by this task
		/// </summary>
		/// <returns>The tag names which are modified by this task</returns>
		public abstract IEnumerable<string> FindProducedTagNames();

		/// <summary>
		/// Adds tag names from a filespec
		/// </summary>
		/// <param name="filespec">A filespec, as can be passed to ResolveFilespec</param>
		/// <returns>Tag names from this filespec</returns>
		protected static IEnumerable<string> FindTagNamesFromFilespec(string filespec)
		{
			if (!String.IsNullOrEmpty(filespec))
			{
				foreach (string pattern in SplitDelimitedList(filespec))
				{
					if (pattern.StartsWith("#", StringComparison.Ordinal))
					{
						yield return pattern;
					}
				}
			}
		}

		/// <summary>
		/// Enumerates tag names from a list
		/// </summary>
		/// <param name="tagList">List of tags separated by semicolons</param>
		/// <returns>Tag names from this filespec</returns>
		protected static IEnumerable<string> FindTagNamesFromList(string? tagList)
		{
			if (!String.IsNullOrEmpty(tagList))
			{
				foreach (string tagName in SplitDelimitedList(tagList))
				{
					yield return tagName;
				}
			}
		}

		/// <summary>
		/// Resolves a single name to a file reference, resolving relative paths to the root of the current path.
		/// </summary>
		/// <param name="name">Name of the file</param>
		/// <returns>Fully qualified file reference</returns>
		public static FileReference ResolveFile(string name)
		{
			if (Path.IsPathRooted(name))
			{
				return new FileReference(name);
			}
			else
			{
				return new FileReference(Path.Combine(CommandUtils.CmdEnv.LocalRoot, name));
			}
		}

		/// <summary>
		/// Resolves a directory reference from the given string. Assumes the root directory is the root of the current branch.
		/// </summary>
		/// <param name="name">Name of the directory. May be null or empty.</param>
		/// <returns>The resolved directory</returns>
		public static DirectoryReference ResolveDirectory(string? name)
		{
			if (String.IsNullOrEmpty(name))
			{
				return Unreal.RootDirectory;
			}
			else if (Path.IsPathRooted(name))
			{
				return new DirectoryReference(name);
			}
			else
			{
				return DirectoryReference.Combine(Unreal.RootDirectory, name);
			}
		}

		/// <summary>
		/// Finds or adds a set containing files with the given tag
		/// </summary>
		/// <param name="tagNameToFileSet">Map of tag names to the set of files they contain</param>
		/// <param name="tagName">The tag name to return a set for. A leading '#' character is required.</param>
		/// <returns>Set of files</returns>
		public static HashSet<FileReference> FindOrAddTagSet(Dictionary<string, HashSet<FileReference>> tagNameToFileSet, string tagName)
		{
			// Make sure the tag name contains a single leading hash
			if (tagName.LastIndexOf('#') != 0)
			{
				throw new AutomationException("Tag name '{0}' is not valid - should contain a single leading '#' character", tagName);
			}

			// Any spaces should be later than the second char - most likely to be a typo if directly after the # character
			if (tagName.IndexOf(' ', StringComparison.Ordinal) == 1)
			{
				throw new AutomationException("Tag name '{0}' is not valid - spaces should only be used to separate words", tagName);
			}

			// Find the files which match this tag
			HashSet<FileReference>? files;
			if (!tagNameToFileSet.TryGetValue(tagName, out files))
			{
				files = new HashSet<FileReference>();
				tagNameToFileSet.Add(tagName, files);
			}

			// If we got a null reference, it's because the tag is not listed as an input for this node (see RunGraph.BuildSingleNode). Fill it in, but only with an error.
			if (files == null)
			{
				Logger.LogError("Attempt to reference tag '{TagName}', which is not listed as a dependency of this node.", tagName);
				files = new HashSet<FileReference>();
				tagNameToFileSet.Add(tagName, files);
			}
			return files;
		}

		/// <summary>
		/// Resolve a list of files, tag names or file specifications separated by semicolons. Supported entries may be:
		///   a) The name of a tag set (eg. #CompiledBinaries)
		///   b) Relative or absolute filenames
		///   c) A simple file pattern (eg. Foo/*.cpp)
		///   d) A full directory wildcard (eg. Engine/...)
		/// Note that wildcards may only match the last fragment in a pattern, so matches like "/*/Foo.txt" and "/.../Bar.txt" are illegal.
		/// </summary>
		/// <param name="defaultDirectory">The default directory to resolve relative paths to</param>
		/// <param name="delimitedPatterns">List of files, tag names, or file specifications to include separated by semicolons.</param>
		/// <param name="tagNameToFileSet">Mapping of tag name to fileset, as passed to the ExecuteAsync() method</param>
		/// <returns>Set of matching files.</returns>
		public static HashSet<FileReference> ResolveFilespec(DirectoryReference defaultDirectory, string delimitedPatterns, Dictionary<string, HashSet<FileReference>> tagNameToFileSet)
		{
			List<string> excludePatterns = new List<string>();
			return ResolveFilespecWithExcludePatterns(defaultDirectory, delimitedPatterns, excludePatterns, tagNameToFileSet);
		}

		/// <summary>
		/// Resolve a list of files, tag names or file specifications separated by semicolons as above, but preserves any directory references for further processing.
		/// </summary>
		/// <param name="defaultDirectory">The default directory to resolve relative paths to</param>
		/// <param name="delimitedPatterns">List of files, tag names, or file specifications to include separated by semicolons.</param>
		/// <param name="excludePatterns">Set of patterns to apply to directory searches. This can greatly speed up enumeration by earlying out of recursive directory searches if large directories are excluded (eg. .../Intermediate/...).</param>
		/// <param name="tagNameToFileSet">Mapping of tag name to fileset, as passed to the ExecuteAsync() method</param>
		/// <returns>Set of matching files.</returns>
		public static HashSet<FileReference> ResolveFilespecWithExcludePatterns(DirectoryReference defaultDirectory, string delimitedPatterns, List<string> excludePatterns, Dictionary<string, HashSet<FileReference>> tagNameToFileSet)
		{
			// Split the argument into a list of patterns
			List<string> patterns = SplitDelimitedList(delimitedPatterns);
			return ResolveFilespecWithExcludePatterns(defaultDirectory, patterns, excludePatterns, tagNameToFileSet);
		}

		/// <summary>
		/// Resolve a list of files, tag names or file specifications as above, but preserves any directory references for further processing.
		/// </summary>
		/// <param name="defaultDirectory">The default directory to resolve relative paths to</param>
		/// <param name="filePatterns">List of files, tag names, or file specifications to include separated by semicolons.</param>
		/// <param name="excludePatterns">Set of patterns to apply to directory searches. This can greatly speed up enumeration by earlying out of recursive directory searches if large directories are excluded (eg. .../Intermediate/...).</param>
		/// <param name="tagNameToFileSet">Mapping of tag name to fileset, as passed to the ExecuteAsync() method</param>
		/// <returns>Set of matching files.</returns>
		public static HashSet<FileReference> ResolveFilespecWithExcludePatterns(DirectoryReference defaultDirectory, List<string> filePatterns, List<string> excludePatterns, Dictionary<string, HashSet<FileReference>> tagNameToFileSet)
		{
			// Parse each of the patterns, and add the results into the given sets
			HashSet<FileReference> files = new HashSet<FileReference>();
			foreach (string pattern in filePatterns)
			{
				// Check if it's a tag name
				if (pattern.StartsWith("#", StringComparison.Ordinal))
				{
					files.UnionWith(FindOrAddTagSet(tagNameToFileSet, pattern));
					continue;
				}

				// If it doesn't contain any wildcards, just add the pattern directly
				int wildcardIdx = FileFilter.FindWildcardIndex(pattern);
				if (wildcardIdx == -1)
				{
					files.Add(FileReference.Combine(defaultDirectory, pattern));
					continue;
				}

				// Find the base directory for the search. We construct this in a very deliberate way including the directory separator itself, so matches
				// against the OS root directory will resolve correctly both on Mac (where / is the filesystem root) and Windows (where / refers to the current drive).
				int lastDirectoryIdx = pattern.LastIndexOfAny(new char[] { Path.DirectorySeparatorChar, Path.AltDirectorySeparatorChar }, wildcardIdx);
				DirectoryReference baseDir = DirectoryReference.Combine(defaultDirectory, pattern.Substring(0, lastDirectoryIdx + 1));

				// Construct the absolute include pattern to match against, re-inserting the resolved base directory to construct a canonical path.
				string includePattern = baseDir.FullName.TrimEnd(new char[] { Path.DirectorySeparatorChar, Path.AltDirectorySeparatorChar }) + "/" + pattern.Substring(lastDirectoryIdx + 1);

				// Construct a filter and apply it to the directory
				if (DirectoryReference.Exists(baseDir))
				{
					FileFilter filter = new FileFilter();
					filter.AddRule(includePattern, FileFilterType.Include);
					if (excludePatterns != null && excludePatterns.Count > 0)
					{
						filter.AddRules(excludePatterns, FileFilterType.Exclude);
					}
					files.UnionWith(filter.ApplyToDirectory(baseDir, baseDir.FullName, true));
				}
			}

			// If we have exclude rules, create and run a filter against all the output files to catch things that weren't added from an include
			if (excludePatterns != null && excludePatterns.Count > 0)
			{
				FileFilter filter = new FileFilter(FileFilterType.Include);
				filter.AddRules(excludePatterns, FileFilterType.Exclude);
				files.RemoveWhere(x => !filter.Matches(x.FullName));
			}
			return files;
		}

		/// <summary>
		/// Splits a string separated by semicolons into a list, removing empty entries
		/// </summary>
		/// <param name="text">The input string</param>
		/// <returns>Array of the parsed items</returns>
		public static List<string> SplitDelimitedList(string text)
		{
			return text.Split(';').Select(x => x.Trim()).Where(x => x.Length > 0).ToList();
		}

		/// <summary>
		/// Name of the environment variable containing cleanup commands
		/// </summary>
		public const string CleanupScriptEnvVarName = "UE_HORDE_CLEANUP";

		/// <summary>
		/// Name of the environment variable containing lease cleanup commands
		/// </summary>
		public const string LeaseCleanupScriptEnvVarName = "UE_HORDE_LEASE_CLEANUP";

		/// <summary>
		/// Add cleanup commands to run after the step completes
		/// </summary>
		/// <param name="newLines">Lines to add to the cleanup script</param>
		/// <param name="lease">Whether to add the commands to run on lease termination</param>
		public static async Task AddCleanupCommandsAsync(IEnumerable<string> newLines, bool lease = false)
		{
			string? cleanupScriptEnvVar = Environment.GetEnvironmentVariable(lease ? LeaseCleanupScriptEnvVarName : CleanupScriptEnvVarName);
			if (!String.IsNullOrEmpty(cleanupScriptEnvVar))
			{
				FileReference cleanupScript = new FileReference(cleanupScriptEnvVar);
				await FileReference.AppendAllLinesAsync(cleanupScript, newLines);
			}
		}

		/// <summary>
		/// Name of the environment variable containing a file to write Horde graph updates to
		/// </summary>
		public const string GraphUpdateEnvVarName = "UE_HORDE_GRAPH_UPDATE";

		/// <summary>
		/// Updates the graph currently used by Horde
		/// </summary>
		/// <param name="job">Context for the current job that is being executed</param>
		public static void UpdateGraphForHorde(JobContext job)
		{
			string? exportGraphFile = Environment.GetEnvironmentVariable(GraphUpdateEnvVarName);
			if (String.IsNullOrEmpty(exportGraphFile))
			{
				throw new Exception($"Missing environment variable {GraphUpdateEnvVarName}. This is required to update graphs on Horde.");
			}

			List<string> newParams = new List<string>();
			newParams.Add("BuildGraph");
			newParams.AddRange(job.OwnerCommand.Params.Select(x => $"-{x}"));
			newParams.RemoveAll(x => x.StartsWith("-SingleNode=", StringComparison.OrdinalIgnoreCase));
			newParams.Add($"-HordeExport={exportGraphFile}");
			newParams.Add($"-ListOnly");

			string newCommandLine = CommandLineArguments.Join(newParams);
			CommandUtils.RunUAT(CommandUtils.CmdEnv, newCommandLine, "bg");
		}
	}

	/// <summary>
	/// Legacy implementation of <see cref="BgTaskImpl"/> which operates synchronously
	/// </summary>
	public abstract class CustomTask : BgTaskImpl
	{
		/// <inheritdoc/>
		public sealed override Task ExecuteAsync(JobContext job, HashSet<FileReference> buildProducts, Dictionary<string, HashSet<FileReference>> tagNameToFileSet)
		{
			Execute(job, buildProducts, tagNameToFileSet);
			return Task.CompletedTask;
		}

		/// <summary>
		/// ExecuteAsync this node.
		/// </summary>
		/// <param name="job">Information about the current job</param>
		/// <param name="buildProducts">Set of build products produced by this node.</param>
		/// <param name="tagNameToFileSet">Mapping from tag names to the set of files they include</param>
		/// <returns>Whether the task succeeded or not. Exiting with an exception will be caught and treated as a failure.</returns>
		public abstract void Execute(JobContext job, HashSet<FileReference> buildProducts, Dictionary<string, HashSet<FileReference>> tagNameToFileSet);
	}
}
