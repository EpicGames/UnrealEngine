// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.IO;
using System.Linq;
using System.Threading.Tasks;
using System.Xml;
using EpicGames.Core;
using Microsoft.Extensions.Logging;
using UnrealBuildTool;

namespace AutomationTool.Tasks
{
	/// <summary>
	/// Parameters for a task that purges data from a symbol store after a given age
	/// </summary>
	public class AgeStoreTaskParameters
	{
		/// <summary>
		/// The target platform to age symbols for.
		/// </summary>
		[TaskParameter]
		public UnrealTargetPlatform Platform { get; set; }

		/// <summary>
		/// The symbol server directory.
		/// </summary>
		[TaskParameter]
		public string StoreDir { get; set; }

		/// <summary>
		/// Number of days worth of symbols to keep.
		/// </summary>
		[TaskParameter]
		public int Days { get; set; }

		/// <summary>
		/// The root of the build directory to check for existing buildversion named directories.
		/// </summary>
		[TaskParameter(Optional = true)]
		public string BuildDir { get; set; }

		/// <summary>
		/// A substring to match in directory file names before deleting symbols. This allows the "age store" task
		/// to avoid deleting symbols from other builds in the case where multiple builds share the same symbol server.
		/// Specific use of the filter value is determined by the symbol server structure defined by the platform toolchain.
		/// </summary>
		[TaskParameter(Optional = true)]
		public string Filter { get; set; }
	}

	/// <summary>
	/// Task that strips symbols from a set of files. This task is named after the AGESTORE utility that comes with the Microsoft debugger tools SDK, but is actually a separate implementation. The main
	/// difference is that it uses the last modified time rather than last access time to determine which files to delete.
	/// </summary>
	[TaskElement("AgeStore", typeof(AgeStoreTaskParameters))]
	public class AgeStoreTask : BgTaskImpl
	{
		/// <summary>
		/// Parameters for this task
		/// </summary>
		readonly AgeStoreTaskParameters _parameters;

		/// <summary>
		/// Construct a spawn task
		/// </summary>
		/// <param name="parameters">Parameters for the task</param>
		public AgeStoreTask(AgeStoreTaskParameters parameters)
		{
			_parameters = parameters;
		}

		private static void TryDelete(DirectoryInfo directory)
		{
			try
			{
				directory.Delete(true);
				Logger.LogInformation("Removed '{Arg0}'", directory.FullName);
			}
			catch
			{
				Logger.LogWarning("Couldn't delete '{Arg0}' - skipping", directory.FullName);
			}
		}

		private static void TryDelete(FileInfo file)
		{
			try
			{
				file.Delete();
				Logger.LogInformation("Removed '{Arg0}'", file.FullName);
			}
			catch
			{
				Logger.LogWarning("Couldn't delete '{Arg0}' - skipping", file.FullName);
			}
		}

		// Checks if an existing build has a version file, returns false to NOT delete if it exists
		private static bool CheckCanDeleteFromVersionFile(HashSet<string> existingBuilds, DirectoryInfo directory, FileInfo individualFile = null)
		{
			// check for any existing version files
			foreach (FileInfo buildVersionFile in directory.EnumerateFiles("*.version"))
			{
				// If the buildversion matches one of the directories in build share provided, don't delete no matter the age.
				string buildVersion = Path.GetFileNameWithoutExtension(buildVersionFile.Name);
				if (existingBuilds.Contains(buildVersion))
				{
					// if checking for an individual file, see if the filename matches what's in the .version file.
					// these file names won't have extensions.
					if (individualFile != null)
					{
						string individualFilePath = individualFile.FullName;
						string filePointerName = File.ReadAllText(buildVersionFile.FullName).Trim();
						if (filePointerName == Path.GetFileNameWithoutExtension(individualFilePath))
						{
							Logger.LogInformation("Found existing build {BuildVersion} in the BuildDir with matching individual file {IndividualFilePath} - skipping.", buildVersion, individualFilePath);
							return false;
						}
					}
					// otherwise it's okay to just mark the entire folder for delete
					else
					{
						Logger.LogInformation("Found existing build {BuildVersion} in the BuildDir - skipping.", buildVersion);
						return false;
					}
				}
			}
			return true;
		}

		private static void RecurseDirectory(DateTime expireTimeUtc, DirectoryInfo currentDirectory, string[] directoryStructure, int level, string filter, HashSet<string> existingBuilds, bool deleteIndividualFiles)
		{
			// Do a file search at the last level.
			if (level == directoryStructure.Length)
			{
				if (deleteIndividualFiles)
				{
					// Delete any file in the directory that is out of date.
					foreach (FileInfo outdatedFile in currentDirectory.EnumerateFiles().Where(x => x.LastWriteTimeUtc < expireTimeUtc && x.Extension != ".version"))
					{
						// check to make sure this file is valid to delete
						if (CheckCanDeleteFromVersionFile(existingBuilds, currentDirectory, outdatedFile))
						{
							TryDelete(outdatedFile);
						}
					}
				}
				// If all files are out of date, delete the directory...
				else if (currentDirectory.EnumerateFiles().Where(x => x.Extension != ".version").All(x => x.LastWriteTimeUtc < expireTimeUtc) && CheckCanDeleteFromVersionFile(existingBuilds, currentDirectory))
				{
					TryDelete(currentDirectory);
				}
			}
			else
			{
				string[] patterns = directoryStructure[level].Split(';');
				foreach (string pattern in patterns)
				{
					string replacedPattern = String.Format(pattern, filter);

					foreach (DirectoryInfo childDirectory in currentDirectory.GetDirectories(replacedPattern, SearchOption.TopDirectoryOnly))
					{
						RecurseDirectory(expireTimeUtc, childDirectory, directoryStructure, level + 1, filter, existingBuilds, deleteIndividualFiles);
					}
				}

				// Delete this directory if it is empty, and it is not the root directory.
				if (level > 0 && !currentDirectory.EnumerateFileSystemInfos().Any())
				{
					TryDelete(currentDirectory);
				}
			}
		}

		/// <summary>
		/// ExecuteAsync the task.
		/// </summary>
		/// <param name="job">Information about the current job</param>
		/// <param name="buildProducts">Set of build products produced by this node.</param>
		/// <param name="tagNameToFileSet">Mapping from tag names to the set of files they include</param>
		public override Task ExecuteAsync(JobContext job, HashSet<FileReference> buildProducts, Dictionary<string, HashSet<FileReference>> tagNameToFileSet)
		{
			// Get the list of symbol file name patterns from the platform.
			Platform targetPlatform = Platform.GetPlatform(_parameters.Platform);
			string[] directoryStructure = targetPlatform.SymbolServerDirectoryStructure;
			if (directoryStructure == null)
			{
				throw new AutomationException("Platform does not specify the symbol server structure. Cannot age the symbol server.");
			}

			string filter = String.IsNullOrWhiteSpace(_parameters.Filter)
				? String.Empty
				: _parameters.Filter.Trim();

			// Eumerate the root directory of builds for buildversions to check against
			// Folder names in the root directory should match the name of the .version files
			HashSet<string> existingBuilds = new HashSet<string>(StringComparer.OrdinalIgnoreCase);
			if (!String.IsNullOrWhiteSpace(_parameters.BuildDir))
			{
				DirectoryReference buildDir = new DirectoryReference(_parameters.BuildDir);
				if (DirectoryReference.Exists(buildDir))
				{
					foreach (string buildName in DirectoryReference.EnumerateDirectories(buildDir).Select(build => build.GetDirectoryName()))
					{
						existingBuilds.Add(buildName);
					}
				}
				else
				{
					Logger.LogWarning("BuildDir of {Arg0} was provided but it doesn't exist! Will not check buildversions against it.", _parameters.BuildDir);
				}
			}

			// Get the time at which to expire files
			DateTime expireTimeUtc = DateTime.UtcNow - TimeSpan.FromDays(_parameters.Days);
			Logger.LogInformation("Expiring all files before {ExpireTimeUtc}...", expireTimeUtc);

			// Scan the store directory and delete old symbol files
			DirectoryReference symbolServerDirectory = ResolveDirectory(_parameters.StoreDir);
			CommandUtils.OptionallyTakeLock(targetPlatform.SymbolServerRequiresLock, symbolServerDirectory, TimeSpan.FromMinutes(15), () =>
			{
				RecurseDirectory(expireTimeUtc, new DirectoryInfo(symbolServerDirectory.FullName), directoryStructure, 0, filter, existingBuilds, targetPlatform.SymbolServerDeleteIndividualFiles);
			});
			return Task.CompletedTask;
		}

		/// <summary>
		/// Output this task out to an XML writer.
		/// </summary>
		public override void Write(XmlWriter writer)
		{
			Write(writer, _parameters);
		}

		/// <summary>
		/// Find all the tags which are used as inputs to this task
		/// </summary>
		/// <returns>The tag names which are read by this task</returns>
		public override IEnumerable<string> FindConsumedTagNames()
		{
			yield break;
		}

		/// <summary>
		/// Find all the tags which are modified by this task
		/// </summary>
		/// <returns>The tag names which are modified by this task</returns>
		public override IEnumerable<string> FindProducedTagNames()
		{
			yield break;
		}
	}

	public static partial class StandardTasks
	{
		/// <summary>
		/// Task that strips symbols from a set of files
		/// </summary>
		/// <param name="platform">Platform to clean</param>
		/// <param name="storeDir">Path to symbol store</param>
		/// <param name="days">Number of days to keep</param>
		/// <param name="buildDir">The root of the build directory to check for existing buildversion named directories</param>
		/// <param name="filter">A substring to match in directory file names before deleting symbols</param>
		/// <returns></returns>
		public static async Task<FileSet> AgeStoreAsync(UnrealTargetPlatform platform, string storeDir, int days, string buildDir, string filter = null)
		{
			AgeStoreTaskParameters parameters = new AgeStoreTaskParameters()
			{
				Platform = platform,
				StoreDir = storeDir,
				Days = days,
				BuildDir = buildDir,
				Filter = filter
			};

			return await ExecuteAsync(new AgeStoreTask(parameters));
		}
	}
}
