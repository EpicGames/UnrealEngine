// Copyright Epic Games, Inc. All Rights Reserved.

using System.Collections.Generic;
using System.Linq;
using System.Text;
using System.Text.Json;
using System.Threading.Tasks;
using System.Xml;
using EpicGames.Core;

namespace AutomationTool.Tasks
{
	/// <summary>
	/// Parameters for an AWS CLI task
	/// </summary>
	public class AwsAssumeRoleTaskParameters
	{
		/// <summary>
		/// Role to assume
		/// </summary>
		[TaskParameter]
		public string Arn { get; set; }

		/// <summary>
		/// Name of this session
		/// </summary>
		[TaskParameter]
		public string Session { get; set; }

		/// <summary>
		/// Duration of the token in seconds
		/// </summary>
		[TaskParameter(Optional = true)]
		public int Duration { get; set; } = 1000;

		/// <summary>
		/// Environment variables
		/// </summary>
		[TaskParameter(Optional = true)]
		public string Environment { get; set; }

		/// <summary>
		/// File to read environment variables from
		/// </summary>
		[TaskParameter(Optional = true)]
		public string EnvironmentFile { get; set; }

		/// <summary>
		/// Output file for the new environment
		/// </summary>
		[TaskParameter]
		public string OutputFile { get; set; }
	}

	/// <summary>
	/// Assumes an AWS role.
	/// </summary>
	[TaskElement("Aws-AssumeRole", typeof(AwsAssumeRoleTaskParameters))]
	public class AwsAssumeRoleTask : SpawnTaskBase
	{
		class AwsSettings
		{
			public AwsCredentials Credentials { get; set; }
		}

		class AwsCredentials
		{
			public string AccessKeyId { get; set; }
			public string SecretAccessKey { get; set; }
			public string SessionToken { get; set; }
		}

		readonly AwsAssumeRoleTaskParameters _parameters;

		/// <summary>
		/// Construct an AWS CLI task
		/// </summary>
		/// <param name="parameters">Parameters for the task</param>
		public AwsAssumeRoleTask(AwsAssumeRoleTaskParameters parameters)
		{
			_parameters = parameters;
		}

		/// <summary>
		/// ExecuteAsync the task.
		/// </summary>
		/// <param name="job">Information about the current job</param>
		/// <param name="buildProducts">Set of build products produced by this node.</param>
		/// <param name="tagNameToFileSet">Mapping from tag names to the set of files they include</param>
		public override async Task ExecuteAsync(JobContext job, HashSet<FileReference> buildProducts, Dictionary<string, HashSet<FileReference>> tagNameToFileSet)
		{
			StringBuilder arguments = new StringBuilder("sts assume-role");
			if (_parameters.Arn != null)
			{
				arguments.Append($" --role-arn {_parameters.Arn}");
			}
			if (_parameters.Session != null)
			{
				arguments.Append($" --role-session-name {_parameters.Session}");
			}
			arguments.Append($" --duration-seconds {_parameters.Duration}");

			Dictionary<string, string> environment = SpawnTaskBase.ParseEnvVars(_parameters.Environment, _parameters.EnvironmentFile);
			IProcessResult result = await SpawnTaskBase.ExecuteAsync("aws", arguments.ToString(), envVars: environment, logOutput: false);

			JsonSerializerOptions options = new JsonSerializerOptions();
			options.PropertyNameCaseInsensitive = true;

			AwsSettings settings = JsonSerializer.Deserialize<AwsSettings>(result.Output, options);
			if (settings.Credentials != null)
			{
				if (settings.Credentials.AccessKeyId != null)
				{
					environment["AWS_ACCESS_KEY_ID"] = settings.Credentials.AccessKeyId;
				}
				if (settings.Credentials.SecretAccessKey != null)
				{
					environment["AWS_SECRET_ACCESS_KEY"] = settings.Credentials.SecretAccessKey;
				}
				if (settings.Credentials.SessionToken != null)
				{
					environment["AWS_SESSION_TOKEN"] = settings.Credentials.SessionToken;
				}
			}

			FileReference outputFile = ResolveFile(_parameters.OutputFile);
			DirectoryReference.CreateDirectory(outputFile.Directory);
			await FileReference.WriteAllLinesAsync(outputFile, environment.OrderBy(x => x.Key).Select(x => $"{x.Key}={x.Value}"));
		}

		/// <summary>
		/// Output this task out to an XML writer.
		/// </summary>
		public override void Write(XmlWriter writer)
		{
			Write(writer, _parameters);
		}

		/// <summary>
		/// Find all the tags which are used as inputs to this task
		/// </summary>
		/// <returns>The tag names which are read by this task</returns>
		public override IEnumerable<string> FindConsumedTagNames()
		{
			yield break;
		}

		/// <summary>
		/// Find all the tags which are modified by this task
		/// </summary>
		/// <returns>The tag names which are modified by this task</returns>
		public override IEnumerable<string> FindProducedTagNames()
		{
			yield break;
		}
	}
}
