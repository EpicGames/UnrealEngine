// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.IO;
using System.Text;
using System.Text.Json;
using System.Threading.Tasks;
using System.Xml;
using EpicGames.Core;
using Microsoft.Extensions.Logging;
using UnrealBuildBase;

namespace AutomationTool.Tasks
{
	/// <summary>
	/// Parameters for a AWS ECS deploy task
	/// </summary>
	public class AwsEcsDeployTaskParameters
	{
		/// <summary>
		/// Task definition file to use
		/// </summary>
		[TaskParameter(Optional = false)]
		public string TaskDefinitionFile { get; set; }

		/// <summary>
		/// Docker image to set in new task definition (will replace %%DOCKER_PATTERN%% with this value)
		/// </summary>
		[TaskParameter(Optional = false)]
		public string DockerImage { get; set; }

		/// <summary>
		/// App version to set in new task definition (will replace %%VERSION%% with this value)
		/// </summary>
		[TaskParameter(Optional = true)]
		public string Version { get; set; }

		/// <summary>
		/// Cluster ARN representing AWS ECS cluster to operate on
		/// </summary>
		[TaskParameter(Optional = false)]
		public string Cluster { get; set; }

		/// <summary>
		/// Service name to update and deploy to
		/// </summary>
		[TaskParameter(Optional = false)]
		public string Service { get; set; }

		/// <summary>
		/// Environment variables
		/// </summary>
		[TaskParameter(Optional = true)]
		public string Environment { get; set; }

		/// <summary>
		/// File to read environment from
		/// </summary>
		[TaskParameter(Optional = true)]
		public string EnvironmentFile { get; set; }

		/// <summary>
		/// Write output to the log
		/// </summary>
		[TaskParameter(Optional = true)]
		public bool LogOutput { get; set; } = false;
	}

	/// <summary>
	/// Creates a new AWS ECS task definition and updates the ECS service to use this new revision of the task def
	/// </summary>
	[TaskElement("Aws-EcsDeploy", typeof(AwsEcsDeployTaskParameters))]
	public class AwsEcsDeployTask : SpawnTaskBase
	{
		/// <summary>
		/// Parameters for this task
		/// </summary>
		readonly AwsEcsDeployTaskParameters _parameters;

		/// <summary>
		/// Construct an AWS ECS deploy task
		/// </summary>
		/// <param name="parameters">Parameters for the task</param>
		public AwsEcsDeployTask(AwsEcsDeployTaskParameters parameters)
		{
			_parameters = parameters;
		}

		/// <summary>
		/// ExecuteAsync the task.
		/// </summary>
		/// <param name="job">Information about the current job</param>
		/// <param name="buildProducts">Set of build products produced by this node.</param>
		/// <param name="tagNameToFileSet">Mapping from tag names to the set of files they include</param>
		public override async Task ExecuteAsync(JobContext job, HashSet<FileReference> buildProducts, Dictionary<string, HashSet<FileReference>> tagNameToFileSet)
		{
			string taskDefTemplate = await File.ReadAllTextAsync(ResolveFile(_parameters.TaskDefinitionFile).FullName);
			string taskDefRendered = taskDefTemplate.Replace("%%DOCKER_IMAGE%%", _parameters.DockerImage, StringComparison.Ordinal);
			if (_parameters.Version != null)
			{
				taskDefRendered = taskDefRendered.Replace("%%VERSION%%", _parameters.Version, StringComparison.Ordinal);
			}

			FileReference tempTaskDefFile = FileReference.Combine(Unreal.RootDirectory, "Engine", "Intermediate", "Build", "AwsEcsDeployTaskTemp.json");
			DirectoryReference.CreateDirectory(tempTaskDefFile.Directory);
			await File.WriteAllTextAsync(tempTaskDefFile.FullName, taskDefRendered, new UTF8Encoding(encoderShouldEmitUTF8Identifier: false));

			IProcessResult createTaskDefResult = await SpawnTaskBase.ExecuteAsync("aws", $"ecs register-task-definition --cli-input-json \"file://{tempTaskDefFile.FullName}\"", envVars: ParseEnvVars(_parameters.Environment, _parameters.EnvironmentFile), logOutput: _parameters.LogOutput);

			JsonDocument taskDefJson = JsonDocument.Parse(createTaskDefResult.Output);
			string taskDefFamily = taskDefJson.RootElement.GetProperty("taskDefinition").GetProperty("family").GetString();
			string taskDefRevision = taskDefJson.RootElement.GetProperty("taskDefinition").GetProperty("revision").ToString();

			string @params = $"ecs update-service --cluster {_parameters.Cluster} --service {_parameters.Service} --task-definition {taskDefFamily}:{taskDefRevision}";
			await SpawnTaskBase.ExecuteAsync("aws", @params, envVars: ParseEnvVars(_parameters.Environment, _parameters.EnvironmentFile), logOutput: _parameters.LogOutput);

			Logger.LogInformation("Service {Service} updated to use new task def {TaskDefFamily}:{TaskDefRevision}", _parameters.Service, taskDefFamily, taskDefRevision);
		}

		/// <summary>
		/// Output this task out to an XML writer.
		/// </summary>
		public override void Write(XmlWriter writer)
		{
			Write(writer, _parameters);
		}

		/// <summary>
		/// Find all the tags which are used as inputs to this task
		/// </summary>
		/// <returns>The tag names which are read by this task</returns>
		public override IEnumerable<string> FindConsumedTagNames()
		{
			yield break;
		}

		/// <summary>
		/// Find all the tags which are modified by this task
		/// </summary>
		/// <returns>The tag names which are modified by this task</returns>
		public override IEnumerable<string> FindProducedTagNames()
		{
			yield break;
		}
	}
}
