// Copyright Epic Games, Inc. All Rights Reserved.

using System.Collections.Generic;
using System.Threading.Tasks;
using System.Xml;
using EpicGames.Core;

namespace AutomationTool.Tasks
{
	/// <summary>
	/// Parameters for a spawn task
	/// </summary>
	public class AwsTaskParameters
	{
		/// <summary>
		/// Arguments for the newly created process.
		/// </summary>
		[TaskParameter(Optional = true)]
		public string Arguments { get; set; }

		/// <summary>
		/// Environment variables
		/// </summary>
		[TaskParameter(Optional = true)]
		public string Environment { get; set; }

		/// <summary>
		/// File to read environment from
		/// </summary>
		[TaskParameter(Optional = true)]
		public string EnvironmentFile { get; set; }

		/// <summary>
		/// Write output to the log
		/// </summary>
		[TaskParameter(Optional = true)]
		public bool LogOutput { get; set; } = false;
	}

	/// <summary>
	/// Spawns AWS CLI and waits for it to complete.
	/// </summary>
	[TaskElement("Aws", typeof(AwsTaskParameters))]
	public class AwsTask : SpawnTaskBase
	{
		readonly AwsTaskParameters _parameters;

		/// <summary>
		/// Construct an AWS CLI task
		/// </summary>
		/// <param name="parameters">Parameters for the task</param>
		public AwsTask(AwsTaskParameters parameters)
		{
			_parameters = parameters;
		}

		/// <summary>
		/// ExecuteAsync the task.
		/// </summary>
		/// <param name="job">Information about the current job</param>
		/// <param name="buildProducts">Set of build products produced by this node.</param>
		/// <param name="tagNameToFileSet">Mapping from tag names to the set of files they include</param>
		public override async Task ExecuteAsync(JobContext job, HashSet<FileReference> buildProducts, Dictionary<string, HashSet<FileReference>> tagNameToFileSet)
		{
			await SpawnTaskBase.ExecuteAsync("aws", _parameters.Arguments, envVars: ParseEnvVars(_parameters.Environment, _parameters.EnvironmentFile), logOutput: _parameters.LogOutput);
		}

		/// <summary>
		/// Output this task out to an XML writer.
		/// </summary>
		public override void Write(XmlWriter writer)
		{
			Write(writer, _parameters);
		}

		/// <summary>
		/// Find all the tags which are used as inputs to this task
		/// </summary>
		/// <returns>The tag names which are read by this task</returns>
		public override IEnumerable<string> FindConsumedTagNames()
		{
			yield break;
		}

		/// <summary>
		/// Find all the tags which are modified by this task
		/// </summary>
		/// <returns>The tag names which are modified by this task</returns>
		public override IEnumerable<string> FindProducedTagNames()
		{
			yield break;
		}
	}
}
