// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Linq;
using System.Text.RegularExpressions;
using System.Threading.Tasks;
using System.Xml;
using EpicGames.Core;
using Microsoft.Extensions.Logging;
using UnrealBuildBase;

namespace AutomationTool.Tasks
{
	/// <summary>
	/// Parameters for a CheckMarkdown task
	/// </summary>
	public class CheckMarkdownTaskParameters
	{
		/// <summary>
		/// Optional filter to be applied to the list of input files.
		/// </summary>
		[TaskParameter(Optional = true, ValidationType = TaskParameterValidationType.FileSpec)]
		public string Files { get; set; }
	}

	/// <summary>
	/// Checks that all markdown links between the given files are valid.
	/// </summary>
	[TaskElement("CheckMarkdown", typeof(CheckMarkdownTaskParameters))]
	public class CheckMarkdownTask : BgTaskImpl
	{
		readonly CheckMarkdownTaskParameters _parameters;

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="parameters">Parameters for this task</param>
		public CheckMarkdownTask(CheckMarkdownTaskParameters parameters)
		{
			_parameters = parameters;
		}

		/// <inheritdoc/>
		public override async Task ExecuteAsync(JobContext job, HashSet<FileReference> buildProducts, Dictionary<string, HashSet<FileReference>> tagNameToFileSet)
		{
			HashSet<FileReference> files = ResolveFilespec(Unreal.RootDirectory, _parameters.Files, tagNameToFileSet);

			HashSet<FileReference> markdownFiles = files.Where(x => x.HasExtension(".md")).ToHashSet();
			Logger.LogInformation("Checking {NumFiles} markdown files...", markdownFiles.Count);

			// Build a set of valid link targets
			HashSet<string> validLinks = new HashSet<string>(StringComparer.Ordinal);
			foreach (FileReference file in files)
			{
				validLinks.Add(file.FullName);
			}

			// Find the anchors in any markdown files
			foreach (FileReference file in markdownFiles)
			{
				string[] lines = await FileReference.ReadAllLinesAsync(file);
				foreach (string line in lines)
				{
					Match match = Regex.Match(line, @"^#+\s+(.*)");
					if (match.Success)
					{
						string anchor = match.Groups[1].Value.ToLowerInvariant().Trim();
						anchor = Regex.Replace(anchor, @"\s+", "-");
						anchor = Regex.Replace(anchor, @"[^a-z0-9-]", "");
						validLinks.Add($"{file.FullName}#{anchor}");
					}
				}
			}

			// Add anchors for any images
			foreach (FileReference file in files)
			{
				if (file.HasExtension(".png") || file.HasExtension(".jpg") || file.HasExtension(".jpeg") || file.HasExtension(".gif"))
				{
					validLinks.Add($"{file.FullName}#gh-dark-mode-only");
					validLinks.Add($"{file.FullName}#gh-light-mode-only");
				}
			}

			// Check the links 
			foreach (FileReference file in markdownFiles)
			{
				string[] lines = await FileReference.ReadAllLinesAsync(file);
				for (int lineIdx = 0; lineIdx < lines.Length; lineIdx++)
				{
					string line = lines[lineIdx];
					foreach (Match match in Regex.Matches(line, @"\[([^\]]+)\]\(([^\)]*)\)"))
					{
						string link = match.Groups[2].Value;
						if (!Regex.IsMatch(link, "^(?:[a-z]+:/)?/"))
						{
							if (!link.Contains('\\', StringComparison.Ordinal))
							{
								int hashIdx = link.IndexOf('#', StringComparison.Ordinal);
								if (hashIdx == -1)
								{
									link = FileReference.Combine(file.Directory, link).FullName;
								}
								else if (hashIdx == 0)
								{
									link = $"{file}{link[hashIdx..]}";
								}
								else
								{
									link = $"{FileReference.Combine(file.Directory, link[0..hashIdx])}{link[hashIdx..]}";
								}
							}

							bool validLink;
							try
							{
								validLink = validLinks.Contains(link);
							}
							catch
							{
								validLink = false;
							}

							if (!validLink)
							{
								Logger.LogWarning("{File}({Line}): Invalid link '{Link}'", file, lineIdx + 1, match.Groups[0].Value);
							}
						}
					}
				}
			}
		}

		/// <inheritdoc/>
		public override void Write(XmlWriter writer)
			=> Write(writer, _parameters);

		/// <inheritdoc/>
		public override IEnumerable<string> FindConsumedTagNames()
			=> FindTagNamesFromFilespec(_parameters.Files);

		/// <inheritdoc/>
		public override IEnumerable<string> FindProducedTagNames()
			=> Array.Empty<string>();
	}
}
