// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Text;
using System.Threading.Tasks;
using System.Xml;
using EpicGames.BuildGraph;
using EpicGames.Core;
using Microsoft.Extensions.Logging;
using UnrealBuildBase;

namespace AutomationTool.Tasks
{
	static class StringExtensions
	{
		public static bool CaseInsensitiveContains(this string text, string value)
		{
			return System.Globalization.CultureInfo.InvariantCulture.CompareInfo.IndexOf(text, value, System.Globalization.CompareOptions.IgnoreCase) >= 0;
		}
	}

	/// <summary>
	/// Parameters for a task that calls another UAT command
	/// </summary>
	public class CommandTaskParameters
	{
		/// <summary>
		/// The command name to execute.
		/// </summary>
		[TaskParameter]
		public string Name { get; set; }

		/// <summary>
		/// Arguments to be passed to the command.
		/// </summary>
		[TaskParameter(Optional = true)]
		public string Arguments { get; set; }

		/// <summary>
		/// If non-null, instructs telemetry from the command to be merged into the telemetry for this UAT instance with the given prefix. May be an empty (non-null) string.
		/// </summary>
		[TaskParameter(Optional = true)]
		public string MergeTelemetryWithPrefix { get; set; }
	}

	/// <summary>
	/// Invokes an AutomationTool child process to run the given command.
	/// </summary>
	[TaskElement("Command", typeof(CommandTaskParameters))]
	public class CommandTask : BgTaskImpl
	{
		/// <summary>
		/// Parameters for this task
		/// </summary>
		readonly CommandTaskParameters _parameters;

		/// <summary>
		/// Construct a new CommandTask.
		/// </summary>
		/// <param name="parameters">Parameters for this task</param>
		public CommandTask(CommandTaskParameters parameters)
		{
			_parameters = parameters;
		}

		/// <summary>
		/// ExecuteAsync the task.
		/// </summary>
		/// <param name="job">Information about the current job</param>
		/// <param name="buildProducts">Set of build products produced by this node.</param>
		/// <param name="tagNameToFileSet">Mapping from tag names to the set of files they include</param>
		public override Task ExecuteAsync(JobContext job, HashSet<FileReference> buildProducts, Dictionary<string, HashSet<FileReference>> tagNameToFileSet)
		{
			// If we're merging telemetry from the child process, get a temp filename for it
			FileReference telemetryFile = null;
			if (_parameters.MergeTelemetryWithPrefix != null)
			{
				telemetryFile = FileReference.Combine(Unreal.RootDirectory, "Engine", "Intermediate", "UAT", "Telemetry.json");
				DirectoryReference.CreateDirectory(telemetryFile.Directory);
			}

			// Run the command
			StringBuilder commandLine = new StringBuilder();
			if (_parameters.Arguments == null || (!_parameters.Arguments.CaseInsensitiveContains("-p4") && !_parameters.Arguments.CaseInsensitiveContains("-nop4")))
			{
				commandLine.AppendFormat("{0} ", CommandUtils.P4Enabled ? "-p4" : "-nop4");
			}
			if (_parameters.Arguments == null || (!_parameters.Arguments.CaseInsensitiveContains("-submit") && !_parameters.Arguments.CaseInsensitiveContains("-nosubmit")))
			{
				if (GlobalCommandLine.Submit)
				{
					commandLine.Append("-submit ");
				}
				if (GlobalCommandLine.NoSubmit)
				{
					commandLine.Append("-nosubmit ");
				}
			}
			if (_parameters.Arguments == null || !_parameters.Arguments.CaseInsensitiveContains("-uselocalbuildstorage"))
			{
				if (GlobalCommandLine.UseLocalBuildStorage)
				{
					commandLine.Append("-uselocalbuildstorage ");
				}
			}

			commandLine.Append("-NoCompile ");
			commandLine.Append(_parameters.Name);
			if (!String.IsNullOrEmpty(_parameters.Arguments))
			{
				commandLine.AppendFormat(" {0}", _parameters.Arguments);
			}
			if (telemetryFile != null)
			{
				commandLine.AppendFormat(" -Telemetry={0}", CommandUtils.MakePathSafeToUseWithCommandLine(telemetryFile.FullName));
			}
			CommandUtils.RunUAT(CommandUtils.CmdEnv, commandLine.ToString(), Identifier: _parameters.Name);

			// Merge in any new telemetry data that was produced
			if (telemetryFile != null && FileReference.Exists(telemetryFile))
			{
				Logger.LogDebug("Merging telemetry from {TelemetryFile}", telemetryFile);

				TelemetryData newTelemetry;
				if (TelemetryData.TryRead(telemetryFile, out newTelemetry))
				{
					CommandUtils.Telemetry.Merge(_parameters.MergeTelemetryWithPrefix, newTelemetry);
				}
				else
				{
					Logger.LogWarning("Unable to read UAT telemetry file from {TelemetryFile}", telemetryFile);
				}
			}
			return Task.CompletedTask;
		}

		/// <summary>
		/// Gets the name of this trace
		/// </summary>
		/// <returns>Name of the trace</returns>
		public override string GetTraceName()
		{
			return String.Format("{0}.{1}", base.GetTraceName(), _parameters.Name.ToLowerInvariant());
		}

		/// <summary>
		/// Output this task out to an XML writer.
		/// </summary>
		public override void Write(XmlWriter writer)
		{
			Write(writer, _parameters);
		}

		/// <summary>
		/// Find all the tags which are used as inputs to this task
		/// </summary>
		/// <returns>The tag names which are read by this task</returns>
		public override IEnumerable<string> FindConsumedTagNames()
		{
			yield break;
		}

		/// <summary>
		/// Find all the tags which are modified by this task
		/// </summary>
		/// <returns>The tag names which are modified by this task</returns>
		public override IEnumerable<string> FindProducedTagNames()
		{
			yield break;
		}
	}

	public static partial class StandardTasks
	{
		/// <summary>
		/// Runs another UAT command
		/// </summary>
		/// <param name="state">The execution state</param>
		/// <param name="name">Name of the command to run</param>
		/// <param name="arguments">Arguments for the command</param>
		/// <param name="mergeTelemetryWithPrefix">If non-null, instructs telemetry from the command to be merged into the telemetry for this UAT instance with the given prefix. May be an empty (non-null) string.</param>
		public static async Task CommandAsync(this BgContext state, string name, string arguments = null, string mergeTelemetryWithPrefix = null)
		{
			_ = state;

			CommandTaskParameters parameters = new CommandTaskParameters();
			parameters.Name = name;
			parameters.Arguments = arguments ?? parameters.Arguments;
			parameters.MergeTelemetryWithPrefix = mergeTelemetryWithPrefix ?? parameters.MergeTelemetryWithPrefix;

			await ExecuteAsync(new CommandTask(parameters));
		}

		/// <summary>
		/// Runs another UAT command
		/// </summary>
		/// <param name="name">Name of the command to run</param>
		/// <param name="arguments">Arguments for the command</param>
		/// <param name="mergeTelemetryWithPrefix">If non-null, instructs telemetry from the command to be merged into the telemetry for this UAT instance with the given prefix. May be an empty (non-null) string.</param>
		public static async Task CommandAsync(string name, string arguments = null, string mergeTelemetryWithPrefix = null)
		{
			CommandTaskParameters parameters = new CommandTaskParameters();
			parameters.Name = name;
			parameters.Arguments = arguments ?? parameters.Arguments;
			parameters.MergeTelemetryWithPrefix = mergeTelemetryWithPrefix ?? parameters.MergeTelemetryWithPrefix;

			await ExecuteAsync(new CommandTask(parameters));
		}
	}
}
