// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Linq;
using System.Threading.Tasks;
using System.Xml;
using EpicGames.BuildGraph;
using EpicGames.Core;
using UnrealBuildTool;

namespace AutomationTool.Tasks
{
	/// <summary>
	/// Parameters for a task which runs a UE commandlet
	/// </summary>
	public class CommandletTaskParameters
	{
		/// <summary>
		/// The commandlet name to execute.
		/// </summary>
		[TaskParameter]
		public string Name { get; set; }

		/// <summary>
		/// The project to run the editor with.
		/// </summary>
		[TaskParameter(Optional = true, ValidationType = TaskParameterValidationType.FileSpec)]
		public string Project { get; set; }

		/// <summary>
		/// Arguments to be passed to the commandlet.
		/// </summary>
		[TaskParameter(Optional = true)]
		public string Arguments { get; set; }

		/// <summary>
		/// The editor executable to use. Defaults to the development UnrealEditor executable for the current platform.
		/// </summary>
		[TaskParameter(Optional = true)]
		public FileReference EditorExe { get; set; }

		/// <summary>
		/// The minimum exit code, which is treated as an error.
		/// </summary>
		[TaskParameter(Optional = true)]
		public int ErrorLevel { get; set; } = 1;
	}

	/// <summary>
	/// Spawns the editor to run a commandlet.
	/// </summary>
	[TaskElement("Commandlet", typeof(CommandletTaskParameters))]
	public class CommandletTask : BgTaskImpl
	{
		readonly CommandletTaskParameters _parameters;

		/// <summary>
		/// Construct a new CommandletTask.
		/// </summary>
		/// <param name="parameters">Parameters for this task</param>
		public CommandletTask(CommandletTaskParameters parameters)
		{
			_parameters = parameters;
		}

		/// <summary>
		/// ExecuteAsync the task.
		/// </summary>
		/// <param name="job">Information about the current job</param>
		/// <param name="buildProducts">Set of build products produced by this node.</param>
		/// <param name="tagNameToFileSet">Mapping from tag names to the set of files they include</param>
		public override Task ExecuteAsync(JobContext job, HashSet<FileReference> buildProducts, Dictionary<string, HashSet<FileReference>> tagNameToFileSet)
		{
			// Get the full path to the project file
			FileReference projectFile = null;
			if (!String.IsNullOrEmpty(_parameters.Project))
			{
				if (_parameters.Project.EndsWith(".uproject", StringComparison.OrdinalIgnoreCase))
				{
					projectFile = ResolveFile(_parameters.Project);
				}
				else
				{
					projectFile = NativeProjects.EnumerateProjectFiles(Log.Logger).FirstOrDefault(x => x.GetFileNameWithoutExtension().Equals(_parameters.Project, StringComparison.OrdinalIgnoreCase));
				}

				if (projectFile == null || !FileReference.Exists(projectFile))
				{
					throw new BuildException("Unable to resolve project '{0}'", _parameters.Project);
				}
			}

			// Get the path to the editor, and check it exists
			FileSystemReference editorExe = _parameters.EditorExe;
			if (editorExe == null)
			{
				editorExe = ProjectUtils.GetEditorForProject(projectFile);
			}

			// Run the commandlet
			CommandUtils.RunCommandlet(projectFile, editorExe.FullName, _parameters.Name, _parameters.Arguments, _parameters.ErrorLevel);
			return Task.CompletedTask;
		}

		/// <summary>
		/// Output this task out to an XML writer.
		/// </summary>
		public override void Write(XmlWriter writer)
		{
			Write(writer, _parameters);
		}

		/// <summary>
		/// Find all the tags which are used as inputs to this task
		/// </summary>
		/// <returns>The tag names which are read by this task</returns>
		public override IEnumerable<string> FindConsumedTagNames()
		{
			yield break;
		}

		/// <summary>
		/// Find all the tags which are modified by this task
		/// </summary>
		/// <returns>The tag names which are modified by this task</returns>
		public override IEnumerable<string> FindProducedTagNames()
		{
			yield break;
		}
	}

	/// <summary>
	/// Task wrapper methods
	/// </summary>
	public static partial class StandardTasks
	{
		/// <summary>
		/// Task which runs a UE commandlet
		/// </summary>
		/// <param name="state"></param>
		/// <param name="name">The commandlet name to execute.</param>
		/// <param name="project">The project to run the editor with.</param>
		/// <param name="arguments">Arguments to be passed to the commandlet.</param>
		/// <param name="editorExe">The editor executable to use. Defaults to the development UnrealEditor executable for the current platform.</param>
		/// <param name="errorLevel">The minimum exit code, which is treated as an error.</param>
		public static async Task CommandletAsync(this BgContext state, string name, FileReference project = null, string arguments = null, FileReference editorExe = null, int errorLevel = 1)
		{
			_ = state;

			CommandletTaskParameters parameters = new CommandletTaskParameters();
			parameters.Name = name;
			parameters.Project = project?.FullName;
			parameters.Arguments = arguments;
			parameters.EditorExe = editorExe;
			parameters.ErrorLevel = errorLevel;
			await ExecuteAsync(new CommandletTask(parameters));
		}
	}
}
