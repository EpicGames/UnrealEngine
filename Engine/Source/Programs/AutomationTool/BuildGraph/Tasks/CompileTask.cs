// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Linq;
using System.Threading.Tasks;
using System.Xml;
using EpicGames.Core;
using OpenTracing;
using UnrealBuildTool;

namespace AutomationTool.Tasks
{
	/// <summary>
	/// Parameters for a compile task
	/// </summary>
	public class CompileTaskParameters
	{
		/// <summary>
		/// The target to compile.
		/// </summary>
		[TaskParameter(Optional = true)]
		public string Target { get; set; }

		/// <summary>
		/// The configuration to compile.
		/// </summary>
		[TaskParameter]
		public UnrealTargetConfiguration Configuration { get; set; }

		/// <summary>
		/// The platform to compile for.
		/// </summary>
		[TaskParameter]
		public UnrealTargetPlatform Platform { get; set; }

		/// <summary>
		/// The project to compile with.
		/// </summary>
		[TaskParameter(Optional = true, ValidationType = TaskParameterValidationType.FileSpec)]
		public string Project { get; set; }

		/// <summary>
		/// Additional arguments for UnrealBuildTool.
		/// </summary>
		[TaskParameter(Optional = true)]
		public string Arguments { get; set; }

		/// <summary>
		/// Whether to allow using XGE for compilation.
		/// </summary>
		[TaskParameter(Optional = true)]
		public bool AllowXGE { get; set; } = true;

		/// <summary>
		/// No longer necessary as UnrealBuildTool is run to compile targets.
		/// </summary>
		[TaskParameter(Optional = true)]
		[Obsolete("This setting is no longer used")]
		public bool AllowParallelExecutor { get; set; } = true;

		/// <summary>
		/// Whether to allow UBT to use all available cores, when AllowXGE is disabled.
		/// </summary>
		[TaskParameter(Optional = true)]
		public bool AllowAllCores { get; set; } = false;

		/// <summary>
		/// Whether to allow cleaning this target. If unspecified, targets are cleaned if the -Clean argument is passed on the command line.
		/// </summary>
		[TaskParameter(Optional = true)]
		public bool? Clean { get; set; } = null;

		/// <summary>
		/// Global flag passed to UBT that can be used to generate target files without fully compiling.
		/// </summary>
		[TaskParameter(Optional = true)]
		public bool SkipBuild { get; set; } = false;

		/// <summary>
		/// Tag to be applied to build products of this task.
		/// </summary>
		[TaskParameter(Optional = true, ValidationType = TaskParameterValidationType.TagList)]
		public string Tag { get; set; }
	}

	/// <summary>
	/// Executor for compile tasks, which can compile multiple tasks simultaneously
	/// </summary>
	public class CompileTaskExecutor : ITaskExecutor
	{
		/// <summary>
		/// List of targets to compile. As well as the target specifically added for this task, additional compile tasks may be merged with it.
		/// </summary>
		readonly List<UnrealBuild.BuildTarget> _targets = new List<UnrealBuild.BuildTarget>();

		/// <summary>
		/// Mapping of receipt filename to its corresponding tag name
		/// </summary>
		readonly Dictionary<UnrealBuild.BuildTarget, string> _targetToTagName = new Dictionary<UnrealBuild.BuildTarget, string>();

		/// <summary>
		/// Whether to allow using XGE for this job
		/// </summary>
		bool _allowXge = true;

		/// <summary>
		/// Whether to allow using all available cores for this job, when bAllowXGE is false
		/// </summary>
		bool _allowAllCores = false;

		/// <summary>
		/// Should SkipBuild be passed to UBT so that only .target files are generated.
		/// </summary>
		bool _skipBuild = false;

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="task">Initial task to execute</param>
		public CompileTaskExecutor(CompileTask task)
		{
			Add(task);
		}

		/// <summary>
		/// Adds another task to this executor
		/// </summary>
		/// <param name="task">Task to add</param>
		/// <returns>True if the task could be added, false otherwise</returns>
		public bool Add(BgTaskImpl task)
		{
			CompileTask compileTask = task as CompileTask;
			if (compileTask == null)
			{
				return false;
			}

			CompileTaskParameters parameters = compileTask.Parameters;
			if (_targets.Count > 0)
			{
				if (_allowXge != parameters.AllowXGE
					|| _skipBuild != parameters.SkipBuild)
				{
					return false;
				}
			}
			else
			{
				_allowXge = parameters.AllowXGE;
				_allowAllCores = parameters.AllowAllCores;
				_skipBuild = parameters.SkipBuild;
			}

			_allowXge &= parameters.AllowXGE;
			_allowAllCores &= parameters.AllowAllCores;

			UnrealBuild.BuildTarget target = new UnrealBuild.BuildTarget { TargetName = parameters.Target, Platform = parameters.Platform, Config = parameters.Configuration, UprojectPath = compileTask.FindProjectFile(), UBTArgs = (parameters.Arguments ?? ""), Clean = parameters.Clean };
			if (!String.IsNullOrEmpty(parameters.Tag))
			{
				_targetToTagName.Add(target, parameters.Tag);
			}
			_targets.Add(target);

			return true;
		}

		/// <summary>
		/// ExecuteAsync all the tasks added to this executor.
		/// </summary>
		/// <param name="job">Information about the current job</param>
		/// <param name="buildProducts">Set of build products produced by this node.</param>
		/// <param name="tagNameToFileSet">Mapping from tag names to the set of files they include</param>
		/// <returns>Whether the task succeeded or not. Exiting with an exception will be caught and treated as a failure.</returns>
		public Task ExecuteAsync(JobContext job, HashSet<FileReference> buildProducts, Dictionary<string, HashSet<FileReference>> tagNameToFileSet)
		{
			// Create the agenda
			UnrealBuild.BuildAgenda agenda = new UnrealBuild.BuildAgenda();
			agenda.Targets.AddRange(_targets);

			// Build everything
			Dictionary<UnrealBuild.BuildTarget, BuildManifest> targetToManifest = new Dictionary<UnrealBuild.BuildTarget, BuildManifest>();
			UnrealBuild builder = new UnrealBuild(job.OwnerCommand);

			bool allCores = (CommandUtils.IsBuildMachine || _allowAllCores);   // Enable using all cores if this is a build agent or the flag was passed in to the task and XGE is disabled.
			builder.Build(agenda, InDeleteBuildProducts: null, InUpdateVersionFiles: false, InForceNoXGE: !_allowXge, InAllCores: allCores, InTargetToManifest: targetToManifest, InSkipBuild: _skipBuild);

			UnrealBuild.CheckBuildProducts(builder.BuildProductFiles);

			// Tag all the outputs
			foreach (KeyValuePair<UnrealBuild.BuildTarget, string> targetTagName in _targetToTagName)
			{
				BuildManifest manifest;
				if (!targetToManifest.TryGetValue(targetTagName.Key, out manifest))
				{
					throw new AutomationException("Missing manifest for target {0} {1} {2}", targetTagName.Key.TargetName, targetTagName.Key.Platform, targetTagName.Key.Config);
				}

				HashSet<FileReference> manifestBuildProducts = manifest.BuildProducts.Select(x => new FileReference(x)).ToHashSet();

				// when we make a Mac/IOS build, Xcode will finalize the .app directory, adding files that UBT has no idea about, so now we recursively add any files in the .app
				// as BuildProducts. look for any .apps that we have any files as BuildProducts, and expand to include all files in the .app
				if (BuildHostPlatform.Current.Platform == UnrealTargetPlatform.Mac)
				{
					HashSet<string> appBundleLocations = new();
					foreach (FileReference file in manifestBuildProducts)
					{
						// look for a ".app/" portion and chop off anything after it
						int appLocation = file.FullName.IndexOf(".app/", StringComparison.InvariantCultureIgnoreCase);
						if (appLocation > 0)
						{
							appBundleLocations.Add(file.FullName.Substring(0, appLocation + 4));
						}
					}

					// now with a unique set of app bundles, add all files in them
					foreach (string appBundleLocation in appBundleLocations)
					{
						manifestBuildProducts.UnionWith(DirectoryReference.EnumerateFiles(new DirectoryReference(appBundleLocation), "*", System.IO.SearchOption.AllDirectories));
					}
				}

				foreach (string tagName in CustomTask.SplitDelimitedList(targetTagName.Value))
				{
					HashSet<FileReference> fileSet = CustomTask.FindOrAddTagSet(tagNameToFileSet, tagName);
					fileSet.UnionWith(manifestBuildProducts);
				}
			}

			// Add everything to the list of build products
			buildProducts.UnionWith(builder.BuildProductFiles.Select(x => new FileReference(x)));
			return Task.CompletedTask;
		}
	}

	/// <summary>
	/// Compiles a target with UnrealBuildTool.
	/// </summary>
	[TaskElement("Compile", typeof(CompileTaskParameters))]
	public class CompileTask : BgTaskImpl
	{
		/// <summary>
		/// Parameters for this task
		/// </summary>
		public CompileTaskParameters Parameters { get; set; }

		/// <summary>
		/// Resolved path to Project file
		/// </summary>
		public FileReference ProjectFile { get; set; } = null;

		/// <summary>
		/// Construct a compile task
		/// </summary>
		/// <param name="parameters">Parameters for this task</param>
		public CompileTask(CompileTaskParameters parameters)
		{
			Parameters = parameters;
		}

		/// <summary>
		/// Resolve the path to the project file
		/// </summary>
		public FileReference FindProjectFile()
		{
			FileReference projectFile = null;

			// Resolve the full path to the project file
			if (!String.IsNullOrEmpty(Parameters.Project))
			{
				if (Parameters.Project.EndsWith(".uproject", StringComparison.OrdinalIgnoreCase))
				{
					projectFile = CustomTask.ResolveFile(Parameters.Project);
				}
				else
				{
					projectFile = NativeProjects.EnumerateProjectFiles(Log.Logger).FirstOrDefault(x => x.GetFileNameWithoutExtension().Equals(Parameters.Project, StringComparison.OrdinalIgnoreCase));
				}

				if (projectFile == null || !FileReference.Exists(projectFile))
				{
					throw new BuildException("Unable to resolve project '{0}'", Parameters.Project);
				}
			}

			return projectFile;
		}

		/// <summary>
		/// ExecuteAsync the task.
		/// </summary>
		/// <param name="job">Information about the current job</param>
		/// <param name="buildProducts">Set of build products produced by this node.</param>
		/// <param name="tagNameToFileSet">Mapping from tag names to the set of files they include</param>
		public override Task ExecuteAsync(JobContext job, HashSet<FileReference> buildProducts, Dictionary<string, HashSet<FileReference>> tagNameToFileSet)
		{
			//
			// Don't do any logic here. You have to do it in the ctor or a getter
			//  otherwise you break the ITaskExecutor pathway, which doesn't call this function!
			//
			return GetExecutor().ExecuteAsync(job, buildProducts, tagNameToFileSet);
		}

		/// <summary>
		/// 
		/// </summary>
		/// <returns></returns>
		public override ITaskExecutor GetExecutor()
		{
			return new CompileTaskExecutor(this);
		}

		/// <summary>
		/// Get properties to include in tracing info
		/// </summary>
		/// <param name="span">The span to add metadata to</param>
		/// <param name="prefix">Prefix for all metadata keys</param>
		public override void GetTraceMetadata(ITraceSpan span, string prefix)
		{
			base.GetTraceMetadata(span, prefix);

			span.AddMetadata(prefix + "target.name", Parameters.Target);
			span.AddMetadata(prefix + "target.config", Parameters.Configuration.ToString());
			span.AddMetadata(prefix + "target.platform", Parameters.Platform.ToString());

			if (Parameters.Project != null)
			{
				span.AddMetadata(prefix + "target.project", Parameters.Project);
			}
		}

		/// <summary>
		/// Get properties to include in tracing info
		/// </summary>
		/// <param name="span">The span to add metadata to</param>
		/// <param name="prefix">Prefix for all metadata keys</param>
		public override void GetTraceMetadata(ISpan span, string prefix)
		{
			base.GetTraceMetadata(span, prefix);

			span.SetTag(prefix + "target.name", Parameters.Target);
			span.SetTag(prefix + "target.config", Parameters.Configuration.ToString());
			span.SetTag(prefix + "target.platform", Parameters.Platform.ToString());

			if (Parameters.Project != null)
			{
				span.SetTag(prefix + "target.project", Parameters.Project);
			}
		}

		/// <summary>
		/// Output this task out to an XML writer.
		/// </summary>
		public override void Write(XmlWriter writer)
		{
			Write(writer, Parameters);
		}

		/// <summary>
		/// Find all the tags which are used as inputs to this task
		/// </summary>
		/// <returns>The tag names which are read by this task</returns>
		public override IEnumerable<string> FindConsumedTagNames()
		{
			yield break;
		}

		/// <summary>
		/// Find all the tags which are modified by this task
		/// </summary>
		/// <returns>The tag names which are modified by this task</returns>
		public override IEnumerable<string> FindProducedTagNames()
		{
			return FindTagNamesFromList(Parameters.Tag);
		}
	}

	public static partial class StandardTasks
	{
		/// <summary>
		/// Compiles a target
		/// </summary>
		/// <param name="target">The target to compile</param>
		/// <param name="configuration">The configuration to compile</param>
		/// <param name="platform">The platform to compile for</param>
		/// <param name="project">The project to compile with</param>
		/// <param name="arguments">Additional arguments for UnrealBuildTool</param>
		/// <param name="allowXge">Whether to allow using XGE for compilation</param>
		/// <param name="clean">Whether to allow cleaning this target. If unspecified, targets are cleaned if the -Clean argument is passed on the command line</param>
		/// <returns>Build products from the compile</returns>
		public static async Task<FileSet> CompileAsync(string target, UnrealTargetPlatform platform, UnrealTargetConfiguration configuration, FileReference project = null, string arguments = null, bool allowXge = true, bool? clean = null)
		{
			CompileTaskParameters parameters = new CompileTaskParameters();
			parameters.Target = target;
			parameters.Platform = platform;
			parameters.Configuration = configuration;
			parameters.Project = project?.FullName;
			parameters.Arguments = arguments;
			parameters.AllowXGE = allowXge;
			parameters.Clean = clean;
			return await ExecuteAsync(new CompileTask(parameters));
		}
	}
}
