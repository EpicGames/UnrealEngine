// Copyright Epic Games, Inc. All Rights Reserved.

using System.Collections.Generic;
using System.Linq;
using System.Threading.Tasks;
using System.Xml;
using EpicGames.Core;
using OpenTracing;
using OpenTracing.Util;

namespace AutomationTool.Tasks
{
	/// <summary>
	/// Parameters for a task that runs the cooker
	/// </summary>
	public class CookTaskParameters
	{
		/// <summary>
		/// Project file to be cooked.
		/// </summary>
		[TaskParameter]
		public string Project { get; set; }

		/// <summary>
		/// The cook platform to target (for example, Windows).
		/// </summary>
		[TaskParameter]
		public string Platform { get; set; }

		/// <summary>
		/// List of maps to be cooked, separated by '+' characters.
		/// </summary>
		[TaskParameter(Optional = true)]
		public string Maps { get; set; }

		/// <summary>
		/// Additional arguments to be passed to the cooker.
		/// </summary>
		[TaskParameter(Optional = true)]
		public bool Versioned { get; set; } = false;

		/// <summary>
		/// Additional arguments to be passed to the cooker.
		/// </summary>
		[TaskParameter(Optional = true)]
		public string Arguments { get; set; } = "";

		/// <summary>
		/// Optional path to what editor executable to run for cooking.
		/// </summary>
		[TaskParameter(Optional = true)]
		public string EditorExe { get; set; } = "";

		/// <summary>
		/// Whether to tag the output from the cook. Since cooks produce a lot of files, it can be detrimental to spend time tagging them if we don't need them in a dependent node.
		/// </summary>
		[TaskParameter(Optional = true)]
		public bool TagOutput { get; set; } = true;

		/// <summary>
		/// Tag to be applied to build products of this task.
		/// </summary>
		[TaskParameter(Optional = true, ValidationType = TaskParameterValidationType.TagList)]
		public string Tag { get; set; }
	}

	/// <summary>
	/// Cook a selection of maps for a certain platform
	/// </summary>
	[TaskElement("Cook", typeof(CookTaskParameters))]
	public class CookTask : BgTaskImpl
	{
		/// <summary>
		/// Parameters for the task
		/// </summary>
		readonly CookTaskParameters _parameters;

		/// <summary>
		/// Constructor.
		/// </summary>
		/// <param name="parameters">Parameters for this task</param>
		public CookTask(CookTaskParameters parameters)
		{
			_parameters = parameters;
		}

		/// <summary>
		/// ExecuteAsync the task.
		/// </summary>
		/// <param name="job">Information about the current job</param>
		/// <param name="buildProducts">Set of build products produced by this node.</param>
		/// <param name="tagNameToFileSet">Mapping from tag names to the set of files they include</param>
		public override Task ExecuteAsync(JobContext job, HashSet<FileReference> buildProducts, Dictionary<string, HashSet<FileReference>> tagNameToFileSet)
		{
			// Figure out the project that this target belongs to
			FileReference projectFile = null;
			if (_parameters.Project != null)
			{
				projectFile = new FileReference(_parameters.Project);
				if (!FileReference.Exists(projectFile))
				{
					throw new AutomationException("Missing project file - {0}", projectFile.FullName);
				}
			}

			// ExecuteAsync the cooker
			using (IScope scope = GlobalTracer.Instance.BuildSpan("Cook").StartActive())
			{
				scope.Span.SetTag("project", projectFile == null ? "UE4" : projectFile.GetFileNameWithoutExtension());
				scope.Span.SetTag("platform", _parameters.Platform);
				string[] maps = (_parameters.Maps == null) ? null : _parameters.Maps.Split(new char[] { '+' });
				string arguments = (_parameters.Versioned ? "" : "-Unversioned ") + _parameters.Arguments;
				string editorExe = (System.String.IsNullOrWhiteSpace(_parameters.EditorExe) ? ProjectUtils.GetEditorForProject(projectFile).FullName : _parameters.EditorExe);
				CommandUtils.CookCommandlet(projectFile, editorExe, maps, null, null, null, _parameters.Platform, arguments);
			}

			// Find all the cooked files
			List<FileReference> cookedFiles = new List<FileReference>();
			if (_parameters.TagOutput)
			{
				foreach (string platform in _parameters.Platform.Split('+'))
				{
					DirectoryReference platformCookedDirectory = DirectoryReference.Combine(projectFile.Directory, "Saved", "Cooked", platform);
					if (!DirectoryReference.Exists(platformCookedDirectory))
					{
						throw new AutomationException("Cook output directory not found ({0})", platformCookedDirectory.FullName);
					}
					List<FileReference> platformCookedFiles = DirectoryReference.EnumerateFiles(platformCookedDirectory, "*", System.IO.SearchOption.AllDirectories).ToList();
					if (platformCookedFiles.Count == 0)
					{
						throw new AutomationException("Cooking did not produce any files in {0}", platformCookedDirectory.FullName);
					}
					cookedFiles.AddRange(platformCookedFiles);
				}
			}

			// Apply the optional tag to the build products
			foreach (string tagName in FindTagNamesFromList(_parameters.Tag))
			{
				FindOrAddTagSet(tagNameToFileSet, tagName).UnionWith(cookedFiles);
			}

			// Add them to the set of build products
			buildProducts.UnionWith(cookedFiles);
			return Task.CompletedTask;
		}

		/// <summary>
		/// Output this task out to an XML writer.
		/// </summary>
		public override void Write(XmlWriter writer)
		{
			Write(writer, _parameters);
		}

		/// <summary>
		/// Find all the tags which are used as inputs to this task
		/// </summary>
		/// <returns>The tag names which are read by this task</returns>
		public override IEnumerable<string> FindConsumedTagNames()
		{
			yield break;
		}

		/// <summary>
		/// Find all the tags which are modified by this task
		/// </summary>
		/// <returns>The tag names which are modified by this task</returns>
		public override IEnumerable<string> FindProducedTagNames()
		{
			return FindTagNamesFromList(_parameters.Tag);
		}
	}
}