// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Linq;
using System.Runtime.InteropServices;
using System.Text;
using System.Threading.Tasks;
using System.Xml;
using EpicGames.Core;
using UnrealBuildBase;
using UnrealBuildTool;

namespace AutomationTool.Tasks
{
	/// <summary>
	/// Copy derived data from one source to one or more targets using a replay file as reference for what gets copied
	/// </summary>
	public class CopyDDCTaskParameters
	{
		/// <summary>
		/// The project to run the tool with.
		/// </summary>
		[TaskParameter(Optional = true, ValidationType = TaskParameterValidationType.FileSpec)]
		public string Project { get; set; }

		/// <summary>
		/// The DDC replay file or tag set to use as a reference for what is to be copied.
		/// </summary>
		[TaskParameter]
		public string Replay { get; set; }

		/// <summary>
		/// The DDC graph name or config to use as the source for the data to copy
		/// </summary>
		[TaskParameter]
		public string Source { get; set; }

		/// <summary>
		/// An array of the the DDC graph names or configs to use as the target for the data to copy, separated with plus (+) symbol
		/// </summary>
		[TaskParameter(Optional = true)]
		public string Targets { get; set; }

		/// <summary>
		/// Optional. Whether to preview the operation without doing any writing of data
		/// </summary>
		[TaskParameter(Optional = true)]
		public bool Preview { get; set; } = false;

		/// <summary>
		/// Optional. Whether to allow overwriting of data in the target(s)
		/// </summary>
		[TaskParameter(Optional = true)]
		public bool Overwrite { get; set; } = false;

		/// <summary>
		/// Optional. Whether to force overwriting of data in the target(s) even when it is there and seems identical
		/// </summary>
		[TaskParameter(Optional = true)]
		public bool ForceOverwrite { get; set; } = false;

		/// <summary>
		/// Arguments for the newly created process.
		/// </summary>
		[TaskParameter(Optional = true)]
		public string Arguments { get; set; }

		/// <summary>
		/// Working directory for spawning the copy task
		/// </summary>
		[TaskParameter(Optional = true)]
		public string WorkingDir { get; set; }

		/// <summary>
		/// Environment variables to set
		/// </summary>
		[TaskParameter(Optional = true)]
		public string Environment { get; set; }

		/// <summary>
		/// File to read environment from
		/// </summary>
		[TaskParameter(Optional = true)]
		public string EnvironmentFile { get; set; }

		/// <summary>
		/// Write output to the log
		/// </summary>
		[TaskParameter(Optional = true)]
		public bool LogOutput { get; set; } = true;

		/// <summary>
		/// The minimum exit code, which is treated as an error.
		/// </summary>
		[TaskParameter(Optional = true)]
		public int ErrorLevel { get; set; } = 1;
	}

	/// <summary>
	/// Spawns an external executable to copy derived data listed in a replay file and waits for it to complete.
	/// </summary>
	[TaskElement("CopyDDC", typeof(CopyDDCTaskParameters))]
	public class CopyDDCTask : SpawnTaskBase
	{
		/// <summary>
		/// Parameters for this task
		/// </summary>
		readonly CopyDDCTaskParameters _parameters;

		/// <summary>
		/// Construct a spawn task
		/// </summary>
		/// <param name="parameters">Parameters for the task</param>
		public CopyDDCTask(CopyDDCTaskParameters parameters)
		{
			_parameters = parameters;
		}

		/// <summary>
		/// ExecuteAsync the task.
		/// </summary>
		/// <param name="job">Information about the current job</param>
		/// <param name="buildProducts">Set of build products produced by this node.</param>
		/// <param name="tagNameToFileSet">Mapping from tag names to the set of files they include</param>
		public override async Task ExecuteAsync(JobContext job, HashSet<FileReference> buildProducts, Dictionary<string, HashSet<FileReference>> tagNameToFileSet)
		{
			FileReference projectFile = null;
			if (!String.IsNullOrEmpty(_parameters.Project))
			{
				if (_parameters.Project.EndsWith(".uproject", StringComparison.OrdinalIgnoreCase))
				{
					projectFile = ResolveFile(_parameters.Project);
				}
				else
				{
					projectFile = NativeProjects.EnumerateProjectFiles(Log.Logger).FirstOrDefault(x => x.GetFileNameWithoutExtension().Equals(_parameters.Project, StringComparison.OrdinalIgnoreCase));
				}

				if (projectFile == null || !FileReference.Exists(projectFile))
				{
					throw new BuildException("Unable to resolve project '{0}'", _parameters.Project);
				}
			}

			if (String.IsNullOrEmpty(_parameters.Source))
			{
				throw new BuildException("Missing or empty Source '{0}'", _parameters.Source);
			}

			if (String.IsNullOrEmpty(_parameters.Replay))
			{
				throw new BuildException("Missing or empty Replay '{0}'", _parameters.Replay);
			}

			// Get the DerivedDataTool executable path
			FileReference derivedDataToolExe = ResolveFile(String.Format("Engine/Binaries/{0}/DerivedDataTool{1}", HostPlatform.Current.HostEditorPlatform.ToString(), RuntimeInformation.IsOSPlatform(OSPlatform.Windows) ? ".exe" : ""));

			if (!FileReference.Exists(derivedDataToolExe))
			{
				throw new BuildException("Unable to resolve DerivedDataTool '{0}' (has it been built?)", derivedDataToolExe);
			}

			StringBuilder args = new StringBuilder();
			args.AppendFormat("{0}Copy -Source=\"{1}\"", projectFile != null ? $"\"{projectFile.FullName}\" " : "", _parameters.Source);

			foreach (string target in _parameters.Targets.Split('+'))
			{
				if (!String.IsNullOrEmpty(target))
				{
					args.Append($" -Target=\"{target}\"");
				}
			}

			if (_parameters.Replay.StartsWith("#", StringComparison.OrdinalIgnoreCase))
			{
				HashSet<FileReference> files = ResolveFilespec(Unreal.RootDirectory, _parameters.Replay, tagNameToFileSet);
				foreach (FileReference file in files)
				{
					args.Append($" -Replay=\"{file}\"");
				}
			}
			else
			{
				args.Append($"{args} -Replay=\"{_parameters.Replay}\"");
			}

			if (_parameters.Preview)
			{
				args.Append(" -Preview");
			}

			if (_parameters.Overwrite)
			{
				args.Append(" -Overwrite");
			}

			if (_parameters.ForceOverwrite)
			{
				args.Append(" -ForceOverwrite");
			}

			args.AppendFormat(" {0}", _parameters.Arguments);
			if (CommandUtils.IsBuildMachine)
			{
				args.Append(" -buildmachine");
			}

			await ExecuteAsync(derivedDataToolExe.FullName, args.ToString(), _parameters.WorkingDir, envVars: ParseEnvVars(_parameters.Environment, _parameters.EnvironmentFile), logOutput: _parameters.LogOutput, errorLevel: _parameters.ErrorLevel);
		}

		/// <summary>
		/// Output this task out to an XML writer.
		/// </summary>
		public override void Write(XmlWriter writer)
		{
			Write(writer, _parameters);
		}

		/// <summary>
		/// Find all the tags which are used as inputs to this task
		/// </summary>
		/// <returns>The tag names which are read by this task</returns>
		public override IEnumerable<string> FindConsumedTagNames()
		{
			yield break;
		}

		/// <summary>
		/// Find all the tags which are modified by this task
		/// </summary>
		/// <returns>The tag names which are modified by this task</returns>
		public override IEnumerable<string> FindProducedTagNames()
		{
			yield break;
		}
	}
}
