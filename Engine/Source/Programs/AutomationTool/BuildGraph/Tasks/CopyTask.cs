// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Linq;
using System.Threading.Tasks;
using System.Xml;
using EpicGames.Core;
using Microsoft.Extensions.Logging;
using UnrealBuildBase;

namespace AutomationTool.Tasks
{
	/// <summary>
	/// Parameters for a copy task
	/// </summary>
	public class CopyTaskParameters
	{
		/// <summary>
		/// Optional filter to be applied to the list of input files.
		/// </summary>
		[TaskParameter(Optional = true, ValidationType = TaskParameterValidationType.FileSpec)]
		public string Files { get; set; }

		/// <summary>
		/// The pattern(s) to copy from (for example, Engine/*.txt).
		/// </summary>
		[TaskParameter(ValidationType = TaskParameterValidationType.FileSpec)]
		public string From { get; set; }

		/// <summary>
		/// The directory to copy to.
		/// </summary>
		[TaskParameter(ValidationType = TaskParameterValidationType.FileSpec)]
		public string To { get; set; }

		/// <summary>
		/// Optional pattern(s) to exclude from the copy for example, Engine/NoCopy*.txt)
		/// </summary>
		[TaskParameter(Optional = true, ValidationType = TaskParameterValidationType.FileSpec)]
		public string Exclude { get; set; }

		/// <summary>
		/// Whether or not to overwrite existing files.
		/// </summary>
		[TaskParameter(Optional = true)]
		public bool Overwrite { get; set; } = true;

		/// <summary>
		/// Tag to be applied to build products of this task.
		/// </summary>
		[TaskParameter(Optional = true, ValidationType = TaskParameterValidationType.TagList)]
		public string Tag { get; set; }

		/// <summary>
		/// Whether or not to throw an error if no files were found to copy
		/// </summary>
		[TaskParameter(Optional = true)]
		public bool ErrorIfNotFound { get; set; } = false;

		/// <summary>
		/// Whether copied files should be tagged in the default set.
		/// </summary>
		[TaskParameter(Optional = true)]
		public bool TagInDefaultSet { get; set; } = true;
	}

	/// <summary>
	/// Copies files from one directory to another.
	/// </summary>
	[TaskElement("Copy", typeof(CopyTaskParameters))]
	public class CopyTask : BgTaskImpl
	{
		readonly CopyTaskParameters _parameters;

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="parameters">Parameters for this task</param>
		public CopyTask(CopyTaskParameters parameters)
		{
			_parameters = parameters;
		}

		/// <summary>
		/// ExecuteAsync the task.
		/// </summary>
		/// <param name="job">Information about the current job</param>
		/// <param name="buildProducts">Set of build products produced by this node.</param>
		/// <param name="tagNameToFileSet">Mapping from tag names to the set of files they include</param>
		public override async Task ExecuteAsync(JobContext job, HashSet<FileReference> buildProducts, Dictionary<string, HashSet<FileReference>> tagNameToFileSet)
		{
			// Parse all the source patterns
			FilePattern sourcePattern = new FilePattern(Unreal.RootDirectory, _parameters.From);

			// Parse the target pattern
			FilePattern targetPattern = new FilePattern(Unreal.RootDirectory, _parameters.To);

			// Apply the filter to the source files
			HashSet<FileReference> files = null;
			if (!String.IsNullOrEmpty(_parameters.Files))
			{
				sourcePattern = sourcePattern.AsDirectoryPattern();
				files = ResolveFilespec(sourcePattern.BaseDirectory, _parameters.Files, tagNameToFileSet);
			}

			try
			{
				// Build the file mapping
				Dictionary<FileReference, FileReference> targetFileToSourceFile = FilePattern.CreateMapping(files, ref sourcePattern, ref targetPattern);

				if (!String.IsNullOrEmpty(_parameters.Exclude))
				{
					FileFilter excludeFilter = new FileFilter();
					foreach (string excludeRule in SplitDelimitedList(_parameters.Exclude))
					{
						// use "Include" because we want to match against the files we will explicitly exclude later
						excludeFilter.Include(excludeRule);
					}

					List<FileReference> excludedFiles = new List<FileReference>();
					foreach (KeyValuePair<FileReference, FileReference> filePair in targetFileToSourceFile)
					{
						if (excludeFilter.Matches(filePair.Value.ToString()))
						{
							excludedFiles.Add(filePair.Key);
						}
					}

					foreach (FileReference excludedFile in excludedFiles)
					{
						targetFileToSourceFile.Remove(excludedFile);
					}
				}

				// Check we got some files
				if (targetFileToSourceFile.Count == 0)
				{
					if (_parameters.ErrorIfNotFound)
					{
						throw new AutomationException("No files found matching '{0}'", sourcePattern);
					}
					else
					{
						Logger.LogInformation("No files found matching '{SourcePattern}'", sourcePattern);
					}
					return;
				}

				// Run the copy
				Logger.LogInformation("Copying {Arg0} file{Arg1} from {Arg2} to {Arg3}...", targetFileToSourceFile.Count, (targetFileToSourceFile.Count == 1) ? "" : "s", sourcePattern.BaseDirectory, targetPattern.BaseDirectory);
				await ExecuteAsync(targetFileToSourceFile, _parameters.Overwrite);

				// Eventually this should be removed once we confirm all our copies do not require the implicit tagging.
				if (_parameters.TagInDefaultSet)
				{
					buildProducts.UnionWith(targetFileToSourceFile.Keys);
				}

				// Apply the optional output tag to them
				foreach (string tagName in FindTagNamesFromList(_parameters.Tag))
				{
					FindOrAddTagSet(tagNameToFileSet, tagName).UnionWith(targetFileToSourceFile.Keys);
				}
			}
			catch (FilePatternSourceFileMissingException ex)
			{
				if (_parameters.ErrorIfNotFound)
				{
					throw new AutomationException(ex, "Error while trying to create file pattern match for '{0}': {1}", sourcePattern, ex.Message);
				}
				else
				{
					Logger.LogInformation(ex, "Error while trying to create file pattern match for '{SourcePattern}': {Message}", sourcePattern, ex.Message);
				}
			}
		}

		/// <summary>
		/// 
		/// </summary>
		/// <param name="targetFileToSourceFile"></param>
		/// <param name="overwrite"></param>
		/// <returns></returns>
		public static async Task ExecuteAsync(Dictionary<FileReference, FileReference> targetFileToSourceFile, bool overwrite)
		{
			//  If we're not overwriting, remove any files where the destination file already exists.
			if (!overwrite)
			{
				Dictionary<FileReference, FileReference> filteredTargetToSourceFile = new Dictionary<FileReference, FileReference>();
				foreach (KeyValuePair<FileReference, FileReference> file in targetFileToSourceFile)
				{
					if (FileReference.Exists(file.Key))
					{
						Logger.LogInformation("Not copying existing file {Arg0}", file.Key);
						continue;
					}
					filteredTargetToSourceFile.Add(file.Key, file.Value);
				}
				if (filteredTargetToSourceFile.Count == 0)
				{
					Logger.LogInformation("All files already exist, exiting early.");
					return;
				}
				targetFileToSourceFile = filteredTargetToSourceFile;
			}

			// If the target is on a network share, retry creating the first directory until it succeeds
			DirectoryReference firstTargetDirectory = targetFileToSourceFile.First().Key.Directory;
			if (!DirectoryReference.Exists(firstTargetDirectory))
			{
				const int MaxNumRetries = 15;
				for (int numRetries = 0; ; numRetries++)
				{
					try
					{
						DirectoryReference.CreateDirectory(firstTargetDirectory);
						if (numRetries == 1)
						{
							Logger.LogInformation("Created target directory {FirstTargetDirectory} after 1 retry.", firstTargetDirectory);
						}
						else if (numRetries > 1)
						{
							Logger.LogInformation("Created target directory {FirstTargetDirectory} after {NumRetries} retries.", firstTargetDirectory, numRetries);
						}
						break;
					}
					catch (Exception ex)
					{
#pragma warning disable CA1508 // False positive about NumRetries alwayws being zero
						if (numRetries == 0)
						{
							Logger.LogInformation("Unable to create directory '{FirstTargetDirectory}' on first attempt. Retrying {MaxNumRetries} times...", firstTargetDirectory, MaxNumRetries);
						}
#pragma warning restore CA1508

						Logger.LogDebug("  {Ex}", ex);

						if (numRetries >= 15)
						{
							throw new AutomationException(ex, "Unable to create target directory '{0}' after {1} retries.", firstTargetDirectory, numRetries);
						}

						await Task.Delay(2000);
					}
				}
			}

			// Copy them all
			KeyValuePair<FileReference, FileReference>[] filePairs = targetFileToSourceFile.ToArray();
			foreach (KeyValuePair<FileReference, FileReference> filePair in filePairs)
			{
				Logger.LogDebug("  {Arg0} -> {Arg1}", filePair.Value, filePair.Key);
			}
			CommandUtils.ThreadedCopyFiles(filePairs.Select(x => x.Value.FullName).ToList(), filePairs.Select(x => x.Key.FullName).ToList(), bQuiet: true, bRetry: true);
		}

		/// <summary>
		/// Output this task out to an XML writer.
		/// </summary>
		public override void Write(XmlWriter writer)
		{
			Write(writer, _parameters);
		}

		/// <summary>
		/// Find all the tags which are used as inputs to this task
		/// </summary>
		/// <returns>The tag names which are read by this task</returns>
		public override IEnumerable<string> FindConsumedTagNames()
		{
			foreach (string tagName in FindTagNamesFromFilespec(_parameters.Files))
			{
				yield return tagName;
			}
		}

		/// <summary>
		/// Find all the tags which are modified by this task
		/// </summary>
		/// <returns>The tag names which are modified by this task</returns>
		public override IEnumerable<string> FindProducedTagNames()
		{
			return FindTagNamesFromList(_parameters.Tag);
		}
	}

	/// <summary>
	/// Extension methods
	/// </summary>
	public static partial class BgStateExtensions
	{
		/// <summary>
		/// Copy files from one location to another
		/// </summary>
		/// <param name="files">The files to copy</param>
		/// <param name="targetDir"></param>
		/// <param name="overwrite">Whether or not to overwrite existing files.</param>
		public static async Task<FileSet> CopyToAsync(this FileSet files, DirectoryReference targetDir, bool? overwrite = null)
		{
			// Run the copy
			Dictionary<FileReference, FileReference> targetFileToSourceFile = files.Flatten(targetDir);
			if (targetFileToSourceFile.Count == 0)
			{
				return FileSet.Empty;
			}

			Log.Logger.LogInformation("Copying {NumFiles} file(s) to {TargetDir}...", targetFileToSourceFile.Count, targetDir);
			await CopyTask.ExecuteAsync(targetFileToSourceFile, overwrite ?? true);
			return FileSet.FromFiles(targetFileToSourceFile.Keys.Select(x => (x.MakeRelativeTo(targetDir), x)));
		}
	}
}
