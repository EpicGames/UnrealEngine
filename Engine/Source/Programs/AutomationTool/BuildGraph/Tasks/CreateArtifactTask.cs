// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Concurrent;
using System.Collections.Generic;
using System.Diagnostics;
using System.Linq;
using System.Threading;
using System.Threading.Tasks;
using System.Xml;
using EpicGames.Core;
using EpicGames.Horde;
using EpicGames.Horde.Artifacts;
using EpicGames.Horde.Commits;
using EpicGames.Horde.Storage;
using EpicGames.Horde.Storage.Nodes;
using EpicGames.Horde.Streams;
using EpicGames.Horde.Symbols;
using Microsoft.Extensions.DependencyInjection;
using Microsoft.Extensions.Logging;

#nullable enable

namespace AutomationTool.Tasks
{
	/// <summary>
	/// Parameters for a <see cref="CreateArtifactTask"/>.
	/// </summary>
	public class CreateArtifactTaskParameters
	{
		/// <summary>
		/// Name of the artifact
		/// </summary>
		[TaskParameter]
		public string Name { get; set; } = null!;

		/// <summary>
		/// The artifact type. Determines the permissions and expiration policy for the artifact.
		/// </summary>
		[TaskParameter]
		public string Type { get; set; } = null!;

		/// <summary>
		/// Description for the artifact. Will be shown through the Horde dashboard.
		/// </summary>
		[TaskParameter(Optional = true)]
		public string? Description { get; set; }

		/// <summary>
		/// Base path for the uploaded files. All the tagged files must be under this directory. Defaults to the workspace root directory.
		/// </summary>
		[TaskParameter(Optional = true)]
		public string? BaseDir { get; set; }

		/// <summary>
		/// Stream containing the artifact.
		/// </summary>
		[TaskParameter(Optional = true)]
		public string? StreamId { get; set; }

		/// <summary>
		/// Commit for the uploaded artifact.
		/// </summary>
		[TaskParameter(Optional = true)]
		public string? Commit { get; set; }

		/// <summary>
		/// Files to include in the artifact.
		/// </summary>
		[TaskParameter(ValidationType = TaskParameterValidationType.FileSpec)]
		public string Files { get; set; } = null!;

		/// <summary>
		/// Queryable keys for this artifact, separated by semicolons.
		/// </summary>
		[TaskParameter(Optional = true)]
		public string? Keys { get; set; }

		/// <summary>
		/// Other metadata for the artifact, separated by semicolons.
		/// </summary>
		[TaskParameter(Optional = true)]
		public string? Metadata { get; set; }

		/// <summary>
		/// Whether to add aliases for symbol files
		/// </summary>
		[TaskParameter(Optional = true)]
		public bool Symbols { get; set; }

		/// <summary>
		/// Whether to reuse blobs from the previous artifact
		/// </summary>
		[TaskParameter(Optional = true)]
		public bool Dedupe { get; set; }
	}

	/// <summary>
	/// Uploads an artifact to Horde
	/// </summary>
	[TaskElement("CreateArtifact", typeof(CreateArtifactTaskParameters))]
	public class CreateArtifactTask : BgTaskImpl
	{
		readonly CreateArtifactTaskParameters _parameters;

		/// <summary> The name of the artifact we are uploading. </summary>
		protected ArtifactName Name { get; private set; }

		/// <summary> The type of the artifact being uploaded </summary>
		protected ArtifactType Type { get; private set; }

		/// <summary> Stream Id resolved from parameters or environment variable </summary>
		protected StreamId? StreamId { get; private set; }

		/// <summary> Commit Id resolved from parameters or environment variable </summary>
		protected CommitId? CommitId { get; private set; }

		/// <summary>
		/// Constructor.
		/// </summary>
		/// <param name="parameters">Parameters for this task.</param>
		public CreateArtifactTask(CreateArtifactTaskParameters parameters)
		{
			_parameters = parameters;
			Name = new ArtifactName(_parameters.Name);
			Type = new ArtifactType(_parameters.Type);
		}

		/// <summary>
		/// ExecuteAsync the task.
		/// </summary>
		/// <param name="job">Information about the current job.</param>
		/// <param name="buildProducts">Set of build products produced by this node.</param>
		/// <param name="tagNameToFileSet">Mapping from tag names to the set of files they include.</param>
		public override async Task ExecuteAsync(JobContext job, HashSet<FileReference> buildProducts, Dictionary<string, HashSet<FileReference>> tagNameToFileSet)
		{
			List<string> keys = (_parameters.Keys ?? String.Empty).Split(';', StringSplitOptions.RemoveEmptyEntries | StringSplitOptions.TrimEntries).ToList();
			List<string> metadata = (_parameters.Metadata ?? String.Empty).Split(';', StringSplitOptions.RemoveEmptyEntries | StringSplitOptions.TrimEntries).ToList();

			// Add keys for the job that's executing
			string? jobId = Environment.GetEnvironmentVariable("UE_HORDE_JOBID");
			if (!String.IsNullOrEmpty(jobId))
			{
				keys.Add($"job:{jobId}");

				string? stepId = Environment.GetEnvironmentVariable("UE_HORDE_STEPID");
				if (!String.IsNullOrEmpty(stepId))
				{
					keys.Add($"job:{jobId}/step:{stepId}");
				}
			}

			// Figure out the current change and stream id
			if (!String.IsNullOrEmpty(_parameters.StreamId))
			{
				StreamId = new StreamId(_parameters.StreamId);
			}
			else
			{
				string? streamIdEnvVar = Environment.GetEnvironmentVariable("UE_HORDE_STREAMID");
				if (!String.IsNullOrEmpty(streamIdEnvVar))
				{
					StreamId = new StreamId(streamIdEnvVar);
				}
				else
				{
					throw new AutomationException("Missing UE_HORDE_STREAMID environment variable; unable to determine current stream.");
				}
			}

			
			if (!String.IsNullOrEmpty(_parameters.Commit))
			{
				CommitId = new CommitId(_parameters.Commit);
			}
			else
			{
				int change = CommandUtils.P4Env.Changelist;
				if (change > 0)
				{
					CommitId = CommitId.FromPerforceChange(CommandUtils.P4Env.Changelist);
				}
				else
				{
					throw new AutomationException("Unknown changelist. Please run with -P4.");
				}
			}

			// Resolve the files to include
			DirectoryReference baseDir = ResolveDirectory(_parameters.BaseDir);
			List<FileReference> files = BgTaskImpl.ResolveFilespec(baseDir, _parameters.Files, tagNameToFileSet).ToList();

			bool validFiles = true;
			foreach (FileReference file in files)
			{
				if (!file.IsUnderDirectory(baseDir))
				{
					Logger.LogError("Artifact file {File} is not under {BaseDir}", file, baseDir);
					validFiles = false;
				}
			}

			if (!validFiles)
			{
				throw new AutomationException($"Unable to create artifact {Name} with given file list.");
			}

			// Find the total file size
			long totalSize = files.Sum(x => x.ToFileInfo().Length);

			// Create the new artifact
			IHordeClient hordeClient = CommandUtils.ServiceProvider.GetRequiredService<IHordeClient>();
			IArtifactBuilder artifact = await hordeClient.Artifacts.CreateAsync(Name, Type, _parameters.Description, StreamId.Value, CommitId, keys, metadata);
			Logger.LogInformation("Creating artifact {ArtifactId} '{ArtifactName}' ({ArtifactType}) from {NumFiles} files ({TotalSize:n1}mb). Namespace: {NamespaceId}, ref {RefName} ({Link})", artifact.Id, Name, Type, files.Count, totalSize / (1024.0 * 1024.0), artifact.NamespaceId, artifact.RefName, new Uri(hordeClient.ServerUrl, $"/api/v1/storage/{artifact.NamespaceId}/refs/{artifact.RefName}").ToString());

			// Upload the files
			Stopwatch timer = Stopwatch.StartNew();

			IHashedBlobRef<DirectoryNode> rootRef;
			await using (IBlobWriter blobWriter = artifact.CreateBlobWriter())
			{
				if (_parameters.Dedupe)
				{
					await using DedupeBlobWriter dedupeWriter = new DedupeBlobWriter(blobWriter);
					try
					{
						IArtifact? prevArtifact = await hordeClient.Artifacts.FindAsync(StreamId, maxCommitId: CommitId, name: Name, type: Type).FirstOrDefaultAsync(x => x.CommitId != CommitId);
						if (prevArtifact == null)
						{
							Logger.LogInformation("Could not find a previous artifact; dedupe will be disabled");
						}
						else if (prevArtifact.NamespaceId != artifact.NamespaceId)
						{
							Logger.LogInformation("Previous artifact {ArtifactId} is from namespace {PrevNamespaceId}, not {NamespaceId}. Will disable dedupe.", artifact.Id, prevArtifact.NamespaceId, artifact.NamespaceId);
						}
						else
						{
							Logger.LogInformation("Caching blobs from previous artifact ({ArtifactId})", prevArtifact.Id);
							await dedupeWriter.AddToCacheAsync(prevArtifact.Content);
						}
					}
					catch (RefNameNotFoundException ex)
					{
						Logger.LogInformation(ex, "Unable to read ref for previous artifact ({RefName}).", ex.Name);
					}
					catch (Exception ex)
					{
						Logger.LogInformation(ex, "Unable to read previous artifact: {Message}", ex.Message);
					}

					rootRef = await dedupeWriter.AddFilesAsync(baseDir, files);

					StorageStats stats = dedupeWriter.GetStats();
					Logger.LogInformation("Stats:");
					stats.Print(Logger);
				}
				else
				{
					rootRef = await blobWriter.AddFilesAsync(baseDir, files);
				}
			}
			double uploadTimeSecs = timer.Elapsed.TotalSeconds;

			// Tag any uploaded symbols
			if (_parameters.Symbols)
			{
				// Find all the hashes
				ConcurrentBag<(IBlobRef Blob, string Alias)> aliases = new ConcurrentBag<(IBlobRef Blob, string Alias)>();
				await Parallel.ForEachAsync(files, (file, cancellationToken) => ComputeHashAsync(baseDir, file, rootRef, aliases, cancellationToken));
				
				// Add them to the artifact
				foreach ((IBlobRef blob, string alias) in aliases)
				{
					await artifact.AddAliasAsync(alias, blob);
				}
			}

			// Finalize the artifact
			await artifact.CompleteAsync(rootRef);
			Logger.LogInformation("Uploaded artifact {ArtifactId} in {Time:n1}s ({Rate:n1}mb/s, {RateMbps:n1}mbps)", artifact.Id, timer.Elapsed.TotalSeconds, totalSize / (uploadTimeSecs * 1024.0 * 1024.0), (totalSize * 8.0) / (uploadTimeSecs * 1024.0 * 1024.0));
		}

		static async ValueTask ComputeHashAsync(DirectoryReference baseDir, FileReference file, IBlobRef<DirectoryNode> rootRef, ConcurrentBag<(IBlobRef, string)> aliases, CancellationToken cancellationToken)
		{
			string? hash = await SymStore.GetHashAsync(file, Logger, cancellationToken);
			if (hash != null)
			{
				string path = file.MakeRelativeTo(baseDir);

				FileEntry? fileEntry = await FindFileAsync(rootRef, path, cancellationToken);
				if (fileEntry == null)
				{
					Logger.LogWarning("Unable to find file {Path} in uploaded data.", path);
				}
				else
				{
					string fileName = file.GetFileName().ToUpperInvariant();
					string alias = $"sym:{fileName}/{hash}/{fileName}";
					Logger.LogInformation("Adding symbol alias: {Alias}", alias);
					aliases.Add((fileEntry.Target, alias));
				}
			}
		}

		static async Task<FileEntry?> FindFileAsync(IBlobRef<DirectoryNode> rootDir, string path, CancellationToken cancellationToken = default)
		{
			string[] fragments = path.Split('/', '\\');
			if (fragments.Length == 0)
			{
				return null;
			}

			IBlobRef<DirectoryNode> directoryRef = rootDir;
			for (int idx = 0; ; idx++)
			{
				DirectoryNode directory = await directoryRef.ReadBlobAsync(cancellationToken);
				if (idx + 1 < fragments.Length)
				{
					if (directory.TryGetDirectoryEntry(fragments[idx], out DirectoryEntry? directoryEntry))
					{
						directoryRef = directoryEntry.Handle;
					}
					else
					{
						return null;
					}
				}
				else
				{
					if (directory.TryGetFileEntry(fragments[idx], out FileEntry? fileEntry))
					{
						return fileEntry;
					}
					else
					{
						return null;
					}
				}
			}
		}

		/// <inheritdoc/>
		public override void Write(XmlWriter writer)
			=> Write(writer, _parameters);

		/// <inheritdoc/>
		public override IEnumerable<string> FindConsumedTagNames()
			=> FindTagNamesFromFilespec(_parameters.Files);

		/// <inheritdoc/>
		public override IEnumerable<string> FindProducedTagNames()
			=> Enumerable.Empty<string>();
	}
}
