// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.IO;
using System.Linq;
using System.Threading;
using System.Threading.Tasks;
using System.Xml;
using EpicGames.Core;
using EpicGames.Horde;
using EpicGames.Horde.Artifacts;
using EpicGames.Horde.Commits;
using EpicGames.Horde.Storage;
using EpicGames.Horde.Storage.Nodes;
using EpicGames.Horde.Streams;
using EpicGames.ProjectStore;
using EpicGames.Serialization;
using Microsoft.Extensions.DependencyInjection;
using Microsoft.Extensions.Logging;
using UnrealBuildBase;
using UnrealBuildTool;

#nullable enable

namespace AutomationTool.Tasks
{
	/// <summary>
	/// Parameters for a <see cref="CreateCloudArtifactTask"/>.
	/// </summary>
	public class CreateCloudArtifactTaskParameters
	{
		/// <summary>
		/// Name of the artifact
		/// </summary>
		[TaskParameter]
		public string Name { get; set; } = null!;

		/// <summary>
		/// An optional override of the id of the artifact as presented in Horde, defaults to name if not set
		/// </summary>
		[TaskParameter(Optional = true)]
		public string HordeArtifactID { get; set; } = null!;

		/// <summary>
		/// An identifier to set for a group of builds that are compatible
		/// </summary>
		[TaskParameter]
		public string BuildGroup { get; set; } = null!;

		/// <summary>
		/// The artifact type. Determines the permissions and expiration policy for the artifact.
		/// </summary>
		[TaskParameter]
		public string Type { get; set; } = null!;

		/// <summary>
		/// Description for the artifact. Will be shown through the Horde dashboard.
		/// </summary>
		[TaskParameter(Optional = true)]
		public string? Description { get; set; }

		/// <summary>
		/// The platform the artifact is created for
		/// </summary>
		[TaskParameter]
		public string Platform { get; set; } = null!;

		/// <summary>
		/// The uproject this artifact is created for
		/// </summary>
		[TaskParameter(Optional = true)]
		public FileReference? Project { get; set; } = null!;

		/// <summary>
		/// Override the project name, defaults to the filename of the uproject
		/// </summary>
		[TaskParameter(Optional = true)]
		public string ProjectNameOverride { get; set; } = "";

		/// <summary>
		/// The platform the artifact is created for
		/// </summary>
		[TaskParameter(Optional = true)]
		public string? Host { get; set; }

		/// <summary>
		/// The platform the artifact is created for
		/// </summary>
		[TaskParameter(Optional = true)]
		public string? Namespace { get; set; }

		/// <summary>
		/// The access token to use
		/// </summary>
		[TaskParameter(Optional = true)]
		public string? AccessToken { get; set; }

		/// <summary>
		/// Base path for the uploaded files. All the tagged files must be under this directory. Defaults to the workspace root directory.
		/// </summary>
		[TaskParameter(Optional = true)]
		public string? BaseDir { get; set; }

		/// <summary>
		/// Stream containing the artifact.
		/// </summary>
		[TaskParameter(Optional = true)]
		public string? StreamId { get; set; }

		/// <summary>
		/// Commit for the uploaded artifact.
		/// </summary>
		[TaskParameter(Optional = true)]
		public string? Commit { get; set; }
		/// <summary>
		/// Files to include in the artifact.
		/// </summary>
		[TaskParameter(ValidationType = TaskParameterValidationType.FileSpec)]
		public string Files { get; set; } = null!;

		/// <summary>
		/// Files to include in the symbols artifact.
		/// </summary>
		[TaskParameter(ValidationType = TaskParameterValidationType.FileSpec, Optional = true)]
		public string? SymbolFiles { get; set; } = null;

		/// <summary>
		/// Other metadata for the artifact, separated by semicolons. Use key=value for each field.
		/// </summary>
		[TaskParameter(Optional = true)]
		public string? Metadata { get; set; }

		/// <summary>
		/// Force the blobs to be uploaded no matter if they exist or not
		/// </summary>
		[TaskParameter(Optional = true)]
		public bool Force { get; set; } = false;

		/// <summary>
		/// The streamid used for your mainline, used to build content as patches on top of this branch if no other baselines exist.
		/// </summary>
		[TaskParameter(Optional = true)]
		public string BaselineBranch { get; set; } = null!;

		/// <summary>
		/// Enable to use multipart endpoints if valuable
		/// </summary>
		[TaskParameter(Optional = true)]
		public bool AllowMultipart { get; set; } = false;

		/// <summary>
		/// Set the explicit http version to use. None to use http handshaking.
		/// </summary>
		[TaskParameter(Optional = true)]
		public string HttpVersion { get; set; } = "None";

		/// <summary>
		/// Verify the contents after upload, essentially downloads the entire build to make sure all data was uploaded correctly. Takes a bit of time and is generally not needed but can be useful to detect issues in the upload.
		/// </summary>
		[TaskParameter(Optional = true)]
		public bool Verify { get; set; } = false;

		/// <summary>
		/// Verbose logging output, can be useful to find out more information when an issue has occured.
		/// </summary>
		[TaskParameter(Optional = true)]
		public bool Verbose { get; set; } = false;

		/// <summary>
		/// Increase the number of worker threads used by zen, may cause machine to be less responsive but will generally improve upload times
		/// </summary>
		[TaskParameter(Optional = true)]
		public bool BoostWorkers { get; set; } = false;

		/// <summary>
		/// Enable to create an Unreal Insights trace of the upload process
		/// </summary>
		public bool EnableTracing { get; set; } = true;
	}

	/// <summary>
	/// Uploads an artifact to Cloud DDC
	/// </summary>
	[TaskElement("CreateCloudArtifact", typeof(CreateCloudArtifactTaskParameters))]
	public class CreateCloudArtifactTask : BgTaskImpl
	{
		readonly CreateCloudArtifactTaskParameters _parameters;

		private readonly HashSet<string> _supportedSymbolsExtensions = [
			".exp",
			".map",
			".pdb"
		];

		/// <summary>
		/// Constructor.
		/// </summary>
		/// <param name="parameters">Parameters for this task.</param>
		public CreateCloudArtifactTask(CreateCloudArtifactTaskParameters parameters)
			=> _parameters = parameters;

		/// <summary>
		/// ExecuteAsync the task.
		/// </summary>
		/// <param name="job">Information about the current job.</param>
		/// <param name="buildProducts">Set of build products produced by this node.</param>
		/// <param name="tagNameToFileSet">Mapping from tag names to the set of files they include.</param>
		public override async Task ExecuteAsync(JobContext job, HashSet<FileReference> buildProducts, Dictionary<string, HashSet<FileReference>> tagNameToFileSet)
		{
			if (String.IsNullOrEmpty(_parameters.Name) && String.IsNullOrEmpty(_parameters.HordeArtifactID))
			{
				throw new AutomationException("No name set in either Name or HordeArtifactID");
			}

			string name = String.IsNullOrEmpty(_parameters.Name) ? _parameters.HordeArtifactID : _parameters.Name;
			ArtifactType type = new ArtifactType(_parameters.Type);
			string sanitizedHordeArtifactParam = ICloudStorage.SanitizeBucketValue(String.IsNullOrEmpty(_parameters.HordeArtifactID) ? name : _parameters.HordeArtifactID);
			ArtifactName hordeArtifactID = new ArtifactName(sanitizedHordeArtifactParam);
			List<string> metadataParam = (_parameters.Metadata ?? String.Empty).Split(';', StringSplitOptions.RemoveEmptyEntries | StringSplitOptions.TrimEntries).ToList();

			Dictionary<string, object> metadata = new Dictionary<string, object>();
			string platform = ICloudStorage.SanitizeBucketValue(_parameters.Platform);
			
			FileReference? projectPath = _parameters.Project;
			string project = projectPath != null ? projectPath.GetFileNameWithoutExtension() : "";

			bool projectPathExists = projectPath != null && File.Exists(projectPath.FullName);
			if (!projectPathExists)
			{
				projectPath = null;
				Logger.LogInformation("\'{Project}\' is not a valid uproject file. Unable to load inis, you will need to specify project options explicitly.", _parameters.Project);
				project = _parameters.ProjectNameOverride;
				if (String.IsNullOrEmpty(project))
				{
					Logger.LogError("No project name override set and unable to locate uproject file '{Project}'. Unable to determine what the project name should be.", _parameters.Project);
					throw new AutomationException("Unable to determine what the project name should be");
				}
			}

			if (!String.IsNullOrEmpty(_parameters.ProjectNameOverride))
			{
				project = _parameters.ProjectNameOverride;
			}

			bool foundConfig = false;
			CloudConfiguration? cloudConfig = null;
			if (projectPath != null)
			{
				(Dictionary<UnrealTargetPlatform, ConfigHierarchy> engineConfigs, Dictionary<UnrealTargetPlatform, ConfigHierarchy> _) = GetIniConfigs(projectPath);
				ConfigHierarchy config = engineConfigs[HostPlatform.Current.HostEditorPlatform];
				foundConfig = config.TryGetValueGeneric("StorageServers", "Cloud", out CloudConfiguration foundCloudConfig);
				cloudConfig = foundCloudConfig;
			}

			foreach (string line in metadataParam)
			{
				int sep = line.IndexOf('=', StringComparison.OrdinalIgnoreCase);
				if (sep != -1)
				{
					string k = line[..sep];
					string v = line[(sep + 1)..];
					metadata.Add(k, v);
					continue;
				}
				Logger.LogWarning("Unable to convert metadata line: \'{Line}\' to key-value. Please use = to separate key and value. Ignoring.", line);
			}

			metadata.Add("type", type);
			if (!String.IsNullOrEmpty(_parameters.Description))
			{
				metadata.Add("description", _parameters.Description);
			}

			// Add keys for the job that's executing
			string? stepId = null;
			string? jobId = Environment.GetEnvironmentVariable("UE_HORDE_JOBID");
			if (!String.IsNullOrEmpty(jobId))
			{
				metadata.Add("job", jobId);

				stepId = Environment.GetEnvironmentVariable("UE_HORDE_STEPID");
				if (!String.IsNullOrEmpty(stepId))
				{
					metadata.Add("step", stepId);
				}
			}

			string? hordeUrl = Environment.GetEnvironmentVariable("UE_HORDE_URL");
			// if we are running in horde and have the required environment variables we append a link back to the horde job into metadata
			if (!String.IsNullOrEmpty(hordeUrl) && !String.IsNullOrEmpty(jobId) && !String.IsNullOrEmpty(stepId))
			{
				metadata.Add("buildurl", $"{hordeUrl}job/{jobId}?step={stepId}");
			}

			if (!String.IsNullOrEmpty(_parameters.BuildGroup))
			{
				metadata.Add("buildgroup", _parameters.BuildGroup);
			}

			string httpVersion = _parameters.HttpVersion;
			if (String.IsNullOrEmpty(httpVersion) || httpVersion == "None")
			{
				string? httpVersionEnvironmentVariable = Environment.GetEnvironmentVariable("UE-CloudPublishHttpVersion");
				if (!String.IsNullOrEmpty(httpVersionEnvironmentVariable))
				{
					httpVersion = httpVersionEnvironmentVariable;
				}
			}
			
			// Figure out the current change and stream id
			StreamId streamIdTyped;
			if (!String.IsNullOrEmpty(_parameters.StreamId))
			{
				streamIdTyped = new StreamId(_parameters.StreamId);
			}
			else
			{
				string? streamIdEnvVar = Environment.GetEnvironmentVariable("UE_HORDE_STREAMID");
				if (!String.IsNullOrEmpty(streamIdEnvVar))
				{
					streamIdTyped = new StreamId(streamIdEnvVar);
				}
				else
				{
					throw new AutomationException("Missing UE_HORDE_STREAMID environment variable; unable to determine current stream.");
				}
			}
			string streamId = ICloudStorage.SanitizeBucketValue(streamIdTyped.ToString());
			metadata.Add("stream", streamId);
			
			string baselineBranch =_parameters.BaselineBranch;
			if (foundConfig && cloudConfig != null && String.IsNullOrEmpty(baselineBranch))
			{
				// no explicit baseline branch set and we have found a storage server config, using that value
				baselineBranch = cloudConfig.Value.BuildsBaselineBranch;
			}
			if (String.IsNullOrEmpty(baselineBranch))
			{
				Logger.LogWarning("No baseline branch set, falling back to current stream as baseline. This will generate in-optimal de-dup for branches, make sure to set this to your mainline by specifying the property or setting this in your StorageServer section in your engine ini");
				baselineBranch = streamId;
			}

			baselineBranch = ICloudStorage.SanitizeBucketValue(baselineBranch);

			CommitId commitId = CommitId.Empty;

			if (!String.IsNullOrEmpty(_parameters.Commit))
			{
				commitId = new CommitId(_parameters.Commit);
				metadata.Add("commit", commitId);
			}
			else
			{
				try
				{
					int change = CommandUtils.P4Env.Changelist;
					if (change > 0)
					{
						commitId = CommitId.FromPerforceChange(CommandUtils.P4Env.Changelist);
						metadata.Add("commit", commitId.GetPerforceChange());
					}
				}
				catch (AutomationException)
				{
					// not an error to not use p4, commit is just metadata
				}
				
				// no commit specified, not required so ignoring
			}

			string publishHostEnvVar = "UE-CloudPublishHost";
			string descriptorHostEnvVar = "UE-CloudPublishDescriptorHost";
			if (!OperatingSystem.IsWindows())
			{
				publishHostEnvVar = publishHostEnvVar.Replace("-", "_", StringComparison.OrdinalIgnoreCase);
				descriptorHostEnvVar = descriptorHostEnvVar.Replace("-", "_", StringComparison.OrdinalIgnoreCase);
			}

			// the host to specify in files published to users, as the host we connect to in the farm maybe different from what users should connect to
			string? descriptorHost = null;
			if (!String.IsNullOrEmpty(Environment.GetEnvironmentVariable(descriptorHostEnvVar)))
			{
				descriptorHost = Environment.GetEnvironmentVariable(descriptorHostEnvVar);
			}

			string? cloudConfigHost = null;
			if (foundConfig && cloudConfig != null && !String.IsNullOrEmpty(cloudConfig.Value.Host))
			{
				cloudConfigHost = cloudConfig.Value.Host;
				if (cloudConfigHost.Contains(';', StringComparison.OrdinalIgnoreCase))
				{
					// if its a list pick the first element
					cloudConfigHost = cloudConfigHost.Split(";").First();
				}

				// if not explicitly set we default the user facing host value to the config setting value
				if (descriptorHost == null)
				{
					descriptorHost = cloudConfigHost;
				}
			}

			string host;
			if (!String.IsNullOrEmpty(_parameters.Host))
			{
				host = _parameters.Host;
			}
			else if (!String.IsNullOrEmpty(Environment.GetEnvironmentVariable(publishHostEnvVar)))
			{
				host = Environment.GetEnvironmentVariable(publishHostEnvVar)!;
			}
			else if (!String.IsNullOrEmpty(cloudConfigHost))
			{
				host = cloudConfigHost;
			}
			else
			{
				throw new AutomationException("Missing UE-CloudPublishHost environment variable; unable to determine cloud host. Please specify this environment variable or define it in your StorageServer section in your engine ini");
			}

			// if we have not yet had the user facing host set then we just assume that is the same as the host we are publishing to
			if (descriptorHost == null)
			{
				descriptorHost = host;
			}

			string publishNamespaceEnvVar = "UE-CloudPublishNamespace";
			if (!OperatingSystem.IsWindows())
			{
				publishNamespaceEnvVar = publishNamespaceEnvVar.Replace("-", "_", StringComparison.OrdinalIgnoreCase);
			}
			string ns;
			if (!String.IsNullOrEmpty(_parameters.Namespace))
			{
				ns = _parameters.Namespace;
			}
			else if (!String.IsNullOrEmpty(Environment.GetEnvironmentVariable(publishNamespaceEnvVar)))
			{
				ns = Environment.GetEnvironmentVariable(publishNamespaceEnvVar)!;
			}
			else if (foundConfig && cloudConfig != null && !String.IsNullOrEmpty(cloudConfig.Value.BuildsNamespace))
			{
				ns = cloudConfig.Value.BuildsNamespace;
			}
			else
			{
				throw new AutomationException("Missing UE-CloudPublishNamespace environment variable; unable to default namespace please specify it in the task or define it in your StorageServer section in your engine ini");
			}

			string accessTokenEnvVar = "UE-CloudDataCacheAccessToken";
			if (!OperatingSystem.IsWindows())
			{
				accessTokenEnvVar = accessTokenEnvVar.Replace("-", "_", StringComparison.OrdinalIgnoreCase);
			}
			string accessToken;
			if (!String.IsNullOrEmpty(_parameters.AccessToken))
			{
				accessToken = _parameters.AccessToken;
			}
			else if (!String.IsNullOrEmpty(Environment.GetEnvironmentVariable(accessTokenEnvVar)))
			{
				accessToken = Environment.GetEnvironmentVariable(accessTokenEnvVar)!;
			}
			else
			{
				throw new AutomationException("Missing UE-CloudDataCacheAccessToken environment variable; unable to find access token to use.");
			}
			
			DirectoryReference baseDir = ResolveDirectory(_parameters.BaseDir);

			if (!Directory.Exists(baseDir.FullName))
			{
				Directory.CreateDirectory(baseDir.FullName);
			}

			long totalSize = 0;

			FileReference manifestFile = FileReference.Combine(baseDir, "zen-manifest.json");
			{
				using JsonWriter writer = new JsonWriter(manifestFile);
				
				void WriteFilesArray(string filePattern, Predicate<string>? filter = null)
				{
					writer.WriteArrayStart("files");
				
					// Resolve the files to include
					List<FileReference> files = ResolveFilespec(baseDir, filePattern, tagNameToFileSet).ToList();

					bool validFiles = files.Count != 0;
					if (files.Count != 0)
					{
						foreach (FileReference file in files)
						{
							if (filter != null && !filter(file.GetFileName()))
							{
								Logger.LogInformation("Skipping file {FileName}", file.GetFileName());
								continue;
							}
							
							if (!file.IsUnderDirectory(baseDir))
							{
								Logger.LogError("Artifact file {File} is not under {BaseDir}", file, baseDir);
								validFiles = false;
							}

							// always use / as path separation in the manifest no matter the host platform
							writer.WriteValue(file.MakeRelativeTo(baseDir).Replace('\\', '/'));
							totalSize += file.ToFileInfo().Length;
						}
					}
					else
					{
						Logger.LogWarning("No files found under {BaseDir}", baseDir);
					}

					if (!validFiles)
					{
						throw new AutomationException($"Unable to create artifact {name} with given file list.");
					}
				
					writer.WriteArrayEnd();
				}

				writer.WriteObjectStart();
				{
					writer.WriteObjectStart("parts");
					{
						writer.WriteObjectStart("default");
						{
							writer.WriteValue("partId", CbObjectId.NewObjectId().ToString());
							WriteFilesArray(_parameters.Files);
						}
						writer.WriteObjectEnd();

						if (!String.IsNullOrEmpty(_parameters.SymbolFiles))
						{
							writer.WriteObjectStart("symbols");
							{
								writer.WriteValue("partId", CbObjectId.NewObjectId().ToString());
								WriteFilesArray(_parameters.SymbolFiles!,
									filter: fileName => _supportedSymbolsExtensions.Contains(Path.GetExtension(fileName).ToLowerInvariant()));
							}
							writer.WriteObjectEnd();
						}
					}
					writer.WriteObjectEnd();
				}
				writer.WriteObjectEnd();
			}

			string bucketId = $"{project}.{type}.{streamId}.{platform}";

			metadata.Add("totalSize", totalSize);
			string createdAtString = DateTime.UtcNow.ToString("O");
			metadata.Add("createdAt", createdAtString);

			Uri? buildUri = null;

			const int MaxAttempts = 3;
			const int WarnAtAttempts = MaxAttempts - 1;
			for (int countOfRetries = 0; countOfRetries < MaxAttempts; countOfRetries++)
			{
				// generate new build and part ids for each attempt so that we do not continue to append to a broken build in case of a retry
				CbObjectId buildId = CbObjectId.NewObjectId();

				FileReference? traceFile = null;
				if (_parameters.EnableTracing)
				{
					// this is automatically uploaded as it's in the Saved folder
					traceFile = FileReference.Combine(Unreal.RootDirectory, $"Engine/Programs/AutomationTool/Saved/Logs/build-upload-{buildId}.utrace");
				}

				try
				{
					BuildUploader.UploadBuild(host, baseDir, ns, bucketId, buildId, manifestFile, metadata, accessToken, new BuildUploader.BuildInfo
					{
						Name = name.ToString(),
						ProjectName = project,
						Branch = streamId,
						BaselineBranch = baselineBranch,
						Platform = platform
					}, allowMultipart: _parameters.AllowMultipart, assumeHttp2: String.Equals(httpVersion, "http2-only", StringComparison.OrdinalIgnoreCase), forceClean: _parameters.Force, verify: _parameters.Verify, verbose: _parameters.Verbose, boostWorkers: _parameters.BoostWorkers, traceFile: traceFile);

					buildUri = new Uri($"{descriptorHost}/api/v2/builds/{ns}/{bucketId}/{buildId}");

					// command successful
					break;
				}	
				catch (CommandUtils.CommandFailedException e)
				{
					// increasingly long time to wait before retrying
					int[] waitDurationSeconds = [60, 300, 900];
					int waitDuration = waitDurationSeconds[countOfRetries];
					int exitCode = (int)e.ErrorCode;
					// 502 or 504 are the old exit codes that indicate errors
					// new exit codes for retries that also work on mac and linux are 69, 92 and 93
					// From zen: 
					// kHttpServiceUnavailable  = 69,	// ServiceUnavailable(503)
					// kHttpBadGateway          = 92,	// BadGateway(502)
					// kHttpGatewayTimeout      = 93,	// GatewayTimeout(504)

					if (exitCode is 502 or 504 or 69 or 92 or 93 or 84)
					{
						// zen returns http status code as exit codes if it had an issue
						Logger.LogInformation("Http Status code {ExitCode} returned from zen. Waiting {WaitDuration}s before trying.", exitCode, waitDuration);
						if (countOfRetries+1 == MaxAttempts)
						{
							// give up
							Logger.LogError("Zen returned http status code {ExitCode} after {RetryAttempts} retries. Giving up.", exitCode, countOfRetries);
							throw;
						}
						else
						{
							if (countOfRetries + 1 == WarnAtAttempts)
							{
								Logger.LogWarning("Zen returned http status code {ExitCode} after {RetryAttempts} retries. This is a indication of an infrastructure issue.", exitCode, countOfRetries);
							}
							Thread.Sleep(TimeSpan.FromSeconds(waitDuration));
							continue; // retry
						}
					}
					else
					{
						Logger.LogWarning("Unknown exit code from zen when command failed: {ExitCode} . Not retrying.", exitCode);
						throw;
					}
				}
			}

			if (buildUri != null)
			{
				try
				{
					await RegisterHordeArtifactAsync(buildUri, hordeArtifactID, type, streamIdTyped, commitId, metadata);
				}
				catch (Exception ex)
				{
					Logger.LogWarning(ex, "Exception registering Horde artifact: {Message}", ex.Message);
				}
			}
		}

		internal static (Dictionary<UnrealTargetPlatform, ConfigHierarchy>, Dictionary<UnrealTargetPlatform, ConfigHierarchy>) GetIniConfigs(FileReference projectPath)
		{
			Logger.LogDebug("Loading ini files for {RawProjectPath}", projectPath);

			Dictionary<UnrealTargetPlatform, ConfigHierarchy> engineConfigs = new Dictionary<UnrealTargetPlatform, ConfigHierarchy>();
			Dictionary<UnrealTargetPlatform, ConfigHierarchy> gameConfigs = new Dictionary<UnrealTargetPlatform, ConfigHierarchy>();
			foreach (UnrealTargetPlatform targetPlatformType in UnrealTargetPlatform.GetValidPlatforms())
			{
				ConfigHierarchy engineConfig = ConfigCache.ReadHierarchy(ConfigHierarchyType.Engine, projectPath.Directory, targetPlatformType);
				engineConfigs.Add(targetPlatformType, engineConfig);

				ConfigHierarchy gameConfig = ConfigCache.ReadHierarchy(ConfigHierarchyType.Game, projectPath.Directory, targetPlatformType); 
				gameConfigs.Add(targetPlatformType, gameConfig);
			}

			return (engineConfigs, gameConfigs);
		}

		async Task RegisterHordeArtifactAsync(Uri buildUri, ArtifactName name, ArtifactType type, StreamId streamId, CommitId commitId, Dictionary<string, object> metadata)
		{		
			// Add keys used to collate and find builds
			List<string> keys = new List<string>();

			// Add keys for the job that's executing
			string? jobId = Environment.GetEnvironmentVariable("UE_HORDE_JOBID");
			if (!String.IsNullOrEmpty(jobId))
			{
				keys.Add($"job:{jobId}");

				string? stepId = Environment.GetEnvironmentVariable("UE_HORDE_STEPID");
				if (!String.IsNullOrEmpty(stepId))
				{
					keys.Add($"job:{jobId}/step:{stepId}");
				}
			}

			List<string> metadataList = new List<string>();

			metadataList.Add("Backend=Zen");
			metadataList.Add($"ZenBuildUri={buildUri.ToString()}");

			foreach (KeyValuePair<string, object> pair in metadata)
			{
				metadataList.Add($"{pair.Key}={pair.Value.ToString()}");
			}
		
			// TODO: Investigate registering in a zen artifact namespace 
			IHordeClient hordeClient = CommandUtils.ServiceProvider.GetRequiredService<IHordeClient>();
			IArtifactBuilder artifact = await hordeClient.Artifacts.CreateAsync(name, type, _parameters.Description, streamId, commitId, keys, metadataList);

			Logger.LogInformation("Registering cloud artifact with Horde, artifact {ArtifactId} '{ArtifactName}' ({ArtifactType}). Namespace: {NamespaceId}, ref {RefName}", artifact.Id, name, type, artifact.NamespaceId, artifact.RefName);

			// TODO: Add manifest and support partial downloads
			IHashedBlobRef<DirectoryNode> rootRef;
			await using (IBlobWriter blobWriter = artifact.CreateBlobWriter())
			{
				DirectoryReference baseDir = ResolveDirectory("");
				List<FileReference> files = new List<FileReference>();
				rootRef = await blobWriter.AddFilesAsync(baseDir, files);
			}

			await artifact.CompleteAsync(rootRef);
		}	

		/// <inheritdoc/>
		public override void Write(XmlWriter writer)
			=> Write(writer, _parameters);

		/// <inheritdoc/>
		public override IEnumerable<string> FindConsumedTagNames()
			=> FindTagNamesFromFilespec(_parameters.Files);

		/// <inheritdoc/>
		public override IEnumerable<string> FindProducedTagNames()
			=> Enumerable.Empty<string>();
	}

	static class BuildUploader
	{
		internal struct BuildInfo
		{
			public string ProjectName { get; set; }
			public string Name { get; set; }
			public string Platform { get; set; }
			public string Branch { get; set; }
			public string BaselineBranch { get; set; }
		}
		public static void UploadBuild(string host, DirectoryReference baseDir, string namespaceId, string bucketId, CbObjectId buildId,
			FileReference manifestFile, Dictionary<string, object> metadata, string accessToken, BuildInfo info,
			bool allowMultipart = false, bool assumeHttp2 = false, bool forceClean = false, bool verify = false, bool verbose = false,
			bool boostWorkers = false, FileReference? traceFile = null)
		{
			string metadataJsonPath = Path.Combine(Path.GetTempPath(), Path.GetTempFileName());
			try
			{
				FileInfo zenExe = new FileInfo("Engine/Binaries/Win64/zen.exe");

				{
					using JsonWriter jsonWriter = new JsonWriter(metadataJsonPath);
					jsonWriter.WriteObjectStart();
					jsonWriter.WriteValue("project", info.ProjectName);
					jsonWriter.WriteValue("branch", info.Branch);
					jsonWriter.WriteValue("baselineBranch", info.BaselineBranch);
					jsonWriter.WriteValue("platform", info.Platform);
					jsonWriter.WriteValue("name", info.Name);

					string[] notAllowedKeys = ["name", "branch", "baselineBranch", "platform", "project"];

					foreach (KeyValuePair<string, object> pair in metadata)
					{
						if (notAllowedKeys.Contains(pair.Key.ToLower()))
						{
							Log.Logger.LogWarning("Metadata contained key '{Key}' which is not allowed as its specified by the system. Ignoring.", pair.Key);
							continue;
						}

						// Use IConvertible + TypeCode to detect any numeric primitive (sbyte through decimal)
						// rather than explicitly checking each type. This makes sure numerical values are written as numbers.
						if (pair.Value is IConvertible conv)
						{
							TypeCode tc = conv.GetTypeCode();
							if (tc is >= TypeCode.SByte and <= TypeCode.Decimal)
							{
								jsonWriter.WriteValue(pair.Key, conv.ToDouble(null));
								continue;
							}
						}
						jsonWriter.WriteValue(pair.Key, pair.Value.ToString());

					}
					jsonWriter.WriteObjectEnd();
				}

				string metadataString = File.ReadAllText(metadataJsonPath);
				string http2Options = assumeHttp2 ? "--assume-http2" : "";
				string verifyOption = verify ? "--verify" : "";
				string verboseOption = verbose ? "--verbose" : "";
				string boostWorkersOption = boostWorkers ? "--boost-workers" : "";
				// trace file option needs to be passed before other arguments including the verbs
				string traceOption = traceFile != null ? $"--tracefile={traceFile.FullName} " : "";
				// pass the access token via environment variable to avoid showing it on the cli
				string cmdline = $"{traceOption}builds upload --url {host} --namespace {namespaceId} --bucket {bucketId} --build-id {buildId} --metadata-path \"{metadataJsonPath}\" --manifest-path \"{manifestFile}\" --access-token-env UE-CloudDataCacheAccessToken --create-build --plain-progress --allow-multipart={allowMultipart} {http2Options} {boostWorkersOption} --clean={forceClean} {verifyOption} {verboseOption} \"{baseDir}\"";
				Log.Logger.LogInformation("Using metadata '{Metadata}'", metadataString);

				try
				{
					// Suppress matcher-based log processing for zen output. Must use Log.WriteLine (not Log.Logger.LogInformation)
					// so the marker is fed into Log.EventParser.WriteLine; routing via ILogger goes through LegacyEventLogger
					// which writes directly to the inner sink and never delivers the marker to the parser.
					Log.WriteLine(LogEventType.Console, "<-- Suspend Log Parsing -->");
					CommandUtils.RunAndLog(CommandUtils.CmdEnv, zenExe.FullName, cmdline, Options: CommandUtils.ERunOptions.Default, EnvVars: new Dictionary<string, string> { { "UE-CloudDataCacheAccessToken", accessToken } });
				}
				finally
				{
					Log.WriteLine(LogEventType.Console, "<-- Resume Log Parsing -->");
				}
			}
			finally
			{
				File.Delete(metadataJsonPath);
			}
		}
	}
	namespace CloudDDC
	{
		class StartMultipartUploadRequest
		{
			[CbField("blobLength")]
			public ulong BlobLength { get; set; }
		}

		class MultipartUploadIdResponse
		{
			[CbField("uploadId")]
			public string UploadId { get; set; } = null!;

			[CbField("blobName")]
			public string BlobName { get; set; } = null!;
		
			[CbField("blobQueryStrings")]
#pragma warning disable CA2227
			public List<string> PartQueryStrings { get; set; } = new List<string>();
#pragma warning restore CA2227
		}

		class PutBuildResponse
		{
			[CbField("chunkSize")]
			public uint ChunkSize { get; set; }
		}

		class NeedsResponse
		{
			public NeedsResponse()
			{
				Needs = new List<IoHash>();
			}

			public NeedsResponse(List<IoHash> needs)
			{
				Needs = needs;
			}

			[CbField("needs")]
			public List<IoHash> Needs { get; set; }
		}

		class CompleteMultipartUploadRequest
		{
			[CbField("blobName")]
			public string BlobName { get; set; } = null!;

			[CbField("uploadId")]
			public string UploadId { get; set; } = null!;

			[CbField("isCompressed")]
			public bool IsCompressed { get; set; }
		}

		static class CustomMediaTypeNames
		{
			/// <summary>
			/// Media type for compact binary
			/// </summary>
			public const string UnrealCompactBinary = "application/x-ue-cb";

			/// <summary>
			/// Media type for compressed buffers
			/// </summary>
			public const string UnrealCompressedBuffer = "application/x-ue-comp";
		}
	}
}
