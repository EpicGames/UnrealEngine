// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.IO;
using System.Linq;
using System.Text.Json;
using System.Threading.Tasks;
using System.Xml;
using EpicGames.Core;
using EpicGames.Horde;
using EpicGames.Horde.Artifacts;
using EpicGames.Horde.Commits;
using EpicGames.Horde.Storage;
using Microsoft.Extensions.DependencyInjection;
using Microsoft.Extensions.Logging;

#nullable enable

namespace AutomationTool.Tasks
{
	/// <summary>
	/// Parameters for a <see cref="CreateSummaryArtifactTask"/>.
	/// Inherits all parameters from <see cref="CreateArtifactTaskParameters"/> and adds report fields.
	/// </summary>
	public class CreateSummaryArtifactTaskParameters : CreateArtifactTaskParameters
	{
		/// <summary>
		/// Where to display the report
		/// </summary>
		[TaskParameter(Optional = true)]
		public string Scope { get; set; } = "Job";

		/// <summary>
		/// Name for the Horde report, shown on the job dashboard. Also used as the base filename
		/// for the .md and .report.json files written to the log folder.
		/// </summary>
		[TaskParameter]
		public string ReportName { get; set; } = null!;

		/// <summary>
		/// Dashboard placement for the report. Defaults to "Summary".
		/// </summary>
		[TaskParameter(Optional = true)]
		public string ReportPlacement { get; set; } = "Summary";

		/// <summary>
		/// Markdown text body for the report. Supports two substitution tokens:
		///   {ArtifactId} — replaced with the uploaded artifact's id.
		///   {HordeUrl}   — replaced with the Horde server base URL (from UE_HORDE_URL env var).
		/// Example: "[View Report]({HordeUrl}/api/v2/artifacts/{ArtifactId}/browse/report.html)"
		/// </summary>
		[TaskParameter]
		public string ReportText { get; set; } = null!;
	}

	/// <summary>
	/// Uploads an artifact to Horde and writes a job-summary report containing a direct link to the artifact.
	/// Extends <see cref="CreateArtifactTask"/> so all artifact upload behaviour is inherited unchanged;
	/// the only addition is writing the .md / .report.json pair to the step log folder after upload completes.
	/// </summary>
	[TaskElement("Horde-CreateSummaryArtifact", typeof(CreateSummaryArtifactTaskParameters))]
	public class CreateSummaryArtifactTask : CreateArtifactTask
	{
		readonly CreateSummaryArtifactTaskParameters _summaryParameters;

		/// <summary>
		/// Constructor.
		/// </summary>
		/// <param name="parameters">Parameters for this task.</param>
		public CreateSummaryArtifactTask(CreateSummaryArtifactTaskParameters parameters) : base(parameters)
			=> _summaryParameters = parameters;

		/// <inheritdoc/>
		public override async Task ExecuteAsync(JobContext job, HashSet<FileReference> buildProducts, Dictionary<string, HashSet<FileReference>> tagNameToFileSet)
		{
			await base.ExecuteAsync(job, buildProducts, tagNameToFileSet);

			IHordeClient hordeClient = CommandUtils.ServiceProvider.GetRequiredService<IHordeClient>();
			string uploadedArtifactid = String.Empty;
			try
			{
				// Find the artifact that was just uploaded.
				IArtifact? uploadedArtifact = await hordeClient.Artifacts.FindAsync(StreamId, minCommitId: CommitId, maxCommitId: CommitId, name: Name, type: Type).FirstOrDefaultAsync();
				if (uploadedArtifact == null)
				{
					Logger.LogError("Could not find the uploaded artifact of {Name}", Name);
				}
				else
				{
					uploadedArtifactid = uploadedArtifact.Id.ToString();
				}
			}
			catch (RefNameNotFoundException ex)
			{
				Logger.LogInformation(ex, "Unable to read ref for artifact ({RefName}).", ex.Name);
			}
			catch (Exception ex)
			{
				Logger.LogInformation(ex, "Unable to read artifact: {Message}", ex.Message);
			}
			if (String.IsNullOrEmpty(uploadedArtifactid))
			{
				Logger.LogError("Unable to find artifact id for uploaded data. Unable to create report.");
				return;
			}

			// Substitute tokens in the caller-supplied report text.
			string hordeUrl = (Environment.GetEnvironmentVariable("UE_HORDE_URL") ?? "").TrimEnd('/');
			string text = _summaryParameters.ReportText
				.Replace("{ArtifactId}", uploadedArtifactid, StringComparison.Ordinal)
				.Replace("{HordeUrl}", hordeUrl, StringComparison.Ordinal);

			// Write the .md and .report.json files that Horde picks up as a job report.
			DirectoryReference logDir = new DirectoryReference(CommandUtils.CmdEnv.LogFolder);
			string reportName = _summaryParameters.ReportName;

			FileReference mdFile   = FileReference.Combine(logDir, $"{reportName}.md");
			FileReference jsonFile = FileReference.Combine(logDir, $"{reportName}.report.json");

			await FileReference.WriteAllTextAsync(mdFile, text);

			using (FileStream jsonStream = FileReference.Open(jsonFile, FileMode.Create, FileAccess.Write, FileShare.Read))
			using (Utf8JsonWriter writer = new Utf8JsonWriter(jsonStream))
			{
				writer.WriteStartObject();
				writer.WriteString("scope",		_summaryParameters.Scope);
				writer.WriteString("name",      reportName);
				writer.WriteString("placement", _summaryParameters.ReportPlacement);
				writer.WriteString("fileName",  mdFile.GetFileName());
				writer.WriteEndObject();
			}

			Logger.LogInformation("Written summary report '{Name}' for artifact {ArtifactId}", reportName, uploadedArtifactid);
		}

		/// <inheritdoc/>
		public override void Write(XmlWriter writer) => Write(writer, _summaryParameters);

		/// <inheritdoc/>
		public override IEnumerable<string> FindConsumedTagNames() => FindTagNamesFromFilespec(_summaryParameters.Files);

		/// <inheritdoc/>
		public override IEnumerable<string> FindProducedTagNames() => Enumerable.Empty<string>();
	}
}
