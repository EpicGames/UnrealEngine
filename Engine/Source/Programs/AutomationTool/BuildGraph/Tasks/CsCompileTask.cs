// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Threading.Tasks;
using System.Xml;
using EpicGames.Core;
using Microsoft.Extensions.Logging;
using UnrealBuildBase;

namespace AutomationTool.Tasks
{
	/// <summary>
	/// Parameters for a task that compiles a C# project
	/// </summary>
	public class CsCompileTaskParameters
	{
		/// <summary>
		/// The C# project file to compile. Using semicolons, more than one project file can be specified.
		/// </summary>
		[TaskParameter]
		public string Project { get; set; }

		/// <summary>
		/// The configuration to compile.
		/// </summary>
		[TaskParameter(Optional = true)]
		public string Configuration { get; set; }

		/// <summary>
		/// The platform to compile.
		/// </summary>
		[TaskParameter(Optional = true)]
		public string Platform { get; set; }

		/// <summary>
		/// The target to build.
		/// </summary>
		[TaskParameter(Optional = true)]
		public string Target { get; set; }

		/// <summary>
		/// Properties for the command
		/// </summary>
		[TaskParameter(Optional = true)]
		public string Properties { get; set; }

		/// <summary>
		/// Additional options to pass to the compiler.
		/// </summary>
		[TaskParameter(Optional = true)]
		public string Arguments { get; set; }

		/// <summary>
		/// Only enumerate build products -- do not actually compile the projects.
		/// </summary>
		[TaskParameter(Optional = true)]
		public bool EnumerateOnly { get; set; }

		/// <summary>
		/// Tag to be applied to build products of this task.
		/// </summary>
		[TaskParameter(Optional = true, ValidationType = TaskParameterValidationType.TagList)]
		public string Tag { get; set; }

		/// <summary>
		/// Tag to be applied to any non-private references the projects have.
		/// (for example, those that are external and not copied into the output directory).
		/// </summary>
		[TaskParameter(Optional = true, ValidationType = TaskParameterValidationType.TagList)]
		public string TagReferences { get; set; }

		/// <summary>
		/// Whether to use the system toolchain rather than the bundled UE SDK
		/// </summary>
		[TaskParameter(Optional = true)]
		public bool UseSystemCompiler { get; set; }
	}

	/// <summary>
	/// Compiles C# project files, and their dependencies.
	/// </summary>
	[TaskElement("CsCompile", typeof(CsCompileTaskParameters))]
	public class CsCompileTask : BgTaskImpl
	{
		readonly CsCompileTaskParameters _parameters;

		/// <summary>
		/// Constructor.
		/// </summary>
		/// <param name="parameters">Parameters for this task</param>
		public CsCompileTask(CsCompileTaskParameters parameters)
		{
			_parameters = parameters;
		}

		/// <summary>
		/// ExecuteAsync the task.
		/// </summary>
		/// <param name="job">Information about the current job</param>
		/// <param name="buildProducts">Set of build products produced by this node.</param>
		/// <param name="tagNameToFileSet">Mapping from tag names to the set of files they include</param>
		public override Task ExecuteAsync(JobContext job, HashSet<FileReference> buildProducts, Dictionary<string, HashSet<FileReference>> tagNameToFileSet)
		{
			// Get the project file
			HashSet<FileReference> projectFiles = ResolveFilespec(Unreal.RootDirectory, _parameters.Project, tagNameToFileSet);
			foreach (FileReference projectFile in projectFiles)
			{
				if (!FileReference.Exists(projectFile))
				{
					throw new AutomationException("Couldn't find project file '{0}'", projectFile.FullName);
				}
				if (!projectFile.HasExtension(".csproj"))
				{
					throw new AutomationException("File '{0}' is not a C# project", projectFile.FullName);
				}
			}

			// Get the default properties
			Dictionary<string, string> properties = new Dictionary<string, string>(StringComparer.InvariantCultureIgnoreCase);
			if (!String.IsNullOrEmpty(_parameters.Platform))
			{
				properties["Platform"] = _parameters.Platform;
			}
			if (!String.IsNullOrEmpty(_parameters.Configuration))
			{
				properties["Configuration"] = _parameters.Configuration;
			}
			if (!String.IsNullOrEmpty(_parameters.Properties))
			{
				foreach (string property in _parameters.Properties.Split(';'))
				{
					if (!String.IsNullOrWhiteSpace(property))
					{
						int equalsIdx = property.IndexOf('=', StringComparison.Ordinal);
						if (equalsIdx == -1)
						{
							Logger.LogWarning("Missing '=' in property assignment");
						}
						else
						{
							properties[property.Substring(0, equalsIdx).Trim()] = property.Substring(equalsIdx + 1).Trim();
						}
					}
				}
			}

			// Build the arguments and run the build
			if (!_parameters.EnumerateOnly)
			{
				List<string> arguments = new List<string>();
				foreach (KeyValuePair<string, string> propertyPair in properties)
				{
					arguments.Add(String.Format("/property:{0}={1}", CommandUtils.MakePathSafeToUseWithCommandLine(propertyPair.Key), CommandUtils.MakePathSafeToUseWithCommandLine(propertyPair.Value)));
				}
				if (!String.IsNullOrEmpty(_parameters.Arguments))
				{
					arguments.Add(_parameters.Arguments);
				}
				if (!String.IsNullOrEmpty(_parameters.Target))
				{
					arguments.Add(String.Format("/target:{0}", CommandUtils.MakePathSafeToUseWithCommandLine(_parameters.Target)));
				}

				arguments.Add("/restore");
				arguments.Add("/verbosity:minimal");
				arguments.Add("/nologo");

				string joinedArguments = String.Join(" ", arguments);

				foreach (FileReference projectFile in projectFiles)
				{
					if (!FileReference.Exists(projectFile))
					{
						throw new AutomationException("Project {0} does not exist!", projectFile);
					}

					if (_parameters.UseSystemCompiler)
					{
						CommandUtils.MsBuild(CommandUtils.CmdEnv, projectFile.FullName, joinedArguments, null);
					}
					else
					{
						CommandUtils.RunAndLog(CommandUtils.CmdEnv, CommandUtils.CmdEnv.DotnetMsbuildPath, $"msbuild {CommandUtils.MakePathSafeToUseWithCommandLine(projectFile.FullName)} {joinedArguments}");
					}
				}
			}

			// Try to figure out the output files
			HashSet<FileReference> projectBuildProducts;
			HashSet<FileReference> projectReferences;
			properties["EngineDirectory"] = Unreal.EngineDirectory.FullName;
			FindBuildProductsAndReferences(projectFiles, properties, out projectBuildProducts, out projectReferences);

			// Apply the optional tag to the produced archive
			foreach (string tagName in FindTagNamesFromList(_parameters.Tag))
			{
				FindOrAddTagSet(tagNameToFileSet, tagName).UnionWith(projectBuildProducts);
			}

			// Apply the optional tag to any references
			if (!String.IsNullOrEmpty(_parameters.TagReferences))
			{
				foreach (string tagName in FindTagNamesFromList(_parameters.TagReferences))
				{
					FindOrAddTagSet(tagNameToFileSet, tagName).UnionWith(projectReferences);
				}
			}

			// Merge them into the standard set of build products
			buildProducts.UnionWith(projectBuildProducts);
			buildProducts.UnionWith(projectReferences);
			return Task.CompletedTask;
		}

		/// <summary>
		/// Output this task out to an XML writer.
		/// </summary>
		public override void Write(XmlWriter writer)
		{
			Write(writer, _parameters);
		}

		/// <summary>
		/// Find all the tags which are used as inputs to this task
		/// </summary>
		/// <returns>The tag names which are read by this task</returns>
		public override IEnumerable<string> FindConsumedTagNames()
		{
			return FindTagNamesFromFilespec(_parameters.Project);
		}

		/// <summary>
		/// Find all the tags which are modified by this task
		/// </summary>
		/// <returns>The tag names which are modified by this task</returns>
		public override IEnumerable<string> FindProducedTagNames()
		{
			foreach (string tagName in FindTagNamesFromList(_parameters.Tag))
			{
				yield return tagName;
			}

			foreach (string tagName in FindTagNamesFromList(_parameters.TagReferences))
			{
				yield return tagName;
			}
		}

		/// <summary>
		/// Find all the build products created by compiling the given project file
		/// </summary>
		/// <param name="projectFiles">Initial project file to read. All referenced projects will also be read.</param>
		/// <param name="initialProperties">Mapping of property name to value</param>
		/// <param name="outBuildProducts">Receives a set of build products on success</param>
		/// <param name="outReferences">Receives a set of non-private references on success</param>
		static void FindBuildProductsAndReferences(HashSet<FileReference> projectFiles, Dictionary<string, string> initialProperties, out HashSet<FileReference> outBuildProducts, out HashSet<FileReference> outReferences)
		{
			// Find all the build products and references
			outBuildProducts = new HashSet<FileReference>();
			outReferences = new HashSet<FileReference>();

			// Read all the project information into a dictionary
			Dictionary<FileReference, IDotnetProjectInfo> fileToProjectInfo = [];
			foreach (FileReference projectFile in projectFiles)
			{
				// Read all the projects
				ReadProjectsRecursively(projectFile, initialProperties, fileToProjectInfo);

				// Find all the outputs for each project
				foreach (KeyValuePair<FileReference, IDotnetProjectInfo> pair in fileToProjectInfo)
				{
					IDotnetProjectInfo projectInfo = pair.Value;

					// Add all the build projects from this project
					DirectoryReference outputDir = projectInfo.GetOutputDir(pair.Key.Directory);
					projectInfo.FindBuildProducts(outputDir, outBuildProducts, fileToProjectInfo);

					// Add any files which are only referenced
					foreach (KeyValuePair<FileReference, bool> reference in projectInfo.References)
					{
						projectInfo.AddReferencedAssemblyAndSupportFiles(reference.Key, outReferences);
					}
				}
			}

			outBuildProducts.RemoveWhere(x => !FileReference.Exists(x));
			outReferences.RemoveWhere(x => !FileReference.Exists(x));
		}

		/// <summary>
		/// Read a project file, plus all the project files it references.
		/// </summary>
		/// <param name="file">Project file to read</param>
		/// <param name="initialProperties">Mapping of property name to value for the initial project</param>
		/// <param name="fileToProjectInfo"></param>
		/// <returns>True if the projects were read correctly, false (and prints an error to the log) if not</returns>
		static void ReadProjectsRecursively(FileReference file, Dictionary<string, string> initialProperties, Dictionary<FileReference, IDotnetProjectInfo> fileToProjectInfo)
		{
			// Early out if we've already read this project
			if (!fileToProjectInfo.ContainsKey(file))
			{
				// Try to read this project
				DotnetProjectInfo projectInfo;
				if (!DotnetProjectInfo.TryRead(file, initialProperties, out projectInfo))
				{
					throw new AutomationException("Couldn't read project '{0}'", file.FullName);
				}

				// Add it to the project lookup, and try to read all the projects it references
				fileToProjectInfo.Add(file, projectInfo);
				foreach (FileReference projectReference in projectInfo.ProjectReferences.Keys)
				{
					if (!FileReference.Exists(projectReference))
					{
						throw new AutomationException("Unable to find project '{0}' referenced by '{1}'", projectReference, file);
					}
					ReadProjectsRecursively(projectReference, initialProperties, fileToProjectInfo);
				}
			}
		}
	}

	/// <summary>
	/// Output from compiling a csproj file
	/// </summary>
	public class CsCompileOutput
	{
		/// <summary>
		/// Empty instance of CsCompileOutput
		/// </summary>
		public static CsCompileOutput Empty { get; } = new CsCompileOutput(FileSet.Empty, FileSet.Empty);

		/// <summary>
		/// Output binaries
		/// </summary>
		public FileSet Binaries { get; }

		/// <summary>
		/// Referenced output
		/// </summary>
		public FileSet References { get; }

		/// <summary>
		/// Constructor
		/// </summary>
		public CsCompileOutput(FileSet binaries, FileSet references)
		{
			Binaries = binaries;
			References = references;
		}

		/// <summary>
		/// Merge all outputs from this project
		/// </summary>
		/// <returns></returns>
		public FileSet Merge()
		{
			return Binaries + References;
		}

		/// <summary>
		/// Merges two outputs together
		/// </summary>
		public static CsCompileOutput operator +(CsCompileOutput lhs, CsCompileOutput rhs)
		{
			return new CsCompileOutput(lhs.Binaries + rhs.Binaries, lhs.References + rhs.References);
		}
	}

	/// <summary>
	/// Extension methods for csproj compilation
	/// </summary>
	public static class CsCompileOutputExtensions
	{
		/// <summary>
		/// 
		/// </summary>
		/// <param name="task"></param>
		/// <returns></returns>
		public static async Task<FileSet> MergeAsync(this Task<CsCompileOutput> task)
		{
			return (await task).Merge();
		}
	}

	public static partial class StandardTasks
	{
		/// <summary>
		/// Compile a C# project
		/// </summary>
		/// <param name="project">The C# project files to compile.</param>
		/// <param name="platform">The platform to compile.</param>
		/// <param name="configuration">The configuration to compile.</param>
		/// <param name="target">The target to build.</param>
		/// <param name="properties">Properties for the command.</param>
		/// <param name="arguments">Additional options to pass to the compiler.</param>
		/// <param name="enumerateOnly">Only enumerate build products -- do not actually compile the projects.</param>
		public static async Task<CsCompileOutput> CsCompileAsync(FileReference project, string platform = null, string configuration = null, string target = null, string properties = null, string arguments = null, bool? enumerateOnly = null)
		{
			CsCompileTaskParameters parameters = new CsCompileTaskParameters();
			parameters.Project = project.FullName;
			parameters.Platform = platform;
			parameters.Configuration = configuration;
			parameters.Target = target;
			parameters.Properties = properties;
			parameters.Arguments = arguments;
			parameters.EnumerateOnly = enumerateOnly ?? parameters.EnumerateOnly;
			parameters.Tag = "#Out";
			parameters.TagReferences = "#Refs";

			HashSet<FileReference> buildProducts = new HashSet<FileReference>();
			Dictionary<string, HashSet<FileReference>> tagNameToFileSet = new Dictionary<string, HashSet<FileReference>>();
			await new CsCompileTask(parameters).ExecuteAsync(new JobContext(null!, null!), buildProducts, tagNameToFileSet);

			FileSet binaries = FileSet.Empty;
			FileSet references = FileSet.Empty;
			if (tagNameToFileSet.TryGetValue(parameters.Tag, out HashSet<FileReference> binaryFiles))
			{
				binaries = FileSet.FromFiles(Unreal.RootDirectory, binaryFiles);
			}
			if (tagNameToFileSet.TryGetValue(parameters.TagReferences, out HashSet<FileReference> referenceFiles))
			{
				references = FileSet.FromFiles(Unreal.RootDirectory, referenceFiles);
			}

			return new CsCompileOutput(binaries, references);
		}
	}
}
