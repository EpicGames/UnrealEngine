// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Linq;
using System.Threading.Tasks;
using System.Xml;
using EpicGames.Core;
using UnrealBuildBase;
using UnrealBuildTool;

namespace AutomationTool.Tasks
{
	/// <summary>
	/// Parameters for updating the derived data via a replay load
	/// </summary>
	public class DDCReplayLoadTaskParameters
	{
		/// <summary>
		/// Executable to spawn.
		/// </summary>
		[TaskParameter]
		public string Exe { get; set; }

		/// <summary>
		/// The project to run the tool with.
		/// </summary>
		[TaskParameter(Optional = true, ValidationType = TaskParameterValidationType.FileSpec)]
		public string Project { get; set; }

		/// <summary>
		/// The DDC replay load file or tag set to use.
		/// </summary>
		[TaskParameter]
		public string DDCReplayFile { get; set; }

		/// <summary>
		/// Arguments for the newly created process.
		/// </summary>
		[TaskParameter(Optional = true)]
		public string Arguments { get; set; }

		/// <summary>
		/// Working directory for spawning the new task
		/// </summary>
		[TaskParameter(Optional = true)]
		public string WorkingDir { get; set; }

		/// <summary>
		/// Environment variables to set
		/// </summary>
		[TaskParameter(Optional = true)]
		public string Environment { get; set; }

		/// <summary>
		/// File to read environment from
		/// </summary>
		[TaskParameter(Optional = true)]
		public string EnvironmentFile { get; set; }

		/// <summary>
		/// Write output to the log
		/// </summary>
		[TaskParameter(Optional = true)]
		public bool LogOutput { get; set; } = true;

		/// <summary>
		/// The minimum exit code, which is treated as an error.
		/// </summary>
		[TaskParameter(Optional = true)]
		public int ErrorLevel { get; set; } = 1;
	}

	/// <summary>
	/// Spawns an external executable to UpdateDerivedData and waits for it to complete.
	/// </summary>
	[TaskElement("DDCReplayLoad", typeof(DDCReplayLoadTaskParameters))]
	public class DDCReplayLoadTask : SpawnTaskBase
	{
		/// <summary>
		/// Parameters for this task
		/// </summary>
		readonly DDCReplayLoadTaskParameters _parameters;

		/// <summary>
		/// Construct a spawn task
		/// </summary>
		/// <param name="parameters">Parameters for the task</param>
		public DDCReplayLoadTask(DDCReplayLoadTaskParameters parameters)
		{
			_parameters = parameters;
		}

		/// <summary>
		/// ExecuteAsync the task.
		/// </summary>
		/// <param name="job">Information about the current job</param>
		/// <param name="buildProducts">Set of build products produced by this node.</param>
		/// <param name="tagNameToFileSet">Mapping from tag names to the set of files they include</param>
		public override async Task ExecuteAsync(JobContext job, HashSet<FileReference> buildProducts, Dictionary<string, HashSet<FileReference>> tagNameToFileSet)
		{
			FileReference projectFile = null;
			if (!String.IsNullOrEmpty(_parameters.Project))
			{
				if (_parameters.Project.EndsWith(".uproject", StringComparison.OrdinalIgnoreCase))
				{
					projectFile = ResolveFile(_parameters.Project);
				}
				else
				{
					projectFile = NativeProjects.EnumerateProjectFiles(Log.Logger).FirstOrDefault(x => x.GetFileNameWithoutExtension().Equals(_parameters.Project, StringComparison.OrdinalIgnoreCase));
				}

				if (projectFile == null || !FileReference.Exists(projectFile))
				{
					throw new BuildException("Unable to resolve project '{0}'", _parameters.Project);
				}
			}

			string commonArgs = String.Format("{0}CreateCache {1}{2}", projectFile != null ? $"{projectFile.FullName} " : "", _parameters.Arguments, CommandUtils.IsBuildMachine ? " -buildmachine" : "");

			if (_parameters.DDCReplayFile.StartsWith("#", StringComparison.OrdinalIgnoreCase))
			{
				HashSet<FileReference> files = ResolveFilespec(Unreal.RootDirectory, _parameters.DDCReplayFile, tagNameToFileSet);
				foreach (FileReference file in files)
				{
					string taskArguments = $"{commonArgs} -DDC-ReplayLoad={file}";
					await ExecuteAsync(_parameters.Exe, taskArguments, _parameters.WorkingDir, envVars: ParseEnvVars(_parameters.Environment, _parameters.EnvironmentFile), logOutput: _parameters.LogOutput, errorLevel: _parameters.ErrorLevel);
				}
			}
			else
			{
				string taskArguments = $"{commonArgs} -DDC-ReplayLoad={_parameters.DDCReplayFile}";
				await ExecuteAsync(_parameters.Exe, taskArguments, _parameters.WorkingDir, envVars: ParseEnvVars(_parameters.Environment, _parameters.EnvironmentFile), logOutput: _parameters.LogOutput, errorLevel: _parameters.ErrorLevel);
			}
		}

		/// <summary>
		/// Output this task out to an XML writer.
		/// </summary>
		public override void Write(XmlWriter writer)
		{
			Write(writer, _parameters);
		}

		/// <summary>
		/// Find all the tags which are used as inputs to this task
		/// </summary>
		/// <returns>The tag names which are read by this task</returns>
		public override IEnumerable<string> FindConsumedTagNames()
		{
			yield break;
		}

		/// <summary>
		/// Find all the tags which are modified by this task
		/// </summary>
		/// <returns>The tag names which are modified by this task</returns>
		public override IEnumerable<string> FindProducedTagNames()
		{
			yield break;
		}
	}
}
