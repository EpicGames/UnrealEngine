// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.IO;
using System.Linq;
using System.Threading.Tasks;
using System.Xml;
using EpicGames.Core;
using Microsoft.Extensions.Logging;
using UnrealBuildBase;

namespace AutomationTool.Tasks
{
	/// <summary>
	/// Parameters for a copy task
	/// </summary>
	public class DeleteTaskParameters
	{
		/// <summary>
		/// List of file specifications separated by semicolons (for example, *.cpp;Engine/.../*.bat), or the name of a tag set
		/// </summary>
		[TaskParameter(Optional = true, ValidationType = TaskParameterValidationType.FileSpec)]
		public string Files { get; set; }

		/// <summary>
		/// List of directory names
		/// </summary>
		[TaskParameter(Optional = true)]
		public string Directories { get; set; }

		/// <summary>
		/// Whether to delete empty directories after deleting the files. Defaults to true.
		/// </summary>
		[TaskParameter(Optional = true)]
		public bool DeleteEmptyDirectories { get; set; } = true;

		/// <summary>
		/// Whether or not to use verbose logging.
		/// </summary>
		[TaskParameter(Optional = true)]
		public bool Verbose { get; set; } = false;
	}

	/// <summary>
	/// Delete a set of files.
	/// </summary>
	[TaskElement("Delete", typeof(DeleteTaskParameters))]
	public class DeleteTask : BgTaskImpl
	{
		readonly DeleteTaskParameters _parameters;

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="parameters">Parameters for this task</param>
		public DeleteTask(DeleteTaskParameters parameters)
		{
			_parameters = parameters;
		}

		/// <summary>
		/// ExecuteAsync the task.
		/// </summary>
		/// <param name="job">Information about the current job</param>
		/// <param name="buildProducts">Set of build products produced by this node.</param>
		/// <param name="tagNameToFileSet">Mapping from tag names to the set of files they include</param>
		public override Task ExecuteAsync(JobContext job, HashSet<FileReference> buildProducts, Dictionary<string, HashSet<FileReference>> tagNameToFileSet)
		{
			if (_parameters.Files != null)
			{
				// Find all the referenced files and delete them
				HashSet<FileReference> files = ResolveFilespec(Unreal.RootDirectory, _parameters.Files, tagNameToFileSet);
				foreach (FileReference file in files)
				{
					if (_parameters.Verbose)
					{
						Logger.LogInformation("Deleting {File}", file.FullName);
					}
					if (!InternalUtils.SafeDeleteFile(file.FullName))
					{
						Logger.LogWarning("Couldn't delete file {Arg0}", file.FullName);
					}
				}

				// Try to delete all the parent directories. Keep track of the directories we've already deleted to avoid hitting the disk.
				if (_parameters.DeleteEmptyDirectories)
				{
					// Find all the directories that we're touching
					HashSet<DirectoryReference> parentDirectories = new HashSet<DirectoryReference>();
					foreach (FileReference file in files)
					{
						parentDirectories.Add(file.Directory);
					}

					// Recurse back up from each of those directories to the root folder
					foreach (DirectoryReference parentDirectory in parentDirectories)
					{
						for (DirectoryReference currentDirectory = parentDirectory; currentDirectory != Unreal.RootDirectory; currentDirectory = currentDirectory.ParentDirectory)
						{
							if (!TryDeleteEmptyDirectory(currentDirectory))
							{
								break;
							}
						}
					}
				}
			}
			if (_parameters.Directories != null)
			{
				foreach (string directory in _parameters.Directories.Split(';'))
				{
					if (!String.IsNullOrEmpty(directory))
					{
						if (_parameters.Verbose)
						{
							Logger.LogInformation("Deleting {Directory}", directory);
						}
						DirectoryReference fullDir = new DirectoryReference(directory);
						if (DirectoryReference.Exists(fullDir))
						{
							FileUtils.ForceDeleteDirectory(fullDir);
						}
					}
				}
			}
			return Task.CompletedTask;
		}

		/// <summary>
		/// Deletes a directory, if it's empty
		/// </summary>
		/// <param name="candidateDirectory">The directory to check</param>
		/// <returns>True if the directory was deleted, false if not</returns>
		static bool TryDeleteEmptyDirectory(DirectoryReference candidateDirectory)
		{
			// Make sure the directory exists
			if (!DirectoryReference.Exists(candidateDirectory))
			{
				return false;
			}

			// Check if there are any files in it. If there are, don't bother trying to delete it.
			if (Directory.EnumerateFiles(candidateDirectory.FullName).Any() || Directory.EnumerateDirectories(candidateDirectory.FullName).Any())
			{
				return false;
			}

			// Try to delete the directory.
			try
			{
				Directory.Delete(candidateDirectory.FullName);
				return true;
			}
			catch (Exception ex)
			{
				Logger.LogWarning("Couldn't delete directory {Arg0} ({Arg1})", candidateDirectory.FullName, ex.Message);
				return false;
			}
		}

		/// <summary>
		/// Output this task out to an XML writer.
		/// </summary>
		public override void Write(XmlWriter writer)
		{
			Write(writer, _parameters);
		}

		/// <summary>
		/// Find all the tags which are used as inputs to this task
		/// </summary>
		/// <returns>The tag names which are read by this task</returns>
		public override IEnumerable<string> FindConsumedTagNames()
		{
			return FindTagNamesFromFilespec(_parameters.Files);
		}

		/// <summary>
		/// Find all the tags which are modified by this task
		/// </summary>
		/// <returns>The tag names which are modified by this task</returns>
		public override IEnumerable<string> FindProducedTagNames()
		{
			yield break;
		}
	}
}
