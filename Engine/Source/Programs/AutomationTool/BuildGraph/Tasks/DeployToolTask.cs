// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.IO;
using System.Text.Json;
using System.Threading.Tasks;
using System.Xml;
using EpicGames.Core;
using EpicGames.Horde;
using EpicGames.Horde.Server;
using EpicGames.Horde.Storage;
using EpicGames.Horde.Storage.Nodes;
using EpicGames.Horde.Tools;
using Microsoft.Extensions.DependencyInjection;
using Microsoft.Extensions.Logging;

#nullable enable

namespace AutomationTool.Tasks
{
	/// <summary>
	/// Parameters for a DeployTool task
	/// </summary>
	public class DeployToolTaskParameters
	{
		/// <summary>
		/// Identifier for the tool
		/// </summary>
		[TaskParameter]
		public string Id { get; set; } = String.Empty;

		/// <summary>
		/// Settings file to use for the deployment. Should be a JSON file containing server name and access token.
		/// </summary>
		[TaskParameter(Optional = true)]
		public string Settings { get; set; } = String.Empty;

		/// <summary>
		/// Version number for the new tool
		/// </summary>
		[TaskParameter]
		public string Version { get; set; } = String.Empty;

		/// <summary>
		/// Duration over which to roll out the tool, in minutes.
		/// </summary>
		[TaskParameter(Optional = true)]
		public int Duration { get; set; } = 0;

		/// <summary>
		/// Whether to create the deployment as paused
		/// </summary>
		[TaskParameter(Optional = true)]
		public bool Paused { get; set; } = false;

		/// <summary>
		/// Zip file containing files to upload
		/// </summary>
		[TaskParameter(Optional = true)]
		public string? File { get; set; } = null!;

		/// <summary>
		/// Directory to upload for the tool
		/// </summary>
		[TaskParameter(Optional = true)]
		public string? Directory { get; set; } = null!;
	}

	/// <summary>
	/// Deploys a tool update through Horde
	/// </summary>
	[TaskElement("DeployTool", typeof(DeployToolTaskParameters))]
	public class DeployToolTask : SpawnTaskBase
	{
		class DeploySettings
		{
			public string Server { get; set; } = String.Empty;
			public string? Token { get; set; }
		}

		/// <summary>
		/// Options for a new deployment
		/// </summary>
		class CreateDeploymentRequest
		{
			public string Version { get; set; } = "Unknown";
			public double? Duration { get; set; }
			public bool? CreatePaused { get; set; }
			public string? Node { get; set; }
		}

		readonly DeployToolTaskParameters _parameters;

		/// <summary>
		/// Construct a Helm task
		/// </summary>
		/// <param name="parameters">Parameters for the task</param>
		public DeployToolTask(DeployToolTaskParameters parameters)
		{
			_parameters = parameters;
		}

		/// <summary>
		/// ExecuteAsync the task.
		/// </summary>
		/// <param name="job">Information about the current job</param>
		/// <param name="buildProducts">Set of build products produced by this node.</param>
		/// <param name="tagNameToFileSet">Mapping from tag names to the set of files they include</param>
		public override async Task ExecuteAsync(JobContext job, HashSet<FileReference> buildProducts, Dictionary<string, HashSet<FileReference>> tagNameToFileSet)
		{
			DeploySettings? settings = null;
			if (!String.IsNullOrEmpty(_parameters.Settings))
			{
				FileReference settingsFile = ResolveFile(_parameters.Settings);
				if (!FileReference.Exists(settingsFile))
				{
					throw new AutomationException($"Settings file '{settingsFile}' does not exist");
				}

				byte[] settingsData = await FileReference.ReadAllBytesAsync(settingsFile);
				JsonSerializerOptions jsonOptions = new JsonSerializerOptions { AllowTrailingCommas = true, ReadCommentHandling = JsonCommentHandling.Skip, PropertyNameCaseInsensitive = true };

				settings = JsonSerializer.Deserialize<DeploySettings>(settingsData, jsonOptions);
				if (settings == null)
				{
					throw new AutomationException($"Unable to read settings file {settingsFile}");
				}
				else if (settings.Server == null)
				{
					throw new AutomationException($"Missing 'server' key from {settingsFile}");
				}
			}

			ToolId toolId = new ToolId(_parameters.Id);

			ServiceCollection serviceCollection = new ServiceCollection();
			serviceCollection.Configure<HordeOptions>(options =>
			{
				if (!String.IsNullOrEmpty(settings?.Server))
				{
					options.ServerUrl = new Uri(settings.Server);
				}
				if (!String.IsNullOrEmpty(settings?.Token))
				{
					options.AccessToken = settings.Token;
				}
			});
			serviceCollection.AddLogging(builder => builder.AddProvider(CommandUtils.ServiceProvider.GetRequiredService<ILoggerProvider>()));
			serviceCollection.AddHttpClient();
			serviceCollection.AddHorde();

			await using ServiceProvider serviceProvider = serviceCollection.BuildServiceProvider();
			IHordeClient hordeClient = (settings == null) 
				? CommandUtils.ServiceProvider.GetRequiredService<IHordeClient>()
				: serviceProvider.GetRequiredService<IHordeClient>();

			using HordeHttpClient hordeHttpClient = hordeClient.CreateHttpClient();

			GetServerInfoResponse infoResponse = await hordeHttpClient.GetServerInfoAsync();
			Logger.LogInformation("Uploading {ToolId} to {ServerUrl} (Version: {Version}, API v{ApiVersion})...", toolId, hordeClient.ServerUrl, infoResponse.ServerVersion, (int)infoResponse.ApiVersion);

			BlobSerializerOptions serializerOptions = BlobSerializerOptions.Create(infoResponse.ApiVersion);

			IHashedBlobRef handle;

			IStorageNamespace storageNamespace = hordeClient.GetStorageNamespace(toolId);
			await using (IBlobWriter blobWriter = storageNamespace.CreateBlobWriter($"{toolId}", serializerOptions: serializerOptions))
			{
				DirectoryNode sandbox = new DirectoryNode();
				if (_parameters.File != null)
				{
					using FileStream stream = FileReference.Open(ResolveFile(_parameters.File), FileMode.Open, FileAccess.Read);
					await sandbox.CopyFromZipStreamAsync(stream, blobWriter, new ChunkingOptions());
				}
				else if (_parameters.Directory != null)
				{
					DirectoryInfo directoryInfo = ResolveDirectory(_parameters.Directory).ToDirectoryInfo();
					await sandbox.AddFilesAsync(directoryInfo, blobWriter);
				}
				else
				{
					throw new AutomationException("Either File=... or Directory=... must be specified");
				}
				handle = await blobWriter.WriteBlobAsync(sandbox);
				await blobWriter.FlushAsync();
			}

			double? duration = null;
			if (_parameters.Duration != 0)
			{
				duration = _parameters.Duration;
			}

			bool? createPaused = null;
			if (_parameters.Paused)
			{
				createPaused = true;
			}

			HashedBlobRefValue locator = handle.GetRefValue();
			ToolDeploymentId deploymentId = await hordeHttpClient.CreateToolDeploymentAsync(toolId, _parameters.Version, duration, createPaused, locator);
			Logger.LogInformation("Created {ToolId} deployment {DeploymentId}", toolId, deploymentId);
		}

		/// <summary>
		/// Output this task out to an XML writer.
		/// </summary>
		public override void Write(XmlWriter writer)
		{
			Write(writer, _parameters);
		}

		/// <summary>
		/// Find all the tags which are used as inputs to this task
		/// </summary>
		/// <returns>The tag names which are read by this task</returns>
		public override IEnumerable<string> FindConsumedTagNames()
		{
			yield break;
		}

		/// <summary>
		/// Find all the tags which are modified by this task
		/// </summary>
		/// <returns>The tag names which are modified by this task</returns>
		public override IEnumerable<string> FindProducedTagNames()
		{
			yield break;
		}
	}
}
