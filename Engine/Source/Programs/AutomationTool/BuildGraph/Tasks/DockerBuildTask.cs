// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Linq;
using System.Text;
using System.Text.RegularExpressions;
using System.Threading.Tasks;
using System.Xml;
using EpicGames.Core;
using Microsoft.Extensions.Logging;
using UnrealBuildBase;

namespace AutomationTool.Tasks
{
	/// <summary>
	/// Parameters for a Docker-Build task
	/// </summary>
	public class DockerBuildTaskParameters
	{
		/// <summary>
		/// Base directory for the build
		/// </summary>
		[TaskParameter]
		public string BaseDir { get; set; }

		/// <summary>
		/// Files to be staged before building the image
		/// </summary>
		[TaskParameter]
		public string Files { get; set; }

		/// <summary>
		/// Path to the Dockerfile. Uses the root of basedir if not specified.
		/// </summary>
		[TaskParameter(Optional = true)]
		public string DockerFile { get; set; }

		/// <summary>
		/// Path to a .dockerignore. Will be copied to basedir if specified.
		/// </summary>
		[TaskParameter(Optional = true)]
		public string DockerIgnoreFile { get; set; }

		/// <summary>
		/// Use BuildKit in Docker
		/// </summary>
		[TaskParameter(Optional = true)]
		public bool UseBuildKit { get; set; }
		
		/// <summary>
		/// Set ulimit in build
		/// </summary>
		[TaskParameter(Optional = true)]
		public string Ulimit { get; set; } = "nofile=100000:100000";

		/// <summary>
		/// Type of progress output (--progress)
		/// </summary>
		[TaskParameter(Optional = true)]
		public string ProgressOutput { get; set; }

		/// <summary>
		/// Tag for the image
		/// </summary>
		[TaskParameter(Optional = true)]
		public string Tag { get; set; }

		/// <summary>
		/// Set the target build stage to build (--target)
		/// </summary>
		[TaskParameter(Optional = true)]
		public string Target { get; set; }

		/// <summary>
		/// Custom output exporter. Requires BuildKit (--output)
		/// </summary>
		[TaskParameter(Optional = true)]
		public string Output { get; set; }

		/// <summary>
		/// Optional arguments
		/// </summary>
		[TaskParameter(Optional = true)]
		public string Arguments { get; set; }

		/// <summary>
		/// List of additional directories to overlay into the staged input files. Allows credentials to be staged, etc...
		/// </summary>
		[TaskParameter(Optional = true)]
		public string OverlayDirs { get; set; }

		/// <summary>
		/// Environment variables to set
		/// </summary>
		[TaskParameter(Optional = true)]
		public string Environment { get; set; }

		/// <summary>
		/// File to read environment variables from
		/// </summary>
		[TaskParameter(Optional = true)]
		public string EnvironmentFile { get; set; }
	}

	/// <summary>
	/// Spawns Docker and waits for it to complete.
	/// </summary>
	[TaskElement("Docker-Build", typeof(DockerBuildTaskParameters))]
	public class DockerBuildTask : SpawnTaskBase
	{
		readonly DockerBuildTaskParameters _parameters;

		/// <summary>
		/// Construct a Docker task
		/// </summary>
		/// <param name="parameters">Parameters for the task</param>
		public DockerBuildTask(DockerBuildTaskParameters parameters)
		{
			_parameters = parameters;
		}

		/// <summary>
		/// ExecuteAsync the task.
		/// </summary>
		/// <param name="job">Information about the current job</param>
		/// <param name="buildProducts">Set of build products produced by this node.</param>
		/// <param name="tagNameToFileSet">Mapping from tag names to the set of files they include</param>
		public override async Task ExecuteAsync(JobContext job, HashSet<FileReference> buildProducts, Dictionary<string, HashSet<FileReference>> tagNameToFileSet)
		{
			Logger.LogInformation("Building Docker image");
			using (LogIndentScope scope = new LogIndentScope("  "))
			{
				DirectoryReference baseDir = ResolveDirectory(_parameters.BaseDir);
				List<FileReference> sourceFiles = ResolveFilespec(baseDir, _parameters.Files, tagNameToFileSet).ToList();
				bool isStagingEnabled = sourceFiles.Count > 0;

				DirectoryReference stagingDir = DirectoryReference.Combine(Unreal.EngineDirectory, "Intermediate", "Docker");
				FileUtils.ForceDeleteDirectoryContents(stagingDir);

				List<FileReference> targetFiles = sourceFiles.ConvertAll(x => FileReference.Combine(stagingDir, x.MakeRelativeTo(baseDir)));
				CommandUtils.ThreadedCopyFiles(sourceFiles, baseDir, stagingDir);

				FileReference dockerIgnoreFileInBaseDir = FileReference.Combine(baseDir, ".dockerignore");
				FileReference.Delete(dockerIgnoreFileInBaseDir);

				if (!String.IsNullOrEmpty(_parameters.OverlayDirs))
				{
					foreach (string overlayDir in _parameters.OverlayDirs.Split(';'))
					{
						CommandUtils.ThreadedCopyFiles(ResolveDirectory(overlayDir), stagingDir);
					}
				}

				StringBuilder arguments = new StringBuilder("build .");
				if (_parameters.Tag != null)
				{
					arguments.Append($" -t {_parameters.Tag}");
				}
				if (_parameters.Target != null)
				{
					arguments.Append($" --target {_parameters.Target}");
				}
				if (_parameters.Output != null)
				{
					if (!_parameters.UseBuildKit)
					{
						throw new AutomationException($"{nameof(_parameters.UseBuildKit)} must be enabled to use '{nameof(_parameters.Output)}' parameter");
					}
					arguments.Append($" --output {_parameters.Output}");
				}
				if (_parameters.DockerFile != null)
				{
					FileReference dockerFile = ResolveFile(_parameters.DockerFile);
					if (!dockerFile.IsUnderDirectory(baseDir))
					{
						throw new AutomationException($"Dockerfile '{dockerFile}' is not under base directory ({baseDir})");
					}
					arguments.Append($" -f {dockerFile.MakeRelativeTo(baseDir).QuoteArgument()}");
				}
				if (_parameters.DockerIgnoreFile != null)
				{
					FileReference dockerIgnoreFile = ResolveFile(_parameters.DockerIgnoreFile);
					FileReference.Copy(dockerIgnoreFile, dockerIgnoreFileInBaseDir);
				}
				if (_parameters.ProgressOutput != null)
				{
					arguments.Append($" --progress={_parameters.ProgressOutput}");
				}
				if (_parameters.Ulimit != null && _parameters.Ulimit.Length > 1)
				{
					arguments.Append($" --ulimit={_parameters.Ulimit}");
				}
				if (_parameters.Arguments != null)
				{
					arguments.Append($" {_parameters.Arguments}");
				}

				Dictionary<string, string> envVars = ParseEnvVars(_parameters.Environment, _parameters.EnvironmentFile);
				if (_parameters.UseBuildKit)
				{
					envVars["DOCKER_BUILDKIT"] = "1";
				}

				string workingDir = isStagingEnabled ? stagingDir.FullName : baseDir.FullName;
				string exe = DockerTask.GetDockerExecutablePath();
				await SpawnTaskBase.ExecuteAsync(exe, arguments.ToString(), envVars: envVars, workingDir: workingDir, spewFilterCallback: FilterOutput);
			}
		}

		static readonly Regex s_filterOutputPattern = new Regex(@"^#\d+ (?:\d+\.\d+ )?");

		static string FilterOutput(string line) => s_filterOutputPattern.Replace(line, "");

		/// <summary>
		/// Output this task out to an XML writer.
		/// </summary>
		public override void Write(XmlWriter writer)
		{
			Write(writer, _parameters);
		}

		/// <summary>
		/// Find all the tags which are used as inputs to this task
		/// </summary>
		/// <returns>The tag names which are read by this task</returns>
		public override IEnumerable<string> FindConsumedTagNames()
		{
			List<string> tagNames = new List<string>();
			tagNames.AddRange(FindTagNamesFromFilespec(_parameters.DockerFile));
			tagNames.AddRange(FindTagNamesFromFilespec(_parameters.Files));
			return tagNames;
		}

		/// <summary>
		/// Find all the tags which are modified by this task
		/// </summary>
		/// <returns>The tag names which are modified by this task</returns>
		public override IEnumerable<string> FindProducedTagNames()
		{
			yield break;
		}
	}
}
