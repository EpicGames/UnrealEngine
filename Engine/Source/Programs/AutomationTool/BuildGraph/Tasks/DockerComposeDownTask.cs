// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Linq;
using System.Text;
using System.Threading.Tasks;
using System.Xml;
using EpicGames.Core;
using Microsoft.Extensions.Logging;

namespace AutomationTool.Tasks
{
	/// <summary>
	/// Parameters for a Docker-Compose task
	/// </summary>
	public class DockerComposeDownTaskParameters
	{
		/// <summary>
		/// Path to the docker-compose file
		/// </summary>
		[TaskParameter]
		public string File { get; set; }

		/// <summary>
		/// Arguments for the command
		/// </summary>
		[TaskParameter(Optional = true)]
		public string Arguments { get; set; }
	}

	/// <summary>
	/// Spawns Docker and waits for it to complete.
	/// </summary>
	[TaskElement("Docker-Compose-Down", typeof(DockerComposeDownTaskParameters))]
	public class DockerComposeDownTask : SpawnTaskBase
	{
		/// <summary>
		/// Parameters for this task
		/// </summary>
		readonly DockerComposeDownTaskParameters _parameters;

		/// <summary>
		/// Construct a Docker-Compose task
		/// </summary>
		/// <param name="parameters">Parameters for the task</param>
		public DockerComposeDownTask(DockerComposeDownTaskParameters parameters)
		{
			_parameters = parameters;
		}

		/// <summary>
		/// ExecuteAsync the task.
		/// </summary>
		/// <param name="job">Information about the current job</param>
		/// <param name="buildProducts">Set of build products produced by this node.</param>
		/// <param name="tagNameToFileSet">Mapping from tag names to the set of files they include</param>
		public override async Task ExecuteAsync(JobContext job, HashSet<FileReference> buildProducts, Dictionary<string, HashSet<FileReference>> tagNameToFileSet)
		{
			StringBuilder arguments = new StringBuilder("--ansi never ");
			if (!String.IsNullOrEmpty(_parameters.File))
			{
				arguments.Append($"--file {_parameters.File.QuoteArgument()} ");
			}
			arguments.Append("down");
			if (!String.IsNullOrEmpty(_parameters.Arguments))
			{
				arguments.Append($" {_parameters.Arguments}");
			}

			Logger.LogInformation("Running docker compose {Arguments}", arguments.ToString());
			using (LogIndentScope scope = new LogIndentScope("  "))
			{
				await SpawnTaskBase.ExecuteAsync("docker-compose", arguments.ToString());
			}
		}

		/// <summary>
		/// Output this task out to an XML writer.
		/// </summary>
		public override void Write(XmlWriter writer)
		{
			Write(writer, _parameters);
		}

		/// <summary>
		/// Find all the tags which are used as inputs to this task
		/// </summary>
		/// <returns>The tag names which are read by this task</returns>
		public override IEnumerable<string> FindConsumedTagNames()
		{
			return Enumerable.Empty<string>();
		}

		/// <summary>
		/// Find all the tags which are modified by this task
		/// </summary>
		/// <returns>The tag names which are modified by this task</returns>
		public override IEnumerable<string> FindProducedTagNames()
		{
			return Enumerable.Empty<string>();
		}
	}
}
