// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Text.Json;
using System.Threading.Tasks;
using System.Xml;
using EpicGames.Core;
using Microsoft.Extensions.Logging;

namespace AutomationTool.Tasks
{
	/// <summary>
	/// Parameters for a Docker-Build task
	/// </summary>
	public class DockerPushTaskParameters
	{
		/// <summary>
		/// Repository
		/// </summary>
		[TaskParameter]
		public string Repository { get; set; }

		/// <summary>
		/// Source image to push
		/// </summary>
		[TaskParameter]
		public string Image { get; set; }

		/// <summary>
		/// Name of the target image
		/// </summary>
		[TaskParameter(Optional = true)]
		public string TargetImage { get; set; }

		/// <summary>
		/// Additional environment variables
		/// </summary>
		[TaskParameter(Optional = true)]
		public string Environment { get; set; }

		/// <summary>
		/// File to read environment from
		/// </summary>
		[TaskParameter(Optional = true)]
		public string EnvironmentFile { get; set; }

		/// <summary>
		/// Whether to login to AWS ECR
		/// </summary>
		[TaskParameter(Optional = true)]
		public bool AwsEcr { get; set; }

		/// <summary>
		/// Path to a json file for authentication to the repository for pushing.
		/// </summary>
		[TaskParameter(Optional = true)]
		public string RepositoryAuthFile { get; set; }
	}

	/// <summary>
	/// Spawns Docker and waits for it to complete.
	/// </summary>
	[TaskElement("Docker-Push", typeof(DockerPushTaskParameters))]
	public class DockerPushTask : SpawnTaskBase
	{
		readonly DockerPushTaskParameters _parameters;

		/// <summary>
		/// Construct a Docker task
		/// </summary>
		/// <param name="parameters">Parameters for the task</param>
		public DockerPushTask(DockerPushTaskParameters parameters)
		{
			_parameters = parameters;
		}

		/// <summary>
		/// ExecuteAsync the task.
		/// </summary>
		/// <param name="job">Information about the current job</param>
		/// <param name="buildProducts">Set of build products produced by this node.</param>
		/// <param name="tagNameToFileSet">Mapping from tag names to the set of files they include</param>
		public override async Task ExecuteAsync(JobContext job, HashSet<FileReference> buildProducts, Dictionary<string, HashSet<FileReference>> tagNameToFileSet)
		{
			Logger.LogInformation("Pushing Docker image");
			using (LogIndentScope scope = new LogIndentScope("  "))
			{
				string exe = DockerTask.GetDockerExecutablePath();
				Dictionary<string, string> environment = ParseEnvVars(_parameters.Environment, _parameters.EnvironmentFile);

				if (_parameters.AwsEcr)
				{
					IProcessResult result = await SpawnTaskBase.ExecuteAsync("aws", "ecr get-login-password", envVars: environment, logOutput: false);
					await ExecuteAsync(exe, $"login {_parameters.Repository} --username AWS --password-stdin", input: result.Output);
				}
				if (!String.IsNullOrEmpty(_parameters.RepositoryAuthFile))
				{
					string repositoryText = CommandUtils.ReadAllText(_parameters.RepositoryAuthFile);
					Dictionary<string, string> authDict = JsonSerializer.Deserialize<Dictionary<string, string>>(repositoryText);
					await ExecuteAsync(exe, $"login {_parameters.Repository} --username {authDict["Username"]} --password-stdin", input: authDict["Token"]);
				}

				string targetImage = _parameters.TargetImage ?? _parameters.Image;
				await ExecuteAsync(exe, $"tag {_parameters.Image} {_parameters.Repository}/{targetImage}", envVars: environment);
				await ExecuteAsync(exe, $"push {_parameters.Repository}/{targetImage}", envVars: environment);
			}
		}

		/// <summary>
		/// Output this task out to an XML writer.
		/// </summary>
		public override void Write(XmlWriter writer)
		{
			Write(writer, _parameters);
		}

		/// <summary>
		/// Find all the tags which are used as inputs to this task
		/// </summary>
		/// <returns>The tag names which are read by this task</returns>
		public override IEnumerable<string> FindConsumedTagNames()
		{
			yield break;
		}

		/// <summary>
		/// Find all the tags which are modified by this task
		/// </summary>
		/// <returns>The tag names which are modified by this task</returns>
		public override IEnumerable<string> FindProducedTagNames()
		{
			yield break;
		}
	}
}
