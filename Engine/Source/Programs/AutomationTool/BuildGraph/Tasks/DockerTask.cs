// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Threading.Tasks;
using System.Xml;
using EpicGames.Core;

namespace AutomationTool.Tasks
{
	/// <summary>
	/// Parameters for a Docker task
	/// </summary>
	public class DockerTaskParameters
	{
		/// <summary>
		/// Docker command line arguments
		/// </summary>
		[TaskParameter]
		public string Arguments { get; set; }

		/// <summary>
		/// Environment variables to set
		/// </summary>
		[TaskParameter(Optional = true)]
		public string Environment { get; set; }

		/// <summary>
		/// File to read environment variables from
		/// </summary>
		[TaskParameter(Optional = true)]
		public string EnvironmentFile { get; set; }

		/// <summary>
		/// Base directory for running the command
		/// </summary>
		[TaskParameter(Optional = true)]
		public string WorkingDir { get; set; }
	}

	/// <summary>
	/// Spawns Docker and waits for it to complete.
	/// </summary>
	[TaskElement("Docker", typeof(DockerTaskParameters))]
	public class DockerTask : SpawnTaskBase
	{
		/// <summary>
		/// Parameters for this task
		/// </summary>
		readonly DockerTaskParameters _parameters;

		/// <summary>
		/// Construct a Docker task
		/// </summary>
		/// <param name="parameters">Parameters for the task</param>
		public DockerTask(DockerTaskParameters parameters)
		{
			_parameters = parameters;
		}

		/// <summary>
		/// ExecuteAsync the task.
		/// </summary>
		/// <param name="job">Information about the current job</param>
		/// <param name="buildProducts">Set of build products produced by this node.</param>
		/// <param name="tagNameToFileSet">Mapping from tag names to the set of files they include</param>
		public override async Task ExecuteAsync(JobContext job, HashSet<FileReference> buildProducts, Dictionary<string, HashSet<FileReference>> tagNameToFileSet)
		{
			await ExecuteAsync(GetDockerExecutablePath(), _parameters.Arguments, envVars: ParseEnvVars(_parameters.Environment, _parameters.EnvironmentFile), workingDir: _parameters.WorkingDir);
		}

		/// <summary>
		/// Output this task out to an XML writer.
		/// </summary>
		public override void Write(XmlWriter writer)
		{
			Write(writer, _parameters);
		}

		/// <summary>
		/// Find all the tags which are used as inputs to this task
		/// </summary>
		/// <returns>The tag names which are read by this task</returns>
		public override IEnumerable<string> FindConsumedTagNames()
		{
			yield break;
		}

		/// <summary>
		/// Find all the tags which are modified by this task
		/// </summary>
		/// <returns>The tag names which are modified by this task</returns>
		public override IEnumerable<string> FindProducedTagNames()
		{
			yield break;
		}

		/// <summary>
		/// Resolve path to Docker executable by using the optional env var "UE_DOCKER_EXEC_PATH"
		/// Will default to "docker" if not set. Allows supporting alternative Docker implementations such as Podman.
		/// </summary>
		/// <returns>Path to Docker executable</returns>
		public static string GetDockerExecutablePath()
		{
			return Environment.GetEnvironmentVariable("UE_DOCKER_EXEC_PATH") ?? "docker";
		}
	}
}
