// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.IO;
using System.Linq;
using System.Threading.Tasks;
using System.Xml;
using EpicGames.Core;
using Microsoft.Extensions.Logging;
using UnrealBuildBase;

namespace AutomationTool.Tasks
{
	/// <summary>
	/// Parameters for a DotNet task
	/// </summary>
	public class DotNetTaskParameters
	{
		/// <summary>
		/// Docker command line arguments
		/// </summary>
		[TaskParameter(Optional = true)]
		public string Arguments { get; set; }

		/// <summary>
		/// Base directory for running the command
		/// </summary>
		[TaskParameter(Optional = true)]
		public string BaseDir { get; set; }

		/// <summary>
		/// Environment variables to set
		/// </summary>
		[TaskParameter(Optional = true)]
		public string Environment { get; set; }

		/// <summary>
		/// File to read environment variables from
		/// </summary>
		[TaskParameter(Optional = true)]
		public string EnvironmentFile { get; set; }

		/// <summary>
		/// The minimum exit code, which is treated as an error.
		/// </summary>
		[TaskParameter(Optional = true)]
		public int ErrorLevel { get; set; } = 1;

		/// <summary>
		/// Override path to dotnet executable
		/// </summary>
		[TaskParameter(Optional = true)]
		public FileReference DotNetPath { get; set; }
	}

	/// <summary>
	/// Spawns Docker and waits for it to complete.
	/// </summary>
	[TaskElement("DotNet", typeof(DotNetTaskParameters))]
	public class DotNetTask : SpawnTaskBase
	{
		readonly DotNetTaskParameters _parameters;

		/// <summary>
		/// Construct a Docker task
		/// </summary>
		/// <param name="parameters">Parameters for the task</param>
		public DotNetTask(DotNetTaskParameters parameters)
		{
			_parameters = parameters;
		}

		/// <summary>
		/// ExecuteAsync the task.
		/// </summary>
		/// <param name="job">Information about the current job</param>
		/// <param name="buildProducts">Set of build products produced by this node.</param>
		/// <param name="tagNameToFileSet">Mapping from tag names to the set of files they include</param>
		public override async Task ExecuteAsync(JobContext job, HashSet<FileReference> buildProducts, Dictionary<string, HashSet<FileReference>> tagNameToFileSet)
		{
			FileReference dotNetFile = _parameters.DotNetPath == null ? Unreal.DotnetPath : _parameters.DotNetPath;
			if (!FileReference.Exists(dotNetFile))
			{
				throw new AutomationException("DotNet is missing from {0}", dotNetFile);
			}

			// Set DOTNET_ROOT to ensure SDK resolution uses the correct .NET installation
			string dotNetRoot = dotNetFile.Directory.FullName;

			Dictionary<string, string> envVars = ParseEnvVars(_parameters.Environment, _parameters.EnvironmentFile);
			
			// Modify PATH:
			// Remove any dotnet in PATH
			string currentPath = Environment.GetEnvironmentVariable("PATH") ?? "";
			string cleanPath = String.Join(Path.PathSeparator.ToString(),
				currentPath
				.Split(Path.PathSeparator)
				.Where(p => !p.Contains("dotnet", StringComparison.OrdinalIgnoreCase)));
			// Add requested dotnet back to PAth
			envVars["PATH"] = $"{dotNetRoot}{Path.PathSeparator}{cleanPath}";

			// Set DOTNET_ROOT to ensure SDK resolution uses the correct .NET installation
			envVars["DOTNET_ROOT"] = dotNetRoot;

			// Disable multilevel lookup to prevent searching system-wide .NET installations
			envVars["DOTNET_MULTILEVEL_LOOKUP"] = "0";

			// Explicitly set the dotnet host path for MSBuild SDK resolver
			envVars["DOTNET_HOST_PATH"] = dotNetFile.FullName;

			// Point the SDK resolver CLI directory to our dotnet installation
			envVars["DOTNET_MSBUILD_SDK_RESOLVER_CLI_DIR"] = dotNetRoot;

			// Allow rolling forward major dotnet versions as a concession to dotnet version upgrades
			envVars["DOTNET_ROLL_FORWARD"] = "Major";

			// Try to find and set the MSBuild SDKs path
			DirectoryReference sdkDir = DirectoryReference.Combine(dotNetFile.Directory, "sdk");
			if (DirectoryReference.Exists(sdkDir))
			{
				foreach (DirectoryReference versionDir in DirectoryReference.EnumerateDirectories(sdkDir).Reverse())
				{
					DirectoryReference sdksPath = DirectoryReference.Combine(versionDir, "Sdks");
					if (DirectoryReference.Exists(sdksPath))
					{
						envVars["DOTNET_MSBUILD_SDK_RESOLVER_SDKS_DIR"] = sdksPath.FullName;
						envVars["MSBuildSDKsPath"] = sdksPath.FullName;
						envVars["MSBuildExtensionsPath"] = versionDir.FullName;
						envVars["MSBUILD_EXE_PATH"] = FileReference.Combine(versionDir, "MSBuild.dll").FullName;

						// Also set the SDK version explicitly
						envVars["DOTNET_MSBUILD_SDK_RESOLVER_SDKS_VER"] = versionDir.GetDirectoryName();

						break;
					}
				}
			}

			// CRITICAL: Disable the workload resolver to prevent it from searching other .NET locations
			// The error originates from Microsoft.NET.Sdk.ImportWorkloads.props
			envVars["MSBuildEnableWorkloadResolver"] = "false";
			
			IProcessResult result = await ExecuteAsync(dotNetFile.FullName, _parameters.Arguments, workingDir: _parameters.BaseDir, envVars: envVars);
			if (result.ExitCode < 0 || result.ExitCode >= _parameters.ErrorLevel)
			{
				Logger.LogError(KnownLogEvents.ExitCode, "Docker terminated with an exit code indicating an error ({ExitCode})", result.ExitCode);
				throw new AutomationException("Docker terminated with an exit code indicating an error ({0})", result.ExitCode) { OutputFormat = AutomationExceptionOutputFormat.Silent };
			}
		}

		/// <summary>
		/// Output this task out to an XML writer.
		/// </summary>
		public override void Write(XmlWriter writer)
		{
			Write(writer, _parameters);
		}

		/// <summary>
		/// Find all the tags which are used as inputs to this task
		/// </summary>
		/// <returns>The tag names which are read by this task</returns>
		public override IEnumerable<string> FindConsumedTagNames()
		{
			yield break;
		}

		/// <summary>
		/// Find all the tags which are modified by this task
		/// </summary>
		/// <returns>The tag names which are modified by this task</returns>
		public override IEnumerable<string> FindProducedTagNames()
		{
			yield break;
		}
	}

	public static partial class StandardTasks
	{
		/// <summary>
		/// Runs a command using dotnet.
		/// </summary>
		/// <param name="arguments">Command-line arguments.</param>
		/// <param name="baseDir">Base directory for running the command.</param>
		/// <param name="environment">Environment variables to set.</param>
		/// <param name="environmentFile">File to read environment variables from.</param>
		/// <param name="errorLevel">The minimum exit code, which is treated as an error.</param>
		/// <param name="dotNetPath">Override path to dotnet executable.</param>
		public static async Task DotNetAsync(string arguments = null, DirectoryReference baseDir = null, string environment = null, FileReference environmentFile = null, int errorLevel = 1, FileReference dotNetPath = null)
		{
			DotNetTaskParameters parameters = new DotNetTaskParameters();
			parameters.Arguments = arguments;
			parameters.BaseDir = baseDir?.FullName;
			parameters.Environment = environment;
			parameters.EnvironmentFile = environmentFile?.FullName;
			parameters.ErrorLevel = errorLevel;
			parameters.DotNetPath = dotNetPath;

			await ExecuteAsync(new DotNetTask(parameters));
		}
	}
}
