// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Linq;
using System.Text;
using System.Threading.Tasks;
using System.Xml;
using EpicGames.Core;
using EpicGames.Perforce;
using Microsoft.Extensions.Logging;

namespace AutomationTool.Tasks
{
	/// <summary>
	/// Parameters for a task that compiles a C# project
	/// </summary>
	public class FindModifiedFilesTaskParameters
	{
		/// <summary>
		///  List of file specifications separated by semicolon (default is ...)
		/// </summary>
		[TaskParameter(Optional = true, ValidationType = TaskParameterValidationType.FileSpec)]
		public string Path { get; set; } = "...";

		/// <summary>
		/// The configuration to compile.
		/// </summary>
		[TaskParameter(Optional = true)]
		public int Change { get; set; }

		/// <summary>
		/// The configuration to compile.
		/// </summary>
		[TaskParameter(Optional = true)]
		public int MinChange { get; set; }

		/// <summary>
		/// The configuration to compile.
		/// </summary>
		[TaskParameter(Optional = true)]
		public int MaxChange { get; set; }

		/// <summary>
		/// The file to write to
		/// </summary>
		[TaskParameter(Optional = true)]
		public FileReference Output { get; set; }
	}

	/// <summary>
	/// Compiles C# project files, and their dependencies.
	/// </summary>
	[TaskElement("FindModifiedFiles", typeof(FindModifiedFilesTaskParameters))]
	public class FindModifiedFilesTask : BgTaskImpl
	{
		/// <summary>
		/// Parameters for the task
		/// </summary>
		readonly FindModifiedFilesTaskParameters _parameters;

		/// <summary>
		/// Constructor.
		/// </summary>
		/// <param name="parameters">Parameters for this task</param>
		public FindModifiedFilesTask(FindModifiedFilesTaskParameters parameters)
		{
			_parameters = parameters;
		}

		/// <summary>
		/// ExecuteAsync the task.
		/// </summary>
		/// <param name="job">Information about the current job</param>
		/// <param name="buildProducts">Set of build products produced by this node.</param>
		/// <param name="tagNameToFileSet">Mapping from tag names to the set of files they include</param>
		public override async Task ExecuteAsync(JobContext job, HashSet<FileReference> buildProducts, Dictionary<string, HashSet<FileReference>> tagNameToFileSet)
		{
			using IPerforceConnection connection = await PerforceConnection.CreateAsync(CommandUtils.P4Settings, Log.Logger);

			SortedSet<string> allLocalFiles = new SortedSet<string>();
			foreach (string path in _parameters.Path.Split(";"))
			{
				StringBuilder filter = new StringBuilder($"//{connection.Settings.ClientName}/{path}");
				if (_parameters.Change > 0)
				{
					filter.Append($"@={_parameters.Change}");
				}
				else if (_parameters.MinChange > 0)
				{
					if (_parameters.MaxChange > 0)
					{
						filter.Append($"@{_parameters.MinChange},{_parameters.MaxChange}");
					}
					else
					{
						filter.Append($"@>={_parameters.MinChange}");
					}
				}
				else
				{
					throw new AutomationException("Change or MinChange must be specified to FindModifiedFiles task");
				}

				StreamRecord streamRecord = await connection.GetStreamAsync(CommandUtils.P4Env.Branch, true);
				PerforceViewMap viewMap = PerforceViewMap.Parse(streamRecord.View);

				HashSet<string> localFiles = new HashSet<string>();

				List<FilesRecord> files = await connection.FilesAsync(FilesOptions.None, filter.ToString());
				foreach (FilesRecord file in files)
				{
					string localFile;
					if (viewMap.TryMapFile(file.DepotFile, StringComparison.OrdinalIgnoreCase, out localFile))
					{
						localFiles.Add(localFile);
					}
					else
					{
						Logger.LogInformation("Unable to map {DepotFile} to workspace; skipping.", file.DepotFile);
					}
				}

				Logger.LogInformation("Found {NumFiles} modified files matching {Filter}", localFiles.Count, filter.ToString());
				foreach (string localFile in localFiles)
				{
					Logger.LogInformation("  {LocalFile}", localFile);
				}
				allLocalFiles.UnionWith(localFiles);
			}

			Logger.LogInformation("Found {NumFiles} total modified files", allLocalFiles.Count);
			if (_parameters.Output != null)
			{
				await FileReference.WriteAllLinesAsync(_parameters.Output, allLocalFiles);
				Logger.LogInformation("Written {OutputFile}", _parameters.Output);
			}
		}

		/// <summary>
		/// Output this task out to an XML writer.
		/// </summary>
		public override void Write(XmlWriter writer)
		{
			Write(writer, _parameters);
		}

		/// <summary>
		/// Find all the tags which are used as inputs to this task
		/// </summary>
		/// <returns>The tag names which are read by this task</returns>
		public override IEnumerable<string> FindConsumedTagNames()
		{
			//			return FindTagNamesFromFilespec(Parameters.Project);
			return Enumerable.Empty<string>();
		}

		/// <summary>
		/// Find all the tags which are modified by this task
		/// </summary>
		/// <returns>The tag names which are modified by this task</returns>
		public override IEnumerable<string> FindProducedTagNames()
		{
			return Enumerable.Empty<string>();
			/*			foreach (string TagName in FindTagNamesFromList(Parameters.Tag))
						{
							yield return TagName;
						}

						foreach (string TagName in FindTagNamesFromList(Parameters.TagReferences))
						{
							yield return TagName;
						}*/
		}
	}
}
