// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.IO;
using System.Threading.Tasks;
using System.Xml;
using EpicGames.Core;
using Microsoft.Extensions.Logging;

namespace AutomationTool.Tasks
{
	/// <summary>
	/// Parameters for <see cref="GatherBuildProductsFromFileTask"/>
	/// </summary>
	public class GatherBuildProductsFromFileTaskParameters
	{
		/// <summary>
		/// Path to the file containing the list of build products
		/// </summary>
		[TaskParameter]
		public string BuildProductsFile { get; set; }

		/// <summary>
		/// Optional base directory to apply to entries in the BuildProducts file
		/// </summary>
		[TaskParameter(Optional = true)]
		public string BaseDirectory { get; set; }
	}

	[TaskElement("GatherBuildProductsFromFile", typeof(GatherBuildProductsFromFileTaskParameters))]
	class GatherBuildProductsFromFileTask : BgTaskImpl
	{
		public GatherBuildProductsFromFileTask(GatherBuildProductsFromFileTaskParameters parameters)
		{
			_parameters = parameters;
		}

		public override async Task ExecuteAsync(JobContext job, HashSet<FileReference> buildProducts, Dictionary<string, HashSet<FileReference>> tagNameToFileSet)
		{
			Logger.LogInformation("Gathering BuildProducts from {Arg0}...", _parameters.BuildProductsFile);

			try
			{
				string[] fileBuildProducts = await File.ReadAllLinesAsync(_parameters.BuildProductsFile);
				foreach (string buildProduct in fileBuildProducts)
				{
					string fullPath = buildProduct;
					if (!String.IsNullOrEmpty(_parameters.BaseDirectory) && !Path.IsPathRooted(buildProduct))
					{
						fullPath = Path.Combine(_parameters.BaseDirectory, buildProduct);
					}
					Logger.LogInformation("Adding file to build products: {BuildProduct}", fullPath);
					buildProducts.Add(new FileReference(fullPath));
				}
			}
			catch (Exception ex)
			{
				Logger.LogInformation("Failed to gather build products: {Arg0}", ex.Message);
			}
		}

		public override void Write(XmlWriter writer)
		{
			Write(writer, _parameters);
		}

		public override IEnumerable<string> FindConsumedTagNames()
		{
			yield break;
		}

		public override IEnumerable<string> FindProducedTagNames()
		{
			yield break;
		}

		public GatherBuildProductsFromFileTaskParameters _parameters;
	}
}
