// Copyright Epic Games, Inc. All Rights Reserved.

using System.Collections.Generic;
using System.Threading.Tasks;
using System.Xml;
using EpicGames.Core;
using Microsoft.Extensions.Logging;
using UnrealBuildBase;

namespace AutomationTool.Tasks
{
	/// <summary>
	/// Parameters for a Git-Checkout task
	/// </summary>
	public class GitCloneTaskParameters
	{
		/// <summary>
		/// Directory for the repository
		/// </summary>
		[TaskParameter]
		public string Dir { get; set; }

		/// <summary>
		/// The remote to add
		/// </summary>
		[TaskParameter(Optional = true)]
		public string Remote { get; set; }

		/// <summary>
		/// The branch to check out on the remote
		/// </summary>
		[TaskParameter]
		public string Branch { get; set; }

		/// <summary>
		/// Configuration file for the repo. This can be used to set up a remote to be fetched and/or provide credentials.
		/// </summary>
		[TaskParameter(Optional = true)]
		public string ConfigFile { get; set; }
	}

	/// <summary>
	/// Clones a Git repository into a local path.
	/// </summary>
	[TaskElement("Git-Clone", typeof(GitCloneTaskParameters))]
	public class GitCloneTask : BgTaskImpl
	{
		/// <summary>
		/// Parameters for this task
		/// </summary>
		readonly GitCloneTaskParameters _parameters;

		/// <summary>
		/// Construct a Git task
		/// </summary>
		/// <param name="parameters">Parameters for the task</param>
		public GitCloneTask(GitCloneTaskParameters parameters)
		{
			_parameters = parameters;
		}

		/// <summary>
		/// ExecuteAsync the task.
		/// </summary>
		/// <param name="job">Information about the current job</param>
		/// <param name="buildProducts">Set of build products produced by this node.</param>
		/// <param name="tagNameToFileSet">Mapping from tag names to the set of files they include</param>
		public override async Task ExecuteAsync(JobContext job, HashSet<FileReference> buildProducts, Dictionary<string, HashSet<FileReference>> tagNameToFileSet)
		{
			FileReference gitExe = CommandUtils.FindToolInPath("git");
			if (gitExe == null)
			{
				throw new AutomationException("Unable to find path to Git. Check you have it installed, and it is on your PATH.");
			}

			DirectoryReference dir = ResolveDirectory(_parameters.Dir);
			Logger.LogInformation("Cloning Git repository into {Dir}", _parameters.Dir);
			using (LogIndentScope scope = new LogIndentScope("  "))
			{
				DirectoryReference gitDir = DirectoryReference.Combine(dir, ".git");
				if (!FileReference.Exists(FileReference.Combine(gitDir, "HEAD")))
				{
					await RunGit(gitExe, $"init \"{dir}\"", Unreal.RootDirectory);
				}

				if (_parameters.ConfigFile != null)
				{
					CommandUtils.CopyFile(_parameters.ConfigFile, FileReference.Combine(gitDir, "config").FullName);
				}

				if (_parameters.Remote != null)
				{
					await RunGit(gitExe, $"remote add origin {_parameters.Remote}", dir);
				}

				await RunGit(gitExe, "clean -dxf", dir);
				await RunGit(gitExe, "fetch --all", dir);
				await RunGit(gitExe, $"reset --hard {_parameters.Branch}", dir);
			}
		}

		/// <summary>
		/// Runs a git command
		/// </summary>
		/// <param name="toolFile"></param>
		/// <param name="arguments"></param>
		/// <param name="workingDir"></param>
		static Task RunGit(FileReference toolFile, string arguments, DirectoryReference workingDir)
		{
			IProcessResult result = CommandUtils.Run(toolFile.FullName, arguments, WorkingDir: workingDir.FullName);
			if (result.ExitCode != 0)
			{
				throw new AutomationException("Git terminated with an exit code indicating an error ({0})", result.ExitCode);
			}
			return Task.CompletedTask;
		}

		/// <summary>
		/// Output this task out to an XML writer.
		/// </summary>
		public override void Write(XmlWriter writer)
		{
			Write(writer, _parameters);
		}

		/// <summary>
		/// Find all the tags which are used as inputs to this task
		/// </summary>
		/// <returns>The tag names which are read by this task</returns>
		public override IEnumerable<string> FindConsumedTagNames()
		{
			yield break;
		}

		/// <summary>
		/// Find all the tags which are modified by this task
		/// </summary>
		/// <returns>The tag names which are modified by this task</returns>
		public override IEnumerable<string> FindProducedTagNames()
		{
			yield break;
		}
	}
}
