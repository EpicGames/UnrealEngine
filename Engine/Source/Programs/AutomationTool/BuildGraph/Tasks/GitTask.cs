// Copyright Epic Games, Inc. All Rights Reserved.

using System.Collections.Generic;
using System.Threading.Tasks;
using System.Xml;
using EpicGames.Core;

namespace AutomationTool.Tasks
{
	/// <summary>
	/// Parameters for a Git task
	/// </summary>
	public class GitTaskParameters
	{
		/// <summary>
		/// Git command line arguments
		/// </summary>
		[TaskParameter(Optional = true)]
		public string Arguments { get; set; }

		/// <summary>
		/// Base directory for running the command
		/// </summary>
		[TaskParameter(Optional = true)]
		public string BaseDir { get; set; }

		/// <summary>
		/// The minimum exit code, which is treated as an error.
		/// </summary>
		[TaskParameter(Optional = true)]
		public int ErrorLevel { get; set; } = 1;
	}

	/// <summary>
	/// Spawns Git and waits for it to complete.
	/// </summary>
	[TaskElement("Git", typeof(GitTaskParameters))]
	public class GitTask : BgTaskImpl
	{
		readonly GitTaskParameters _parameters;

		/// <summary>
		/// Construct a Git task
		/// </summary>
		/// <param name="parameters">Parameters for the task</param>
		public GitTask(GitTaskParameters parameters)
		{
			_parameters = parameters;
		}

		/// <summary>
		/// ExecuteAsync the task.
		/// </summary>
		/// <param name="job">Information about the current job</param>
		/// <param name="buildProducts">Set of build products produced by this node.</param>
		/// <param name="tagNameToFileSet">Mapping from tag names to the set of files they include</param>
		public override Task ExecuteAsync(JobContext job, HashSet<FileReference> buildProducts, Dictionary<string, HashSet<FileReference>> tagNameToFileSet)
		{
			FileReference toolFile = CommandUtils.FindToolInPath("git");
			if (toolFile == null)
			{
				throw new AutomationException("Unable to find path to Git. Check you have it installed, and it is on your PATH.");
			}

			IProcessResult result = CommandUtils.Run(toolFile.FullName, _parameters.Arguments, WorkingDir: _parameters.BaseDir);
			if (result.ExitCode < 0 || result.ExitCode >= _parameters.ErrorLevel)
			{
				throw new AutomationException("Git terminated with an exit code indicating an error ({0})", result.ExitCode);
			}

			return Task.CompletedTask;
		}

		/// <summary>
		/// Output this task out to an XML writer.
		/// </summary>
		public override void Write(XmlWriter writer)
		{
			Write(writer, _parameters);
		}

		/// <summary>
		/// Find all the tags which are used as inputs to this task
		/// </summary>
		/// <returns>The tag names which are read by this task</returns>
		public override IEnumerable<string> FindConsumedTagNames()
		{
			yield break;
		}

		/// <summary>
		/// Find all the tags which are modified by this task
		/// </summary>
		/// <returns>The tag names which are modified by this task</returns>
		public override IEnumerable<string> FindProducedTagNames()
		{
			yield break;
		}
	}
}
