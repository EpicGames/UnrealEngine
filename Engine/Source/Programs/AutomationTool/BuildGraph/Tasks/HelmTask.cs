// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Threading.Tasks;
using System.Xml;
using EpicGames.Core;
using UnrealBuildBase;

namespace AutomationTool.Tasks
{
	/// <summary>
	/// Parameters for a Helm task
	/// </summary>
	public class HelmTaskParameters
	{
		/// <summary>
		/// Helm command line arguments
		/// </summary>
		[TaskParameter]
		public string Chart { get; set; }

		/// <summary>
		/// Name of the release
		/// </summary>
		[TaskParameter]
		public string Deployment { get; set; }

		/// <summary>
		/// The Kubernetes namespace
		/// </summary>
		[TaskParameter(Optional = true)]
		public string Namespace { get; set; }

		/// <summary>
		/// The kubectl context
		/// </summary>
		[TaskParameter(Optional = true)]
		public string KubeContext { get; set; }

		/// <summary>
		/// The kubectl config file to use
		/// </summary>
		[TaskParameter(Optional = true)]
		public string KubeConfig { get; set; }

		/// <summary>
		/// Values to set for running the chart
		/// </summary>
		[TaskParameter(Optional = true)]
		public string Values { get; set; }

		/// <summary>
		/// Values to set for running the chart
		/// </summary>
		[TaskParameter(Optional = true)]
		public string ValuesFile { get; set; }

		/// <summary>
		/// Environment variables to set
		/// </summary>
		[TaskParameter(Optional = true)]
		public string Environment { get; set; }

		/// <summary>
		/// File to parse environment variables from
		/// </summary>
		[TaskParameter(Optional = true)]
		public string EnvironmentFile { get; set; }

		/// <summary>
		/// Additional arguments
		/// </summary>
		[TaskParameter(Optional = true)]
		public string Arguments { get; set; }

		/// <summary>
		/// Base directory for running the command
		/// </summary>
		[TaskParameter(Optional = true)]
		public string WorkingDir { get; set; }
	}

	/// <summary>
	/// Spawns Helm and waits for it to complete.
	/// </summary>
	[TaskElement("Helm", typeof(HelmTaskParameters))]
	public class HelmTask : SpawnTaskBase
	{
		readonly HelmTaskParameters _parameters;

		/// <summary>
		/// Construct a Helm task
		/// </summary>
		/// <param name="parameters">Parameters for the task</param>
		public HelmTask(HelmTaskParameters parameters)
		{
			_parameters = parameters;
		}

		/// <summary>
		/// ExecuteAsync the task.
		/// </summary>
		/// <param name="job">Information about the current job</param>
		/// <param name="buildProducts">Set of build products produced by this node.</param>
		/// <param name="tagNameToFileSet">Mapping from tag names to the set of files they include</param>
		public override async Task ExecuteAsync(JobContext job, HashSet<FileReference> buildProducts, Dictionary<string, HashSet<FileReference>> tagNameToFileSet)
		{

			// Build the argument list
			List<string> arguments = new List<string>();
			arguments.Add("upgrade");
			arguments.Add(_parameters.Deployment);
			arguments.Add(new FileReference(_parameters.Chart).FullName);
			arguments.Add("--install");
			arguments.Add("--reset-values");
			if (_parameters.Namespace != null)
			{
				arguments.Add("--namespace");
				arguments.Add(_parameters.Namespace);
			}
			if (_parameters.KubeContext != null)
			{
				arguments.Add("--kube-context");
				arguments.Add(_parameters.KubeContext);
			}
			if (_parameters.KubeConfig != null)
			{
				arguments.Add("--kubeconfig");
				arguments.Add(_parameters.KubeConfig);
			}
			if (!String.IsNullOrEmpty(_parameters.Values))
			{
				foreach (string value in SplitDelimitedList(_parameters.Values))
				{
					arguments.Add("--set");
					arguments.Add(value);
				}
			}
			if (!String.IsNullOrEmpty(_parameters.ValuesFile))
			{
				foreach (FileReference valuesFile in ResolveFilespec(Unreal.RootDirectory, _parameters.ValuesFile, tagNameToFileSet))
				{
					arguments.Add("--values");
					arguments.Add(valuesFile.FullName);
				}
			}

			string additionalArguments = String.IsNullOrEmpty(_parameters.Arguments) ? String.Empty : $" {_parameters.Arguments}";
			await SpawnTaskBase.ExecuteAsync("helm", CommandLineArguments.Join(arguments) + additionalArguments, workingDir: _parameters.WorkingDir, envVars: ParseEnvVars(_parameters.Environment, _parameters.EnvironmentFile));
		}

		/// <summary>
		/// Output this task out to an XML writer.
		/// </summary>
		public override void Write(XmlWriter writer)
		{
			Write(writer, _parameters);
		}

		/// <summary>
		/// Find all the tags which are used as inputs to this task
		/// </summary>
		/// <returns>The tag names which are read by this task</returns>
		public override IEnumerable<string> FindConsumedTagNames()
		{
			yield break;
		}

		/// <summary>
		/// Find all the tags which are modified by this task
		/// </summary>
		/// <returns>The tag names which are modified by this task</returns>
		public override IEnumerable<string> FindProducedTagNames()
		{
			yield break;
		}
	}
}
