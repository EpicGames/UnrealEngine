// Copyright Epic Games, Inc. All Rights Reserved.

using System.Collections.Generic;
using System.IO;
using System.Linq;
using System.Text.Json;
using System.Threading.Tasks;
using System.Xml;
using EpicGames.Core;
using Microsoft.Extensions.Logging;

namespace AutomationTool.Tasks
{
	/// <summary>
	/// Parameters for a <see cref="HordeCreateReportTask"/>.
	/// </summary>
	public class HordeCreateReportTaskParameters
	{
		/// <summary>
		/// Name for the report
		/// </summary>
		[TaskParameter]
		public string Name { get; set; }

		/// <summary>
		/// Where to display the report
		/// </summary>
		[TaskParameter]
		public string Scope { get; set; }

		/// <summary>
		/// Where to show the report
		/// </summary>
		[TaskParameter]
		public string Placement { get; set; }

		/// <summary>
		/// Text to be displayed
		/// </summary>
		[TaskParameter]
		public string Text { get; set; }
	}

	/// <summary>
	/// Creates a Horde report file, which will be displayed on the dashboard with any job running this task.
	/// </summary>
	[TaskElement("Horde-CreateReport", typeof(HordeCreateReportTaskParameters))]
	public class HordeCreateReportTask : BgTaskImpl
	{
		/// <summary>
		/// Parameters for this task.
		/// </summary>
		readonly HordeCreateReportTaskParameters _parameters;

		/// <summary>
		/// Constructor.
		/// </summary>
		/// <param name="parameters">Parameters for this task.</param>
		public HordeCreateReportTask(HordeCreateReportTaskParameters parameters)
		{
			_parameters = parameters;
		}

		/// <summary>
		/// ExecuteAsync the task.
		/// </summary>
		/// <param name="job">Information about the current job.</param>
		/// <param name="buildProducts">Set of build products produced by this node.</param>
		/// <param name="tagNameToFileSet">Mapping from tag names to the set of files they include.</param>
		public override async Task ExecuteAsync(JobContext job, HashSet<FileReference> buildProducts, Dictionary<string, HashSet<FileReference>> tagNameToFileSet)
		{
			FileReference reportTextFile = FileReference.Combine(new DirectoryReference(CommandUtils.CmdEnv.LogFolder), $"{_parameters.Name}.md");
			await FileReference.WriteAllTextAsync(reportTextFile, _parameters.Text);

			FileReference reportJsonFile = FileReference.Combine(new DirectoryReference(CommandUtils.CmdEnv.LogFolder), $"{_parameters.Name}.report.json");
			using (FileStream reportJsonStream = FileReference.Open(reportJsonFile, FileMode.Create, FileAccess.Write, FileShare.Read))
			{
				using (Utf8JsonWriter writer = new Utf8JsonWriter(reportJsonStream))
				{
					writer.WriteStartObject();
					writer.WriteString("scope", _parameters.Scope);
					writer.WriteString("name", _parameters.Name);
					writer.WriteString("placement", _parameters.Placement);
					writer.WriteString("fileName", reportTextFile.GetFileName());
					writer.WriteEndObject();
				}
			}

			Logger.LogInformation("Written report to {TextFile} and {JsonFile}: \"{Text}\"", reportTextFile, reportJsonFile, _parameters.Text);
		}

		/// <summary>
		/// Output this task out to an XML writer.
		/// </summary>
		public override void Write(XmlWriter writer)
		{
			Write(writer, _parameters);
		}

		/// <summary>
		/// Find all the tags which are used as inputs to this task
		/// </summary>
		/// <returns>The tag names which are read by this task</returns>
		public override IEnumerable<string> FindConsumedTagNames() => Enumerable.Empty<string>();

		/// <summary>
		/// Find all the tags which are modified by this task
		/// </summary>
		/// <returns>The tag names which are modified by this task</returns>
		public override IEnumerable<string> FindProducedTagNames() => Enumerable.Empty<string>();
	}

	public static partial class StandardTasks
	{
		/// <summary>
		/// Compiles a target
		/// </summary>
		/// <param name="name">Name for the report</param>
		/// <param name="scope"> Where to display the report</param>
		/// <param name="placement">Where to show the report</param>
		/// <param name="text">Text to be displayed</param>
		/// <returns>Creates a Horde report file, which will be displayed on the dashboard with any job running this task.</returns>
		public static async Task<FileSet> HordeCreateReportAsync(string name, string scope, string placement, string text)
		{
			HordeCreateReportTaskParameters parameters = new()
			{
				Name = name,
				Scope = scope,
				Placement = placement,
				Text = text
			};
			return await ExecuteAsync(new HordeCreateReportTask(parameters));
		}
	}
}