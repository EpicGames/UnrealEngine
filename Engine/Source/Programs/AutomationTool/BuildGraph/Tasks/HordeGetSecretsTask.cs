// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Linq;
using System.Net.Http;
using System.Threading.Tasks;
using System.Xml;
using EpicGames.Core;
using EpicGames.Horde;
using EpicGames.Horde.Secrets;
using Microsoft.Extensions.DependencyInjection;
using Microsoft.Extensions.Logging;

#nullable enable

namespace AutomationTool.Tasks
{
	/// <summary>
	/// Parameters for a <see cref="HordeGetSecretsTask"/>.
	/// </summary>
	public class HordeGetSecretsTaskParameters
	{
		/// <summary>
		/// File to update with secrets
		/// </summary>
		[TaskParameter]
		public string File { get; set; } = String.Empty;

		/// <summary>
		/// Text to update with secrets
		/// </summary>
		[TaskParameter(Optional = true)]
		public string? Text { get; set; }

		/// <summary>
		/// Pairs of strings and secret names to expand in the text file, in the form SOURCE_TEXT=secret-name;SOURCE_TEXT_2=secret-name-2. 
		/// If not specified, secrets embedded inline in the text will be expanded from {{secret-name.value}} strings.
		/// </summary>
		[TaskParameter(Optional = true)]
		public string? Replace { get; set; }
	}

	/// <summary>
	/// Replaces strings in a text file with secrets obtained from Horde
	/// </summary>
	[TaskElement("Horde-GetSecrets", typeof(HordeGetSecretsTaskParameters))]
	public class HordeGetSecretsTask : BgTaskImpl
	{
		record class ReplacementInfo(string Property, string Variable);

		readonly HordeGetSecretsTaskParameters _parameters;

		/// <summary>
		/// Constructor.
		/// </summary>
		/// <param name="parameters">Parameters for this task.</param>
		public HordeGetSecretsTask(HordeGetSecretsTaskParameters parameters)
		{
			_parameters = parameters;
		}

		/// <summary>
		/// ExecuteAsync the task.
		/// </summary>
		/// <param name="job">Information about the current job.</param>
		/// <param name="buildProducts">Set of build products produced by this node.</param>
		/// <param name="tagNameToFileSet">Mapping from tag names to the set of files they include.</param>
		public override async Task ExecuteAsync(JobContext job, HashSet<FileReference> buildProducts, Dictionary<string, HashSet<FileReference>> tagNameToFileSet)
		{
			FileReference file = ResolveFile(_parameters.File);

			// Read the input text
			string? text = _parameters.Text;
			if (String.IsNullOrEmpty(text))
			{
				text = await FileReference.ReadAllTextAsync(file);
			}

			// Parse the secrets to replace
			Dictionary<SecretId, List<ReplacementInfo>> secretToReplacementInfo = new Dictionary<SecretId, List<ReplacementInfo>>();
			if (String.IsNullOrEmpty(_parameters.Replace))
			{
				int pos = 0;
				for (; ; )
				{
					pos = text.IndexOf("{{", pos, StringComparison.Ordinal);
					if (pos == -1)
					{
						break;
					}

					pos += 2;

					int endPos = text.IndexOf("}}", pos, StringComparison.Ordinal);
					if (endPos == -1)
					{
						continue;
					}

					string variable = text.Substring(pos - 2, (endPos + 2) - (pos - 2));
					string replacement = text.Substring(pos, endPos - pos);
					if (!ParseReplacementInfo(variable, replacement, secretToReplacementInfo))
					{
						throw new AutomationException($"Invalid replacement clause for secret in Horde-GetSecrets task: {replacement} (expected VARIABLE=Secret.Property)");
					}

					pos = endPos + 2;
				}
			}
			else
			{
				foreach (string replacement in _parameters.Replace.Split(';', StringSplitOptions.RemoveEmptyEntries | StringSplitOptions.TrimEntries))
				{
					int idx = replacement.LastIndexOf('=');
					if (idx == -1)
					{
						throw new AutomationException($"Invalid replacement clause in Horde-GetSecrets task: {replacement} (expected VARIABLE=Secret.Property)");
					}

					string variable = replacement.Substring(0, idx);
					if (!text.Contains(variable, StringComparison.Ordinal))
					{
						Logger.LogWarning("Variable '{Variable}' not found in {File}", variable, file);
						continue;
					}

					if (!ParseReplacementInfo(variable, replacement.Substring(idx + 1), secretToReplacementInfo))
					{
						throw new AutomationException($"Invalid replacement clause for secret in Horde-GetSecrets task: {replacement} (expected VARIABLE=Secret.Property)");
					}
				}
			}

			// Read the secrets from Horde, and substitute them in the output file
			if (secretToReplacementInfo.Count > 0)
			{
				ServiceCollection serviceCollection = new ServiceCollection();
				serviceCollection.AddLogging(builder => builder.AddEpicDefault());
				serviceCollection.AddHorde(options => options.AllowAuthPrompt = !CommandUtils.IsBuildMachine);

				await using (ServiceProvider serviceProvider = serviceCollection.BuildServiceProvider())
				{
					IHordeClient hordeClient = serviceProvider.GetRequiredService<IHordeClient>();

					using HordeHttpClient hordeHttpClient = hordeClient.CreateHttpClient();

					foreach ((SecretId secretId, List<ReplacementInfo> replacements) in secretToReplacementInfo)
					{
						GetSecretResponse secret;
						try
						{
							secret = await hordeHttpClient.GetSecretAsync(secretId);
						}
						catch (HttpRequestException ex) when (ex.StatusCode == System.Net.HttpStatusCode.NotFound)
						{
							throw new AutomationException(ex, $"Secret '{secretId}' was not found on {hordeClient.ServerUrl}");
						}
						catch (HttpRequestException ex) when (ex.StatusCode == System.Net.HttpStatusCode.Forbidden)
						{
							throw new AutomationException(ex, $"User does not have permissions to read '{secretId}' on {hordeClient.ServerUrl}");
						}

						foreach (ReplacementInfo replacement in replacements)
						{
							string? value;
							if (secret.Data.TryGetValue(replacement.Property, out value))
							{
								text = text.Replace(replacement.Variable, value, StringComparison.Ordinal);
							}
							else
							{
								Logger.LogWarning("Property '{PropertyName}' not found in secret {SecretId}", replacement.Property, secretId);
							}
						}
					}
				}
			}

			// Write the output file
			DirectoryReference.CreateDirectory(file.Directory);
			await FileReference.WriteAllTextAsync(file, text);
			Logger.LogInformation("Updated {File} with secrets from Horde.", file);
		}

		static bool ParseReplacementInfo(string variable, string replacement, Dictionary<SecretId, List<ReplacementInfo>> secretToReplacementInfo)
		{
			int propertyIdx = replacement.IndexOf('.', StringComparison.Ordinal);
			if (propertyIdx == -1)
			{
				return false;
			}

			SecretId secretId = new SecretId(replacement.Substring(0, propertyIdx));
			string propertyName = replacement.Substring(propertyIdx + 1);

			List<ReplacementInfo>? replacements;
			if (!secretToReplacementInfo.TryGetValue(secretId, out replacements))
			{
				replacements = new List<ReplacementInfo>();
				secretToReplacementInfo.Add(secretId, replacements);
			}

			replacements.Add(new ReplacementInfo(propertyName, variable));
			return true;
		}

		/// <summary>
		/// Output this task out to an XML writer.
		/// </summary>
		public override void Write(XmlWriter writer)
		{
			Write(writer, _parameters);
		}

		/// <summary>
		/// Find all the tags which are used as inputs to this task
		/// </summary>
		/// <returns>The tag names which are read by this task</returns>
		public override IEnumerable<string> FindConsumedTagNames() => Enumerable.Empty<string>();

		/// <summary>
		/// Find all the tags which are modified by this task
		/// </summary>
		/// <returns>The tag names which are modified by this task</returns>
		public override IEnumerable<string> FindProducedTagNames() => Enumerable.Empty<string>();
	}
}
