// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Linq;
using System.Net.Http;
using System.Text.RegularExpressions;
using System.Threading.Tasks;
using System.Xml;
using EpicGames.Core;
using EpicGames.Horde;
using EpicGames.Horde.Secrets;
using Microsoft.Extensions.DependencyInjection;
using Microsoft.Extensions.Logging;

#nullable enable

namespace AutomationTool.Tasks
{
	/// <summary>
	/// Parameters for a <see cref="HordeGetSecretsTask"/>.
	/// </summary>
	public class HordeSetSecretEnvVarTaskParameters
	{
		/// <summary>
		/// Name of the environment variable to set
		/// </summary>
		[TaskParameter]
		public string Name { get; set; } = String.Empty;

		/// <summary>
		/// Name of the secret to fetch
		/// </summary>
		[TaskParameter]
		public string Secret { get; set; } = String.Empty;
	}

	/// <summary>
	/// Replaces strings in a text file with secrets obtained from Horde
	/// </summary>
	[TaskElement("Horde-SetSecretEnvVar", typeof(HordeSetSecretEnvVarTaskParameters))]
	public class HordeSetSecretEnvVarTask : BgTaskImpl
	{
		readonly HordeSetSecretEnvVarTaskParameters _parameters;

		/// <summary>
		/// Constructor.
		/// </summary>
		/// <param name="parameters">Parameters for this task.</param>
		public HordeSetSecretEnvVarTask(HordeSetSecretEnvVarTaskParameters parameters)
		{
			_parameters = parameters;
		}

		/// <summary>
		/// ExecuteAsync the task.
		/// </summary>
		/// <param name="job">Information about the current job.</param>
		/// <param name="buildProducts">Set of build products produced by this node.</param>
		/// <param name="tagNameToFileSet">Mapping from tag names to the set of files they include.</param>
		public override async Task ExecuteAsync(JobContext job, HashSet<FileReference> buildProducts, Dictionary<string, HashSet<FileReference>> tagNameToFileSet)
		{
			Match match = Regex.Match(_parameters.Secret, @"^([^\.]+)\.([^\.]+)$");
			if (!match.Success)
			{
				throw new AutomationException($"Invalid secret name '{_parameters.Secret}'. Expected a secret id followed by property name, eg. 'my-secret.property'.");
			}

			SecretId secretId = new SecretId(match.Groups[1].Value);
			string propertyName = match.Groups[2].Value;

			ServiceCollection serviceCollection = new ServiceCollection();
			serviceCollection.AddLogging(builder => builder.AddEpicDefault());
			serviceCollection.AddHorde(options => options.AllowAuthPrompt = !CommandUtils.IsBuildMachine);

			await using (ServiceProvider serviceProvider = serviceCollection.BuildServiceProvider())
			{
				IHordeClient hordeClient = serviceProvider.GetRequiredService<IHordeClient>();

				using HordeHttpClient hordeHttpClient = hordeClient.CreateHttpClient();

				GetSecretResponse secret;
				try
				{
					secret = await hordeHttpClient.GetSecretAsync(secretId);
				}
				catch (HttpRequestException ex) when (ex.StatusCode == System.Net.HttpStatusCode.NotFound)
				{
					throw new AutomationException(ex, $"Secret '{secretId}' was not found on {hordeClient.ServerUrl}");
				}
				catch (HttpRequestException ex) when (ex.StatusCode == System.Net.HttpStatusCode.Forbidden)
				{
					throw new AutomationException(ex, $"User does not have permissions to read '{secretId}' on {hordeClient.ServerUrl}");
				}

				string? value;
				if (!secret.Data.TryGetValue(propertyName, out value))
				{
					throw new AutomationException($"Property '{propertyName}' not found in secret {secretId}");
				}

				Logger.LogInformation("Setting environment variable {Name} to value of secret {Secret}", _parameters.Name, _parameters.Secret);
				Environment.SetEnvironmentVariable(_parameters.Name, value);
			}
		}

		/// <summary>
		/// Output this task out to an XML writer.
		/// </summary>
		public override void Write(XmlWriter writer)
		{
			Write(writer, _parameters);
		}

		/// <summary>
		/// Find all the tags which are used as inputs to this task
		/// </summary>
		/// <returns>The tag names which are read by this task</returns>
		public override IEnumerable<string> FindConsumedTagNames() => Enumerable.Empty<string>();

		/// <summary>
		/// Find all the tags which are modified by this task
		/// </summary>
		/// <returns>The tag names which are modified by this task</returns>
		public override IEnumerable<string> FindProducedTagNames() => Enumerable.Empty<string>();
	}
}
