// Copyright Epic Games, Inc. All Rights Reserved.

using System.Collections.Generic;
using System.Threading.Tasks;
using System.Xml;
using EpicGames.Core;

namespace AutomationTool.Tasks
{
	/// <summary>
	/// Parameters for a Kubectl task
	/// </summary>
	public class KubectlTaskParameters
	{
		/// <summary>
		/// Command line arguments
		/// </summary>
		[TaskParameter]
		public string Arguments { get; set; }

		/// <summary>
		/// Base directory for running the command
		/// </summary>
		[TaskParameter(Optional = true)]
		public string BaseDir { get; set; }
	}

	/// <summary>
	/// Spawns Kubectl and waits for it to complete.
	/// </summary>
	[TaskElement("Kubectl", typeof(KubectlTaskParameters))]
	public class KubectlTask : BgTaskImpl
	{
		readonly KubectlTaskParameters _parameters;

		/// <summary>
		/// Construct a Kubectl task
		/// </summary>
		/// <param name="parameters">Parameters for the task</param>
		public KubectlTask(KubectlTaskParameters parameters)
		{
			_parameters = parameters;
		}

		/// <summary>
		/// ExecuteAsync the task.
		/// </summary>
		/// <param name="job">Information about the current job</param>
		/// <param name="buildProducts">Set of build products produced by this node.</param>
		/// <param name="tagNameToFileSet">Mapping from tag names to the set of files they include</param>
		public override Task ExecuteAsync(JobContext job, HashSet<FileReference> buildProducts, Dictionary<string, HashSet<FileReference>> tagNameToFileSet)
		{
			FileReference kubectlExe = CommandUtils.FindToolInPath("kubectl");
			if (kubectlExe == null)
			{
				throw new AutomationException("Unable to find path to Kubectl. Check you have it installed, and it is on your PATH.");
			}

			IProcessResult result = CommandUtils.Run(kubectlExe.FullName, _parameters.Arguments, null, WorkingDir: _parameters.BaseDir);
			if (result.ExitCode != 0)
			{
				throw new AutomationException("Kubectl terminated with an exit code indicating an error ({0})", result.ExitCode);
			}

			return Task.CompletedTask;
		}

		/// <summary>
		/// Output this task out to an XML writer.
		/// </summary>
		public override void Write(XmlWriter writer)
		{
			Write(writer, _parameters);
		}

		/// <summary>
		/// Find all the tags which are used as inputs to this task
		/// </summary>
		/// <returns>The tag names which are read by this task</returns>
		public override IEnumerable<string> FindConsumedTagNames()
		{
			yield break;
		}

		/// <summary>
		/// Find all the tags which are modified by this task
		/// </summary>
		/// <returns>The tag names which are modified by this task</returns>
		public override IEnumerable<string> FindProducedTagNames()
		{
			yield break;
		}
	}
}
