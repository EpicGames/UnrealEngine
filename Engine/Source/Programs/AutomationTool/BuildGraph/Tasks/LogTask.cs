// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Linq;
using System.Threading.Tasks;
using System.Xml;
using EpicGames.Core;
using Microsoft.Extensions.Logging;
using UnrealBuildBase;

namespace AutomationTool.Tasks
{
	/// <summary>
	/// Parameters for the log task
	/// </summary>
	public class LogTaskParameters
	{
		/// <summary>
		/// Message to print out.
		/// </summary>
		[TaskParameter(Optional = true)]
		public string Message { get; set; }

		/// <summary>
		/// If specified, causes the given list of files to be printed after the given message.
		/// </summary>
		[TaskParameter(Optional = true, ValidationType = TaskParameterValidationType.FileSpec)]
		public string Files { get; set; }

		/// <summary>
		/// If specified, causes the contents of the given files to be printed out.
		/// </summary>
		[TaskParameter(Optional = true)]
		public bool IncludeContents { get; set; }
	}

	/// <summary>
	/// Print a message (and other optional diagnostic information) to the output log.
	/// </summary>
	[TaskElement("Log", typeof(LogTaskParameters))]
	public class LogTask : BgTaskImpl
	{
		/// <summary>
		/// Parameters for the task
		/// </summary>
		readonly LogTaskParameters _parameters;

		/// <summary>
		/// Constructor.
		/// </summary>
		/// <param name="parameters">Parameters for the task</param>
		public LogTask(LogTaskParameters parameters)
		{
			_parameters = parameters;
		}

		/// <summary>
		/// ExecuteAsync the task.
		/// </summary>
		/// <param name="job">Information about the current job</param>
		/// <param name="buildProducts">Set of build products produced by this node.</param>
		/// <param name="tagNameToFileSet">Mapping from tag names to the set of files they include</param>
		public override async Task ExecuteAsync(JobContext job, HashSet<FileReference> buildProducts, Dictionary<string, HashSet<FileReference>> tagNameToFileSet)
		{
			// Print the message
			if (!String.IsNullOrEmpty(_parameters.Message))
			{
				Logger.LogInformation("{Text}", _parameters.Message);
			}

			// Print the contents of the given tag, if specified
			if (!String.IsNullOrEmpty(_parameters.Files))
			{
				HashSet<FileReference> files = ResolveFilespec(Unreal.RootDirectory, _parameters.Files, tagNameToFileSet);
				foreach (FileReference file in files.OrderBy(x => x.FullName))
				{
					Logger.LogInformation("  {Arg0}", file.FullName);
					if (_parameters.IncludeContents)
					{
						string[] lines = await System.IO.File.ReadAllLinesAsync(file.FullName);
						foreach (string line in lines)
						{
							Logger.LogInformation("    {Line}", line);
						}
					}
				}
			}
		}

		/// <summary>
		/// Output this task out to an XML writer.
		/// </summary>
		public override void Write(XmlWriter writer)
		{
			Write(writer, _parameters);
		}

		/// <summary>
		/// Find all the tags which are used as inputs to this task
		/// </summary>
		/// <returns>The tag names which are read by this task</returns>
		public override IEnumerable<string> FindConsumedTagNames()
		{
			return FindTagNamesFromFilespec(_parameters.Files);
		}

		/// <summary>
		/// Find all the tags which are modified by this task
		/// </summary>
		/// <returns>The tag names which are modified by this task</returns>
		public override IEnumerable<string> FindProducedTagNames()
		{
			yield break;
		}
	}
}
