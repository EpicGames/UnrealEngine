// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Threading.Tasks;
using System.Xml;
using EpicGames.Core;
using UnrealBuildTool;

namespace AutomationTool.Tasks
{
	/// <summary>
	/// Parameters for a ModifyConfig task
	/// </summary>
	public class ModifyConfigTaskParameters
	{
		/// <summary>
		/// Path to the config file
		/// </summary>
		[TaskParameter(ValidationType = TaskParameterValidationType.FileSpec)]
		public string File { get; set; }

		/// <summary>
		/// The section name to modify
		/// </summary>
		[TaskParameter]
		public string Section { get; set; }

		/// <summary>
		/// The property name to set
		/// </summary>
		[TaskParameter]
		public string Key { get; set; }

		/// <summary>
		/// The property value to set
		/// </summary>
		[TaskParameter]
		public string Value { get; set; }

		/// <summary>
		/// Tag to be applied to the extracted files
		/// </summary>
		[TaskParameter(Optional = true, ValidationType = TaskParameterValidationType.TagList)]
		public string Tag { get; set; }
	}

	/// <summary>
	/// Modifies a config file
	/// </summary>
	[TaskElement("ModifyConfig", typeof(ModifyConfigTaskParameters))]
	public class ModifyConfigTask : BgTaskImpl
	{
		readonly ModifyConfigTaskParameters _parameters;

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="parameters">Parameters for this task</param>
		public ModifyConfigTask(ModifyConfigTaskParameters parameters)
		{
			_parameters = parameters;
		}

		/// <summary>
		/// ExecuteAsync the task.
		/// </summary>
		/// <param name="job">Information about the current job</param>
		/// <param name="buildProducts">Set of build products produced by this node.</param>
		/// <param name="tagNameToFileSet">Mapping from tag names to the set of files they include</param>
		public override Task ExecuteAsync(JobContext job, HashSet<FileReference> buildProducts, Dictionary<string, HashSet<FileReference>> tagNameToFileSet)
		{
			FileReference configFileLocation = ResolveFile(_parameters.File);

			ConfigFile configFile;
			if (FileReference.Exists(configFileLocation))
			{
				configFile = new ConfigFile(configFileLocation);
			}
			else
			{
				configFile = new ConfigFile();
			}

			ConfigFileSection section = configFile.FindOrAddSection(_parameters.Section);
			section.Lines.RemoveAll(x => String.Equals(x.Key, _parameters.Key, StringComparison.OrdinalIgnoreCase));
			section.Lines.Add(new ConfigLine(ConfigLineAction.Set, _parameters.Key, _parameters.Value));

			FileReference.MakeWriteable(configFileLocation);
			configFile.Write(configFileLocation);

			// Apply the optional tag to the produced archive
			foreach (string tagName in FindTagNamesFromList(_parameters.Tag))
			{
				FindOrAddTagSet(tagNameToFileSet, tagName).Add(configFileLocation);
			}

			// Add the archive to the set of build products
			buildProducts.Add(configFileLocation);
			return Task.CompletedTask;
		}

		/// <summary>
		/// Output this task out to an XML writer.
		/// </summary>
		public override void Write(XmlWriter writer)
		{
			Write(writer, _parameters);
		}

		/// <summary>
		/// Find all the tags which are used as inputs to this task
		/// </summary>
		/// <returns>The tag names which are read by this task</returns>
		public override IEnumerable<string> FindConsumedTagNames()
		{
			return FindTagNamesFromFilespec(_parameters.File);
		}

		/// <summary>
		/// Find all the tags which are modified by this task
		/// </summary>
		/// <returns>The tag names which are modified by this task</returns>
		public override IEnumerable<string> FindProducedTagNames()
		{
			return FindTagNamesFromList(_parameters.Tag);
		}
	}
}
