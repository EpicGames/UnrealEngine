// Copyright Epic Games, Inc. All Rights Reserved.

using System.Collections.Generic;
using System.IO;
using System.Linq;
using System.Text;
using System.Text.Json;
using System.Threading.Tasks;
using System.Xml;
using EpicGames.Core;
using UnrealBuildBase;

namespace AutomationTool.Tasks
{
	/// <summary>
	/// Parameters for a ModifyJsonValue task
	/// </summary>
	public class ModifyJsonValueParameters
	{
		/// <summary>
		/// json file paths which will be modified
		/// </summary>
		[TaskParameter(ValidationType = TaskParameterValidationType.FileSpec)]
		public string Files { get; set; }

		/// <summary>
		/// json key path to find in each file
		/// </summary>
		[TaskParameter(ValidationType = TaskParameterValidationType.Default)]
		public string KeyPath { get; set; }

		/// <summary>
		/// new value to apply
		/// </summary>
		[TaskParameter(ValidationType = TaskParameterValidationType.Default)]
		public int NewValue { get; set; }
	}

	/// <summary>
	/// Modifies json files by setting a value specified in the key path
	/// </summary>
	[TaskElement("ModifyJsonValue", typeof(ModifyJsonValueParameters))]
	public class ModifyJsonValue : BgTaskImpl
	{
		readonly ModifyJsonValueParameters _parameters;

		/// <summary>
		/// Create a new ModifyJsonValue.
		/// </summary>
		/// <param name="parameters">Parameters for this task.</param>
		public ModifyJsonValue(ModifyJsonValueParameters parameters)
		{
			_parameters = parameters;
		}

		/// <summary>
		/// Placeholder comment
		/// </summary>
		public override async Task ExecuteAsync(JobContext job, HashSet<FileReference> buildProducts, Dictionary<string, HashSet<FileReference>> tagNameToFileSet)
		{
			string[] keys = _parameters.KeyPath.Split('.');
			if (keys.Length == 0)
			{
				return;
			}
			HashSet<FileReference> files = ResolveFilespec(Unreal.RootDirectory, _parameters.Files, tagNameToFileSet);
			foreach (string jsonFile in files.Select(f => f.FullName))
			{
				string oldContents = await File.ReadAllTextAsync(jsonFile);
				IDictionary<string, object> paramObj = fastJSON.JSON.Instance.Parse(oldContents) as IDictionary<string, object>;
				IDictionary<string, object> currObj = paramObj;
				for (int i = 0; i < keys.Length - 1; i++)
				{
					if (!currObj.TryGetValue(keys[i], out object nextNode))
					{
						currObj[keys[i]] = nextNode = new Dictionary<string, object>();
					}
					currObj = (IDictionary<string, object>)nextNode;
				}

				currObj[keys[keys.Length - 1]] = _parameters.NewValue;

				string newContents = JsonSerializer.Serialize(paramObj, new JsonSerializerOptions { WriteIndented = true });
				await File.WriteAllTextAsync(jsonFile, newContents, new UTF8Encoding(encoderShouldEmitUTF8Identifier: false));
			}
		}

		/// <summary>
		/// Placeholder comment
		/// </summary>
		public override void Write(XmlWriter writer)
		{
			Write(writer, _parameters);
		}

		/// <summary>
		/// Placeholder comment
		/// </summary>
		public override IEnumerable<string> FindConsumedTagNames()
		{
			foreach (string tagName in FindTagNamesFromFilespec(_parameters.Files))
			{
				yield return tagName;
			}
		}

		/// <summary>
		/// Placeholder comment
		/// </summary>
		public override IEnumerable<string> FindProducedTagNames()
		{
			yield break;
		}
	}
}
