// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Linq;
using System.Threading.Tasks;
using System.Xml;
using EpicGames.Core;
using Microsoft.Extensions.Logging;
using UnrealBuildBase;

namespace AutomationTool.Tasks
{
	/// <summary>
	/// Parameters for a move task
	/// </summary>
	public class MoveTaskParameters
	{
		/// <summary>
		/// Optional filter to be applied to the list of input files.
		/// </summary>
		[TaskParameter(Optional = true, ValidationType = TaskParameterValidationType.FileSpec)]
		public string Files { get; set; }

		/// <summary>
		/// The pattern(s) to copy from (for example, Engine/*.txt).
		/// </summary>
		[TaskParameter(ValidationType = TaskParameterValidationType.FileSpec)]
		public string From { get; set; }

		/// <summary>
		/// The directory to copy to.
		/// </summary>
		[TaskParameter(ValidationType = TaskParameterValidationType.FileSpec)]
		public string To { get; set; }

		/// <summary>
		/// Optionally if files should be overwritten, defaults to false.
		/// </summary>
		[TaskParameter(Optional = true)]
		public bool Overwrite { get; set; } = false;

		/// <summary>
		/// Tag to be applied to build products of this task.
		/// </summary>
		[TaskParameter(Optional = true, ValidationType = TaskParameterValidationType.TagList)]
		public string Tag { get; set; }

		/// <summary>
		/// Whether or not to throw an error if no files were found to copy
		/// </summary>
		[TaskParameter(Optional = true)]
		public bool ErrorIfNotFound { get; set; } = false;
	}

	/// <summary>
	/// Moves files from one directory to another.
	/// </summary>
	[TaskElement("Move", typeof(MoveTaskParameters))]
	public class MoveTask : BgTaskImpl
	{
		readonly MoveTaskParameters _parameters;

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="parameters">Parameters for this task</param>
		public MoveTask(MoveTaskParameters parameters)
		{
			_parameters = parameters;
		}

		/// <summary>
		/// ExecuteAsync the task.
		/// </summary>
		/// <param name="job">Information about the current job</param>
		/// <param name="buildProducts">Set of build products produced by this node.</param>
		/// <param name="tagNameToFileSet">Mapping from tag names to the set of files they include</param>
		public override Task ExecuteAsync(JobContext job, HashSet<FileReference> buildProducts, Dictionary<string, HashSet<FileReference>> tagNameToFileSet)
		{
			// Parse all the source patterns
			FilePattern sourcePattern = new FilePattern(Unreal.RootDirectory, _parameters.From);

			// Parse the target pattern
			FilePattern targetPattern = new FilePattern(Unreal.RootDirectory, _parameters.To);

			// Apply the filter to the source files
			HashSet<FileReference> files = null;
			if (!String.IsNullOrEmpty(_parameters.Files))
			{
				sourcePattern = sourcePattern.AsDirectoryPattern();
				files = ResolveFilespec(sourcePattern.BaseDirectory, _parameters.Files, tagNameToFileSet);
			}

			// Build the file mapping
			Dictionary<FileReference, FileReference> targetFileToSourceFile;

			try
			{
				targetFileToSourceFile = FilePattern.CreateMapping(files, ref sourcePattern, ref targetPattern);

				// Check we got some files
				if (targetFileToSourceFile.Count == 0)
				{
					if (_parameters.ErrorIfNotFound)
					{
						throw new AutomationException("No files found matching '{0}'", sourcePattern);
					}
					else
					{
						Logger.LogInformation("No files found matching '{SourcePattern}'", sourcePattern);
					}
					return Task.CompletedTask;
				}
				// Copy them all
				Logger.LogInformation("Moving {Arg0} file{Arg1} from {Arg2} to {Arg3}...", targetFileToSourceFile.Count, (targetFileToSourceFile.Count == 1) ? "" : "s", sourcePattern.BaseDirectory, targetPattern.BaseDirectory);
				CommandUtils.ParallelMoveFiles(targetFileToSourceFile.Select(x => new KeyValuePair<FileReference, FileReference>(x.Value, x.Key)), _parameters.Overwrite);

				// Update the list of build products
				buildProducts.UnionWith(targetFileToSourceFile.Keys);

				// Apply the optional output tag to them
				foreach (string tagName in FindTagNamesFromList(_parameters.Tag))
				{
					FindOrAddTagSet(tagNameToFileSet, tagName).UnionWith(targetFileToSourceFile.Keys);
				}
			}
			catch (FilePatternSourceFileMissingException ex)
			{
				if (_parameters.ErrorIfNotFound)
				{
					throw new AutomationException(ex, "Error while trying to create file pattern match for '{0}': {1}", sourcePattern, ex.Message);
				}
				else
				{
					Logger.LogInformation(ex, "Error while trying to create file pattern match for '{SourcePattern}': {Message}", sourcePattern, ex.Message);
				}
			}

			return Task.CompletedTask;
		}

		/// <summary>
		/// Output this task out to an XML writer.
		/// </summary>
		public override void Write(XmlWriter writer)
		{
			Write(writer, _parameters);
		}

		/// <summary>
		/// Find all the tags which are used as inputs to this task
		/// </summary>
		/// <returns>The tag names which are read by this task</returns>
		public override IEnumerable<string> FindConsumedTagNames()
		{
			foreach (string tagName in FindTagNamesFromFilespec(_parameters.Files))
			{
				yield return tagName;
			}
		}

		/// <summary>
		/// Find all the tags which are modified by this task
		/// </summary>
		/// <returns>The tag names which are modified by this task</returns>
		public override IEnumerable<string> FindProducedTagNames()
		{
			return FindTagNamesFromList(_parameters.Tag);
		}
	}
}
