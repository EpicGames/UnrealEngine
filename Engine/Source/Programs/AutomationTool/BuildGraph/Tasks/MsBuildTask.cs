// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Linq;
using System.Threading.Tasks;
using System.Xml;
using EpicGames.Core;
using UnrealBuildBase;

namespace AutomationTool.Tasks
{
	/// <summary>
	/// Parameters for a task that executes MSBuild
	/// </summary>
	public class MsBuildTaskParameters
	{
		/// <summary>
		/// The C# project file to compile. Using semicolons, more than one project file can be specified.
		/// </summary>
		[TaskParameter]
		public string Project { get; set; }

		/// <summary>
		/// The configuration to compile.
		/// </summary>
		[TaskParameter(Optional = true)]
		public string Configuration { get; set; }

		/// <summary>
		/// The platform to compile.
		/// </summary>
		[TaskParameter(Optional = true)]
		public string Platform { get; set; }

		/// <summary>
		/// Additional options to pass to the compiler.
		/// </summary>
		[TaskParameter(Optional = true)]
		public string Arguments { get; set; }

		/// <summary>
		/// The MSBuild output verbosity.
		/// </summary>
		[TaskParameter(Optional = true)]
		public string Verbosity { get; set; } = "minimal";
	}

	/// <summary>
	/// Executes MsBuild
	/// </summary>
	[TaskElement("MsBuild", typeof(MsBuildTaskParameters))]
	public class MsBuildTask : BgTaskImpl
	{
		/// <summary>
		/// Parameters for the task
		/// </summary>
		readonly MsBuildTaskParameters _parameters;

		/// <summary>
		/// Constructor.
		/// </summary>
		/// <param name="parameters">Parameters for this task</param>
		public MsBuildTask(MsBuildTaskParameters parameters)
		{
			_parameters = parameters;
		}

		/// <summary>
		/// ExecuteAsync the task.
		/// </summary>
		/// <param name="job">Information about the current job</param>
		/// <param name="buildProducts">Set of build products produced by this node.</param>
		/// <param name="tagNameToFileSet">Mapping from tag names to the set of files they include</param>
		public override Task ExecuteAsync(JobContext job, HashSet<FileReference> buildProducts, Dictionary<string, HashSet<FileReference>> tagNameToFileSet)
		{
			// Get the project file
			HashSet<FileReference> projectFiles = ResolveFilespec(Unreal.RootDirectory, _parameters.Project, tagNameToFileSet);
			foreach (FileReference projectFile in projectFiles)
			{
				if (!FileReference.Exists(projectFile))
				{
					throw new AutomationException("Couldn't find project file '{0}'", projectFile.FullName);
				}
			}

			// Build the argument list
			List<string> arguments = new List<string>();
			if (!String.IsNullOrEmpty(_parameters.Platform))
			{
				arguments.Add(String.Format("/p:Platform={0}", CommandUtils.MakePathSafeToUseWithCommandLine(_parameters.Platform)));
			}
			if (!String.IsNullOrEmpty(_parameters.Configuration))
			{
				arguments.Add(String.Format("/p:Configuration={0}", CommandUtils.MakePathSafeToUseWithCommandLine(_parameters.Configuration)));
			}
			if (!String.IsNullOrEmpty(_parameters.Arguments))
			{
				arguments.Add(_parameters.Arguments);
			}
			if (!String.IsNullOrEmpty(_parameters.Verbosity))
			{
				arguments.Add(String.Format("/verbosity:{0}", _parameters.Verbosity));
			}
			arguments.Add("/nologo");

			// Build all the projects
			foreach (FileReference projectFile in projectFiles)
			{
				CommandUtils.MsBuild(CommandUtils.CmdEnv, projectFile.FullName, String.Join(" ", arguments), null);
			}

			return Task.CompletedTask;
		}

		/// <summary>
		/// Output this task out to an XML writer.
		/// </summary>
		public override void Write(XmlWriter writer)
		{
			Write(writer, _parameters);
		}

		/// <summary>
		/// Find all the tags which are used as inputs to this task
		/// </summary>
		/// <returns>The tag names which are read by this task</returns>
		public override IEnumerable<string> FindConsumedTagNames()
		{
			return FindTagNamesFromFilespec(_parameters.Project);
		}

		/// <summary>
		/// Find all the tags which are modified by this task
		/// </summary>
		/// <returns>The tag names which are modified by this task</returns>
		public override IEnumerable<string> FindProducedTagNames()
		{
			return Enumerable.Empty<string>();
		}
	}
}
