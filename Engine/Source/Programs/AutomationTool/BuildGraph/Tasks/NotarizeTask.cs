// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Text.RegularExpressions;
using System.Threading.Tasks;
using System.Xml;
using EpicGames.Core;
using Microsoft.Extensions.Logging;
using UnrealBuildTool;

#pragma warning disable SYSLIB0014

namespace AutomationTool.Tasks
{
	/// <summary>
	/// Parameters for a task that notarizes a dmg via the apple notarization process
	/// </summary>
	public class NotarizeTaskParameters
	{
		/// <summary>
		/// Path to the dmg to notarize
		/// </summary>
		[TaskParameter]
		public string DmgPath { get; set; }

		/// <summary>
		/// primary bundle ID
		/// </summary>
		[TaskParameter]
		public string BundleID { get; set; }

		/// <summary>
		/// Apple ID Username
		/// </summary>
		[TaskParameter]
		public string UserName { get; set; }

		/// <summary>
		/// The keychain ID
		/// </summary>
		[TaskParameter]
		public string KeyChainID { get; set; }

		/// <summary>
		/// When true the notarization ticket will be stapled
		/// </summary>
		[TaskParameter(Optional = true)]
		public bool RequireStapling { get; set; } = false;
	}

	[TaskElement("Notarize", typeof(NotarizeTaskParameters))]
	class NotarizeTask : BgTaskImpl
	{
		/// <summary>
		/// Parameters for the task
		/// </summary>
		readonly NotarizeTaskParameters _parameters;

		/// <summary>
		/// Constructor.
		/// </summary>
		/// <param name="parameters">Parameters for the task</param>
		public NotarizeTask(NotarizeTaskParameters parameters)
		{
			_parameters = parameters;
		}

		/// <summary>
		/// ExecuteAsync the task.
		/// </summary>
		/// <param name="job">Information about the current job</param>
		/// <param name="buildProducts">Set of build products produced by this node.</param>
		/// <param name="tagNameToFileSet">Mapping from tag names to the set of files they include</param>
		public override async Task ExecuteAsync(JobContext job, HashSet<FileReference> buildProducts, Dictionary<string, HashSet<FileReference>> tagNameToFileSet)
		{
			// Ensure running on a mac.
			if (BuildHostPlatform.Current.Platform != UnrealTargetPlatform.Mac)
			{
				throw new AutomationException("Notarization can only be run on a Mac!");
			}

			// Ensure file exists
			FileReference dmg = new FileReference(_parameters.DmgPath);
			if (!FileReference.Exists(dmg))
			{
				throw new AutomationException("Couldn't find a file to notarize at {0}", dmg.FullName);
			}

			int exitCode;
			Logger.LogInformation("Uploading {Arg0} to the notarization server...", dmg.FullName);

			// The notarytool will timeout after 5 retries or 1 hour. Whichever comes first.
			const int MaxNumRetries = 5;
			const int MaxTimeoutInMilliseconds = 3600000;
			long timeoutInMilliseconds = MaxTimeoutInMilliseconds;
			string output = "";

			System.Diagnostics.Stopwatch timeoutStopwatch = System.Diagnostics.Stopwatch.StartNew();

			for (int numRetries = 0; numRetries < MaxNumRetries; numRetries++)
			{
				string commandLine = String.Format("notarytool submit \"{0}\" --keychain-profile \"{1}\" --wait --timeout \"{2}\"", dmg.FullName, _parameters.KeyChainID, timeoutInMilliseconds);
				output = CommandUtils.RunAndLog("xcrun", commandLine, out exitCode);

				if (exitCode == 0)
				{
					break;
				}

				if (timeoutStopwatch.ElapsedMilliseconds >= timeoutInMilliseconds)
				{
					Logger.LogInformation("notarytool timed out after {TimeoutInMilliseconds}ms.", timeoutInMilliseconds);
					timeoutStopwatch.Stop();
				}
				else if (numRetries < MaxNumRetries)
				{
					Logger.LogInformation("notarytool failed with exit {ExitCode} attempting retry {NumRetries} of {MaxNumRetries}", exitCode, numRetries, MaxNumRetries);
					await Task.Delay(2000);
					timeoutInMilliseconds = MaxTimeoutInMilliseconds - timeoutStopwatch.ElapsedMilliseconds;
					continue;
				}

				Logger.LogInformation("Retries have been exhausted");
				throw new AutomationException("notarytool failed with exit {0}", exitCode);
			}

			// Grab the UUID from the log
			string requestUuid;
			try
			{
				requestUuid = Regex.Match(output, "id: ([a-zA-Z0-9]{8}-[a-zA-Z0-9]{4}-[a-zA-Z0-9]{4}-[a-zA-Z0-9]{4}-[a-zA-Z0-9]{12})").Groups[1].Value.Trim();
			}
			catch (Exception ex)
			{
				throw new AutomationException(ex, "Couldn't get UUID from the log output {0}", output);
			}

			try
			{
				MatchCollection statusMatches = Regex.Matches(output, "(?<=status: ).+");
				// The last status update is the right one.
				string status = statusMatches[statusMatches.Count - 1].Value.ToLower();

				if (status == "accepted")
				{
					if (_parameters.RequireStapling)
					{
						// once we have a log file, print it out, staple, and we're done.
						Logger.LogInformation("{Text}", GetRequestLogs(requestUuid));
						string commandLine = String.Format("stapler staple {0}", dmg.FullName);
						output = CommandUtils.RunAndLog("xcrun", commandLine, out exitCode);
						if (exitCode != 0)
						{
							throw new AutomationException("stapler failed with exit {0}", exitCode);
						}
					}
				}
				else
				{
					Logger.LogError("{Text}", GetRequestLogs(requestUuid));
					throw new AutomationException($"Could not notarize the app. Request status: {0}. See log output above.", status);
				}
			}
			catch (Exception ex)
			{
				if (ex is AutomationException)
				{
					throw;
				}
				else
				{
					throw new AutomationException(ex, "Querying for the notarization result failed, output: {0}", output);
				}
			}
		}

		private string GetRequestLogs(string requestUuid)
		{
			try
			{
				string logCommand = String.Format("notarytool log {0} --keychain-profile \"{1}\"", requestUuid, _parameters.KeyChainID);
				IProcessResult logResult = CommandUtils.Run("xcrun", logCommand);

				string responseContent = null;
				if (logResult.bExitCodeSuccess)
				{
					responseContent = logResult.Output;
				}

				return responseContent;
			}
			catch (Exception ex)
			{
				throw new AutomationException(ex, String.Format("Couldn't complete the request, error: {0}", ex.Message));
			}
		}

		/// <summary>
		/// Output this task out to an XML writer.
		/// </summary>
		public override void Write(XmlWriter writer)
		{
			Write(writer, _parameters);
		}

		/// <summary>
		/// Find all the tags which are used as inputs to this task
		/// </summary>
		/// <returns>The tag names which are read by this task</returns>
		public override IEnumerable<string> FindConsumedTagNames()
		{
			yield break;
		}

		/// <summary>
		/// Find all the tags which are modified by this task
		/// </summary>
		/// <returns>The tag names which are modified by this task</returns>
		public override IEnumerable<string> FindProducedTagNames()
		{
			yield break;
		}
	}
}
