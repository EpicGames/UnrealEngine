// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Data;
using System.IO;
using System.Linq;
using System.Net.Http;
using System.Text;
using System.Text.Json;
using System.Text.RegularExpressions;
using System.Threading.Tasks;
using System.Xml;
using EpicGames.Core;
using Microsoft.Extensions.Logging;
using UnrealBuildBase;

namespace AutomationTool.Tasks
{
	/// <summary>
	/// Parameters for a NuGetLicenseCheck task
	/// </summary>
	public class NuGetLicenseCheckTaskParameters
	{
		/// <summary>
		/// Base directory for running the command
		/// </summary>
		[TaskParameter]
		public string BaseDir { get; set; }

		/// <summary>
		/// Specifies a list of packages to ignore for version checks, separated by semicolons. Optional version number may be specified with 'name@version' syntax.
		/// </summary>
		[TaskParameter(Optional = true)]
		public string IgnorePackages { get; set; }

		/// <summary>
		/// Directory containing allowed licenses
		/// </summary>
		[TaskParameter(Optional = true)]
		public DirectoryReference LicenseDir { get; set; }

		/// <summary>
		/// Path to a csv file to write with list of packages and their licenses
		/// </summary>
		[TaskParameter(Optional = true)]
		public FileReference CsvFile { get; set; }

		/// <summary>
		/// Override path to dotnet executable
		/// </summary>
		[TaskParameter(Optional = true)]
		public FileReference DotNetPath { get; set; }
	}

	/// <summary>
	/// Verifies which licenses are in use by nuget dependencies
	/// </summary>
	[TaskElement("NuGet-LicenseCheck", typeof(NuGetLicenseCheckTaskParameters))]
	public class NuGetLicenseCheckTask : SpawnTaskBase
	{
		class LicenseConfig
		{
			public List<string> Urls { get; set; } = new List<string>();
		}

		readonly NuGetLicenseCheckTaskParameters _parameters;

		/// <summary>
		/// Construct a NuGetLicenseCheckTask task
		/// </summary>
		/// <param name="parameters">Parameters for the task</param>
		public NuGetLicenseCheckTask(NuGetLicenseCheckTaskParameters parameters)
		{
			_parameters = parameters;
		}

		enum PackageState
		{
			None,
			IgnoredViaArgs,
			MissingPackageFolder,
			MissingPackageDescriptor,
			Valid,
		}

		class PackageInfo
		{
			public string _name;
			public string _version;
			public string _projectUrl;
			public LicenseInfo _license;
			public string _licenseSource;
			public PackageState _state;
			public FileReference _descriptor;
		}

		class LicenseInfo
		{
			public IoHash _hash;
			public string _text;
			public string _normalizedText;
			public string _extension;
			public bool _approved;
			public FileReference _file;
		}

		static LicenseInfo FindOrAddLicense(Dictionary<IoHash, LicenseInfo> licenses, string text, string extension)
		{
			string normalizedText = text;
			normalizedText = Regex.Replace(normalizedText, @"^\s+", "", RegexOptions.Multiline);
			normalizedText = Regex.Replace(normalizedText, @"\s+$", "", RegexOptions.Multiline);
			normalizedText = Regex.Replace(normalizedText, "^(?:MIT License|The MIT License \\(MIT\\))\n", "", RegexOptions.Multiline);
			normalizedText = Regex.Replace(normalizedText, "^Copyright \\(c\\)[^\n]*\\s*(?:All rights reserved\\.?\\s*)?", "", RegexOptions.Multiline);
			normalizedText = Regex.Replace(normalizedText, @"\s+", " ");
			normalizedText = normalizedText.Trim();

			byte[] data = Encoding.UTF8.GetBytes(normalizedText);
			IoHash hash = IoHash.Compute(data);

			LicenseInfo licenseInfo;
			if (!licenses.TryGetValue(hash, out licenseInfo))
			{
				licenseInfo = new LicenseInfo();
				licenseInfo._hash = hash;
				licenseInfo._text = text;
				licenseInfo._normalizedText = normalizedText;
				licenseInfo._extension = extension;
				licenses.Add(hash, licenseInfo);
			}
			return licenseInfo;
		}

		/// <summary>
		/// ExecuteAsync the task.
		/// </summary>
		/// <param name="job">Information about the current job</param>
		/// <param name="buildProducts">Set of build products produced by this node.</param>
		/// <param name="tagNameToFileSet">Mapping from tag names to the set of files they include</param>
		public override async Task ExecuteAsync(JobContext job, HashSet<FileReference> buildProducts, Dictionary<string, HashSet<FileReference>> tagNameToFileSet)
		{
			FileReference dotNetPath = _parameters.DotNetPath ?? Unreal.DotnetPath;

			IProcessResult nuGetOutput = await ExecuteAsync(dotNetPath.FullName, $"nuget locals global-packages --list", logOutput: false);
			if (nuGetOutput.ExitCode != 0)
			{
				throw new AutomationException("DotNet terminated with an exit code indicating an error ({0})", nuGetOutput.ExitCode);
			}

			List<DirectoryReference> nuGetPackageDirs = new List<DirectoryReference>();
			foreach (string line in nuGetOutput.Output.Split('\n'))
			{
				int colonIdx = line.IndexOf(':', StringComparison.Ordinal);
				if (colonIdx != -1)
				{
					DirectoryReference nuGetPackageDir = new DirectoryReference(line.Substring(colonIdx + 1).Trim());
					Logger.LogInformation("Using NuGet package directory: {Path}", nuGetPackageDir);
					nuGetPackageDirs.Add(nuGetPackageDir);
				}
			}

			const string UnknownPrefix = "Unknown-";

			IProcessResult packageListOutput = await ExecuteAsync(dotNetPath.FullName, "list package --include-transitive", workingDir: _parameters.BaseDir, logOutput: false);
			if (packageListOutput.ExitCode != 0)
			{
				throw new AutomationException("DotNet terminated with an exit code indicating an error ({0})", packageListOutput.ExitCode);
			}

			Dictionary<string, PackageInfo> packages = new Dictionary<string, PackageInfo>();
			foreach (string line in packageListOutput.Output.Split('\n'))
			{
				Match match = Regex.Match(line, @"^\s*>\s*([^\s]+)\s+(?:[^\s]+\s+)?([^\s]+)\s*$");
				if (match.Success)
				{
					PackageInfo info = new PackageInfo();
					info._name = match.Groups[1].Value;
					info._version = match.Groups[2].Value;
					packages.TryAdd($"{info._name}@{info._version}", info);
				}
			}

			DirectoryReference packageRootDir = DirectoryReference.Combine(DirectoryReference.GetSpecialFolder(Environment.SpecialFolder.UserProfile), ".nuget", "packages");
			if (!DirectoryReference.Exists(packageRootDir))
			{
				throw new AutomationException("Missing NuGet package cache at {0}", packageRootDir);
			}

			HashSet<string> licenseUrls = new HashSet<string>(StringComparer.OrdinalIgnoreCase);

			Dictionary<IoHash, LicenseInfo> licenses = new Dictionary<IoHash, LicenseInfo>();
			if (_parameters.LicenseDir != null)
			{
				Logger.LogInformation("Reading allowed licenses from {LicenseDir}", _parameters.LicenseDir);
				foreach (FileReference file in DirectoryReference.EnumerateFiles(_parameters.LicenseDir))
				{
					if (!file.GetFileName().StartsWith(UnknownPrefix, StringComparison.OrdinalIgnoreCase))
					{
						try
						{
							if (file.HasExtension(".json"))
							{
								byte[] data = await FileReference.ReadAllBytesAsync(file);
								LicenseConfig config = JsonSerializer.Deserialize<LicenseConfig>(data, new JsonSerializerOptions { PropertyNameCaseInsensitive = true, AllowTrailingCommas = true, ReadCommentHandling = JsonCommentHandling.Skip });
								licenseUrls.UnionWith(config.Urls);
							}
							else if (file.HasExtension(".txt") || file.HasExtension(".html") || file.HasExtension(".md"))
							{
								string text = await FileReference.ReadAllTextAsync(file);
								LicenseInfo license = FindOrAddLicense(licenses, text, file.GetFileNameWithoutExtension());
								license._file = file;
								license._approved = true;
							}
						}
						catch (Exception ex)
						{
							throw new AutomationException(ex, $"Error parsing {file}: {ex.Message}");
						}
					}
				}
			}

			HashSet<string> ignorePackages = new HashSet<string>(StringComparer.OrdinalIgnoreCase);
			if (_parameters.IgnorePackages != null)
			{
				ignorePackages.UnionWith(_parameters.IgnorePackages.Split(';'));
			}

			Dictionary<string, LicenseInfo> licenseUrlToInfo = new Dictionary<string, LicenseInfo>(StringComparer.OrdinalIgnoreCase);
			foreach (PackageInfo info in packages.Values)
			{
				if (ignorePackages.Contains(info._name) || ignorePackages.Contains($"{info._name}@{info._version}"))
				{
					info._state = PackageState.IgnoredViaArgs;
					continue;
				}

				DirectoryReference packageDir = nuGetPackageDirs.Select(x => DirectoryReference.Combine(x, info._name.ToLowerInvariant(), info._version.ToLowerInvariant())).FirstOrDefault(x => DirectoryReference.Exists(x));
				if (packageDir == null)
				{
					info._state = PackageState.MissingPackageFolder;
					continue;
				}

				info._descriptor = FileReference.Combine(packageDir, $"{info._name.ToLowerInvariant()}.nuspec");
				if (!FileReference.Exists(info._descriptor))
				{
					info._state = PackageState.MissingPackageDescriptor;
					continue;
				}

				using (Stream stream = FileReference.Open(info._descriptor, FileMode.Open, FileAccess.Read, FileShare.Read))
				{
					using XmlTextReader xmlReader = new XmlTextReader(stream);
					xmlReader.Namespaces = false;

					XmlDocument xmlDocument = new XmlDocument();
					xmlDocument.Load(xmlReader);

					XmlNode projectUrlNode = xmlDocument.SelectSingleNode("/package/metadata/projectUrl");
					info._projectUrl = projectUrlNode?.InnerText;

					if (info._license == null)
					{
						XmlNode licenseNode = xmlDocument.SelectSingleNode("/package/metadata/license");
						if (licenseNode?.Attributes["type"]?.InnerText?.Equals("file", StringComparison.Ordinal) ?? false)
						{
							FileReference licenseFile = FileReference.Combine(packageDir, licenseNode.InnerText);
							if (FileReference.Exists(licenseFile))
							{
								string text = await FileReference.ReadAllTextAsync(licenseFile);
								info._license = FindOrAddLicense(licenses, text, licenseFile.GetExtension());
								info._licenseSource = licenseFile.FullName;
							}
						}
					}

					if (info._license == null)
					{
						XmlNode licenseUrlNode = xmlDocument.SelectSingleNode("/package/metadata/licenseUrl");

						string licenseUrl = licenseUrlNode?.InnerText;
						if (licenseUrl != null)
						{
							licenseUrl = Regex.Replace(licenseUrl, @"^https://github.com/(.*)/blob/(.*)$", @"https://raw.githubusercontent.com/$1/$2");
							info._licenseSource = licenseUrl;

							if (!licenseUrlToInfo.TryGetValue(licenseUrl, out info._license))
							{
								using (HttpClient client = new HttpClient())
								{
									using HttpResponseMessage response = await client.GetAsync(licenseUrl);
									if (!response.IsSuccessStatusCode)
									{
										Logger.LogError("Unable to fetch license from {LicenseUrl}", licenseUrl);
									}
									else
									{
										string text = await response.Content.ReadAsStringAsync();
										string type = (response.Content.Headers.ContentType?.MediaType == "text/html") ? ".html" : ".txt";
										info._license = FindOrAddLicense(licenses, text, type);
										if (!info._license._approved)
										{
											info._license._approved = licenseUrls.Contains(licenseUrl);
										}
										licenseUrlToInfo.Add(licenseUrl, info._license);
									}
								}
							}
						}
					}
				}

				info._state = PackageState.Valid;
			}

			Logger.LogInformation("Referenced Packages:");
			Logger.LogInformation("");
			foreach (PackageInfo info in packages.Values.OrderBy(x => x._name).ThenBy(x => x._version))
			{
				switch (info._state)
				{
					case PackageState.IgnoredViaArgs:
						Logger.LogInformation("  {Name,-60} {Version,-10} Explicitly ignored via task arguments", info._name, info._version);
						break;
					case PackageState.MissingPackageFolder:
						Logger.LogInformation("  {Name,-60} {Version,-10} NuGet package not found", info._name, info._version);
						break;
					case PackageState.MissingPackageDescriptor:
						Logger.LogWarning("  {Name,-60} {Version,-10} Missing package descriptor: {NuSpecFile}", info._name, info._version, info._descriptor);
						break;
					case PackageState.Valid:
						if (info._license == null)
						{
							Logger.LogError("  {Name,-60} {Version,-10} No license metadata found", info._name, info._version);
						}
						else if (!info._license._approved)
						{
							Logger.LogWarning("  {Name,-60} {Version,-10} {Hash}", info._name, info._version, info._license._hash);
						}
						else
						{
							Logger.LogInformation("  {Name,-60} {Version,-10} {Hash}", info._name, info._version, info._license._hash);
						}
						break;
					default:
						Logger.LogError("  {Name,-60} {Version,-10} Unhandled state: {State}", info._name, info._version, info._state);
						break;
				}
			}

			Dictionary<LicenseInfo, List<PackageInfo>> missingLicenses = new Dictionary<LicenseInfo, List<PackageInfo>>();
			foreach (PackageInfo packageInfo in packages.Values)
			{
				if (packageInfo._license != null && !packageInfo._license._approved)
				{
					List<PackageInfo> licensePackages;
					if (!missingLicenses.TryGetValue(packageInfo._license, out licensePackages))
					{
						licensePackages = new List<PackageInfo>();
						missingLicenses.Add(packageInfo._license, licensePackages);
					}
					licensePackages.Add(packageInfo);
				}
			}

			if (missingLicenses.Count > 0)
			{
				DirectoryReference licenseDir = _parameters.LicenseDir ?? DirectoryReference.Combine(Unreal.RootDirectory, "Engine", "Saved", "Licenses");
				DirectoryReference.CreateDirectory(licenseDir);

				Logger.LogInformation("");
				Logger.LogInformation("Missing licenses:");
				foreach ((LicenseInfo missingLicense, List<PackageInfo> missingLicensePackages) in missingLicenses.OrderBy(x => x.Key._hash))
				{
					FileReference outputFile = FileReference.Combine(licenseDir, $"{UnknownPrefix}{missingLicense._hash}{missingLicense._extension}");
					await FileReference.WriteAllTextAsync(outputFile, missingLicense._text);

					Logger.LogInformation("");
					Logger.LogInformation("  {LicenseFile}", outputFile);
					foreach (PackageInfo licensePackage in missingLicensePackages)
					{
						Logger.LogInformation("  -> {Name} {Version} ({Source})", licensePackage._name, licensePackage._version, licensePackage._licenseSource);
					}
				}
			}

			if (_parameters.CsvFile != null)
			{
				Logger.LogInformation("Writing {File}", _parameters.CsvFile);
				DirectoryReference.CreateDirectory(_parameters.CsvFile.Directory);
				using (StreamWriter writer = new StreamWriter(_parameters.CsvFile.FullName))
				{
					await writer.WriteLineAsync($"Package,Version,Project Url,License Url,License Hash,License File");
					foreach (PackageInfo packageInfo in packages.Values)
					{
						string relativeLicensePath = "";
						if (packageInfo._license?._file != null)
						{
							relativeLicensePath = packageInfo._license._file.MakeRelativeTo(_parameters.CsvFile.Directory);
						}

						string licenseUrl = "";
						if (packageInfo._licenseSource != null && packageInfo._licenseSource.StartsWith("http", StringComparison.OrdinalIgnoreCase))
						{
							licenseUrl = packageInfo._licenseSource;
						}

						await writer.WriteLineAsync($"\"{packageInfo._name}\",\"{packageInfo._version}\",{packageInfo._projectUrl},{licenseUrl},{packageInfo._license?._hash},{relativeLicensePath}");
					}
				}
			}
		}

		/// <summary>
		/// Output this task out to an XML writer.
		/// </summary>
		public override void Write(XmlWriter writer)
		{
			Write(writer, _parameters);
		}

		/// <summary>
		/// Find all the tags which are used as inputs to this task
		/// </summary>
		/// <returns>The tag names which are read by this task</returns>
		public override IEnumerable<string> FindConsumedTagNames()
		{
			yield break;
		}

		/// <summary>
		/// Find all the tags which are modified by this task
		/// </summary>
		/// <returns>The tag names which are modified by this task</returns>
		public override IEnumerable<string> FindProducedTagNames()
		{
			yield break;
		}
	}
}
