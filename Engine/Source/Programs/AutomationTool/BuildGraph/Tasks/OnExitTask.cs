// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Linq;
using System.Threading.Tasks;
using System.Xml;
using EpicGames.Core;

namespace AutomationTool.Tasks
{
	/// <summary>
	/// Parameters for a spawn task
	/// </summary>
	public class OnExitTaskParameters
	{
		/// <summary>
		/// Executable to spawn.
		/// </summary>
		[TaskParameter]
		public string Command { get; set; } = String.Empty;

		/// <summary>
		/// Whether to execute on lease termination
		/// </summary>
		[TaskParameter(Optional = true)]
		public bool Lease { get; set; } = false;
	}

	/// <summary>
	/// Spawns an external executable and waits for it to complete.
	/// </summary>
	[TaskElement("OnExit", typeof(OnExitTaskParameters))]
	public class OnExitTask : BgTaskImpl
	{
		readonly OnExitTaskParameters _parameters;

		/// <summary>
		/// Construct a spawn task
		/// </summary>
		/// <param name="parameters">Parameters for the task</param>
		public OnExitTask(OnExitTaskParameters parameters)
		{
			_parameters = parameters;
		}

		/// <inheritdoc/>
		public override async Task ExecuteAsync(JobContext job, HashSet<FileReference> buildProducts, Dictionary<string, HashSet<FileReference>> tagNameToFileSet)
		{
			string[] commands = _parameters.Command.Split('\n').Select(x => x.Trim()).ToArray();
			await AddCleanupCommandsAsync(commands, _parameters.Lease);
		}

		/// <inheritdoc/>
		public override void Write(XmlWriter writer)
		{
			Write(writer, _parameters);
		}

		/// <inheritdoc/>
		public override IEnumerable<string> FindConsumedTagNames()
		{
			yield break;
		}

		/// <inheritdoc/>
		public override IEnumerable<string> FindProducedTagNames()
		{
			yield break;
		}
	}
}
