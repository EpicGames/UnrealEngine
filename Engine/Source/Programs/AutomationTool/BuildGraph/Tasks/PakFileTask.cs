// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.IO;
using System.Text;
using System.Threading.Tasks;
using System.Xml;
using EpicGames.Core;
using Microsoft.Extensions.Logging;
using UnrealBuildBase;
using UnrealBuildTool;

namespace AutomationTool.Tasks
{
	/// <summary>
	/// Parameters for a task that runs the cooker
	/// </summary>
	public class PakFileTaskParameters
	{
		/// <summary>
		/// List of files, wildcards, and tag sets to add to the pak file, separated by ';' characters.
		/// </summary>
		[TaskParameter(ValidationType = TaskParameterValidationType.FileSpec)]
		public string Files { get; set; }

		/// <summary>
		/// PAK file to output.
		/// </summary>
		[TaskParameter]
		public FileReference Output { get; set; }

		/// <summary>
		/// Path to a Response File that contains a list of files to add to the pak file -- instead of specifying them individually.
		/// </summary>
		[TaskParameter(Optional = true)]
		public FileReference ResponseFile { get; set; }

		/// <summary>
		/// Directories to rebase the files relative to. If specified, the shortest path under a listed directory will be used for each file.
		/// </summary>
		[TaskParameter(Optional = true)]
		public HashSet<DirectoryReference> RebaseDir { get; set; }

		/// <summary>
		/// Script that gives the order of files.
		/// </summary>
		[TaskParameter(Optional = true)]
		public FileReference Order { get; set; }

		/// <summary>
		/// Encryption keys for this pak file.
		/// </summary>
		[TaskParameter(Optional = true)]
		public FileReference Sign { get; set; }

		/// <summary>
		/// Whether to compress files.
		/// </summary>
		[TaskParameter(Optional = true)]
		public bool Compress { get; set; } = true;

		/// <summary>
		/// Additional arguments to pass to UnrealPak.
		/// </summary>
		[TaskParameter(Optional = true)]
		public string Arguments { get; set; } = "";

		/// <summary>
		/// Tag to be applied to build products of this task.
		/// </summary>
		[TaskParameter(Optional = true, ValidationType = TaskParameterValidationType.TagList)]
		public string Tag { get; set; }
	}

	/// <summary>
	/// Creates a PAK file from a given set of files.
	/// </summary>
	[TaskElement("PakFile", typeof(PakFileTaskParameters))]
	public class PakFileTask : BgTaskImpl
	{
		readonly PakFileTaskParameters _parameters;

		/// <summary>
		/// Constructor.
		/// </summary>
		/// <param name="parameters">Parameters for this task</param>
		public PakFileTask(PakFileTaskParameters parameters)
		{
			_parameters = parameters;
		}

		/// <summary>
		/// ExecuteAsync the task.
		/// </summary>
		/// <param name="job">Information about the current job</param>
		/// <param name="buildProducts">Set of build products produced by this node.</param>
		/// <param name="tagNameToFileSet">Mapping from tag names to the set of files they include</param>
		public override async Task ExecuteAsync(JobContext job, HashSet<FileReference> buildProducts, Dictionary<string, HashSet<FileReference>> tagNameToFileSet)
		{
			// Find the directories we're going to rebase relative to
			HashSet<DirectoryReference> rebaseDirs = new HashSet<DirectoryReference> { Unreal.RootDirectory };
			if (_parameters.RebaseDir != null)
			{
				rebaseDirs.UnionWith(_parameters.RebaseDir);
			}

			// Get the output parameter
			FileReference outputFile = _parameters.Output;

			// Check for a ResponseFile parameter
			FileReference responseFile = _parameters.ResponseFile;
			if (responseFile == null)
			{
				// Get a unique filename for the response file
				responseFile = FileReference.Combine(new DirectoryReference(CommandUtils.CmdEnv.LogFolder), String.Format("PakList_{0}.txt", outputFile.GetFileNameWithoutExtension()));
				for (int idx = 2; FileReference.Exists(responseFile); idx++)
				{
					responseFile = FileReference.Combine(responseFile.Directory, String.Format("PakList_{0}_{1}.txt", outputFile.GetFileNameWithoutExtension(), idx));
				}

				// Write out the response file
				HashSet<FileReference> files = ResolveFilespec(Unreal.RootDirectory, _parameters.Files, tagNameToFileSet);
				using (StreamWriter writer = new StreamWriter(responseFile.FullName, false, new System.Text.UTF8Encoding(true)))
				{
					foreach (FileReference file in files)
					{
						string relativePath = FindShortestRelativePath(file, rebaseDirs);
						if (relativePath == null)
						{
							throw new AutomationException("Couldn't find relative path for '{0}' - not under any rebase directories", file.FullName);
						}

						string compressArg = _parameters.Compress ? " -compress" : "";
						await writer.WriteLineAsync($"\"{file.FullName}\" \"{relativePath}\"{compressArg}");
					}
				}
			}

			// Format the command line
			StringBuilder commandLine = new StringBuilder();
			commandLine.AppendFormat("{0} -create={1}", CommandUtils.MakePathSafeToUseWithCommandLine(outputFile.FullName), CommandUtils.MakePathSafeToUseWithCommandLine(responseFile.FullName));
			if (_parameters.Sign != null)
			{
				commandLine.AppendFormat(" -sign={0}", CommandUtils.MakePathSafeToUseWithCommandLine(_parameters.Sign.FullName));
			}
			if (_parameters.Order != null)
			{
				commandLine.AppendFormat(" -order={0}", CommandUtils.MakePathSafeToUseWithCommandLine(_parameters.Order.FullName));
			}
			if (Unreal.IsEngineInstalled())
			{
				commandLine.Append(" -installed");
			}

			// Get the executable path
			FileReference unrealPakExe;
			if (HostPlatform.Current.HostEditorPlatform == UnrealTargetPlatform.Win64)
			{
				unrealPakExe = ResolveFile("Engine/Binaries/Win64/UnrealPak.exe");
			}
			else
			{
				unrealPakExe = ResolveFile(String.Format("Engine/Binaries/{0}/UnrealPak", HostPlatform.Current.HostEditorPlatform.ToString()));
			}

			// Run it
			Logger.LogInformation("Running '{Arg0} {Arg1}'", CommandUtils.MakePathSafeToUseWithCommandLine(unrealPakExe.FullName), commandLine.ToString());
			CommandUtils.RunAndLog(CommandUtils.CmdEnv, unrealPakExe.FullName, commandLine.ToString(), Options: CommandUtils.ERunOptions.Default | CommandUtils.ERunOptions.UTF8Output);
			buildProducts.Add(outputFile);

			// Apply the optional tag to the output file
			foreach (string tagName in FindTagNamesFromList(_parameters.Tag))
			{
				FindOrAddTagSet(tagNameToFileSet, tagName).Add(outputFile);
			}
		}

		/// <summary>
		/// Find the shortest relative path of the given file from a set of base directories.
		/// </summary>
		/// <param name="file">Full path to a file</param>
		/// <param name="rebaseDirs">Possible base directories</param>
		/// <returns>The shortest relative path, or null if the file is not under any of them</returns>
		public static string FindShortestRelativePath(FileReference file, IEnumerable<DirectoryReference> rebaseDirs)
		{
			string relativePath = null;
			foreach (DirectoryReference rebaseDir in rebaseDirs)
			{
				if (file.IsUnderDirectory(rebaseDir))
				{
					string newRelativePath = file.MakeRelativeTo(rebaseDir);
					if (relativePath == null || newRelativePath.Length < relativePath.Length)
					{
						relativePath = newRelativePath;
					}
				}
			}
			return relativePath;
		}

		/// <summary>
		/// Output this task out to an XML writer.
		/// </summary>
		public override void Write(XmlWriter writer)
		{
			Write(writer, _parameters);
		}

		/// <summary>
		/// Find all the tags which are used as inputs to this task
		/// </summary>
		/// <returns>The tag names which are read by this task</returns>
		public override IEnumerable<string> FindConsumedTagNames()
		{
			return FindTagNamesFromFilespec(_parameters.Files);
		}

		/// <summary>
		/// Find all the tags which are modified by this task
		/// </summary>
		/// <returns>The tag names which are modified by this task</returns>
		public override IEnumerable<string> FindProducedTagNames()
		{
			return FindTagNamesFromList(_parameters.Tag);
		}
	}
}
