// Copyright Epic Games, Inc. All Rights Reserved.

using System.Collections.Generic;
using System.Linq;
using System.Threading.Tasks;
using System.Xml;
using EpicGames.Core;
using UnrealBuildBase;
using UnrealBuildTool;

#nullable enable

namespace AutomationTool.Tasks
{
	/// <summary>
	/// Parameters to query for all the targets in a project
	/// </summary>
	public class QueryTargetsTaskParameters
	{
		/// <summary>
		/// Path to the project file to query
		/// </summary>
		[TaskParameter(Optional = true)]
		public FileReference? ProjectFile { get; set; }

		/// <summary>
		/// Path to the output file to receive information about the targets
		/// </summary>
		[TaskParameter(Optional = true)]
		public FileReference? OutputFile { get; set; } = null;

		/// <summary>
		/// Write out all targets, even if a default is specified in the BuildSettings section of the Default*.ini files. 
		/// </summary>
		[TaskParameter(Optional = true)]
		public bool IncludeAllTargets { get; set; } = false;

		/// <summary>
		/// Tag to be applied to build products of this task.
		/// </summary>
		[TaskParameter(Optional = true, ValidationType = TaskParameterValidationType.TagList)]
		public string? Tag { get; set; }
	}

	/// <summary>
	/// Runs UBT to query all the targets for a particular project
	/// </summary>
	[TaskElement("QueryTargets", typeof(QueryTargetsTaskParameters))]
	public class QueryTargetsTask : BgTaskImpl
	{
		/// <summary>
		/// Parameters for this task
		/// </summary>
		readonly QueryTargetsTaskParameters _parameters;

		/// <summary>
		/// Construct a spawn task
		/// </summary>
		/// <param name="parameters">Parameters for the task</param>
		public QueryTargetsTask(QueryTargetsTaskParameters parameters)
		{
			_parameters = parameters;
		}

		/// <summary>
		/// ExecuteAsync the task.
		/// </summary>
		/// <param name="job">Information about the current job</param>
		/// <param name="buildProducts">Set of build products produced by this node.</param>
		/// <param name="tagNameToFileSet">Mapping from tag names to the set of files they include</param>
		public override Task ExecuteAsync(JobContext job, HashSet<FileReference> buildProducts, Dictionary<string, HashSet<FileReference>> tagNameToFileSet)
		{
			// Get the output file
			FileReference? outputFile = _parameters.OutputFile;
			if (outputFile == null)
			{
				if (_parameters.ProjectFile == null)
				{
					outputFile = FileReference.Combine(Unreal.EngineDirectory, "Intermediate", "TargetInfo.json");
				}
				else
				{
					outputFile = FileReference.Combine(_parameters.ProjectFile.Directory, "Intermediate", "TargetInfo.json");
				}
			}
			FileUtils.ForceDeleteFile(outputFile);

			// Run UBT to generate the target info
			List<string> arguments = new List<string> { "-Mode=QueryTargets" };
			if (_parameters.ProjectFile != null)
			{
				arguments.Add($"-Project={_parameters.ProjectFile}");
			}
			if (_parameters.IncludeAllTargets)
			{
				arguments.Add("-IncludeAllTargets");
			}
			CommandUtils.RunUBT(CommandUtils.CmdEnv, Unreal.UnrealBuildToolDllPath, CommandLineArguments.Join(arguments));

			// Check the output file exists
			if (!FileReference.Exists(outputFile))
			{
				throw new BuildException($"Missing {outputFile}");
			}

			// Apply the optional tag to the build products
			foreach (string tagName in FindTagNamesFromList(_parameters.Tag))
			{
				FindOrAddTagSet(tagNameToFileSet, tagName).Add(outputFile);
			}

			// Add the target files to the set of build products
			buildProducts.Add(outputFile);
			return Task.CompletedTask;
		}

		/// <summary>
		/// Output this task out to an XML writer.
		/// </summary>
		public override void Write(XmlWriter writer)
		{
			Write(writer, _parameters);
		}

		/// <summary>
		/// Find all the tags which are used as inputs to this task
		/// </summary>
		/// <returns>The tag names which are read by this task</returns>
		public override IEnumerable<string> FindConsumedTagNames()
		{
			return Enumerable.Empty<string>();
		}

		/// <summary>
		/// Find all the tags which are modified by this task
		/// </summary>
		/// <returns>The tag names which are modified by this task</returns>
		public override IEnumerable<string> FindProducedTagNames()
		{
			return FindTagNamesFromList(_parameters.Tag);
		}
	}
}
