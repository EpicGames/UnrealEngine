// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Threading.Tasks;
using System.Xml;
using EpicGames.Core;
using Microsoft.Extensions.Logging;

namespace AutomationTool.Tasks
{
	/// <summary>
	/// Parameters for <see cref="RandomDataTask"/>.
	/// </summary>
	public class RandomDataTaskParameters
	{
		/// <summary>
		/// Seed for the data generation
		/// </summary>
		[TaskParameter(Optional = true)]
		public int Seed { get; set; } = -1;

		/// <summary>
		/// The size of each file.
		/// </summary>
		[TaskParameter(Optional = true)]
		public int Size { get; set; } = 1024;

		/// <summary>
		/// Number of files to write.
		/// </summary>
		[TaskParameter(Optional = true)]
		public int Count { get; set; } = 50;

		/// <summary>
		/// Whether to generate different data for each output file.
		/// </summary>
		[TaskParameter(Optional = true)]
		public bool Different { get; set; } = true;

		/// <summary>
		/// Output directory
		/// </summary>
		[TaskParameter(Optional = true)]
		public string OutputDir { get; set; }

		/// <summary>
		/// Optional filter to be applied to the list of input files.
		/// </summary>
		[TaskParameter(Optional = true)]
		public string Tag { get; set; }
	}

	/// <summary>
	/// Creates files containing random data in the specified output directory. Used for generating test data for the temp storage system.
	/// </summary>
	[TaskElement("RandomData", typeof(RandomDataTaskParameters))]
	public class RandomDataTask : BgTaskImpl
	{
		readonly RandomDataTaskParameters _parameters;

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="parameters">Parameters for this task</param>
		public RandomDataTask(RandomDataTaskParameters parameters)
		{
			_parameters = parameters;
		}

		/// <summary>
		/// ExecuteAsync the task.
		/// </summary>
		/// <param name="job">Information about the current job</param>
		/// <param name="buildProducts">Set of build products produced by this node.</param>
		/// <param name="tagNameToFileSet">Mapping from tag names to the set of files they include</param>
		public override async Task ExecuteAsync(JobContext job, HashSet<FileReference> buildProducts, Dictionary<string, HashSet<FileReference>> tagNameToFileSet)
		{
			DirectoryReference outputDir = ResolveDirectory(_parameters.OutputDir);
			DirectoryReference.CreateDirectory(outputDir);

			Random random;
			if (_parameters.Seed >= 0)
			{
				random = new Random(_parameters.Seed);
			}
			else
			{
				random = new Random();
			}

			byte[] buffer = Array.Empty<byte>();
			for (int idx = 0; idx < _parameters.Count; idx++)
			{
				if (idx == 0 || _parameters.Different)
				{
					buffer = new byte[_parameters.Size];
					random.NextBytes(buffer);
				}

				FileReference file = FileReference.Combine(outputDir, $"test-{_parameters.Size}-{idx}.dat");
				await FileReference.WriteAllBytesAsync(file, buffer);
				buildProducts.Add(file);
			}

			Logger.LogInformation("Created {NumFiles:n0} files of {Size:n0} bytes in {OutputDir} (Different={Different})", _parameters.Count, _parameters.Size, outputDir, _parameters.Different);

			// Apply the optional output tag to them
			foreach (string tagName in FindTagNamesFromList(_parameters.Tag))
			{
				FindOrAddTagSet(tagNameToFileSet, tagName).UnionWith(buildProducts);
			}
		}

		/// <summary>
		/// Output this task out to an XML writer.
		/// </summary>
		public override void Write(XmlWriter writer)
		{
			Write(writer, _parameters);
		}

		/// <summary>
		/// Find all the tags which are used as inputs to this task
		/// </summary>
		/// <returns>The tag names which are read by this task</returns>
		public override IEnumerable<string> FindConsumedTagNames()
		{
			yield break;
		}

		/// <summary>
		/// Find all the tags which are modified by this task
		/// </summary>
		/// <returns>The tag names which are modified by this task</returns>
		public override IEnumerable<string> FindProducedTagNames()
		{
			return FindTagNamesFromList(_parameters.Tag);
		}
	}
}
