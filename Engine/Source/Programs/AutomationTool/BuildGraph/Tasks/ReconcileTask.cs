// Copyright Epic Games, Inc. All Rights Reserved.

using EpicGames.Core;
using Microsoft.Extensions.Logging;
using System;
using System.Collections.Generic;
using System.Text;
using System.Text.RegularExpressions;
using System.Threading.Tasks;
using System.Xml;
using UnrealBuildBase;

namespace AutomationTool.Tasks
{
	/// <summary>
	/// Parameters for the submit task
	/// </summary>
	public class ReconcileTaskParameters
	{
		/// <summary>
		/// The description for the submitted changelist.
		/// </summary>
		[TaskParameter]
		public string Description { get; set; }

		/// <summary>
		/// The files to reconcile
		/// </summary>
		[TaskParameter(Optional = true, ValidationType = TaskParameterValidationType.FileSpec)]
		public string Files { get; set; }

		/// <summary>
		/// The directories to reconcile, semi colon delimited, relative p4 syntax
		/// </summary>
		[TaskParameter(Optional = true)]
		public string Directories { get; set; }

		/// <summary>
		/// The workspace name. If specified, a new workspace will be created using the given stream and root directory to submit the files. If not, the current workspace will be used.
		/// </summary>
		[TaskParameter(Optional=true)]
		public string Workspace { get; set; }

		/// <summary>
		/// The stream for the workspace -- defaults to the current stream. Ignored unless the Workspace attribute is also specified.
		/// </summary>
		[TaskParameter(Optional = true)]
		public string Stream { get; set; }

		/// <summary>
		/// Branch for the workspace (legacy P4 depot path). May not be used in conjunction with Stream.
		/// </summary>
		[TaskParameter(Optional = true)]
		public string Branch { get; set; }

		/// <summary>
		/// Root directory for the stream. If not specified, defaults to the current root directory.
		/// </summary>
		[TaskParameter(Optional = true)]
		public DirectoryReference RootDir { get; set; }

		/// <summary>
		/// Force the submit to happen -- even if a resolve is needed (always accept current version).
		/// </summary>
		[TaskParameter(Optional = true)]
		public bool Force { get; set; }

		/// <summary>
		/// Allow verbose P4 output (spew).
		/// </summary>
		[TaskParameter(Optional = true)]
		public bool P4Verbose { get; set; }

		/// <summary>
		/// Runs a reconcile preview, does not submit.
		/// </summary>
		[TaskParameter(Optional = false)]
		public bool Preview { get; set; }
	}

	/// <summary>
	/// Creates a new changelist and reconciles a set of files to submit to a Perforce stream.
	/// </summary>
	[TaskElement("Reconcile", typeof(ReconcileTaskParameters))]
	public class ReconcileTask : BgTaskImpl
	{
		/// <summary>
		/// Parameters for the task
		/// </summary>
		readonly ReconcileTaskParameters _parameters;

		/// <summary>
		/// Construct a version task
		/// </summary>
		/// <param name="parameters">Parameters for this task</param>
		public ReconcileTask(ReconcileTaskParameters parameters)
		{
			_parameters = parameters;
		}

		/// <summary>
		/// Execute the task.
		/// </summary>
		/// <param name="job">Information about the current job</param>
		/// <param name="buildProducts">Set of build products produced by this node.</param>
		/// <param name="tagNameToFileSet">Mapping from tag names to the set of files they include</param>
		public override Task ExecuteAsync(JobContext job, HashSet<FileReference> buildProducts, Dictionary<string, HashSet<FileReference>> tagNameToFileSet)
		{
			HashSet<FileReference> files = ResolveFilespec(Unreal.RootDirectory, _parameters.Files ?? String.Empty, tagNameToFileSet);
			if (files.Count == 0 && String.IsNullOrEmpty(_parameters.Directories))
			{
				throw new AutomationException("No file(s) or directory path(s) specified to reconcile.");
			}

			if (!CommandUtils.AllowSubmit)
			{
				Logger.LogWarning("Submitting to Perforce is disabled by default. Run with the -submit argument to allow.");
			}
			
			try
			{
				// Get the connection that we're going to submit with
				P4Connection submitP4 = CommandUtils.P4;
				if (_parameters.Workspace != null)
				{
					// Create a brand new workspace
					P4ClientInfo client = new P4ClientInfo();
					client.Owner = CommandUtils.P4Env.User;
					client.Host = Environment.MachineName;
					client.RootPath = _parameters.RootDir.FullName ?? Unreal.RootDirectory.FullName;
					client.Name = $"{_parameters.Workspace}_{Regex.Replace(client.Host, "[^a-zA-Z0-9]", "-")}_{ContentHash.MD5((CommandUtils.P4Env.ServerAndPort ?? "").ToUpperInvariant())}";
					client.Options = P4ClientOption.NoAllWrite | P4ClientOption.Clobber | P4ClientOption.NoCompress | P4ClientOption.Unlocked | P4ClientOption.NoModTime | P4ClientOption.RmDir;
					client.LineEnd = P4LineEnd.Local;
					if (!String.IsNullOrEmpty(_parameters.Branch))
					{
						client.View.Add(new KeyValuePair<string, string>($"{_parameters.Branch}/...", "/..."));
					}
					else
					{
						client.Stream = _parameters.Stream ?? CommandUtils.P4Env.Branch;
					}
					CommandUtils.P4.CreateClient(client, AllowSpew: _parameters.P4Verbose);

					// Create a new connection for it
					submitP4 = new P4Connection(client.Owner, client.Name);
				}

				StringBuilder p4Path = new StringBuilder();
				p4Path.Append(String.Join(" ", files));
				p4Path.Append(String.Join(" ", SplitDelimitedList(_parameters.Directories)));

				// Scan the relevant portion of our workspace for any changes.
				if (CommandUtils.AllowSubmit && !_parameters.Preview)
				{
					int newCL = submitP4.CreateChange(Description: _parameters.Description.Replace("\\n", "\n", StringComparison.Ordinal));
					submitP4.Reconcile(newCL, p4Path.ToString());

					int filesChanged = submitP4.ChangeFiles(newCL, out bool bPending).Count;
					if (filesChanged == 0)
					{
						// No files were changed, so we'll log this, delete our temporary P4 changelist, and early exit this method
						Logger.LogInformation("No changes to submit.");
						submitP4.DeleteChange(newCL);
					}
					else
					{
						// Submit it
						submitP4.Submit(newCL, out int submittedCL, Force: _parameters.Force);
						if (submittedCL <= 0)
						{
							throw new AutomationException("Submit failed.");
						}

						Logger.LogInformation("Submitted in changelist {SubmittedCL}", submittedCL);
					}
				}
				else
				{
					submitP4.ReconcilePreview(p4Path.ToString());
				}
			}
			catch (P4Exception ex)
			{
				Logger.LogError(KnownLogEvents.Systemic_Perforce, "{Message}", ex.Message);
				throw new AutomationException(ex.ErrorCode, ex, ex.Message) { OutputFormat = AutomationExceptionOutputFormat.Silent };
			}
			return Task.CompletedTask;
		}

		/// <summary>
		/// Output this task out to an XML writer.
		/// </summary>
		public override void Write(XmlWriter writer)
		{
			Write(writer, _parameters);
		}

		/// <summary>
		/// Find all the tags which are used as inputs to this task
		/// </summary>
		/// <returns>The tag names which are read by this task</returns>
		public override IEnumerable<string> FindConsumedTagNames()
		{
			return FindTagNamesFromFilespec(_parameters.Files);
		}

		/// <summary>
		/// Find all the tags which are modified by this task
		/// </summary>
		/// <returns>The tag names which are modified by this task</returns>
		public override IEnumerable<string> FindProducedTagNames()
		{
			yield break;
		}
	}
}
