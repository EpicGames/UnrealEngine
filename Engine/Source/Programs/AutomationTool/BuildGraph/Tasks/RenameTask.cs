// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.IO;
using System.Text;
using System.Text.RegularExpressions;
using System.Threading.Tasks;
using System.Xml;
using EpicGames.Core;
using UnrealBuildBase;

namespace AutomationTool.Tasks
{
	/// <summary>
	/// Parameters for a copy task
	/// </summary>
	public class RenameTaskParameters
	{
		/// <summary>
		/// The file or files to rename.
		/// </summary>
		[TaskParameter(ValidationType = TaskParameterValidationType.FileSpec)]
		public string Files { get; set; }

		/// <summary>
		/// The current file name, or pattern to match (for example, *.txt). Should not include any path separators.
		/// </summary>
		[TaskParameter(Optional = true)]
		public string From { get; set; }

		/// <summary>
		/// The new name for the file(s). Should not include any path separators.
		/// </summary>
		[TaskParameter]
		public string To { get; set; }

		/// <summary>
		/// Tag to be applied to the renamed files.
		/// </summary>
		[TaskParameter(Optional = true, ValidationType = TaskParameterValidationType.TagList)]
		public string Tag { get; set; }
	}

	/// <summary>
	/// Renames a file, or group of files.
	/// </summary>
	[TaskElement("Rename", typeof(RenameTaskParameters))]
	public class RenameTask : BgTaskImpl
	{
		readonly RenameTaskParameters _parameters;

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="parameters">Parameters for this task</param>
		public RenameTask(RenameTaskParameters parameters)
		{
			_parameters = parameters;
		}

		/// <summary>
		/// ExecuteAsync the task.
		/// </summary>
		/// <param name="job">Information about the current job</param>
		/// <param name="buildProducts">Set of build products produced by this node.</param>
		/// <param name="tagNameToFileSet">Mapping from tag names to the set of files they include</param>
		public override Task ExecuteAsync(JobContext job, HashSet<FileReference> buildProducts, Dictionary<string, HashSet<FileReference>> tagNameToFileSet)
		{
			// Get the pattern to match against. If it's a simple pattern (eg. *.cpp, Engine/Build/...), automatically infer the source wildcard
			string fromPattern = _parameters.From;
			if (fromPattern == null)
			{
				List<string> patterns = SplitDelimitedList(_parameters.Files);
				if (patterns.Count != 1 || patterns[0].StartsWith("#", StringComparison.Ordinal))
				{
					throw new AutomationException("Missing 'From' attribute specifying pattern to match source files against");
				}

				fromPattern = patterns[0];

				int slashIdx = fromPattern.LastIndexOfAny(new char[] { Path.DirectorySeparatorChar, Path.AltDirectorySeparatorChar });
				if (slashIdx != -1)
				{
					fromPattern = fromPattern.Substring(slashIdx + 1);
				}
				if (fromPattern.StartsWith("...", StringComparison.Ordinal))
				{
					fromPattern = "*" + fromPattern.Substring(3);
				}
			}

			// Convert the source pattern into a regex
			string escapedFromPattern = "^" + Regex.Escape(fromPattern) + "$";
			escapedFromPattern = escapedFromPattern.Replace("\\*", "(.*)", StringComparison.Ordinal);
			escapedFromPattern = escapedFromPattern.Replace("\\?", "(.)", StringComparison.Ordinal);
			Regex fromRegex = new Regex(escapedFromPattern, RegexOptions.IgnoreCase | RegexOptions.CultureInvariant);

			// Split the output pattern into fragments that we can insert captures between
			string[] fromFragments = fromPattern.Split('*', '?');
			string[] toFragments = _parameters.To.Split('*', '?');
			if (fromFragments.Length < toFragments.Length)
			{
				throw new AutomationException("Too few capture groups in source pattern '{0}' to rename to '{1}'", fromPattern, _parameters.To);
			}

			// Find the input files
			HashSet<FileReference> inputFiles = ResolveFilespec(Unreal.RootDirectory, _parameters.Files, tagNameToFileSet);

			// Find all the corresponding output files
			Dictionary<FileReference, FileReference> renameFiles = new Dictionary<FileReference, FileReference>();
			foreach (FileReference inputFile in inputFiles)
			{
				Match match = fromRegex.Match(inputFile.GetFileName());
				if (match.Success)
				{
					StringBuilder outputName = new StringBuilder(toFragments[0]);
					for (int idx = 1; idx < toFragments.Length; idx++)
					{
						outputName.Append(match.Groups[idx].Value);
						outputName.Append(toFragments[idx]);
					}
					renameFiles[inputFile] = FileReference.Combine(inputFile.Directory, outputName.ToString());
				}
			}

			// Print out everything we're going to do
			foreach (KeyValuePair<FileReference, FileReference> pair in renameFiles)
			{
				CommandUtils.RenameFile(pair.Key.FullName, pair.Value.FullName, true);
			}

			// Add the build product
			buildProducts.UnionWith(renameFiles.Values);

			// Apply the optional output tag to them
			foreach (string tagName in FindTagNamesFromList(_parameters.Tag))
			{
				FindOrAddTagSet(tagNameToFileSet, tagName).UnionWith(renameFiles.Values);
			}

			return Task.CompletedTask;
		}

		/// <summary>
		/// Output this task out to an XML writer.
		/// </summary>
		public override void Write(XmlWriter writer)
		{
			Write(writer, _parameters);
		}

		/// <summary>
		/// Find all the tags which are used as inputs to this task
		/// </summary>
		/// <returns>The tag names which are read by this task</returns>
		public override IEnumerable<string> FindConsumedTagNames()
		{
			return FindTagNamesFromFilespec(_parameters.Files);
		}

		/// <summary>
		/// Find all the tags which are modified by this task
		/// </summary>
		/// <returns>The tag names which are modified by this task</returns>
		public override IEnumerable<string> FindProducedTagNames()
		{
			return FindTagNamesFromList(_parameters.Tag);
		}
	}
}
