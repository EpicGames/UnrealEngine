// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Diagnostics;
using System.Linq;
using System.Threading;
using System.Threading.Tasks;
using System.Xml;
using EpicGames.Core;
using EpicGames.Horde;
using EpicGames.Horde.Artifacts;
using EpicGames.Horde.Commits;
using EpicGames.Horde.Storage.Nodes;
using EpicGames.Horde.Streams;
using Microsoft.Extensions.DependencyInjection;
using Microsoft.Extensions.Logging;

#nullable enable

namespace AutomationTool.Tasks
{
	/// <summary>
	/// Parameters for a <see cref="CreateArtifactTask"/>.
	/// </summary>
	public class RetrieveArtifactTaskParameters
	{
		/// <summary>
		/// Stream containing the artifact
		/// </summary>
		[TaskParameter(Optional = true)]
		public string? Stream { get; set; } = null!;

		/// <summary>
		/// Change number for the artifact
		/// </summary>
		[TaskParameter(Optional = true)]
		public string? Commit { get; set; }

		/// <summary>
		/// Maximum commit for the artifact
		/// </summary>
		[TaskParameter(Optional = true)]
		public string? MaxCommit { get; set; }

		/// <summary>
		/// Name of the artifact
		/// </summary>
		[TaskParameter(Optional = true)]
		public string Name { get; set; } = null!;

		/// <summary>
		/// The artifact type. Determines the permissions and expiration policy for the artifact.
		/// </summary>
		[TaskParameter]
		public string Type { get; set; } = null!;

		/// <summary>
		/// Keys for the artifact
		/// </summary>
		[TaskParameter(Optional = true)]
		public string Keys { get; set; } = null!;

		/// <summary>
		/// Output directory for 
		/// </summary>
		[TaskParameter]
		public string? OutputDir { get; set; }

		/// <summary>
		/// Whether or not to throw an error if the artifact was not found
		/// </summary>
		/// <remarks>For backwards compatibility defaults to true</remarks>
		[TaskParameter(Optional = true)]
		public bool ErrorIfNotFound { get; set; } = true;

		/// <summary>
		/// Append the environment variable UE_HORDE_JOBID to the artifact query
		/// </summary>
		[TaskParameter(Optional = true)]
		public bool ScopeToJobId { get; set; }
	}

	/// <summary>
	/// Retrieves an artifact from Horde
	/// </summary>
	[TaskElement("RetrieveArtifact", typeof(RetrieveArtifactTaskParameters))]
	public class RetrieveArtifactTask : BgTaskImpl
	{
		readonly RetrieveArtifactTaskParameters _parameters;

		/// <summary>
		/// Constructor.
		/// </summary>
		/// <param name="parameters">Parameters for this task.</param>
		public RetrieveArtifactTask(RetrieveArtifactTaskParameters parameters)
			=> _parameters = parameters;

		/// <inheritdoc/>
		public override async Task ExecuteAsync(JobContext job, HashSet<FileReference> buildProducts, Dictionary<string, HashSet<FileReference>> tagNameToFileSet)
		{
			if (!String.IsNullOrEmpty(_parameters.Commit) && !String.IsNullOrEmpty(_parameters.MaxCommit))
			{
				throw new AutomationException("Cannot specify both Commit and MaxCommit parameters for retrieving an artifact.");
			}

			// Figure out the current change and stream id
			StreamId streamId;
			if (!String.IsNullOrEmpty(_parameters.Stream))
			{
				streamId = new StreamId(_parameters.Stream);
			}
			else
			{
				string? streamIdEnvVar = Environment.GetEnvironmentVariable("UE_HORDE_STREAMID");
				if (!String.IsNullOrEmpty(streamIdEnvVar))
				{
					streamId = new StreamId(streamIdEnvVar);
				}
				else
				{
					throw new AutomationException("Missing UE_HORDE_STREAMID environment variable; unable to determine current stream.");
				}
			}

			// Get the current commit id
			CommitId? minCommitId = null;
			CommitId maxCommitId;
			if (!String.IsNullOrEmpty(_parameters.MaxCommit))
			{
				maxCommitId = new CommitId(_parameters.MaxCommit);
			}
			else if (!String.IsNullOrEmpty(_parameters.Commit))
			{
				minCommitId = maxCommitId = new CommitId(_parameters.Commit);
			}
			else
			{
				int change = CommandUtils.P4Env.Changelist;
				if (change > 0)
				{
					minCommitId = maxCommitId = CommitId.FromPerforceChange(CommandUtils.P4Env.Changelist);
				}
				else
				{
					throw new AutomationException("Unknown changelist. Please run with -P4.");
				}
			}

			// Other parameters
			ArtifactName? name = null;
			if (!String.IsNullOrEmpty(_parameters.Name))
			{
				name = new ArtifactName(_parameters.Name);
			}

			ArtifactType type = new ArtifactType(_parameters.Type);
			List<string> keys = (_parameters.Keys ?? String.Empty).Split(';', StringSplitOptions.RemoveEmptyEntries | StringSplitOptions.TrimEntries).ToList();

			if (_parameters.ScopeToJobId)
			{
				string? jobIdEnvVar = Environment.GetEnvironmentVariable("UE_HORDE_JOBID");
				if (!String.IsNullOrEmpty(jobIdEnvVar))
				{
					keys.Add($"job:{jobIdEnvVar}");
				}
			}

			DirectoryReference outputDir = ResolveDirectory(_parameters.OutputDir);

			// Format the list of search parameters
			List<string> arguments = new List<string>();
			arguments.Add($"Stream: {streamId}");
			if (minCommitId == maxCommitId)
			{
				arguments.Add($"Commit: {maxCommitId}");
			}
			else if (minCommitId != null)
			{
				arguments.Add($"MinCommit: {minCommitId}");
				arguments.Add($"MaxCommit: {maxCommitId}");
			}
			else
			{
				arguments.Add($"MaxCommit: {maxCommitId}");
			}
			if (name != null)
			{
				arguments.Add($"Name: {name}");
			}
			arguments.Add($"Type: {type}");
			if (keys.Count > 0)
			{
				arguments.Add($"Keys: {String.Join(" ", keys.Select(x => $"\"{x}\""))}");
			}

			string argumentsList = String.Join(", ", arguments);
			Logger.LogInformation("Locating artifact ({Arguments})", argumentsList);

			// Find the new artifact
			IHordeClient hordeClient = CommandUtils.ServiceProvider.GetRequiredService<IHordeClient>();
			IArtifact? artifact = await hordeClient.Artifacts.FindAsync(streamId, minCommitId, maxCommitId, name, type, keys, 1).FirstOrDefaultAsync();
			if (artifact == null)
			{
				if (_parameters.ErrorIfNotFound)
				{
					throw new AutomationException("Unable to find any artifact matching given criteria.");
				}
				else
				{
					Logger.LogInformation("Unable to find any artifact matching given criteria.");
				}
				return;
			}

			Logger.LogInformation("Found artifact {ArtifactId}", artifact.Id);

			// Upload the files
			Stopwatch timer = Stopwatch.StartNew();
			await artifact.Content.ExtractAsync(outputDir.ToDirectoryInfo(), new ExtractOptions { Progress = new ExtractStatsLogger(Logger) }, Logger, CancellationToken.None);
			Logger.LogInformation("Retrieved artifact {ArtifactId} in {Time:n1}s", artifact.Id, timer.Elapsed.TotalSeconds);
		}

		/// <inheritdoc/>
		public override void Write(XmlWriter writer)
			=> Write(writer, _parameters);

		/// <inheritdoc/>
		public override IEnumerable<string> FindConsumedTagNames()
			=> Enumerable.Empty<string>();

		/// <inheritdoc/>
		public override IEnumerable<string> FindProducedTagNames()
			=> Enumerable.Empty<string>();
	}
}
