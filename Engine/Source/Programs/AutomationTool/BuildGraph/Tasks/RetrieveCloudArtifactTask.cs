// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.IO;
using System.Linq;
using System.Threading;
using System.Threading.Tasks;
using System.Xml;
using AutomationUtils;
using EpicGames.Core;
using EpicGames.Horde.Artifacts;
using EpicGames.Horde.Commits;
using EpicGames.Horde.Storage;
using EpicGames.Horde.Streams;
using EpicGames.ProjectStore;
using EpicGames.Serialization;
using Microsoft.Extensions.DependencyInjection;
using Microsoft.Extensions.Logging;
using UnrealBuildBase;

#nullable enable

namespace AutomationTool.Tasks
{
	/// <summary>
	/// Parameters for a <see cref="RetrieveCloudArtifactTask"/>.
	/// </summary>
	public class RetrieveCloudArtifactTaskParameters
	{
		/// <summary>
		/// Stream containing the artifact
		/// </summary>
		[TaskParameter(Optional = true)]
		public string? StreamId { get; set; } = null!;

		/// <summary>
		/// Change number for the artifact
		/// </summary>
		[TaskParameter(Optional = true)]
		public string? Commit { get; set; }

		/// <summary>
		/// Requires that the current synced commit is the same as the artifacts commit
		/// </summary>
		[TaskParameter(Optional = true)]
		public bool? RequireMatchingCommit { get; set; }

		/// <summary>
		/// Name of the artifact
		/// </summary>
		[TaskParameter(Optional = true)]
		public string Name { get; set; } = null!;

		/// <summary>
		/// Optional filter to require the artifact to be in build group
		/// </summary>
		[TaskParameter(Optional = true)]
		public string? BuildGroup { get; set; } = null!;

		/// <summary>
		/// The artifact type. Determines the permissions and expiration policy for the artifact.
		/// </summary>
		[TaskParameter]
		public string Type { get; set; } = null!;

		/// <summary>
		/// Keys for the artifact
		/// </summary>
		[TaskParameter(Optional = true)]
		public string Keys { get; set; } = null!;

		/// <summary>
		/// Output directory for 
		/// </summary>
		[TaskParameter]
		public string? OutputDir { get; set; }

		/// <summary>
		/// The platform the artifact is created for
		/// </summary>
		[TaskParameter]
		public string Platform { get; set; } = null!;

		/// <summary>
		/// The path to the uproject this artifact is created for, this is optional if UAT is run with the -project option
		/// </summary>
		[TaskParameter(Optional = true)]
		public FileReference? Project { get; set; } = null!;

		/// <summary>
		/// The platform the artifact is created for
		/// </summary>
		[TaskParameter(Optional = true)]
		public string? Host { get; set; }

		/// <summary>
		/// The platform the artifact is created for
		/// </summary>
		[TaskParameter(Optional = true)]
		public string? Namespace { get; set; }

		/// <summary>
		/// The access token to use
		/// </summary>
		[TaskParameter(Optional = true)]
		public string? AccessToken { get; set; }

		/// <summary>
		/// Set this to use the latest match if multiple artifacts are possible matches
		/// </summary>
		[TaskParameter(Optional = true)]
		public bool AllowMultipleMatches { get; set; } = false;

		/// <summary>
		/// Enable to use multipart endpoints if valuable
		/// </summary>
		[TaskParameter(Optional = true)]
		public bool AllowMultipart { get; set; } = false;

		/// <summary>
		/// Set the explicit http version to use. None to use http handshaking.
		/// </summary>
		[TaskParameter(Optional = true)]
		public string HttpVersion { get; set; } = "None";

		/// <summary>
		/// Increase the number of worker threads used by zen, may cause machine to be less responsive but will generally improve download times
		/// </summary>
		[TaskParameter(Optional = true)]
		public bool BoostWorkers { get; set; } = false;

		/// <summary>
		/// Enable to create an Unreal Insights trace of the download process
		/// </summary>
		public bool EnableTracing { get; set; } = true;

		/// <summary>
		/// Optional pattern(s) to download from the artifact, if omitted the entire artifact is downloaded
		/// </summary>
		[TaskParameter(Optional = true, ValidationType = TaskParameterValidationType.FileSpec)]
		public string? Include { get; set; } = null;

		/// <summary>
		/// Optional pattern(s) to exclude from the copy for example, Engine/NoCopy*.txt)
		/// </summary>
		[TaskParameter(Optional = true, ValidationType = TaskParameterValidationType.FileSpec)]
		public string? Exclude { get; set; } = null;

		/// <summary>
		/// Whether to append downloads to an existing directory (true) or to clear the directory before download (false)
		/// Set this to true if you want to download an artifact to a directory, then download an additional artifact from another without wiping state between
		/// </summary>
		[TaskParameter(Optional = true)]
		public bool Append { get; set; } = false;

		/// <summary>
		/// Delete all data in target folder that is not part of the downloaded content
		/// </summary>
		[TaskParameter(Optional = true)]
		public bool Clean { get; set; } = false;
	}

	/// <summary>
	/// Retrieves an artifact from Cloud DDC
	/// </summary>
	[TaskElement("RetrieveCloudArtifact", typeof(RetrieveCloudArtifactTaskParameters))]
	public class RetrieveCloudArtifactTask : BgTaskImpl
	{
		readonly RetrieveCloudArtifactTaskParameters _parameters;

		/// <summary>
		/// Parameters for the task.
		/// </summary>
		protected RetrieveCloudArtifactTaskParameters Parameters => _parameters;

		/// <summary>
		/// Constructor.
		/// </summary>
		/// <param name="parameters">Parameters for this task.</param>
		public RetrieveCloudArtifactTask(RetrieveCloudArtifactTaskParameters parameters)
			=> _parameters = parameters;

		/// <summary>
		/// ExecuteAsync the task.
		/// </summary>
		/// <param name="job">Information about the current job.</param>
		/// <param name="buildProducts">Set of build products produced by this node.</param>
		/// <param name="tagNameToFileSet">Mapping from tag names to the set of files they include.</param>
		public override async Task ExecuteAsync(JobContext job, HashSet<FileReference> buildProducts, Dictionary<string, HashSet<FileReference>> tagNameToFileSet)
		{
			string? defaultProject = job.OwnerCommand.ParseParamValue("Project");
			FileReference? defaultProjectFile = null;
			if (defaultProject != null && File.Exists(defaultProject))
			{
				defaultProjectFile = new FileReference(defaultProject);
			}

			FileReference? projectPath = _parameters.Project != null ? _parameters.Project : defaultProjectFile;
			string? project = projectPath != null ? projectPath.GetFileNameWithoutExtension() : null;
			if (projectPath == null || !File.Exists(projectPath.FullName))
			{
				// if the project is not an uproject that exists we can not scan for inis
				projectPath = null;
				Logger.LogInformation("\'{Project}\' is not a valid file. Not parsing ini files, config needs to be explicitly provided.", project);
			}

			if (project == null)
			{
				Logger.LogError("Project is null and no ProjectFile set for build graph. Unable to determine project for this upload.");
				return;
			}
			string platform = _parameters.Platform;

			ICloudStorage? cloudStorage = CommandUtils.ServiceScope.ServiceProvider.GetService<ICloudStorage>();
			CloudBuildOptionsOverride optionsOverride = new CloudBuildOptionsOverride();

			if (cloudStorage == null)
			{
				// TODO: If this actually happens we could likely initialize cloud storage ourselves
				throw new NotImplementedException("Failed to retrieve ICloudStorage from service provider, unable to continue");
			}

			// Figure out the current change and stream id
			StreamId streamId;
			if (!String.IsNullOrEmpty(_parameters.StreamId))
			{
				streamId = new StreamId(_parameters.StreamId);
			}
			else
			{
				string? streamIdEnvVar = Environment.GetEnvironmentVariable("UE_HORDE_STREAMID");
				if (!String.IsNullOrEmpty(streamIdEnvVar))
				{
					streamId = new StreamId(streamIdEnvVar);
				}
				else
				{
					throw new AutomationException("Missing UE_HORDE_STREAMID environment variable; unable to determine current stream.");
				}
			}

			string? cloudHostEnvVar = Environment.GetEnvironmentVariable(OperatingSystem.IsWindows() ? "UE-CloudPublishHost" : "UE_CloudPublishHost");
			
			if (!String.IsNullOrEmpty(_parameters.Host))
			{
				optionsOverride.OverrideHost = _parameters.Host;
			}
			else if (!String.IsNullOrEmpty(cloudHostEnvVar))
			{
				optionsOverride.OverrideHost = cloudHostEnvVar;
			}

			string? cloudDefaultNamespaceEnv = Environment.GetEnvironmentVariable(OperatingSystem.IsWindows() ? "UE-CloudPublishNamespace" : "UE_CloudPublishNamespace");
				
			if (!String.IsNullOrEmpty(_parameters.Namespace))
			{
				optionsOverride.NamespaceId = new NamespaceId(_parameters.Namespace);
			}
			else if (!String.IsNullOrEmpty(cloudDefaultNamespaceEnv))
			{
				optionsOverride.NamespaceId = new NamespaceId(cloudDefaultNamespaceEnv);
			}

			string httpVersion = _parameters.HttpVersion;
			if (String.IsNullOrEmpty(httpVersion) || httpVersion == "None")
			{
				string? httpVersionEnvironmentVariable = Environment.GetEnvironmentVariable(OperatingSystem.IsWindows() ? "UE-CloudPublishHttpVersion": "UE_CloudPublishHttpVersion");
				if (!String.IsNullOrEmpty(httpVersionEnvironmentVariable))
				{
					httpVersion = httpVersionEnvironmentVariable;
				}
			}

			if (!String.IsNullOrEmpty(_parameters.AccessToken))
			{
				optionsOverride.AccessToken = _parameters.AccessToken;
			}
			else
			{
				string? cloudAccessTokenEnvVar = Environment.GetEnvironmentVariable(OperatingSystem.IsWindows() ? "UE-CloudDataCacheAccessToken" : "UE_CloudDataCacheAccessToken");
				if (!String.IsNullOrEmpty(cloudAccessTokenEnvVar))
				{
					optionsOverride.AccessToken = cloudAccessTokenEnvVar;
				}
			}

			// Get the current commit id
			CommitId? requiredCommitId = null;
			if (_parameters.RequireMatchingCommit.GetValueOrDefault(true))
			{
				// the commit needs to match, check the commit option or fallback to P4 in case it has not been specified
				if (!String.IsNullOrEmpty(_parameters.Commit))
				{
					requiredCommitId = new CommitId(_parameters.Commit);
				}
				else
				{
					try
					{
						int change = CommandUtils.P4Env.Changelist;
						if (change > 0)
						{
							requiredCommitId = CommitId.FromPerforceChange(CommandUtils.P4Env.Changelist);
						}
					}
					catch (AutomationException)
					{
						// not an error to run without p4
					}
				}
			}

			await cloudStorage.Reconfigure(options =>
			{
				if (projectPath != null && File.Exists(projectPath.FullName))
				{
					// initialize from the explicitly defined project name property for backwards compatibility reasons, this is usually read by the -ProjectFile argument to UAT
					options.ReadFromIni(projectPath);
				}
			});

			await cloudStorage.LoginAsync(unattended: true, optionsOverride);

			string? name = null;
			if (!String.IsNullOrEmpty(_parameters.Name))
			{
				name = _parameters.Name;
			}

			ArtifactType type = new ArtifactType(_parameters.Type);
			List<string> keys = (_parameters.Keys ?? String.Empty).Split(';', StringSplitOptions.RemoveEmptyEntries | StringSplitOptions.TrimEntries).ToList();

			CloudStorageQueryBuilder queryBuilder = new();

			string branch = ICloudStorage.SanitizeBucketValue(streamId.ToString());
			queryBuilder.Equals("platform", ICloudStorage.SanitizeBucketValue(platform));
			
			if (requiredCommitId != null)
			{
				queryBuilder.Equals("commit", requiredCommitId.ToString());
			}

			if (name != null)
			{
				queryBuilder.Equals("name", name);
			}

			if (!String.IsNullOrEmpty(_parameters.BuildGroup))
			{
				queryBuilder.Equals("buildgroup", _parameters.BuildGroup);
			}

			queryBuilder.Equals("type", type.ToString());

			if (keys.Count > 0)
			{
				queryBuilder.In("keys", keys);
			}

			DirectoryReference outputDir = ResolveDirectory(_parameters.OutputDir);

			List<FilePattern>? includeFilter = null;
			if (!String.IsNullOrEmpty(_parameters.Include))
			{
				includeFilter = new List<FilePattern>();

				foreach (string filter in SplitDelimitedList(_parameters.Include))
				{
					includeFilter.Add(new FilePattern(outputDir, filter));
				}
			}

			List<FilePattern>? excludeFilter = null;
			if (!String.IsNullOrEmpty(_parameters.Exclude))
			{
				excludeFilter = new List<FilePattern>();
				foreach (string filter in SplitDelimitedList(_parameters.Exclude))
				{
					excludeFilter.Add(new FilePattern(outputDir, filter));
				}
			}

			CbObjectId artifact;
			FindBuildResponse foundBuild;
			
			{
				CbObject queryObject = queryBuilder.Done();
				List<FindBuildResponse> matchingBuilds = await cloudStorage.FindBuildsAsync(project.ToLower(), type: type.ToString(), branch: branch, queryObject, searchOptions: new CloudBuildSearchOptions {Limit  = 2}, optionsOverride: optionsOverride, cancellationToken: CancellationToken.None).ToListAsync();

				if (matchingBuilds.Count == 0)
				{
					throw new AutomationException($"Unable to find any artifact matching given criteria in namespace {cloudStorage.GetCurrentNamespace(optionsOverride)} for project \"{project}\" and type \"{type}\" using criteria: {queryObject.ToJson()}");
				}

				if (!_parameters.AllowMultipleMatches && matchingBuilds.Count != 1)
				{
					throw new AutomationException("More then one matching artifact given criteria, set \"AllowMultipleMatches\" option if you want to use the newest matching artifact or refine the search");
				}

				foundBuild = matchingBuilds.First();
				artifact = foundBuild.BuildId;
			}

			FileReference? traceFile = null;
			if (_parameters.EnableTracing)
			{
				// this is automatically uploaded by Horde as it's in the Saved folder
				traceFile = FileReference.Combine(Unreal.RootDirectory, $"Engine/Programs/AutomationTool/Saved/Logs/build-download-{artifact}.utrace");
			}

			Logger.LogInformation("Downloading artifact {BuildId} from bucket {BucketId}", foundBuild.BuildId, foundBuild.BucketId);
			await cloudStorage.DownloadBuildAsync(foundBuild.BucketId, foundBuild.BuildId, outputDir, includeFilter, excludeFilter, new CloudBuildDownloadOptions
				{
					AllowMultipart = _parameters.AllowMultipart, 
					AssumeHttp2 = String.Equals(httpVersion, "http2-only", StringComparison.OrdinalIgnoreCase),
					BoostWorkers = CommandUtils.IsBuildMachine || _parameters.BoostWorkers,
					// boost worker memory if boosting workers as well
					BoostWorkerMemory = CommandUtils.IsBuildMachine || _parameters.BoostWorkers,
					TraceFile = traceFile,
					Append = _parameters.Append,
					Clean = _parameters.Clean
				}, optionsOverride: optionsOverride);
		}

		/// <inheritdoc/>
		public override void Write(XmlWriter writer)
			=> Write(writer, _parameters);

		/// <inheritdoc/>
		public override IEnumerable<string> FindConsumedTagNames()
			=> Enumerable.Empty<string>();

		/// <inheritdoc/>
		public override IEnumerable<string> FindProducedTagNames()
			=> Enumerable.Empty<string>();
	}
}
