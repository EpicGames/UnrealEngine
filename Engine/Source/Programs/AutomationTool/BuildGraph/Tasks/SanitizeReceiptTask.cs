// Copyright Epic Games, Inc. All Rights Reserved.

using System.Collections.Generic;
using System.Linq;
using System.Threading.Tasks;
using System.Xml;
using EpicGames.Core;
using Microsoft.Extensions.Logging;
using UnrealBuildBase;
using UnrealBuildTool;

namespace AutomationTool.Tasks
{
	/// <summary>
	/// Parameters for the Tag Receipt task.
	/// </summary>
	public class SanitizeReceiptTaskParameters
	{
		/// <summary>
		/// Set of receipt files (*.target) to read, including wildcards and tag names, separated by semicolons.
		/// </summary>
		[TaskParameter(ValidationType = TaskParameterValidationType.FileSpec)]
		public string Files { get; set; }

		/// <summary>
		/// Path to the Engine folder, used to expand $(EngineDir) properties in receipt files. Defaults to the Engine directory for the current workspace.
		/// </summary>
		[TaskParameter(Optional = true)]
		public DirectoryReference EngineDir { get; set; }
	}

	/// <summary>
	/// Task that tags build products and/or runtime dependencies by reading from *.target files.
	/// </summary>
	[TaskElement("SanitizeReceipt", typeof(SanitizeReceiptTaskParameters))]
	class SanitizeReceiptTask : BgTaskImpl
	{
		readonly SanitizeReceiptTaskParameters _parameters;

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="parameters">Parameters to select which files to search</param>
		public SanitizeReceiptTask(SanitizeReceiptTaskParameters parameters)
		{
			_parameters = parameters;
		}

		/// <summary>
		/// ExecuteAsync the task.
		/// </summary>
		/// <param name="job">Information about the current job</param>
		/// <param name="buildProducts">Set of build products produced by this node.</param>
		/// <param name="tagNameToFileSet">Mapping from tag names to the set of files they include</param>
		public override async Task ExecuteAsync(JobContext job, HashSet<FileReference> buildProducts, Dictionary<string, HashSet<FileReference>> tagNameToFileSet)
		{
			// Set the Engine directory
			DirectoryReference engineDir = _parameters.EngineDir ?? Unreal.EngineDirectory;

			// Resolve the input list
			IEnumerable<FileReference> targetFiles = ResolveFilespec(Unreal.RootDirectory, _parameters.Files, tagNameToFileSet);
			await Execute(targetFiles, engineDir);
		}

		public static Task Execute(IEnumerable<FileReference> targetFiles, DirectoryReference engineDir)
		{
			engineDir ??= Unreal.EngineDirectory;

			foreach (FileReference targetFile in targetFiles)
			{
				// check all files are .target files
				if (targetFile.GetExtension() != ".target")
				{
					throw new AutomationException("Invalid file passed to TagReceipt task ({0})", targetFile.FullName);
				}

				// Print the name of the file being scanned
				Logger.LogInformation("Sanitizing {TargetFile}", targetFile);
				using (new LogIndentScope("  "))
				{
					// Read the receipt
					TargetReceipt receipt;
					if (!TargetReceipt.TryRead(targetFile, engineDir, out receipt))
					{
						Logger.LogWarning("Unable to load file using TagReceipt task ({Arg0})", targetFile.FullName);
						continue;
					}

					// Remove any build products that don't exist
					List<BuildProduct> newBuildProducts = new List<BuildProduct>(receipt.BuildProducts.Count);
					foreach (BuildProduct buildProduct in receipt.BuildProducts)
					{
						if (FileReference.Exists(buildProduct.Path))
						{
							newBuildProducts.Add(buildProduct);
						}
						else
						{
							Logger.LogInformation("Removing build product: {BuildProduct}", buildProduct.Path);
						}
					}
					receipt.BuildProducts = newBuildProducts;

					// Remove any runtime dependencies that don't exist
					RuntimeDependencyList newRuntimeDependencies = new RuntimeDependencyList();
					foreach (RuntimeDependency runtimeDependency in receipt.RuntimeDependencies)
					{
						if (FileReference.Exists(runtimeDependency.Path))
						{
							newRuntimeDependencies.Add(runtimeDependency);
						}
						else
						{
							Logger.LogInformation("Removing runtime dependency: {RuntimeDependency}", runtimeDependency.Path);
						}
					}
					receipt.RuntimeDependencies = newRuntimeDependencies;

					// Save the new receipt
					receipt.Write(targetFile, engineDir);
				}
			}
			return Task.CompletedTask;
		}

		/// <summary>
		/// Output this task out to an XML writer.
		/// </summary>
		public override void Write(XmlWriter writer)
		{
			Write(writer, _parameters);
		}

		/// <summary>
		/// Find all the tags which are required by this task
		/// </summary>
		/// <returns>The tag names which are required by this task</returns>
		public override IEnumerable<string> FindConsumedTagNames()
		{
			return FindTagNamesFromFilespec(_parameters.Files);
		}

		/// <summary>
		/// Find all the referenced tags from tasks in this task
		/// </summary>
		/// <returns>The tag names which are produced/modified by this task</returns>
		public override IEnumerable<string> FindProducedTagNames()
		{
			return Enumerable.Empty<string>();
		}
	}

	/// <summary>
	/// Extension methods
	/// </summary>
	public static class SanitizeReceiptExtensions
	{
		/// <summary>
		/// Sanitize the given receipt files, removing any files that don't exist in the current workspace
		/// </summary>
		public static async Task SanitizeReceiptsAsync(this FileSet targetFiles, DirectoryReference engineDir = null)
		{
			await SanitizeReceiptTask.Execute(targetFiles, engineDir);
		}
	}
}
