// Copyright Epic Games, Inc. All Rights Reserved.

using System.Collections.Generic;
using System.Threading.Tasks;
using System.Xml;
using EpicGames.Core;
using UnrealBuildBase;

namespace AutomationTool.Tasks
{
	/// <summary>
	/// Parameters for the version task
	/// </summary>
	public class SetVersionTaskParameters
	{
		/// <summary>
		/// The changelist to set in the version files.
		/// </summary>
		[TaskParameter]
		public int Change { get; set; }

		/// <summary>
		/// The engine compatible changelist to set in the version files.
		/// </summary>
		[TaskParameter(Optional = true)]
		public int CompatibleChange { get; set; }

		/// <summary>
		/// The branch string.
		/// </summary>
		[TaskParameter]
		public string Branch { get; set; }

		/// <summary>
		/// The build version string.
		/// </summary>
		[TaskParameter(Optional = true)]
		public string Build { get; set; }

		/// <summary>
		/// The URL for a running continuous integration job.
		/// </summary>
		[TaskParameter(Optional = true)]
		public string BuildURL { get; set; }

		/// <summary>
		/// Whether to set the IS_LICENSEE_VERSION flag to true.
		/// </summary>
		[TaskParameter(Optional = true)]
		public bool Licensee { get; set; }

		/// <summary>
		/// Whether to set the ENGINE_IS_PROMOTED_BUILD flag to true.
		/// </summary>
		[TaskParameter(Optional = true)]
		public bool Promoted { get; set; } = true;

		/// <summary>
		/// If set, do not write to the files -- just return the version files that would be updated. Useful for local builds.
		/// </summary>
		[TaskParameter(Optional = true)]
		public bool SkipWrite { get; set; }

		/// <summary>
		/// Tag to be applied to build products of this task.
		/// </summary>
		[TaskParameter(Optional = true, ValidationType = TaskParameterValidationType.TagList)]
		public string Tag { get; set; }
	}

	/// <summary>
	/// Updates the local version files (Engine/Source/Runtime/Launch/Resources/Version.h, Engine/Build/Build.version, and Engine/Source/Programs/Shared/Metadata.cs) with the given version information.
	/// </summary>
	[TaskElement("SetVersion", typeof(SetVersionTaskParameters))]
	public class SetVersionTask : BgTaskImpl
	{
		readonly SetVersionTaskParameters _parameters;

		/// <summary>
		/// Construct a version task
		/// </summary>
		/// <param name="parameters">Parameters for this task</param>
		public SetVersionTask(SetVersionTaskParameters parameters)
		{
			_parameters = parameters;
		}

		/// <summary>
		/// ExecuteAsync the task.
		/// </summary>
		/// <param name="job">Information about the current job</param>
		/// <param name="buildProducts">Set of build products produced by this node.</param>
		/// <param name="tagNameToFileSet">Mapping from tag names to the set of files they include</param>
		public override Task ExecuteAsync(JobContext job, HashSet<FileReference> buildProducts, Dictionary<string, HashSet<FileReference>> tagNameToFileSet)
		{
			// Update the version files
			List<FileReference> versionFiles = UnrealBuild.StaticUpdateVersionFiles(_parameters.Change, _parameters.CompatibleChange, _parameters.Branch, _parameters.Build, _parameters.BuildURL, _parameters.Licensee, _parameters.Promoted, !_parameters.SkipWrite);

			// Apply the optional tag to them
			foreach (string tagName in FindTagNamesFromList(_parameters.Tag))
			{
				FindOrAddTagSet(tagNameToFileSet, tagName).UnionWith(versionFiles);
			}

			// Add them to the list of build products
			buildProducts.UnionWith(versionFiles);
			return Task.CompletedTask;
		}

		/// <summary>
		/// Output this task out to an XML writer.
		/// </summary>
		public override void Write(XmlWriter writer)
		{
			Write(writer, _parameters);
		}

		/// <summary>
		/// Find all the tags which are used as inputs to this task
		/// </summary>
		/// <returns>The tag names which are read by this task</returns>
		public override IEnumerable<string> FindConsumedTagNames()
		{
			yield break;
		}

		/// <summary>
		/// Find all the tags which are modified by this task
		/// </summary>
		/// <returns>The tag names which are modified by this task</returns>
		public override IEnumerable<string> FindProducedTagNames()
		{
			return FindTagNamesFromList(_parameters.Tag);
		}
	}

	/// <summary>
	/// Task wrapper methods
	/// </summary>
	public static partial class StandardTasks
	{
		/// <summary>
		/// ExecuteAsync a task instance
		/// </summary>
		/// <param name="task"></param>
		/// <returns></returns>
		public static async Task<FileSet> ExecuteAsync(BgTaskImpl task)
		{
			HashSet<FileReference> buildProducts = new HashSet<FileReference>();
			await task.ExecuteAsync(new JobContext(null!, null!), buildProducts, new Dictionary<string, HashSet<FileReference>>());
			return FileSet.FromFiles(Unreal.RootDirectory, buildProducts);
		}

		/// <summary>
		/// Updates the current engine version
		/// </summary>
		public static async Task<FileSet> SetVersionAsync(int change, string branch, int? compatibleChange = null, string build = null, string buildUrl = null, bool? licensee = null, bool? promoted = null, bool? skipWrite = null)
		{
			SetVersionTaskParameters parameters = new SetVersionTaskParameters();
			parameters.Change = change;
			parameters.CompatibleChange = compatibleChange ?? parameters.CompatibleChange;
			parameters.Branch = branch ?? parameters.Branch;
			parameters.Build = build ?? parameters.Build;
			parameters.BuildURL = buildUrl ?? parameters.BuildURL;
			parameters.Licensee = licensee ?? parameters.Licensee;
			parameters.Promoted = promoted ?? parameters.Promoted;
			parameters.SkipWrite = skipWrite ?? parameters.SkipWrite;
			return await ExecuteAsync(new SetVersionTask(parameters));
		}
	}
}
