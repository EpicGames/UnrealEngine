// Copyright Epic Games, Inc. All Rights Reserved.

using System.Collections.Generic;
using System.Linq;
using System.Threading.Tasks;
using System.Xml;
using EpicGames.Core;
using UnrealBuildBase;

namespace AutomationTool.Tasks
{
	/// <summary>
	/// Parameters for a task that signs a set of files.
	/// </summary>
	public class SignTaskParameters
	{
		/// <summary>
		/// List of file specifications separated by semicolons (for example, *.cpp;Engine/.../*.bat), or the name of a tag set.
		/// </summary>
		[TaskParameter(ValidationType = TaskParameterValidationType.FileSpec)]
		public string Files { get; set; }

		/// <summary>
		/// Optional description for the signed content
		/// </summary>
		[TaskParameter(Optional = true)]
		public string Description { get; set; }

		/// <summary>
		/// Tag to be applied to build products of this task.
		/// </summary>
		[TaskParameter(Optional = true, ValidationType = TaskParameterValidationType.TagList)]
		public string Tag { get; set; }

		/// <summary>
		/// If true, the calls to the signing tool will be performed in parallel.
		/// </summary>
		[TaskParameter(Optional = true)]
		public bool Parallel { get; set; }
	}

	/// <summary>
	/// Signs a set of executable files with an installed certificate.
	/// </summary>
	[TaskElement("Sign", typeof(SignTaskParameters))]
	public class SignTask : BgTaskImpl
	{
		/// <summary>
		/// Parameters for this task
		/// </summary>
		readonly SignTaskParameters _parameters;

		/// <summary>
		/// Construct a spawn task
		/// </summary>
		/// <param name="parameters">Parameters for the task</param>
		public SignTask(SignTaskParameters parameters)
		{
			_parameters = parameters;
		}

		/// <summary>
		/// ExecuteAsync the task.
		/// </summary>
		/// <param name="job">Information about the current job</param>
		/// <param name="buildProducts">Set of build products produced by this node.</param>
		/// <param name="tagNameToFileSet">Mapping from tag names to the set of files they include</param>
		public override Task ExecuteAsync(JobContext job, HashSet<FileReference> buildProducts, Dictionary<string, HashSet<FileReference>> tagNameToFileSet)
		{
			// Find the matching files
			FileReference[] files = ResolveFilespec(Unreal.RootDirectory, _parameters.Files, tagNameToFileSet).OrderBy(x => x.FullName).ToArray();

			// Sign all the files
			CodeSign.SignMultipleIfEXEOrDLL(job.OwnerCommand, (files.Select(x => x.FullName).ToList()), Description: _parameters.Description, _parameters.Parallel);

			// Apply the optional tag to the build products
			foreach (string tagName in FindTagNamesFromList(_parameters.Tag))
			{
				FindOrAddTagSet(tagNameToFileSet, tagName).UnionWith(files);
			}

			// Add them to the list of build products
			buildProducts.UnionWith(files);
			return Task.CompletedTask;
		}

		/// <summary>
		/// Output this task out to an XML writer.
		/// </summary>
		public override void Write(XmlWriter writer)
		{
			Write(writer, _parameters);
		}

		/// <summary>
		/// Find all the tags which are used as inputs to this task
		/// </summary>
		/// <returns>The tag names which are read by this task</returns>
		public override IEnumerable<string> FindConsumedTagNames()
		{
			return FindTagNamesFromFilespec(_parameters.Files);
		}

		/// <summary>
		/// Find all the tags which are modified by this task
		/// </summary>
		/// <returns>The tag names which are modified by this task</returns>
		public override IEnumerable<string> FindProducedTagNames()
		{
			return FindTagNamesFromList(_parameters.Tag);
		}
	}
}
