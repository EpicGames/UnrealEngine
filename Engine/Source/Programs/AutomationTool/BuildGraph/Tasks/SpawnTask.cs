// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.IO;
using System.Threading.Tasks;
using System.Xml;
using EpicGames.Core;

namespace AutomationTool.Tasks
{
	/// <summary>
	/// Parameters for a spawn task
	/// </summary>
	public class SpawnTaskParameters
	{
		/// <summary>
		/// Executable to spawn.
		/// </summary>
		[TaskParameter]
		public string Exe { get; set; }

		/// <summary>
		/// Arguments for the newly created process.
		/// </summary>
		[TaskParameter(Optional = true)]
		public string Arguments { get; set; }

		/// <summary>
		/// Working directory for spawning the new task
		/// </summary>
		[TaskParameter(Optional = true)]
		public string WorkingDir { get; set; }

		/// <summary>
		/// Environment variables to set
		/// </summary>
		[TaskParameter(Optional = true)]
		public string Environment { get; set; }

		/// <summary>
		/// File to read environment from
		/// </summary>
		[TaskParameter(Optional = true)]
		public string EnvironmentFile { get; set; }

		/// <summary>
		/// Write output to the log
		/// </summary>
		[TaskParameter(Optional = true)]
		public bool LogOutput { get; set; } = true;

		/// <summary>
		/// The minimum exit code, which is treated as an error.
		/// </summary>
		[TaskParameter(Optional = true)]
		public int ErrorLevel { get; set; } = 1;
	}

	/// <summary>
	/// Base class for tasks that run an external tool
	/// </summary>
	public abstract class SpawnTaskBase : BgTaskImpl
	{
		/// <summary>
		/// ExecuteAsync a command
		/// </summary>
		protected static Task<IProcessResult> ExecuteAsync(string exe, string arguments, string workingDir = null, Dictionary<string, string> envVars = null, bool logOutput = true, int errorLevel = 1, string input = null, ProcessResult.SpewFilterCallbackType spewFilterCallback = null)
		{
			if (workingDir != null)
			{
				workingDir = ResolveDirectory(workingDir).FullName;
			}

			CommandUtils.ERunOptions options = CommandUtils.ERunOptions.Default;
			if (!logOutput)
			{
				options |= CommandUtils.ERunOptions.SpewIsVerbose;
			}

			IProcessResult result = CommandUtils.Run(exe, arguments, Env: envVars, WorkingDir: workingDir, Options: options, Input: input, SpewFilterCallback: spewFilterCallback);
			if (result.ExitCode < 0 || result.ExitCode >= errorLevel)
			{
				throw new AutomationException("{0} terminated with an exit code indicating an error ({1})", Path.GetFileName(exe), result.ExitCode);
			}

			return Task.FromResult(result);
		}

		/// <summary>
		/// Parses environment from a property and file
		/// </summary>
		/// <param name="environment"></param>
		/// <param name="environmentFile"></param>
		/// <returns></returns>
		protected static Dictionary<string, string> ParseEnvVars(string environment, string environmentFile)
		{
			Dictionary<string, string> envVars = new Dictionary<string, string>();
			if (environment != null)
			{
				ParseEnvironment(environment, ';', envVars);
			}
			if (!String.IsNullOrEmpty(environmentFile))
			{
				ParseEnvironment(FileUtils.ReadAllText(ResolveFile(environmentFile)), '\n', envVars);
			}
			return envVars;
		}

		/// <summary>
		/// Parse environment from a string
		/// </summary>
		/// <param name="environment"></param>
		/// <param name="separator"></param>
		/// <param name="envVars"></param>
		static void ParseEnvironment(string environment, char separator, Dictionary<string, string> envVars)
		{
			for (int baseIdx = 0; baseIdx < environment.Length;)
			{
				int endIdx = environment.IndexOf(separator, baseIdx);
				if (endIdx == -1)
				{
					endIdx = environment.Length;
				}

				string line = environment.Substring(baseIdx, endIdx - baseIdx);
				if (!String.IsNullOrWhiteSpace(line))
				{
					int equalsIdx = line.IndexOf('=', StringComparison.Ordinal);
					if (equalsIdx == -1)
					{
						throw new AutomationException("Missing value in environment variable string '{0}'", environment.Substring(baseIdx, endIdx - baseIdx));
					}

					string name = line.Substring(0, equalsIdx).Trim();
					string value = line.Substring(equalsIdx + 1).Trim();
					envVars[name] = value;
				}

				baseIdx = endIdx + 1;
			}
		}
	}

	/// <summary>
	/// Spawns an external executable and waits for it to complete.
	/// </summary>
	[TaskElement("Spawn", typeof(SpawnTaskParameters))]
	public class SpawnTask : SpawnTaskBase
	{
		/// <summary>
		/// Parameters for this task
		/// </summary>
		readonly SpawnTaskParameters _parameters;

		/// <summary>
		/// Construct a spawn task
		/// </summary>
		/// <param name="parameters">Parameters for the task</param>
		public SpawnTask(SpawnTaskParameters parameters)
		{
			_parameters = parameters;
		}

		/// <summary>
		/// ExecuteAsync the task.
		/// </summary>
		/// <param name="job">Information about the current job</param>
		/// <param name="buildProducts">Set of build products produced by this node.</param>
		/// <param name="tagNameToFileSet">Mapping from tag names to the set of files they include</param>
		public override async Task ExecuteAsync(JobContext job, HashSet<FileReference> buildProducts, Dictionary<string, HashSet<FileReference>> tagNameToFileSet)
		{
			await ExecuteAsync(_parameters.Exe, _parameters.Arguments, _parameters.WorkingDir, envVars: ParseEnvVars(_parameters.Environment, _parameters.EnvironmentFile), logOutput: _parameters.LogOutput, errorLevel: _parameters.ErrorLevel);
		}

		/// <summary>
		/// Output this task out to an XML writer.
		/// </summary>
		public override void Write(XmlWriter writer)
		{
			Write(writer, _parameters);
		}

		/// <summary>
		/// Find all the tags which are used as inputs to this task
		/// </summary>
		/// <returns>The tag names which are read by this task</returns>
		public override IEnumerable<string> FindConsumedTagNames()
		{
			yield break;
		}

		/// <summary>
		/// Find all the tags which are modified by this task
		/// </summary>
		/// <returns>The tag names which are modified by this task</returns>
		public override IEnumerable<string> FindProducedTagNames()
		{
			yield break;
		}
	}

	public static partial class StandardTasks
	{
		/// <summary>
		/// ExecuteAsync an external program
		/// </summary>
		/// <param name="exe">Executable to spawn.</param>
		/// <param name="arguments">Arguments for the newly created process.</param>
		/// <param name="workingDir">Working directory for spawning the new task.</param>
		/// <param name="environment">Environment variables to set.</param>
		/// <param name="environmentFile">File to read environment from.</param>
		/// <param name="logOutput">Write output to the log.</param>
		/// <param name="errorLevel">The minimum exit code which is treated as an error.</param>
		public static async Task SpawnAsync(string exe, string arguments = null, string workingDir = null, string environment = null, string environmentFile = null, bool? logOutput = null, int? errorLevel = null)
		{
			SpawnTaskParameters parameters = new SpawnTaskParameters();
			parameters.Exe = exe;
			parameters.Arguments = arguments;
			parameters.WorkingDir = workingDir;
			parameters.Environment = environment;
			parameters.EnvironmentFile = environmentFile;
			parameters.LogOutput = logOutput ?? parameters.LogOutput;
			parameters.ErrorLevel = errorLevel ?? parameters.ErrorLevel;

			await ExecuteAsync(new SpawnTask(parameters));
		}
	}
}
