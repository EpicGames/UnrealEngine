// Copyright Epic Games, Inc. All Rights Reserved.

using System.Collections.Generic;
using System.Linq;
using System.Threading.Tasks;
using System.Xml;
using EpicGames.Core;
using UnrealBuildBase;
using UnrealBuildTool;

namespace AutomationTool.Tasks
{
	/// <summary>
	/// Parameters for the staging task
	/// </summary>
	public class StageTaskParameters
	{
		/// <summary>
		/// The project that this target belongs to.
		/// </summary>
		[TaskParameter(Optional = true)]
		public FileReference Project { get; set; }

		/// <summary>
		/// Name of the target to stage.
		/// </summary>
		[TaskParameter]
		public string Target { get; set; }

		/// <summary>
		/// Platform to stage.
		/// </summary>
		[TaskParameter]
		public UnrealTargetPlatform Platform { get; set; }

		/// <summary>
		/// Configuration to be staged.
		/// </summary>
		[TaskParameter]
		public UnrealTargetConfiguration Configuration { get; set; }

		/// <summary>
		/// Architecture to be staged.
		/// </summary>
		[TaskParameter(Optional = true)]
		public string Architecture { get; set; }

		/// <summary>
		/// Directory that the receipt files should be staged to.
		/// </summary>
		[TaskParameter]
		public DirectoryReference ToDir { get; set; }

		/// <summary>
		/// Whether to overwrite existing files.
		/// </summary>
		[TaskParameter(Optional = true)]
		public bool Overwrite { get; set; }

		/// <summary>
		/// Tag to be applied to build products of this task.
		/// </summary>
		[TaskParameter(Optional = true, ValidationType = TaskParameterValidationType.TagList)]
		public string Tag { get; set; }
	}

	/// <summary>
	/// Stages files listed in a build receipt to an output directory.
	/// </summary>
	[TaskElement("Stage", typeof(StageTaskParameters))]
	public class StageTask : BgTaskImpl
	{
		readonly StageTaskParameters _parameters;

		/// <summary>
		/// Constructor.
		/// </summary>
		/// <param name="parameters">Parameters for this task</param>
		public StageTask(StageTaskParameters parameters)
		{
			_parameters = parameters;
		}

		/// <summary>
		/// ExecuteAsync the task.
		/// </summary>
		/// <param name="job">Information about the current job</param>
		/// <param name="buildProducts">Set of build products produced by this node.</param>
		/// <param name="tagNameToFileSet">Mapping from tag names to the set of files they include</param>
		public override Task ExecuteAsync(JobContext job, HashSet<FileReference> buildProducts, Dictionary<string, HashSet<FileReference>> tagNameToFileSet)
		{
			// Get the project path, and check it exists
			FileReference projectFile = _parameters.Project;
			if (_parameters.Project != null && !FileReference.Exists(projectFile))
			{
				throw new AutomationException("Couldn't find project '{0}'", projectFile.FullName);
			}

			// Get the directories used for staging this project
			DirectoryReference sourceEngineDir = Unreal.EngineDirectory;
			DirectoryReference sourceProjectDir = (projectFile == null) ? sourceEngineDir : projectFile.Directory;

			// Get the output directories. We flatten the directory structure on output.
			DirectoryReference targetDir = _parameters.ToDir;
			DirectoryReference targetEngineDir = DirectoryReference.Combine(targetDir, "Engine");
			DirectoryReference targetProjectDir = (projectFile == null) ? targetEngineDir : DirectoryReference.Combine(targetDir, projectFile.GetFileNameWithoutExtension());

			// Get the path to the receipt
			FileReference receiptFileName = TargetReceipt.GetDefaultPath(sourceProjectDir, _parameters.Target, _parameters.Platform, _parameters.Configuration, UnrealArchitectures.FromString(_parameters.Architecture, _parameters.Platform));

			// Try to load it
			TargetReceipt receipt;
			if (!TargetReceipt.TryRead(receiptFileName, out receipt))
			{
				throw new AutomationException("Couldn't read receipt '{0}'", receiptFileName);
			}

			// Stage all the build products needed at runtime
			HashSet<FileReference> sourceFiles = new HashSet<FileReference>();
			foreach (BuildProduct buildProduct in receipt.BuildProducts)
			{
				sourceFiles.Add(buildProduct.Path);
			}
			foreach (RuntimeDependency runtimeDependency in receipt.RuntimeDependencies.Where(x => x.Type != StagedFileType.UFS))
			{
				sourceFiles.Add(runtimeDependency.Path);
			}

			// Get all the target files
			List<FileReference> targetFiles = new List<FileReference>();
			foreach (FileReference sourceFile in sourceFiles)
			{
				// Get the destination file to copy to, mapping to the new engine and project directories as appropriate
				FileReference targetFile;
				if (sourceFile.IsUnderDirectory(sourceEngineDir))
				{
					targetFile = FileReference.Combine(targetEngineDir, sourceFile.MakeRelativeTo(sourceEngineDir));
				}
				else
				{
					targetFile = FileReference.Combine(targetProjectDir, sourceFile.MakeRelativeTo(sourceProjectDir));
				}

				// Only copy the output file if it doesn't already exist. We can stage multiple targets to the same output directory.
				if (_parameters.Overwrite || !FileReference.Exists(targetFile))
				{
					DirectoryReference.CreateDirectory(targetFile.Directory);
					CommandUtils.CopyFile(sourceFile.FullName, targetFile.FullName);
					// Force all destination files to not readonly.
					CommandUtils.SetFileAttributes(targetFile.FullName, ReadOnly: false);
				}

				// Add it to the list of target files
				targetFiles.Add(targetFile);
			}

			// Apply the optional tag to the build products
			foreach (string tagName in FindTagNamesFromList(_parameters.Tag))
			{
				FindOrAddTagSet(tagNameToFileSet, tagName).UnionWith(targetFiles);
			}

			// Add the target file to the list of build products
			buildProducts.UnionWith(targetFiles);
			return Task.CompletedTask;
		}

		/// <summary>
		/// Output this task out to an XML writer.
		/// </summary>
		public override void Write(XmlWriter writer)
		{
			Write(writer, _parameters);
		}

		/// <summary>
		/// Find all the tags which are used as inputs to this task
		/// </summary>
		/// <returns>The tag names which are read by this task</returns>
		public override IEnumerable<string> FindConsumedTagNames()
		{
			yield break;
		}

		/// <summary>
		/// Find all the tags which are modified by this task
		/// </summary>
		/// <returns>The tag names which are modified by this task</returns>
		public override IEnumerable<string> FindProducedTagNames()
		{
			return FindTagNamesFromList(_parameters.Tag);
		}
	}
}
