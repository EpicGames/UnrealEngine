// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Linq;
using System.Threading.Tasks;
using System.Xml;
using EpicGames.Core;
using Microsoft.Extensions.Logging;
using UnrealBuildTool;

namespace AutomationTool.Tasks
{
	/// <summary>
	/// Parameters for a task that strips symbols from a set of files
	/// </summary>
	public class StripTaskParameters
	{
		/// <summary>
		/// The platform toolchain to strip binaries.
		/// </summary>
		[TaskParameter]
		public UnrealTargetPlatform Platform { get; set; }

		/// <summary>
		/// The directory to find files in.
		/// </summary>
		[TaskParameter(Optional = true)]
		public DirectoryReference BaseDir { get; set; }

		/// <summary>
		/// List of file specifications separated by semicolons (for example, Engine/.../*.pdb), or the name of a tag set.
		/// </summary>
		[TaskParameter(ValidationType = TaskParameterValidationType.FileSpec)]
		public string Files { get; set; }

		/// <summary>
		/// Output directory for the stripped files. Defaults to the input path, overwriting the input files.
		/// </summary>
		[TaskParameter(Optional = true)]
		public DirectoryReference OutputDir { get; set; }

		/// <summary>
		/// Tag to be applied to build products of this task.
		/// </summary>
		[TaskParameter(Optional = true, ValidationType = TaskParameterValidationType.TagList)]
		public string Tag { get; set; }

		/// <summary>
		/// Whether all symbols should be stripped. Not relevant to all platforms.
		/// </summary>
		[TaskParameter(Optional = true)]
		public bool StripAll { get; set; }
	}

	/// <summary>
	/// Strips debugging information from a set of files.
	/// </summary>
	[TaskElement("Strip", typeof(StripTaskParameters))]
	public class StripTask : BgTaskImpl
	{
		readonly StripTaskParameters _parameters;

		/// <summary>
		/// Construct a spawn task
		/// </summary>
		/// <param name="parameters">Parameters for the task</param>
		public StripTask(StripTaskParameters parameters)
		{
			_parameters = parameters;
		}

		/// <summary>
		/// ExecuteAsync the task.
		/// </summary>
		/// <param name="job">Information about the current job</param>
		/// <param name="buildProducts">Set of build products produced by this node.</param>
		/// <param name="tagNameToFileSet">Mapping from tag names to the set of files they include</param>
		public override Task ExecuteAsync(JobContext job, HashSet<FileReference> buildProducts, Dictionary<string, HashSet<FileReference>> tagNameToFileSet)
		{
			// Get the base directory
			DirectoryReference baseDir = _parameters.BaseDir;

			// Get the output directory
			DirectoryReference outputDir = _parameters.OutputDir;

			// Find the matching files
			FileReference[] sourceFiles = ResolveFilespec(baseDir, _parameters.Files, tagNameToFileSet).OrderBy(x => x.FullName).ToArray();

			// Create the matching target files
			FileReference[] targetFiles = sourceFiles.Select(x => FileReference.Combine(outputDir, x.MakeRelativeTo(baseDir))).ToArray();

			// Run the stripping command
			Platform targetPlatform = Platform.GetPlatform(_parameters.Platform);
			for (int idx = 0; idx < sourceFiles.Length; idx++)
			{
				DirectoryReference.CreateDirectory(targetFiles[idx].Directory);
				if (sourceFiles[idx] == targetFiles[idx])
				{
					Logger.LogInformation("Stripping symbols: {Arg0}", sourceFiles[idx].FullName);
				}
				else
				{
					Logger.LogInformation("Stripping symbols: {Arg0} -> {Arg1}", sourceFiles[idx].FullName, targetFiles[idx].FullName);
				}

				if (_parameters.StripAll)
				{
					targetPlatform.StripAllSymbols(sourceFiles[idx], targetFiles[idx]);
				}
				else
				{
					targetPlatform.StripSymbols(sourceFiles[idx], targetFiles[idx]);
				}
			}

			// Apply the optional tag to the build products
			foreach (string tagName in FindTagNamesFromList(_parameters.Tag))
			{
				FindOrAddTagSet(tagNameToFileSet, tagName).UnionWith(targetFiles);
			}

			// Add the target files to the set of build products
			buildProducts.UnionWith(targetFiles);
			return Task.CompletedTask;
		}

		/// <summary>
		/// Output this task out to an XML writer.
		/// </summary>
		public override void Write(XmlWriter writer)
		{
			Write(writer, _parameters);
		}

		/// <summary>
		/// Find all the tags which are used as inputs to this task
		/// </summary>
		/// <returns>The tag names which are read by this task</returns>
		public override IEnumerable<string> FindConsumedTagNames()
		{
			return FindTagNamesFromFilespec(_parameters.Files);
		}

		/// <summary>
		/// Find all the tags which are modified by this task
		/// </summary>
		/// <returns>The tag names which are modified by this task</returns>
		public override IEnumerable<string> FindProducedTagNames()
		{
			return FindTagNamesFromList(_parameters.Tag);
		}
	}

	public static partial class StandardTasks
	{
		/// <summary>
		/// Strips symbols from a set of files.
		/// </summary>
		/// <param name="files"></param>
		/// <param name="platform">The platform toolchain to strip binaries.</param>
		/// <param name="baseDir">The directory to find files in.</param>
		/// <param name="outputDir">Output directory for the stripped files. Defaults to the input path, overwriting the input files.</param>
		/// <returns></returns>
		public static async Task<FileSet> StripAsync(FileSet files, UnrealTargetPlatform platform, DirectoryReference baseDir = null, DirectoryReference outputDir = null)
		{
			StripTaskParameters parameters = new StripTaskParameters();
			parameters.Platform = platform;
			parameters.BaseDir = baseDir;
			parameters.Files = String.Join(";", files.Flatten().Values.Select(x => x.FullName));
			parameters.OutputDir = outputDir;
			return await ExecuteAsync(new StripTask(parameters));
		}
	}
}
