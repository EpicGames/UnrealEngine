// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Text.RegularExpressions;
using System.Threading.Tasks;
using System.Xml;
using EpicGames.Core;
using Microsoft.Extensions.Logging;
using UnrealBuildBase;

namespace AutomationTool.Tasks
{
	/// <summary>
	/// Parameters for the submit task
	/// </summary>
	public class SubmitTaskParameters
	{
		/// <summary>
		/// The description for the submitted changelist.
		/// </summary>
		[TaskParameter]
		public string Description { get; set; }

		/// <summary>
		/// The files to submit.
		/// </summary>
		[TaskParameter(ValidationType = TaskParameterValidationType.FileSpec)]
		public string Files { get; set; }

		/// <summary>
		/// The Perforce file type for the submitted files (for example, binary+FS32).
		/// </summary>
		[TaskParameter(Optional = true)]
		public string FileType { get; set; }

		/// <summary>
		/// The workspace name. If specified, a new workspace will be created using the given stream and root directory to submit the files. If not, the current workspace will be used.
		/// </summary>
		[TaskParameter(Optional = true)]
		public string Workspace { get; set; }

		/// <summary>
		/// The stream for the workspace -- defaults to the current stream. Ignored unless the Workspace attribute is also specified.
		/// </summary>
		[TaskParameter(Optional = true)]
		public string Stream { get; set; }

		/// <summary>
		/// Branch for the workspace (legacy P4 depot path). May not be used in conjunction with Stream.
		/// </summary>
		[TaskParameter(Optional = true)]
		public string Branch { get; set; }

		/// <summary>
		/// Root directory for the stream. If not specified, defaults to the current root directory.
		/// </summary>
		[TaskParameter(Optional = true)]
		public DirectoryReference RootDir { get; set; }

		/// <summary>
		/// Whether to revert unchanged files before attempting to submit.
		/// </summary>
		[TaskParameter(Optional = true)]
		public bool RevertUnchanged { get; set; }

		/// <summary>
		/// Force the submit to happen -- even if a resolve is needed (always accept current version).
		/// </summary>
		[TaskParameter(Optional = true)]
		public bool Force { get; set; }

		/// <summary>
		/// Allow verbose P4 output (spew).
		/// </summary>
		[TaskParameter(Optional = true)]
		public bool P4Verbose { get; set; }
	}

	/// <summary>
	/// Creates a new changelist and submits a set of files to a Perforce stream.
	/// </summary>
	[TaskElement("Submit", typeof(SubmitTaskParameters))]
	public class SubmitTask : BgTaskImpl
	{
		readonly SubmitTaskParameters _parameters;

		/// <summary>
		/// Construct a version task
		/// </summary>
		/// <param name="parameters">Parameters for this task</param>
		public SubmitTask(SubmitTaskParameters parameters)
		{
			_parameters = parameters;
		}

		/// <summary>
		/// ExecuteAsync the task.
		/// </summary>
		/// <param name="job">Information about the current job</param>
		/// <param name="buildProducts">Set of build products produced by this node.</param>
		/// <param name="tagNameToFileSet">Mapping from tag names to the set of files they include</param>
		public override Task ExecuteAsync(JobContext job, HashSet<FileReference> buildProducts, Dictionary<string, HashSet<FileReference>> tagNameToFileSet)
		{
			HashSet<FileReference> files = ResolveFilespec(Unreal.RootDirectory, _parameters.Files, tagNameToFileSet);
			if (files.Count == 0)
			{
				Logger.LogInformation("No files to submit.");
			}
			else if (!CommandUtils.AllowSubmit)
			{
				Logger.LogWarning("Submitting to Perforce is disabled by default. Run with the -submit argument to allow.");
			}
			else
			{
				try
				{
					// Get the connection that we're going to submit with
					P4Connection submitP4 = CommandUtils.P4;
					if (_parameters.Workspace != null)
					{
						// Create a brand new workspace
						P4ClientInfo client = new P4ClientInfo();
						client.Owner = CommandUtils.P4Env.User;
						client.Host = Unreal.MachineName;
						client.RootPath = _parameters.RootDir.FullName ?? Unreal.RootDirectory.FullName;
						client.Name = $"{_parameters.Workspace}_{Regex.Replace(client.Host, "[^a-zA-Z0-9]", "-")}_{ContentHash.MD5((CommandUtils.P4Env.ServerAndPort ?? "").ToUpperInvariant())}";
						client.Options = P4ClientOption.NoAllWrite | P4ClientOption.Clobber | P4ClientOption.NoCompress | P4ClientOption.Unlocked | P4ClientOption.NoModTime | P4ClientOption.RmDir;
						client.LineEnd = P4LineEnd.Local;
						if (!String.IsNullOrEmpty(_parameters.Branch))
						{
							client.View.Add(new KeyValuePair<string, string>($"{_parameters.Branch}/...", $"/..."));
						}
						else
						{
							client.Stream = _parameters.Stream ?? CommandUtils.P4Env.Branch;
						}
						CommandUtils.P4.CreateClient(client, AllowSpew: _parameters.P4Verbose);

						// Create a new connection for it
						submitP4 = new P4Connection(client.Owner, client.Name);
					}

					// Get the latest version of it
					int newCl = submitP4.CreateChange(Description: _parameters.Description.Replace("\\n", "\n", StringComparison.Ordinal));
					foreach (FileReference file in files)
					{
						submitP4.Revert(String.Format("-k \"{0}\"", file.FullName), AllowSpew: _parameters.P4Verbose);
						submitP4.Sync(String.Format("-k \"{0}\"", file.FullName), AllowSpew: _parameters.P4Verbose);
						submitP4.Add(newCl, String.Format("\"{0}\"", file.FullName));
						submitP4.Edit(newCl, String.Format("\"{0}\"", file.FullName), AllowSpew: _parameters.P4Verbose);
						if (_parameters.FileType != null)
						{
							submitP4.P4(String.Format("reopen -t \"{0}\" \"{1}\"", _parameters.FileType, file.FullName), AllowSpew: _parameters.P4Verbose);
						}
					}

					// Revert any unchanged files
					if (_parameters.RevertUnchanged)
					{
						submitP4.RevertUnchanged(newCl);
						if (submitP4.TryDeleteEmptyChange(newCl))
						{
							Logger.LogInformation("No files to submit; ignored.");
							return Task.CompletedTask;
						}
					}

					// Submit it
					int submittedCl;
					submitP4.Submit(newCl, out submittedCl, Force: _parameters.Force);
					if (submittedCl <= 0)
					{
						throw new AutomationException("Submit failed.");
					}

					Logger.LogInformation("Submitted in changelist {SubmittedCL}", submittedCl);
				}
				catch (P4Exception ex)
				{
					Logger.LogError(KnownLogEvents.Systemic_Perforce, "{Message}", ex.Message);
					throw new AutomationException(ex.ErrorCode, ex, "{0}", ex.Message) { OutputFormat = AutomationExceptionOutputFormat.Silent };
				}
			}
			return Task.CompletedTask;
		}

		/// <summary>
		/// Output this task out to an XML writer.
		/// </summary>
		public override void Write(XmlWriter writer)
		{
			Write(writer, _parameters);
		}

		/// <summary>
		/// Find all the tags which are used as inputs to this task
		/// </summary>
		/// <returns>The tag names which are read by this task</returns>
		public override IEnumerable<string> FindConsumedTagNames()
		{
			return FindTagNamesFromFilespec(_parameters.Files);
		}

		/// <summary>
		/// Find all the tags which are modified by this task
		/// </summary>
		/// <returns>The tag names which are modified by this task</returns>
		public override IEnumerable<string> FindProducedTagNames()
		{
			yield break;
		}
	}
}
