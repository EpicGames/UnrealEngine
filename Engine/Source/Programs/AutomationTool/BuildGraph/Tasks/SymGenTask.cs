// Copyright Epic Games, Inc. All Rights Reserved.

using AutomationTool;
using System;
using System.IO;
using System.Collections.Concurrent;
using System.Collections.Generic;
using System.Linq;
using System.Threading.Tasks;
using System.Xml;
using EpicGames.Core;
using UnrealBuildTool;
using UnrealBuildBase;
using Microsoft.Extensions.Logging;

namespace BuildGraph.Tasks
{
	/// <summary>
	/// Parameters for a task that generates debugging symbols from a set of files
	/// </summary>
	public class SymGenTaskParameters
	{
		/// <summary>
		/// List of file specifications separated by semicolons (eg. *.cpp;Engine/.../*.bat), or the name of a tag set
		/// </summary>
		[TaskParameter(ValidationType = TaskParameterValidationType.FileSpec)]
		public string Files { get; set; }

		/// <summary>
		/// Tag to be applied to build products of this task
		/// </summary>
		[TaskParameter(Optional = true, ValidationType = TaskParameterValidationType.TagList)]
		public string Tag { get; set; }

		/// <summary>
		/// If set, this will use the rad debugger pdb symbol dumper as well as the rad symbol_path_fixer.
		/// </summary>
		[TaskParameter(Optional = true)]
		public bool UseRadSym { get; set; } = false;

}

	/// <summary>
	/// Generates a portable symbol dump file from the specified binaries
	/// </summary>
	[TaskElement("SymGen", typeof(SymGenTaskParameters))]
	public class SymGenTask : CustomTask
	{
		/// <summary>
		/// Parameters for this task
		/// </summary>
		readonly SymGenTaskParameters _parameters;

		/// <summary>
		/// Construct a spawn task
		/// </summary>
		/// <param name="parameters">Parameters for the task</param>
		public SymGenTask(SymGenTaskParameters parameters)
		{
			_parameters = parameters;
		}

		static UnrealArchitectures ArchitecturesInBinary(FileReference binary)
		{
			if (BuildHostPlatform.Current.Platform != UnrealTargetPlatform.Mac)
			{
				return null;
			}

			List<UnrealArch> arches = new();
			string output = Utils.RunLocalProcessAndReturnStdOut("sh", $"-c 'file \"{binary.FullName}\"'");
			if (output.Contains("arm64", StringComparison.InvariantCulture))
			{
				arches.Add(UnrealArch.Arm64);
			}
			if (output.Contains("x86_64", StringComparison.InvariantCulture))
			{
				arches.Add(UnrealArch.X64);
			}
			return new UnrealArchitectures(arches);
		}

		/// <summary>
		/// Execute the task.
		/// </summary>
		/// <param name="job">Information about the current job</param>
		/// <param name="buildProducts">Set of build products produced by this node.</param>
		/// <param name="tagNameToFileSet">Mapping from tag names to the set of files they include</param>
		public override void Execute(JobContext job, HashSet<FileReference> buildProducts, Dictionary<string, HashSet<FileReference>> tagNameToFileSet)
		{
			bool bUseRadSym = _parameters.UseRadSym;

			// Path to Breakpad's dump_syms executable
			string symbolDumperExecutable = null;

			// Find the matching files
			FileReference[] sourceFiles = ResolveFilespec(Unreal.RootDirectory, _parameters.Files, tagNameToFileSet).OrderBy(x => x.FullName).ToArray();

			string radSymDymperExecuable = Unreal.RootDirectory + @"\Engine\Extras\rad\Binaries\Win64\radbin.exe";
			string radProcessSymExecuable = Unreal.RootDirectory + @"\Engine\Extras\rad\Binaries\Win64\symbol_path_fixer.exe";

			// Filter out all the symbol files
			FileReference[] symbolSourceFiles;
			string workingDirectory = null;
			if (BuildHostPlatform.Current.Platform == UnrealTargetPlatform.Win64)
			{
				symbolDumperExecutable = Unreal.RootDirectory + @"\Engine\Source\ThirdParty\Breakpad\src\tools\windows\binaries\dump_syms.exe";
				string[] symbolFileExtensions = { ".pdb", ".nss", ".nrs" };
				symbolSourceFiles = sourceFiles.Where(x => symbolFileExtensions.Contains(x.GetExtension())).ToArray();
				// set working dir to find our version of msdia140.dll
				workingDirectory = Unreal.RootDirectory + @"\Engine\Binaries\Win64";
			}
			else if (BuildHostPlatform.Current.Platform == UnrealTargetPlatform.Mac)
			{
				symbolDumperExecutable = Unreal.RootDirectory + "/Engine/Source/ThirdParty/Breakpad/src/tools/mac/binaries/dump_syms";
				List<FileReference> files = sourceFiles.Where(x => x.HasExtension(".dSYM")).ToList();

				// find any zipped bundles
				Directory.CreateDirectory(Unreal.RootDirectory + "/Engine/Intermediate/Unzipped");
				FileReference[] zippedFiles = sourceFiles.Where(x => x.FullName.Contains(".dSYM.zip", StringComparison.InvariantCulture)).ToArray();
				foreach (FileReference sourceFile in zippedFiles)
				{
					string[] unzippedFiles = CommandUtils.UnzipFiles(sourceFile.FullName, Unreal.RootDirectory + "/Engine/Intermediate/Unzipped").ToArray();
					files.Add(new FileReference(Unreal.RootDirectory + "/Engine/Intermediate/Unzipped/" + sourceFile.GetFileNameWithoutExtension()));
				}
				foreach (FileReference sourceFile in files)
				{
					Logger.LogInformation("Source File: {Arg0}", sourceFile.FullName);
				}
				symbolSourceFiles = files.Where(x => x.HasExtension(".dSYM")).ToArray();
			}
			else if (BuildHostPlatform.Current.Platform == UnrealTargetPlatform.Linux)
			{
				symbolDumperExecutable = Unreal.RootDirectory + "/Engine/Binaries/Linux/dump_syms";
				string[] symbolFileExtensions = { ".debug" };
				symbolSourceFiles = sourceFiles.Where(x => symbolFileExtensions.Contains(x.GetExtension())).ToArray();
			}
			else
			{
				throw new AutomationException("Symbol generation failed: Unknown platform {0}", BuildHostPlatform.Current.Platform);
			}

			// Remove any existing symbol files
			foreach (string fileName in symbolSourceFiles.Select(x => Path.ChangeExtension(x.FullName, ".psym")))
			{
				if (File.Exists(fileName))
				{
					try
					{
						File.Delete(fileName);
					}
					catch (Exception ex)
					{
						throw new AutomationException("Symbol generation failed: Unable to delete existing symbol file: \"{0}\". Error: {1}", fileName, ex.Message.TrimEnd());
					}
				}
			}

			if (symbolSourceFiles.Length == 0)
			{
				Logger.LogInformation("No symbol files to convert.");
			}

			// Generate portable symbols from the symbol source files
			ConcurrentBag<FileReference> symbolFiles = new ConcurrentBag<FileReference>();

			Parallel.ForEach(symbolSourceFiles, (sourceFile) =>
			{
				string symbolFileName = Path.ChangeExtension(sourceFile.FullName, ".psym");
				string radSymbolTemp = Path.ChangeExtension(sourceFile.FullName, ".radpsym");

				// Check if higher priority debug file or binary already created symbols 
				if (File.Exists(symbolFileName))
				{
					return;
				}

				Logger.LogInformation("Dumping Symbols: {Arg0} to {SymbolFileName}", sourceFile.FullName, symbolFileName);

				string dumpSymsArgs;

				string symbolDumperExeForFile = symbolDumperExecutable;
				if (bUseRadSym &&
					sourceFile.GetExtension() == ".pdb")
				{
					symbolDumperExeForFile = radSymDymperExecuable;
					dumpSymsArgs = $"--breakpad {sourceFile.FullName} -out:{radSymbolTemp}";
				}
				else
				{
					string extraOptions = "";
					if (BuildHostPlatform.Current.Platform == UnrealTargetPlatform.Mac)
					{
						// dump_syms has a bug where if a universal binary is fed into it, on an Intel mac, it will fail to find the current architecture
						// (but not on Arm macs). Specify the host architecture as a param to cause expected behavior (until we make one output per Arch)
						if (ArchitecturesInBinary(sourceFile).bIsMultiArch)
						{
							// ExtraOptions = $"-a {MacExports.HostArchitecture.AppleName} ";
							// Since IBs are universal and we typically only care about arm symbols, force
							// the arch to always be arm.
							extraOptions = $"-a arm64 ";
						}
					}

					dumpSymsArgs = extraOptions + "\"" + sourceFile.FullName + "\"";
				}

				IProcessResult result = CommandUtils.Run(symbolDumperExeForFile, dumpSymsArgs, null, CommandUtils.ERunOptions.AppMustExist, null, FilterSpew, null, workingDirectory);
				if (result.ExitCode == 0)
				{
					if (bUseRadSym)
					{
						// rad dumper outputs to a file, we thunk to a custom exe to do symbol munging for speed.
						CommandUtils.Run(radProcessSymExecuable, radSymbolTemp + " " + Unreal.RootDirectory.FullName, null, CommandUtils.ERunOptions.AppMustExist, null, FilterSpew, null, workingDirectory);

						File.Move(radSymbolTemp, symbolFileName);
						symbolFiles.Add(new FileReference(symbolFileName));

					}
					else
					{
						// Use streaming approach to avoid 2GB string limitation
						// Read directly from the process output without materializing it as a string
						try
						{
							using (StringBuilderReader reader = result.GetOutputReader())
							{
								if (reader != null)
								{
									using (StreamWriter writer = new StreamWriter(symbolFileName))
									{
										ProcessSymbols(reader, writer);
										symbolFiles.Add(new FileReference(symbolFileName));
									}
								}
							}
						}
						catch (Exception ex)
						{
							// There was a problem generating symbols with the dump_syms tool
							throw new AutomationException($"Symbol generation failed: Error Generating Symbols for {symbolFileName}, Error: {ExceptionUtils.FormatException(ex)}");
						}
					}
				}
				else
				{
					if (BuildHostPlatform.Current.Platform != UnrealTargetPlatform.Mac && !bUseRadSym)
					{
						// If we fail, lets re-run with a verbose, -v to check for the error we are seeing
						// -v not available on Mac
						CommandUtils.Run(symbolDumperExecutable, "-v " + sourceFile.FullName, null, CommandUtils.ERunOptions.AppMustExist, null, null, null, workingDirectory);
					}

					// There was a problem generating symbols with the dump_syms tool
					throw new AutomationException("Symbol generation failed: Error Generating Symbols: {0}", symbolFileName);
				}
			});

			// Apply the optional tag to the build products
			foreach (string tagName in FindTagNamesFromList(_parameters.Tag))
			{
				FindOrAddTagSet(tagNameToFileSet, tagName).UnionWith(symbolFiles);
			}

			// Add them to the list of build products
			buildProducts.UnionWith(symbolFiles);
		}

		/// <summary>
		/// Processes the raw symbol dump
		/// </summary>

		static bool ProcessSymbols(StringBuilderReader reader, TextWriter writer)
		{
			char[] fieldSeparator = { ' ' };
			string rootDirectory = CommandUtils.ConvertSeparators(PathSeparator.Slash, Unreal.RootDirectory.FullName).TrimEnd('/');

			bool bSawModule = false;
			while (reader.ReadLine(out ReadOnlySpan<char> lineSpan))
			{
				if (lineSpan.Contains(" = ", StringComparison.InvariantCulture))
				{
					Logger.LogInformation("{Text}", lineSpan.ToString());
					continue;
				}
				// Ignore any output from symbol dump before MODULE, these may included erroneous warnings, etc
				if (!bSawModule)
				{
					if (!lineSpan.StartsWith("MODULE", StringComparison.InvariantCulture))
					{
						continue;
					}

					bSawModule = true;
				}

				// Process source reference FILE blocks
				if (lineSpan.StartsWith("FILE", StringComparison.InvariantCulture))
				{
					// Convert to string only for FILE lines that need processing
					string line = lineSpan.ToString();
					string[] fields = line.Split(fieldSeparator, 3);

					// Check we got all the expected fields
					if (fields.Length == 3)
					{
						string fileName = CommandUtils.ConvertSeparators(PathSeparator.Slash, fields[2]);

						// If the file exists locally, and is within the root, convert path
						if (File.Exists(fileName) && fileName.StartsWith(rootDirectory, StringComparison.OrdinalIgnoreCase))
						{
							// Restore proper filename case on Windows (the symbol dump filenames are all lowercase)
							if (BuildHostPlatform.Current.Platform == UnrealTargetPlatform.Win64)
							{
								fileName = FileUtils.FindCorrectCase(new FileInfo(fileName)).FullName;
							}

							// Shave off the root directory
							writer.WriteLine(String.Format("FILE {0} {1}", fields[1], fileName.Substring(rootDirectory.Length + 1).Replace('\\', '/')));
							continue;
						}
					}
				}

				writer.WriteLine(lineSpan);
			}

			return true;

		}

		/// <summary>
		///  Filters the output from the dump_syms executable, which depending on the platform can be pretty spammy
		/// </summary>
		string FilterSpew(string message)
		{
			foreach (string filterString in s_outputFilterStrings)
			{
				if (message.Contains(filterString, StringComparison.InvariantCulture))
				{
					return null;
				}
			}

			return message;
		}

		/// <summary>
		/// Array of source strings to filter from output
		/// </summary>
		static readonly string[] s_outputFilterStrings = new string[] { "the DIE at offset", "warning: function", "warning: failed", ": in compilation unit" };

		/// <summary>
		/// Output this task out to an XML writer.
		/// </summary>
		public override void Write(XmlWriter writer)
		{
			Write(writer, _parameters);
		}

		/// <summary>
		/// Find all the tags which are used as inputs to this task
		/// </summary>
		/// <returns>The tag names which are read by this task</returns>
		public override IEnumerable<string> FindConsumedTagNames()
		{
			return FindTagNamesFromFilespec(_parameters.Files);
		}

		/// <summary>
		/// Find all the tags which are modified by this task
		/// </summary>
		/// <returns>The tag names which are modified by this task</returns>
		public override IEnumerable<string> FindProducedTagNames()
		{
			return FindTagNamesFromList(_parameters.Tag);
		}
	}
}

namespace BuildScripts.Automation
{
	class GeneratePsyms : BuildCommand
	{
		public override ExitCode Execute()
		{
			BuildGraph.Tasks.SymGenTaskParameters symGenParams = new BuildGraph.Tasks.SymGenTaskParameters();
			symGenParams.Files = ParseRequiredStringParam("Files");

			BuildGraph.Tasks.SymGenTask symGenTask = new BuildGraph.Tasks.SymGenTask(symGenParams);
			symGenTask.Execute(null, new HashSet<FileReference>(), new Dictionary<string, HashSet<FileReference>>());

			return ExitCode.Success;
		}
	}
}

