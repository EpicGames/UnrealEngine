// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Linq;
using System.Threading.Tasks;
using System.Xml;
using EpicGames.Core;
using Microsoft.Extensions.Logging;
using UnrealBuildBase;
using UnrealBuildTool;

namespace AutomationTool.Tasks
{
	/// <summary>
	/// Parameters for a task that uploads symbols to a symbol server
	/// </summary>
	public class SymStoreTaskParameters
	{
		/// <summary>
		/// The platform toolchain required to handle symbol files.
		/// </summary>
		[TaskParameter]
		public UnrealTargetPlatform Platform { get; set; }

		/// <summary>
		/// List of output files. PDBs will be extracted from this list.
		/// </summary>
		[TaskParameter]
		public string Files { get; set; }

		/// <summary>
		/// Output directory for the compressed symbols.
		/// </summary>
		[TaskParameter]
		public string StoreDir { get; set; }

		/// <summary>
		/// Name of the product for the symbol store records.
		/// </summary>
		[TaskParameter]
		public string Product { get; set; }

		/// <summary>
		/// Name of the Branch to base all the depot source files from.
		/// Used when IndexSources is true (may be used only on some platforms).
		/// </summary>
		[TaskParameter(Optional = true)]
		public string Branch { get; set; }

		/// <summary>
		/// Changelist to which all the depot source files have been synced to.
		/// Used when IndexSources is true (may be used only on some platforms).
		/// </summary>
		[TaskParameter(Optional = true)]
		public int Change { get; set; }

		/// <summary>
		/// BuildVersion associated with these symbols. Used for clean-up in AgeStore by matching this version against a directory name in a build share.
		/// </summary>
		[TaskParameter(Optional = true)]
		public string BuildVersion { get; set; }

		/// <summary>
		/// Whether to include the source code index in the uploaded symbols.
		/// When enabled, the task will generate data required by a source server (only some platforms and source control servers are supported).
		/// The source server allows debuggers to automatically fetch the matching source code when debbugging builds or analyzing dumps.
		/// </summary>
		[TaskParameter(Optional = true)]
		public bool IndexSources { get; set; } = false;

		/// <summary>
		/// Filter for the depot source files that are to be indexed.
		/// It's a semicolon-separated list of perforce filter e.g. Engine/....cpp;Engine/....h.
		/// It may also be a name of a previously defined tag e.g. "#SourceFiles
		/// Used when IndexSources is true (may be used only on some platforms).
		/// </summary>
		[TaskParameter(Optional = true)]
		public string SourceFiles { get; set; }

		/// <summary>
		/// List of file path mappings to be applied during source indexing. It is needed only when using VFS (Virtual File System)
		/// feature when compiling the binary. VFS replaces the real local paths used with standardized paths like Z:\UEVFS\...
		/// Since these virtualized paths don't match paths used by the local workspace, we need to map them manually so that
		/// paths in object files/symbols matched the data we have in source indexing.
		/// It's a semicolon-separated list of path assignments e.g.
		///      "[Root]\Samples\Games\SampleGame=Z:\UEVFS\SampleGame;[Root]=Z:\UEVFS\Root"
		/// 
		/// [Root] will be replaced with Unreal.RootDirectory.
		/// You need to ensure that the mapping used here matches the conventions used by Unreal Build Tool.
		/// We may improve it in the future but current the VFS mapping used during compilation is not available outside of UBT.
		/// </summary>
		[TaskParameter(Optional = true)]
		public string VFSMapping { get; set; }
	}

	/// <summary>
	/// Task that strips symbols from a set of files.
	/// </summary>
	[TaskElement("SymStore", typeof(SymStoreTaskParameters))]
	public class SymStoreTask : BgTaskImpl
	{
		/// <summary>
		/// Parameters for this task
		/// </summary>
		readonly SymStoreTaskParameters _parameters;

		/// <summary>
		/// Construct a spawn task
		/// </summary>
		/// <param name="parameters">Parameters for the task</param>
		public SymStoreTask(SymStoreTaskParameters parameters)
		{
			_parameters = parameters;
		}

		/// <summary>
		/// ExecuteAsync the task.
		/// </summary>
		/// <param name="job">Information about the current job</param>
		/// <param name="buildProducts">Set of build products produced by this node.</param>
		/// <param name="tagNameToFileSet">Mapping from tag names to the set of files they include</param>
		public override Task ExecuteAsync(JobContext job, HashSet<FileReference> buildProducts, Dictionary<string, HashSet<FileReference>> tagNameToFileSet)
		{
			// Find the matching files
			HashSet<FileReference> files = ResolveFilespec(Unreal.RootDirectory, _parameters.Files, tagNameToFileSet);

			// Get the symbol store directory
			DirectoryReference storeDir = ResolveDirectory(_parameters.StoreDir);

			// Take the lock before accessing the symbol server, if required by the platform
			Platform targetPlatform = Platform.GetPlatform(_parameters.Platform);

			List<FileReference> sourceFiles = new List<FileReference>();

			if (_parameters.IndexSources && targetPlatform.SymbolServerSourceIndexingRequiresListOfSourceFiles)
			{
				Logger.LogInformation("Discovering source code files...");

				sourceFiles = ResolveFilespec(Unreal.RootDirectory, _parameters.SourceFiles, tagNameToFileSet).ToList();
			}

			CommandUtils.OptionallyTakeLock(targetPlatform.SymbolServerRequiresLock, storeDir, TimeSpan.FromMinutes(60), () =>
			{
				// symstore has issues with processing same file in multiple folders, so run multiple passes with only unique filenames
				bool anyError = false;
				while (files.Count > 0)
				{
					List<FileReference> publishFiles = files.DistinctBy(x => x.GetFileName().ToUpperInvariant()).ToList();
					files = [.. files.Except(publishFiles)];

					if (!targetPlatform.PublishSymbols(storeDir, publishFiles, _parameters.IndexSources, sourceFiles,
						_parameters.Product, _parameters.Branch, _parameters.Change, _parameters.BuildVersion, _parameters.VFSMapping))
					{
						anyError = true;
					}
				}

				if (anyError)
				{
					throw new AutomationException("Failure publishing symbol files.");
				}
			});

			return Task.CompletedTask;
		}

		/// <summary>
		/// Output this task out to an XML writer.
		/// </summary>
		public override void Write(XmlWriter writer)
		{
			Write(writer, _parameters);
		}

		/// <summary>
		/// Find all the tags which are used as inputs to this task
		/// </summary>
		/// <returns>The tag names which are read by this task</returns>
		public override IEnumerable<string> FindConsumedTagNames()
		{
			return FindTagNamesFromFilespec(_parameters.Files);
		}

		/// <summary>
		/// Find all the tags which are modified by this task
		/// </summary>
		/// <returns>The tag names which are modified by this task</returns>
		public override IEnumerable<string> FindProducedTagNames()
		{
			yield break;
		}
	}
}
