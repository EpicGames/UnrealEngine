// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Threading.Tasks;
using System.Xml;
using EpicGames.Core;
using Microsoft.Extensions.Logging;
using UnrealBuildBase;
using UnrealBuildTool;

namespace AutomationTool.Tasks
{
	/// <summary>
	/// Parameters for the Tag Receipt task.
	/// </summary>
	public class TagReceiptTaskParameters
	{
		/// <summary>
		/// Set of receipt files (*.target) to read, including wildcards and tag names, separated by semicolons.
		/// </summary>
		[TaskParameter(ValidationType = TaskParameterValidationType.FileSpec)]
		public string Files { get; set; }

		/// <summary>
		/// Path to the Engine folder, used to expand $(EngineDir) properties in receipt files. Defaults to the Engine directory for the current workspace.
		/// </summary>
		[TaskParameter(Optional = true)]
		public DirectoryReference EngineDir { get; set; }

		/// <summary>
		/// Path to the project folder, used to expand $(ProjectDir) properties in receipt files. Defaults to the Engine directory for the current workspace -- DEPRECATED.
		/// </summary>
		[TaskParameter(Optional = true)]
		public DirectoryReference ProjectDir { get; set; }

		/// <summary>
		/// Whether to tag the Build Products listed in receipts.
		/// </summary>
		[TaskParameter(Optional = true)]
		public bool BuildProducts { get; set; }

		/// <summary>
		/// Which type of Build Products to tag (see TargetReceipt.cs - UnrealBuildTool.BuildProductType for valid values).
		/// </summary>
		[TaskParameter(Optional = true)]
		public string BuildProductType { get; set; }

		/// <summary>
		/// Whether to tag the Runtime Dependencies listed in receipts.
		/// </summary>
		[TaskParameter(Optional = true)]
		public bool RuntimeDependencies { get; set; }

		/// <summary>
		/// Which type of Runtime Dependencies to tag (see TargetReceipt.cs - UnrealBuildTool.StagedFileType for valid values).
		/// </summary>
		[TaskParameter(Optional = true)]
		public string StagedFileType { get; set; }

		/// <summary>
		/// Name of the tag to apply.
		/// </summary>
		[TaskParameter(ValidationType = TaskParameterValidationType.TagList)]
		public string With { get; set; }
	}

	/// <summary>
	/// Task that tags build products and/or runtime dependencies by reading from *.target files.
	/// </summary>
	[TaskElement("TagReceipt", typeof(TagReceiptTaskParameters))]
	class TagReceiptTask : BgTaskImpl
	{
		/// <summary>
		/// Parameters to this task
		/// </summary>
		readonly TagReceiptTaskParameters _parameters;

		/// <summary>
		/// The type of build products to enumerate. May be null.
		/// </summary>
		readonly BuildProductType? _buildProductType;

		/// <summary>
		/// The type of staged files to enumerate. May be null,
		/// </summary>
		readonly StagedFileType? _stagedFileType;

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="parameters">Parameters to select which files to search</param>
		public TagReceiptTask(TagReceiptTaskParameters parameters)
		{
			_parameters = parameters;

			if (!String.IsNullOrEmpty(_parameters.BuildProductType))
			{
				_buildProductType = (BuildProductType)Enum.Parse(typeof(BuildProductType), _parameters.BuildProductType);
			}
			if (!String.IsNullOrEmpty(_parameters.StagedFileType))
			{
				_stagedFileType = (StagedFileType)Enum.Parse(typeof(StagedFileType), _parameters.StagedFileType);
			}
		}

		/// <summary>
		/// ExecuteAsync the task.
		/// </summary>
		/// <param name="job">Information about the current job</param>
		/// <param name="buildProducts">Set of build products produced by this node.</param>
		/// <param name="tagNameToFileSet">Mapping from tag names to the set of files they include</param>
		public override async Task ExecuteAsync(JobContext job, HashSet<FileReference> buildProducts, Dictionary<string, HashSet<FileReference>> tagNameToFileSet)
		{
			// Output a warning if the project directory is specified
			if (_parameters.ProjectDir != null)
			{
				Logger.LogWarning("The ProjectDir argument to the TagReceipt parameter is deprecated. This path is now determined automatically from the receipt.");
			}

			// Set the Engine directory
			DirectoryReference engineDir = _parameters.EngineDir ?? Unreal.EngineDirectory;

			// Resolve the input list
			IEnumerable<FileReference> targetFiles = ResolveFilespec(Unreal.RootDirectory, _parameters.Files, tagNameToFileSet);

			// Filter the files
			HashSet<FileReference> files = await Execute(engineDir, targetFiles, _parameters.BuildProducts, _buildProductType, _parameters.RuntimeDependencies, _stagedFileType);

			// Apply the tag to all the matching files
			FindOrAddTagSet(tagNameToFileSet, _parameters.With).UnionWith(files);
		}

		public static Task<HashSet<FileReference>> Execute(DirectoryReference engineDir, IEnumerable<FileReference> targetFiles, bool buildProducts, BuildProductType? buildProductType, bool runtimeDependencies, StagedFileType? stagedFileType = null)
		{
			HashSet<FileReference> files = new HashSet<FileReference>();

			foreach (FileReference targetFile in targetFiles)
			{
				// check all files are .target files
				if (targetFile.GetExtension() != ".target")
				{
					throw new AutomationException("Invalid file passed to TagReceipt task ({0})", targetFile.FullName);
				}

				// Read the receipt
				TargetReceipt receipt;
				if (!TargetReceipt.TryRead(targetFile, engineDir, out receipt))
				{
					Logger.LogWarning("Unable to load file using TagReceipt task ({Arg0})", targetFile.FullName);
					continue;
				}

				if (buildProducts)
				{
					foreach (BuildProduct buildProduct in receipt.BuildProducts)
					{
						if (buildProductType.HasValue && buildProduct.Type != buildProductType.Value)
						{
							continue;
						}
						if (stagedFileType.HasValue && TargetReceipt.GetStageTypeFromBuildProductType(buildProduct) != stagedFileType.Value)
						{
							continue;
						}
						files.Add(buildProduct.Path);
					}
				}

				if (runtimeDependencies)
				{
					foreach (RuntimeDependency runtimeDependency in receipt.RuntimeDependencies)
					{
						// Skip anything that doesn't match the files we want
						if (buildProductType.HasValue)
						{
							continue;
						}
						if (stagedFileType.HasValue && runtimeDependency.Type != stagedFileType.Value)
						{
							continue;
						}

						// Check which files exist, and warn about any that don't. Ignore debug files, as they are frequently excluded for size (eg. UE on GitHub). This matches logic during staging.
						FileReference dependencyPath = runtimeDependency.Path;
						if (FileReference.Exists(dependencyPath))
						{
							files.Add(dependencyPath);
						}
						else if (runtimeDependency.Type != UnrealBuildTool.StagedFileType.DebugNonUFS)
						{
							Logger.LogWarning("File listed as RuntimeDependency in {Arg0} does not exist ({Arg1})", targetFile.FullName, dependencyPath.FullName);
						}
					}
				}
			}

			return Task.FromResult(files);
		}

		/// <summary>
		/// Output this task out to an XML writer.
		/// </summary>
		public override void Write(XmlWriter writer)
		{
			Write(writer, _parameters);
		}

		/// <summary>
		/// Find all the tags which are required by this task
		/// </summary>
		/// <returns>The tag names which are required by this task</returns>
		public override IEnumerable<string> FindConsumedTagNames()
		{
			return FindTagNamesFromFilespec(_parameters.Files);
		}

		/// <summary>
		/// Find all the referenced tags from tasks in this task
		/// </summary>
		/// <returns>The tag names which are produced/modified by this task</returns>
		public override IEnumerable<string> FindProducedTagNames()
		{
			return FindTagNamesFromList(_parameters.With);
		}
	}

	/// <summary>
	/// Extension methods
	/// </summary>
	public static class TaskExtensions
	{
		/// <summary>
		/// Task that tags build products and/or runtime dependencies by reading from *.target files.
		/// </summary>
		public static async Task<FileSet> TagReceiptsAsync(this FileSet files, DirectoryReference engineDir = null, bool buildProducts = false, BuildProductType? buildProductType = null, bool runtimeDependencies = false, StagedFileType? stagedFileType = null)
		{
			HashSet<FileReference> result = await TagReceiptTask.Execute(engineDir ?? Unreal.EngineDirectory, files, buildProducts, buildProductType, runtimeDependencies, stagedFileType);
			return FileSet.FromFiles(Unreal.RootDirectory, result);
		}
	}
}
