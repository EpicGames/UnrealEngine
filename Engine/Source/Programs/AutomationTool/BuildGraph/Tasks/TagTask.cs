// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Threading.Tasks;
using System.Xml;
using EpicGames.Core;
using UnrealBuildBase;

namespace AutomationTool.Tasks
{
	/// <summary>
	/// Parameters for the Tag task.
	/// </summary>
	public class TagTaskParameters
	{
		/// <summary>
		/// Set the base directory to resolve relative paths and patterns against. If set, any absolute patterns (for example, /Engine/Build/...) are taken to be relative to this path. If not, they are taken to be truly absolute.
		/// </summary>
		[TaskParameter(Optional = true)]
		public DirectoryReference BaseDir { get; set; }

		/// <summary>
		/// Set of files to work from, including wildcards and tag names, separated by semicolons. If set, resolved relative to BaseDir, otherwise resolved to the branch root directory.
		/// </summary>
		[TaskParameter(Optional = true, ValidationType = TaskParameterValidationType.FileSpec)]
		public string Files { get; set; }

		/// <summary>
		/// Set of text files to add additional files from. Each file list should have one file per line.
		/// </summary>
		[TaskParameter(Optional = true, ValidationType = TaskParameterValidationType.FileSpec)]
		public string FileLists { get; set; }

		/// <summary>
		/// Patterns to filter the list of files by, including tag names or wildcards. If set, may include patterns that apply to the base directory. If not specified, defaults to all files.
		/// </summary>
		[TaskParameter(Optional = true, ValidationType = TaskParameterValidationType.FileSpec)]
		public string Filter { get; set; }

		/// <summary>
		/// Set of patterns to exclude from the matched list. May include tag names of patterns that apply to the base directory.
		/// </summary>
		[TaskParameter(Optional = true, ValidationType = TaskParameterValidationType.FileSpec)]
		public string Except { get; set; }

		/// <summary>
		/// Name of the tag to apply.
		/// </summary>
		[TaskParameter(ValidationType = TaskParameterValidationType.TagList)]
		public string With { get; set; }
	}

	/// <summary>
	/// Applies a tag to a given set of files. The list of files is found by enumerating the tags and file specifications given by the 'Files' 
	/// parameter. From this list, any files not matched by the 'Filter' parameter are removed, followed by any files matched by the 'Except' parameter.
	/// </summary>
	[TaskElement("Tag", typeof(TagTaskParameters))]
	class TagTask : BgTaskImpl
	{
		readonly TagTaskParameters _parameters;

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="parameters">Parameters to select which files to match</param>
		public TagTask(TagTaskParameters parameters)
		{
			_parameters = parameters;
		}

		/// <summary>
		/// ExecuteAsync the task.
		/// </summary>
		/// <param name="job">Information about the current job</param>
		/// <param name="buildProducts">Set of build products produced by this node.</param>
		/// <param name="tagNameToFileSet">Mapping from tag names to the set of files they include</param>
		public override async Task ExecuteAsync(JobContext job, HashSet<FileReference> buildProducts, Dictionary<string, HashSet<FileReference>> tagNameToFileSet)
		{
			// Get the base directory
			DirectoryReference baseDir = _parameters.BaseDir ?? Unreal.RootDirectory;

			// Parse all the exclude rules
			List<string> excludeRules = ParseRules(baseDir, _parameters.Except ?? "", tagNameToFileSet);

			// Resolve the input list
			HashSet<FileReference> files = new HashSet<FileReference>();
			if (!String.IsNullOrEmpty(_parameters.Files))
			{
				files.UnionWith(ResolveFilespecWithExcludePatterns(baseDir, _parameters.Files, excludeRules, tagNameToFileSet));
			}

			// Resolve the input file lists
			if (!String.IsNullOrEmpty(_parameters.FileLists))
			{
				HashSet<FileReference> fileLists = ResolveFilespec(baseDir, _parameters.FileLists, tagNameToFileSet);
				foreach (FileReference fileList in fileLists)
				{
					if (!FileReference.Exists(fileList))
					{
						throw new AutomationException("Specified file list '{0}' does not exist", fileList);
					}

					string[] lines = await FileReference.ReadAllLinesAsync(fileList);
					foreach (string line in lines)
					{
						string trimLine = line.Trim();
						if (trimLine.Length > 0)
						{
							files.Add(FileReference.Combine(baseDir, trimLine));
						}
					}
				}
			}

			// Limit to matches against the 'Filter' parameter, if set
			if (_parameters.Filter != null)
			{
				FileFilter filter = new FileFilter();
				filter.AddRules(ParseRules(baseDir, _parameters.Filter, tagNameToFileSet));
				files.RemoveWhere(x => !filter.Matches(x.FullName));
			}

			// Apply the tag to all the matching files
			foreach (string tagName in FindTagNamesFromList(_parameters.With))
			{
				FindOrAddTagSet(tagNameToFileSet, tagName).UnionWith(files);
			}
		}

		/// <summary>
		/// Add rules matching a given set of patterns to a file filter. Patterns are added as absolute paths from the root.
		/// </summary>
		/// <param name="baseDir">The base directory for relative paths.</param>
		/// <param name="delimitedPatterns">List of patterns to add, separated by semicolons.</param>
		/// <param name="tagNameToFileSet">Mapping of tag name to a set of files.</param>
		/// <returns>List of rules, suitable for adding to a FileFilter object</returns>
		static List<string> ParseRules(DirectoryReference baseDir, string delimitedPatterns, Dictionary<string, HashSet<FileReference>> tagNameToFileSet)
		{
			// Split up the list of patterns
			List<string> patterns = SplitDelimitedList(delimitedPatterns);

			// Parse them into a list of rules
			List<string> rules = new List<string>();
			foreach (string pattern in patterns)
			{
				if (pattern.StartsWith("#", StringComparison.Ordinal))
				{
					// Add the files in a specific set to the filter
					HashSet<FileReference> files = FindOrAddTagSet(tagNameToFileSet, pattern);
					foreach (FileReference file in files)
					{
						rules.Add(file.FullName);
					}
				}
				else
				{
					// Parse a wildcard filter
					if (pattern.StartsWith("...", StringComparison.Ordinal))
					{
						rules.Add(pattern);
					}
					else if (!pattern.Contains('/', StringComparison.Ordinal))
					{
						rules.Add(".../" + pattern);
					}
					else if (!pattern.StartsWith("/", StringComparison.Ordinal))
					{
						rules.Add(baseDir.FullName + "/" + pattern);
					}
					else
					{
						rules.Add(baseDir.FullName + pattern);
					}
				}
			}
			return rules;
		}

		/// <summary>
		/// Output this task out to an XML writer.
		/// </summary>
		public override void Write(XmlWriter writer)
		{
			Write(writer, _parameters);
		}

		/// <summary>
		/// Find all the tags which are modified by this task
		/// </summary>
		/// <returns>The tag names which are read by this task</returns>
		public override IEnumerable<string> FindConsumedTagNames()
		{
			foreach (string tagName in FindTagNamesFromFilespec(_parameters.Files))
			{
				yield return tagName;
			}

			if (!String.IsNullOrEmpty(_parameters.Filter))
			{
				foreach (string tagName in FindTagNamesFromFilespec(_parameters.Filter))
				{
					yield return tagName;
				}
			}

			if (!String.IsNullOrEmpty(_parameters.Except))
			{
				foreach (string tagName in FindTagNamesFromFilespec(_parameters.Except))
				{
					yield return tagName;
				}
			}
		}

		/// <summary>
		/// Find all the referenced tags from tasks in this task
		/// </summary>
		/// <returns>The tag names which are modified by this task</returns>
		public override IEnumerable<string> FindProducedTagNames()
		{
			return FindTagNamesFromList(_parameters.With);
		}
	}
}
