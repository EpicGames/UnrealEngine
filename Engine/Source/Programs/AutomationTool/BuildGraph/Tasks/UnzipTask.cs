// Copyright Epic Games, Inc. All Rights Reserved.

using System.Collections.Generic;
using System.Linq;
using System.Threading.Tasks;
using System.Xml;
using EpicGames.Core;
using UnrealBuildBase;

namespace AutomationTool.Tasks
{
	/// <summary>
	/// Parameters for a zip task
	/// </summary>
	public class UnzipTaskParameters
	{
		/// <summary>
		/// Path to the zip file to extract.
		/// </summary>
		[TaskParameter(ValidationType = TaskParameterValidationType.FileSpec)]
		public string ZipFile { get; set; }

		/// <summary>
		/// Output directory for the extracted files.
		/// </summary>
		[TaskParameter]
		public DirectoryReference ToDir { get; set; }

		/// <summary>
		/// Whether or not to use the legacy unzip code.
		/// </summary>
		[TaskParameter(Optional = true)]
		public bool UseLegacyUnzip { get; set; } = false;

		/// <summary>
		/// Whether or not to overwrite files during unzip.
		/// </summary>
		[TaskParameter(Optional = true)]
		public bool OverwriteFiles { get; set; } = true;

		/// <summary>
		/// Tag to be applied to the extracted files.
		/// </summary>
		[TaskParameter(Optional = true, ValidationType = TaskParameterValidationType.TagList)]
		public string Tag { get; set; }
	}

	/// <summary>
	/// Extract files from a zip archive.
	/// </summary>
	[TaskElement("Unzip", typeof(UnzipTaskParameters))]
	public class UnzipTask : BgTaskImpl
	{
		readonly UnzipTaskParameters _parameters;

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="parameters">Parameters for this task</param>
		public UnzipTask(UnzipTaskParameters parameters)
		{
			_parameters = parameters;
		}

		/// <summary>
		/// ExecuteAsync the task.
		/// </summary>
		/// <param name="job">Information about the current job</param>
		/// <param name="buildProducts">Set of build products produced by this node.</param>
		/// <param name="tagNameToFileSet">Mapping from tag names \to the set of files they include</param>
		public override Task ExecuteAsync(JobContext job, HashSet<FileReference> buildProducts, Dictionary<string, HashSet<FileReference>> tagNameToFileSet)
		{
			DirectoryReference toDir = _parameters.ToDir;

			// Find all the zip files
			IEnumerable<FileReference> zipFiles = ResolveFilespec(Unreal.RootDirectory, _parameters.ZipFile, tagNameToFileSet);

			// Extract the files
			HashSet<FileReference> outputFiles = new HashSet<FileReference>();
			foreach (FileReference zipFile in zipFiles)
			{
				if (_parameters.UseLegacyUnzip)
				{
					outputFiles.UnionWith(CommandUtils.LegacyUnzipFiles(zipFile.FullName, toDir.FullName, _parameters.OverwriteFiles).Select(x => new FileReference(x)));
				}
				else
				{
					outputFiles.UnionWith(CommandUtils.UnzipFiles(zipFile, toDir, _parameters.OverwriteFiles));
				}
			}

			// Apply the optional tag to the produced archive
			foreach (string tagName in FindTagNamesFromList(_parameters.Tag))
			{
				FindOrAddTagSet(tagNameToFileSet, tagName).UnionWith(outputFiles);
			}

			// Add the archive to the set of build products
			buildProducts.UnionWith(outputFiles);
			return Task.CompletedTask;
		}

		/// <summary>
		/// Output this task out to an XML writer.
		/// </summary>
		public override void Write(XmlWriter writer)
		{
			Write(writer, _parameters);
		}

		/// <summary>
		/// Find all the tags which are used as inputs to this task
		/// </summary>
		/// <returns>The tag names which are read by this task</returns>
		public override IEnumerable<string> FindConsumedTagNames()
		{
			return FindTagNamesFromFilespec(_parameters.ZipFile);
		}

		/// <summary>
		/// Find all the tags which are modified by this task
		/// </summary>
		/// <returns>The tag names which are modified by this task</returns>
		public override IEnumerable<string> FindProducedTagNames()
		{
			return FindTagNamesFromList(_parameters.Tag);
		}
	}
}
