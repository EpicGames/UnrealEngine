// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.IO;
using System.Linq;
using System.Net;
using System.Net.Http;
using System.Net.Http.Headers;
using System.Text;
using System.Threading.Tasks;
using System.Xml;
using EpicGames.Compression;
using EpicGames.Core;
using EpicGames.Serialization;
using Microsoft.Extensions.Logging;
using UnrealBuildBase;

#nullable enable

namespace AutomationTool.Tasks
{
	/// <summary>
	/// Parameters for a task that uploads symbols to the cloud
	/// </summary>
	public class UploadCloudSymbolsTaskParameters
	{
		/// <summary>
		/// List of output files. PDBs will be extracted from this list.
		/// </summary>
		[TaskParameter(ValidationType = TaskParameterValidationType.FileSpec)]
		public string Files { get; set; } = null!;

		/// <summary>
		/// The cloud host to which symbols should be uploaded
		/// </summary>
		[TaskParameter(Optional = true)]
		public string? Host { get; set; }

		/// <summary>
		/// The access token to use
		/// </summary>
		[TaskParameter(Optional = true)]
		public string? AccessToken { get; set; }

		/// <summary>
		/// The destination namespace to which symbols should be uploaded
		/// </summary>
		[TaskParameter]
		public string? Namespace { get; set; } = null!;
	}

	/// <summary>
	/// Task that strips symbols from a set of files.
	/// </summary>
	[TaskElement("UploadCloudSymbols", typeof(UploadCloudSymbolsTaskParameters))]
	public class UploadCloudSymbolsTask : BgTaskImpl
	{
		/// <summary>
		/// Parameters for this task
		/// </summary>
		readonly UploadCloudSymbolsTaskParameters _parameters;

		/// <summary>
		/// Construct a spawn task
		/// </summary>
		/// <param name="parameters">Parameters for the task</param>
		public UploadCloudSymbolsTask(UploadCloudSymbolsTaskParameters parameters)
		{
			_parameters = parameters;
		}

		/// <summary>
		/// ExecuteAsync the task.
		/// </summary>
		/// <param name="job">Information about the current job</param>
		/// <param name="buildProducts">Set of build products produced by this node.</param>
		/// <param name="tagNameToFileSet">Mapping from tag names to the set of files they include</param>
		public override async Task ExecuteAsync(JobContext job, HashSet<FileReference> buildProducts, Dictionary<string, HashSet<FileReference>> tagNameToFileSet)
		{
			// Find the matching files
			HashSet<FileReference> files = ResolveFilespec(Unreal.RootDirectory, _parameters.Files, tagNameToFileSet);
			
			using HttpClient httpClient = new HttpClient();
			httpClient.BaseAddress = GetCloudHost();
			httpClient.DefaultRequestHeaders.Accept.Clear();
			httpClient.DefaultRequestHeaders.Accept.Add(new MediaTypeWithQualityHeaderValue(CustomMediaTypeNames.UnrealCompactBinary));
			httpClient.DefaultRequestHeaders.Authorization = new AuthenticationHeaderValue("Bearer", GetAccessToken());
			
			foreach (FileReference file in files)
			{
				string moduleName = file.GetFileName();

				if (!Path.GetExtension(moduleName).Equals(".pdb", StringComparison.OrdinalIgnoreCase))
				{
					Logger.LogInformation("Skipping file {FileName}", moduleName);
					continue;
				}
				
				Logger.LogInformation("Uploading module {ModuleName}", moduleName);

#pragma warning disable CA2000 // BinaryReader leaves the Stream open, so this `using` statement will dispose it 
				await using Stream fileStream = File.OpenRead(file.FullName);
#pragma warning restore CA2000
				
				List<ChunkMetadata> chunks = await ChunkAsync(fileStream);
				Dictionary<IoHash, ChunkMetadata> chunksByHash = chunks.ToDictionary(c => c.Hash, c => c);

				NeedsResponse putModuleResponse = await PutModuleAsync(httpClient, moduleName, fileStream, chunks);
				
				foreach (IoHash ioHash in putModuleResponse.Needs)
				{
					if (chunksByHash.TryGetValue(ioHash, out ChunkMetadata chunkMetadata))
					{
						Logger.LogInformation("Uploading hash {IoHash} for module {ModuleName} with size {Size}", chunkMetadata.Hash, moduleName, chunkMetadata.Length);
						
						byte[] chunk = new byte[chunkMetadata.Length];
						fileStream.Seek(chunkMetadata.Offset, SeekOrigin.Begin);
						await fileStream.ReadExactlyAsync(chunk, 0, chunk.Length);
						
						await PutBlobAsync(httpClient, moduleName, chunkMetadata.Hash, chunk);
					}
					else
					{
						throw new AutomationException("Needed chunk with hash {0} was not found in file {1}",  ioHash, file.FullName);
					}
				}
			}
		}

		private Uri GetCloudHost()
		{
			string publishHostEnvVar = "UE-CloudPublishHost";
			
			if (!OperatingSystem.IsWindows())
			{
				publishHostEnvVar = publishHostEnvVar.Replace("-", "_", StringComparison.OrdinalIgnoreCase);
			}

			string result;

			if (!String.IsNullOrEmpty(_parameters.Host))
			{
				result = _parameters.Host;
			}
			else if (!String.IsNullOrEmpty(Environment.GetEnvironmentVariable(publishHostEnvVar)))
			{
				result = Environment.GetEnvironmentVariable(publishHostEnvVar)!;
			}
			else
			{
				throw new AutomationException($"Missing {publishHostEnvVar} environment variable; unable to determine cloud host.");
			}

			return new Uri(result);
		}

		private string GetAccessToken()
		{
			string accessTokenEnvVar = "UE-CloudDataCacheAccessToken";
			
			if (!OperatingSystem.IsWindows())
			{
				accessTokenEnvVar = accessTokenEnvVar.Replace("-", "_", StringComparison.OrdinalIgnoreCase);
			}

			string result;
			
			if (!String.IsNullOrEmpty(_parameters.AccessToken))
			{
				result = _parameters.AccessToken;
			}
			else if (!String.IsNullOrEmpty(Environment.GetEnvironmentVariable(accessTokenEnvVar)))
			{
				result = Environment.GetEnvironmentVariable(accessTokenEnvVar)!;
			}
			else
			{
				throw new AutomationException($"Missing {accessTokenEnvVar} environment variable; unable to find access token to use.");
			}

			return result;
		}

		private readonly struct ChunkMetadata(IoHash hash, long offset, long length)
		{
			public IoHash Hash { get; } = hash;
			public long Offset { get; } = offset;
			public long Length { get; } = length;
		}

		private static async Task<List<ChunkMetadata>> ChunkAsync(Stream stream, int fixedChunkSize = 64 * 1024 * 1024)
		{
			stream.Seek(0, SeekOrigin.Begin);
			
			long minLastChunkSize = Math.Min(128 * 1024, fixedChunkSize / 32);
			byte[] buffer = new byte[fixedChunkSize + minLastChunkSize];
			
			List<ChunkMetadata> results = new List<ChunkMetadata>();

			long offset = 0;

			while (offset < stream.Length)
			{
				long bytesLeft = stream.Length - offset;
				long chunkSize = Math.Min(bytesLeft, fixedChunkSize);

				if (bytesLeft - chunkSize < minLastChunkSize)
				{
					chunkSize = bytesLeft;
				}

				await stream.ReadExactlyAsync(buffer, 0, (int)chunkSize);
				
				IoHash hash = IoHash.Compute(new ReadOnlySpan<byte>(buffer, 0, (int)chunkSize));
				results.Add(new ChunkMetadata(hash, offset, chunkSize));
				
				offset += chunkSize;
			}

			return results;
		}

		private async Task<NeedsResponse> PutModuleAsync(HttpClient httpClient, string moduleName, Stream stream, List<ChunkMetadata> chunks)
		{
			(string pdbIdentifier, int pdbAge) = ExtractModuleInformation(moduleName, stream);
			byte[] putSymbolsPayload = CreatePutSymbolsPayload(chunks, moduleName, pdbIdentifier, pdbAge, stream.Length);
    
			using HttpResponseMessage response = await CallWithRetriesAsync(async () =>
			{
				using ByteArrayContent requestContent = new ByteArrayContent(putSymbolsPayload);
				requestContent.Headers.ContentType = new MediaTypeHeaderValue(CustomMediaTypeNames.UnrealCompactBinary);
				return await httpClient.PutAsync(new Uri($"api/v1/symbols/{_parameters.Namespace}/{moduleName}", UriKind.Relative), requestContent);
			});
    
			return await response.Content.ReadAsCompactBinaryAsync<NeedsResponse>();
		}

		private async Task PutBlobAsync(HttpClient httpClient, string moduleName, IoHash hash, byte[] chunk)
		{
			using MemoryStream compressedStream = new MemoryStream();
			CompressedBuffer.CompressContent(compressedStream, OodleCompressorType.Mermaid, OodleCompressionLevel.VeryFast, chunk);
			byte[] compressedChunk = compressedStream.ToArray();
			
			using HttpResponseMessage response = await CallWithRetriesAsync(async () =>
			{
				using ByteArrayContent requestContent = new ByteArrayContent(compressedChunk);
				requestContent.Headers.ContentType = new MediaTypeHeaderValue(CustomMediaTypeNames.UnrealCompressedBuffer);
				return await httpClient.PutAsync(new Uri($"api/v1/symbols/{_parameters.Namespace}/{moduleName}/blobs/{hash}", UriKind.Relative), requestContent);
			});
		}

		private static async Task<HttpResponseMessage> CallWithRetriesAsync(Func<Task<HttpResponseMessage>> func, int maxAttempts = 10, double initialBackoffSeconds = 0.5, double maxBackoffSeconds = 60.0)
		{
			TimeSpan backoff = TimeSpan.FromSeconds(initialBackoffSeconds);
			TimeSpan maxBackoff = TimeSpan.FromSeconds(maxBackoffSeconds);
			
			for (int attempt = 0; attempt < maxAttempts; attempt++)
			{
				HttpResponseMessage? result = null;

				try
				{
					result = await func();
					result.EnsureSuccessStatusCode();
					return result;
				}
				catch (HttpRequestException e)
				{
					result?.Dispose();
					
					switch (e.StatusCode)
					{
						case HttpStatusCode.RequestTimeout:
						case HttpStatusCode.TooManyRequests:
						case HttpStatusCode.InternalServerError:
						case HttpStatusCode.BadGateway:
						case HttpStatusCode.ServiceUnavailable:
						case HttpStatusCode.GatewayTimeout:
							break;
						default:
							throw new AutomationException("Response with status code {0} cannot be retried", e.StatusCode ?? 0);
					}
				}
				
				if (attempt < maxAttempts - 1)
				{	
					await Task.Delay(backoff);

					backoff *= 2;

					if (backoff > maxBackoff)
					{
						backoff = maxBackoff;
					}
				}
			}

			throw new AutomationException("Exhausted retry attempts");
		}

		private static byte[] CreatePutSymbolsPayload(List<ChunkMetadata> pdbChunks, string moduleName, string pdbIdentifier, int pdbAge, long pdbSize)
		{
			CbWriter writer = new CbWriter();
			writer.BeginObject();
			
			writer.BeginArray("pdbChunks");
			
			foreach (ChunkMetadata chunkMetadata in pdbChunks)
			{
				writer.WriteBinaryAttachmentValue(chunkMetadata.Hash);
			}
			
			writer.EndArray();
			
			writer.WriteString("moduleName", moduleName);
			writer.WriteString("pdbIdentifier", pdbIdentifier);
			writer.WriteInteger("pdbAge", pdbAge);
			writer.WriteInteger("pdbSize", pdbSize);
							
			writer.EndObject();
		
			byte[] putObjectRequestPayload = writer.ToByteArray();
			return putObjectRequestPayload;
		}

		private static (string pdbGuid, int pdbAge) ExtractModuleInformation(string moduleName, Stream stream)
		{
			string extension = Path.GetExtension(moduleName);
			switch (extension)
			{
				case ".pdb":
					return ExtractModuleInformationPdb(stream);
				default:
					throw new NotImplementedException($"Unhandled extension type: {extension}");
			}
		}

		private static (string pdbGuid, int pdbAge) ExtractModuleInformationPdb(Stream stream)
		{
			// the pdb format is somewhat documented here: 
			// https://llvm.org/docs/PDB/MsfFile.html
			// as well as here
			// https://github.com/microsoft/microsoft-pdb

			stream.Seek(0, SeekOrigin.Begin);
			using BinaryReader reader = new BinaryReader(stream, Encoding.ASCII, leaveOpen: true);
			
			// extract magic
			const string MagicHeader = "Microsoft C/C++ MSF 7.00\r\n\u001aDS\0\0\0";
			byte[] magicBytes = reader.ReadBytes(MagicHeader.Length);
			string magicString = Encoding.ASCII.GetString(magicBytes);
			if (!String.Equals(magicString, MagicHeader, StringComparison.OrdinalIgnoreCase))
			{
				throw new Exception("Failed to find expected magic for pdb header");
			}

			// parse the super block
			int blockSize = reader.ReadInt32();
			int freeBlockMapIndex = reader.ReadInt32();
			int blockCount = reader.ReadInt32();
			int directoryStreamLength = reader.ReadInt32();
			int _ = reader.ReadInt32();
			int hintBlock = reader.ReadInt32();

			switch (blockSize)
			{
				case 512:
				case 1024:
				case 2048:
				case 4096:
					break;
				default:
					throw new Exception($"Unsupported block size: {blockSize}");
			}

			if (freeBlockMapIndex != 1 && freeBlockMapIndex != 2)
			{
				throw new Exception("Unexpected free block map index");
			}

			if (blockCount * (long)blockSize != reader.BaseStream.Length)
			{
				throw new Exception("Unexpected pdb stream length");
			}

			// build the stream directory
			long freeBlockOffset = (long)blockSize * hintBlock;
			reader.BaseStream.Seek(freeBlockOffset, SeekOrigin.Begin);
			
			int directoryStreamBlockCount = (int)Math.Ceiling((double)directoryStreamLength / blockSize);
			int[] directoryBlocks = new int[directoryStreamBlockCount];
			for (int i = 0; i < directoryStreamBlockCount; ++i)
			{
				directoryBlocks[i] = reader.ReadInt32();
			}
			
			byte[] streamDirectory = new byte[directoryStreamBlockCount * blockSize];
			int streamDirectoryOffset = 0;

			foreach (int directoryBlock in directoryBlocks)
			{
				reader.BaseStream.Seek((long)blockSize * directoryBlock, SeekOrigin.Begin);
				streamDirectoryOffset += reader.Read(streamDirectory, streamDirectoryOffset, blockSize);
			}
			
			// read the stream directory
			using Stream streamDirectoryStream = new MemoryStream(streamDirectory);
			using BinaryReader streamDirectoryReader = new BinaryReader(streamDirectoryStream);
			int streamCount = streamDirectoryReader.ReadInt32();

			int[] streamLengths = new int[streamCount];
			for (int i = 0; i < streamCount; ++i)
			{
				streamLengths[i] = streamDirectoryReader.ReadInt32();
			}

			// calculate the blocks for each size
			List<List<int>> blocks = new List<List<int>>();
			foreach (int streamLength in streamLengths)
			{
				List<int> blocklist = new List<int>();

				if (streamLength > 0)
				{
					int streamBlockCount = (int)Math.Ceiling((double)streamLength / blockSize);

					for (int j = 0; j < streamBlockCount; ++j)
					{
						blocklist.Add(streamDirectoryReader.ReadInt32());
					}
				}

				blocks.Add(blocklist);
			}

			int? pdbVersion = null;
			int? pdbSignature = null;
			int? pdbAge = null;
			Guid? pdbGuid = null;

			bool infoFound = false;
			// start reading each stream
			for (int i = 0; i < streamCount; ++i)
			{
				if (infoFound)
				{
					break;
				}

				int streamLength = streamLengths[i];

				if (streamLength <= 0)
				{
					continue;
				}
				
				byte[] streamBuffer = new byte[streamLength];
				int destinationIndex = 0;

				List<int> streamBlocks = blocks[i];
				foreach (int streamBlock in streamBlocks)
				{
					reader.BaseStream.Seek(streamBlock * (long)blockSize, SeekOrigin.Begin);
					destinationIndex += reader.Read(streamBuffer, destinationIndex, Math.Min(blockSize, streamBuffer.Length - destinationIndex));
				}

				// all blocks are combined into a contiguous stream
				MemoryStream ms = new MemoryStream(streamBuffer);
				using BinaryReader streamReader = new BinaryReader(ms);
				switch (i)
				{
					case 1:
						// PdbInfoStream
						pdbVersion = streamReader.ReadInt32();
						pdbSignature = streamReader.ReadInt32();
						pdbAge = streamReader.ReadInt32();

						pdbGuid = new Guid(streamReader.ReadBytes(16));

						infoFound = true;
						break;

					case 3:
						// DBIStream
						break;
					default:
						break;
				}
			}

			if (pdbGuid == null || pdbAge == null)
			{
				throw new Exception("No PDBInfoStream found, was this really a pdb?");
			}

			return (pdbGuid.Value.ToString("N").ToUpperInvariant(), pdbAge.Value);
		}

		/// <summary>
		/// Output this task out to an XML writer.
		/// </summary>
		public override void Write(XmlWriter writer)
		{
			Write(writer, _parameters);
		}

		/// <summary>
		/// Find all the tags which are used as inputs to this task
		/// </summary>
		/// <returns>The tag names which are read by this task</returns>
		public override IEnumerable<string> FindConsumedTagNames()
		{
			return FindTagNamesFromFilespec(_parameters.Files);
		}

		/// <summary>
		/// Find all the tags which are modified by this task
		/// </summary>
		/// <returns>The tag names which are modified by this task</returns>
		public override IEnumerable<string> FindProducedTagNames()
		{
			yield break;
		}

		class NeedsResponse
		{
			public NeedsResponse()
			{
				Needs = new List<IoHash>();
			}

			public NeedsResponse(List<IoHash> needs)
			{
				Needs = needs;
			}

			[CbField("needs")]
			public List<IoHash> Needs { get; set; }
		}

		static class CustomMediaTypeNames
		{
			/// <summary>
			/// Media type for compact binary
			/// </summary>
			public const string UnrealCompactBinary = "application/x-ue-cb";

			/// <summary>
			/// Media type for compressed buffers
			/// </summary>
			public const string UnrealCompressedBuffer = "application/x-ue-comp";
		}
	}
}
