// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Threading.Tasks;
using System.Xml;
using EpicGames.Core;

namespace AutomationTool.Tasks
{
	/// <summary>
	/// Parameters for a wait task
	/// </summary>
	public class WaitTaskParameters
	{
		/// <summary>
		/// Number of seconds to wait.
		/// </summary>
		[TaskParameter]
		public int Seconds { get; set; }
	}

	/// <summary>
	/// Waits a defined number of seconds.
	/// </summary>
	[TaskElement("Wait", typeof(WaitTaskParameters))]
	public class WaitTask : BgTaskImpl
	{
		readonly WaitTaskParameters _parameters;

		/// <summary>
		/// Construct a wait task
		/// </summary>
		/// <param name="parameters">Parameters for the task</param>
		public WaitTask(WaitTaskParameters parameters)
		{
			_parameters = parameters;
		}

		/// <summary>
		/// ExecuteAsync the task.
		/// </summary>
		/// <param name="job">Information about the current job</param>
		/// <param name="buildProducts">Set of build products produced by this node.</param>
		/// <param name="tagNameToFileSet">Mapping from tag names to the set of files they include</param>
		public override async Task ExecuteAsync(JobContext job, HashSet<FileReference> buildProducts, Dictionary<string, HashSet<FileReference>> tagNameToFileSet)
		{
			await Task.Delay(TimeSpan.FromSeconds(_parameters.Seconds));
		}

		/// <summary>
		/// Output this task out to an XML writer.
		/// </summary>
		public override void Write(XmlWriter writer)
		{
			Write(writer, _parameters);
		}

		/// <summary>
		/// Find all the tags which are used as inputs to this task
		/// </summary>
		/// <returns>The tag names which are read by this task</returns>
		public override IEnumerable<string> FindConsumedTagNames()
		{
			yield break;
		}

		/// <summary>
		/// Find all the tags which are modified by this task
		/// </summary>
		/// <returns>The tag names which are modified by this task</returns>
		public override IEnumerable<string> FindProducedTagNames()
		{
			yield break;
		}
	}
}
