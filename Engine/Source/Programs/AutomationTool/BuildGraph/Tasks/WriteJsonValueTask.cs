// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Text.Json;
using System.Text.Json.Nodes;
using System.Threading.Tasks;
using System.Xml;
using EpicGames.Core;
using UnrealBuildBase;

#nullable enable

namespace AutomationTool.Tasks
{
	using JsonObject = System.Text.Json.Nodes.JsonObject;

	/// <summary>
	/// Parameters for <see cref="WriteJsonValueTask"/> task
	/// </summary>
	public class WriteJsonValueTaskParameters
	{
		/// <summary>
		/// Json file(s) which will be modified
		/// </summary>
		[TaskParameter(ValidationType = TaskParameterValidationType.FileSpec)]
		public string File { get; set; } = null!;

		/// <summary>
		/// Json element to set in each file. Syntax for this string is a limited subset of JsonPath notation, and may support object properties and
		/// array indices. Any array indices which are omitted or out of range will add a new element to the array (eg. '$.foo.bar[]' will add
		/// an element to the 'bar' array in the 'foo' object).
		/// </summary>
		[TaskParameter(ValidationType = TaskParameterValidationType.Default)]
		public string Key { get; set; } = null!;

		/// <summary>
		/// New value to set. May be any value JSON value (string, array, object, number, boolean or null).
		/// </summary>
		[TaskParameter(ValidationType = TaskParameterValidationType.Default)]
		public string Value { get; set; } = null!;
	}

	/// <summary>
	/// Modifies json files by setting a value specified in the key path
	/// </summary>
	[TaskElement("WriteJsonValue", typeof(WriteJsonValueTaskParameters))]
	public class WriteJsonValueTask : BgTaskImpl
	{
		readonly WriteJsonValueTaskParameters _parameters;

		/// <summary>
		/// Create a new ModifyJsonValue.
		/// </summary>
		/// <param name="parameters">Parameters for this task.</param>
		public WriteJsonValueTask(WriteJsonValueTaskParameters parameters)
		{
			_parameters = parameters;
		}

		/// <summary>
		/// Placeholder comment
		/// </summary>
		public override async Task ExecuteAsync(JobContext job, HashSet<FileReference> buildProducts, Dictionary<string, HashSet<FileReference>> tagNameToFileSet)
		{
			HashSet<FileReference> files = ResolveFilespec(Unreal.RootDirectory, _parameters.File, tagNameToFileSet);

			JsonNode? valueNode;
			try
			{
				valueNode = String.IsNullOrEmpty(_parameters.Value) ? null : JsonNode.Parse(_parameters.Value);
			}
			catch (Exception ex)
			{
				throw new AutomationException(ex, $"Unable to parse '{_parameters.Value}': {ex.Message}");
			}

			foreach (FileReference jsonFile in files)
			{
				string jsonText = FileReference.Exists(jsonFile) ? await FileReference.ReadAllTextAsync(jsonFile) : "{}";

				if (!_parameters.Key.StartsWith("$", StringComparison.Ordinal))
				{
					throw new AutomationException("Key must be in JsonPath format (eg. $.Foo.Bar[123])");
				}

				JsonNode? rootNode;
				try
				{
					rootNode = JsonNode.Parse(jsonText, documentOptions: new JsonDocumentOptions { CommentHandling = JsonCommentHandling.Skip });
				}
				catch (Exception ex)
				{
					throw new AutomationException($"Error parsing {jsonFile}: {ex.Message}");
				}
				rootNode = MergeValue(_parameters.Key, 1, rootNode, valueNode);

				string newJsonText = rootNode?.ToJsonString(new JsonSerializerOptions { WriteIndented = true }) ?? String.Empty;

				DirectoryReference.CreateDirectory(jsonFile.Directory);
				await FileReference.WriteAllTextAsync(jsonFile, newJsonText);
			}
		}

		static JsonNode? MergeValue(string key, int minIdx, JsonNode? prevValue, JsonNode? value)
		{
			if (minIdx == key.Length)
			{
				return value;
			}

			// Find the length of the next token
			int maxIdx = minIdx + 1;
			while (maxIdx < key.Length && key[maxIdx] != '[' && key[maxIdx] != '.')
			{
				maxIdx++;
			}

			// Handle different types of element
			if (key[minIdx] == '.')
			{
				JsonObject? obj = prevValue as JsonObject;
				if (obj != null)
				{
					obj = obj.Deserialize<JsonObject>(); // Clone so we can reattach
				}
				obj ??= new JsonObject();

				string propertyName = key.Substring(minIdx + 1, maxIdx - (minIdx + 1));

				JsonNode? nextNode;
				obj.TryGetPropertyValue(propertyName, out nextNode);
				obj[propertyName] = MergeValue(key, maxIdx, nextNode, value);

				return obj;
			}
			else if (key[minIdx] == '[')
			{
				if (key[maxIdx - 1] != ']')
				{
					throw new AutomationException("Missing ']' in array subscript in Json path expression '{Key}'");
				}

				string indexStr = key.Substring(minIdx + 1, (maxIdx - 1) - (minIdx + 1)).Trim();

				int index = Int32.MaxValue;
				if (indexStr.Length > 0)
				{
					index = Int32.Parse(indexStr);
				}

				JsonArray? array = prevValue as JsonArray;
				if (array != null)
				{
					array = array.Deserialize<JsonArray>();
				}
				array ??= new JsonArray();

				if (index < array.Count)
				{
					array[index] = MergeValue(key, maxIdx, array[index], value);
				}
				else
				{
					array.Add(MergeValue(key, maxIdx, null, value));
				}

				return array;
			}
			else
			{
				throw new AutomationException($"Unable to parse JSON path after '{key}'");
			}
		}

		/// <summary>
		/// Placeholder comment
		/// </summary>
		public override void Write(XmlWriter writer)
		{
			Write(writer, _parameters);
		}

		/// <summary>
		/// Placeholder comment
		/// </summary>
		public override IEnumerable<string> FindConsumedTagNames()
		{
			foreach (string tagName in FindTagNamesFromFilespec(_parameters.File))
			{
				yield return tagName;
			}
		}

		/// <summary>
		/// Placeholder comment
		/// </summary>
		public override IEnumerable<string> FindProducedTagNames()
		{
			yield break;
		}
	}
}
