// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Linq;
using System.Threading.Tasks;
using System.Xml;
using EpicGames.Core;
using Microsoft.Extensions.Logging;
using UnrealBuildBase;

namespace AutomationTool.Tasks
{
	/// <summary>
	/// Parameters for a <see cref="WriteTextFileTask"/>.
	/// </summary>
	public class WriteTextFileTaskParameters
	{
		/// <summary>
		/// Path to the file to write.
		/// </summary>
		[TaskParameter]
		public FileReference File { get; set; }

		/// <summary>
		/// Optional, whether or not to append to the file rather than overwrite.
		/// </summary>
		[TaskParameter(Optional = true)]
		public bool Append { get; set; }

		/// <summary>
		/// The text to write to the file.
		/// </summary>
		[TaskParameter(Optional = true)]
		public string Text { get; set; }

		/// <summary>
		/// If specified, causes the given list of files to be printed after the given message.
		/// </summary>
		[TaskParameter(Optional = true, ValidationType = TaskParameterValidationType.FileSpec)]
		public string Files { get; set; }

		/// <summary>
		/// Tag to be applied to build products of this task.
		/// </summary>
		[TaskParameter(Optional = true, ValidationType = TaskParameterValidationType.TagList)]
		public string Tag { get; set; }
	}

	/// <summary>
	/// Writes text to a file.
	/// </summary>
	[TaskElement("WriteTextFile", typeof(WriteTextFileTaskParameters))]
	public class WriteTextFileTask : BgTaskImpl
	{
		/// <summary>
		/// Parameters for this task.
		/// </summary>
		readonly WriteTextFileTaskParameters _parameters;

		/// <summary>
		/// Constructor.
		/// </summary>
		/// <param name="parameters">Parameters for this task.</param>
		public WriteTextFileTask(WriteTextFileTaskParameters parameters)
		{
			_parameters = parameters;
		}

		/// <summary>
		/// ExecuteAsync the task.
		/// </summary>
		/// <param name="job">Information about the current job.</param>
		/// <param name="buildProducts">Set of build products produced by this node.</param>
		/// <param name="tagNameToFileSet">Mapping from tag names to the set of files they include.</param>
		public override async Task ExecuteAsync(JobContext job, HashSet<FileReference> buildProducts, Dictionary<string, HashSet<FileReference>> tagNameToFileSet)
		{
			string fileText = _parameters.Text;

			// If any files or tagsets are provided, add them to the text output.
			if (!String.IsNullOrEmpty(_parameters.Files))
			{
				if (!String.IsNullOrWhiteSpace(fileText))
				{
					fileText += Environment.NewLine;
				}

				HashSet<FileReference> files = ResolveFilespec(Unreal.RootDirectory, _parameters.Files, tagNameToFileSet);
				if (files.Any())
				{
					fileText += String.Join(Environment.NewLine, files.Select(f => f.FullName));
				}
			}

			// Make sure output folder exists.
			if (!DirectoryReference.Exists(_parameters.File.Directory))
			{
				DirectoryReference.CreateDirectory(_parameters.File.Directory);
			}

			if (_parameters.Append)
			{
				Logger.LogInformation("{Text}", String.Format("Appending text to file '{0}': {1}", _parameters.File, fileText));
				await FileReference.AppendAllTextAsync(_parameters.File, Environment.NewLine + fileText);
			}
			else
			{
				Logger.LogInformation("{Text}", String.Format("Writing text to file '{0}': {1}", _parameters.File, fileText));
				await FileReference.WriteAllTextAsync(_parameters.File, fileText);
			}

			// Apply the optional tag to the build products
			foreach (string tagName in FindTagNamesFromList(_parameters.Tag))
			{
				FindOrAddTagSet(tagNameToFileSet, tagName).Add(_parameters.File);
			}

			// Add them to the set of build products
			buildProducts.Add(_parameters.File);
		}

		/// <summary>
		/// Output this task out to an XML writer.
		/// </summary>
		public override void Write(XmlWriter writer)
		{
			Write(writer, _parameters);
		}

		/// <summary>
		/// Find all the tags which are used as inputs to this task
		/// </summary>
		/// <returns>The tag names which are read by this task</returns>
		public override IEnumerable<string> FindConsumedTagNames()
		{
			foreach (string tagName in FindTagNamesFromFilespec(_parameters.Files))
			{
				yield return tagName;
			}
		}

		/// <summary>
		/// Find all the tags which are modified by this task
		/// </summary>
		/// <returns>The tag names which are modified by this task</returns>
		public override IEnumerable<string> FindProducedTagNames()
		{
			return FindTagNamesFromList(_parameters.Tag);
		}
	}
}
