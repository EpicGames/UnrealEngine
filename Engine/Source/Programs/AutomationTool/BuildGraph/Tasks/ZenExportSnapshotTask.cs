// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Diagnostics.CodeAnalysis;
using System.IO;
using System.Linq;
using System.Runtime.InteropServices;
using System.Text;
using System.Text.Json;
using System.Text.Json.Serialization;
using System.Threading;
using System.Threading.Tasks;
using System.Xml;
using EpicGames.Core;
using EpicGames.ProjectStore;
using EpicGames.Serialization;
using Microsoft.Extensions.Logging;
using UnrealBuildTool;

namespace AutomationTool.Tasks
{
	/// <summary>
	/// Enumeration of different storage options for snapshots.
	/// </summary>
	public enum SnapshotStorageType
	{
		/// <summary>
		/// A reserved non-valid storage type for snapshots.
		/// </summary>
		Invalid,
		/// <summary>
		/// Snapshot stored in cloud repository such as Unreal Cloud DDC.
		/// </summary>
		Cloud,
		/// <summary>
		/// Snapshot stored in builds repository such as Unreal Cloud DDC.
		/// </summary>
		Builds,
		/// <summary>
		/// Snapshot stored in a zenserver.
		/// </summary>
		Zen,
		/// <summary>
		/// Snapshot stored as a file on disk.
		/// </summary>
		File,
	}

	/// <summary>
	/// Parameters for a task that exports an snapshot from ZenServer
	/// </summary>
	public class ZenExportSnapshotTaskParameters
	{
		/// <summary>
		/// The project from which to export the snapshot
		/// </summary>
		[TaskParameter(Optional = true)]
		public FileReference Project { get; set; }

		/// <summary>
		/// The target platform(s) to export the snapshot for
		/// </summary>
		[TaskParameter(Optional = true)]
		public string Platform { get; set; }

		/// <summary>
		/// The metadata to associate with the snapshot
		/// </summary>
		[TaskParameter(Optional = true)]
		public string Metadata { get; set; }

		/// <summary>
		/// A file to read with information about the snapshot that should be used as a base when exporting this new snapshot
		/// </summary>
		[TaskParameter(Optional = true)]
		public FileReference SnapshotBaseDescriptorFile { get; set; }

		/// <summary>
		/// A file to create with information about the snapshot that was exported
		/// </summary>
		[TaskParameter(Optional = true)]
		public FileReference SnapshotDescriptorFile { get; set; }

		/// <summary>
		/// The type of destination to export the snapshot to (cloud, ...)
		/// </summary>
		[TaskParameter]
		public string DestinationStorageType { get; set; }

		/// <summary>
		/// The identifier to use when exporting to a destination
		/// </summary>
		[TaskParameter(Optional = true)]
		public string DestinationIdentifier { get; set; }

		/// <summary>
		/// The host name to use when exporting to a cloud destination
		/// </summary>
		[TaskParameter(Optional = true)]
		public string DestinationCloudHost { get; set; }

		/// <summary>
		/// The host name to use when writing a snapshot descriptor for a cloud destination
		/// </summary>
		[TaskParameter(Optional = true)]
		public string SnapshotDescriptorCloudHost { get; set; }

		/// <summary>
		/// The target platform to use when writing a snapshot descriptor
		/// </summary>
		[TaskParameter(Optional = true)]
		public string SnapshotDescriptorPlatform { get; set; }

		/// <summary>
		/// The http version to use when exporting to a cloud destination
		/// </summary>
		[TaskParameter(Optional = true)]
		public string DestinationCloudHttpVersion { get; set; }

		/// <summary>
		/// The http version to use when writing a snapshot descriptor for a cloud destination
		/// </summary>
		[TaskParameter(Optional = true)]
		public string SnapshotDescriptorCloudHttpVersion { get; set; }

		/// <summary>
		/// The namespace to use when exporting to a cloud destination
		/// </summary>
		[TaskParameter(Optional = true)]
		public string DestinationCloudNamespace { get; set; }

		/// <summary>
		/// A custom bucket name to use when exporting to a cloud destination
		/// </summary>
		[TaskParameter(Optional = true)]
		public string DestinationCloudBucket { get; set; }

		/// <summary>
		/// The host name to use when exporting to a zen destination
		/// </summary>
		[TaskParameter(Optional = true)]
		public string DestinationZenHost { get; set; }

		/// <summary>
		/// The directory to use when exporting to a file destination
		/// </summary>
		[TaskParameter(Optional = true)]
		public DirectoryReference DestinationFileDir { get; set; }

		/// <summary>
		/// The filename to use when exporting to a file destination
		/// </summary>
		[TaskParameter(Optional = true)]
		public string DestinationFileName { get; set; }

		/// <summary>
		/// Optional. Where to look for the ue.projectstore
		/// The pattern {Platform} can be used for exporting multiple platforms at once.
		/// </summary>
		[TaskParameter(Optional = true)]
		public string OverridePlatformCookedDir { get; set; }

		/// <summary>
		/// Optional. Whether to force export of data even if the destination claims to have them.
		/// </summary>
		[TaskParameter(Optional = true)]
		public bool ForceExport { get; set; } = false;

		/// <summary>
		/// Optional. Whether to entirely bypass the exporting of data and write a snapshot descriptor as if the data had been exported.
		/// </summary>
		[TaskParameter(Optional = true)]
		public bool SkipExport { get; set; } = false;
	}

	/// <summary>
	/// Exports an snapshot from Zen to a specified destination.
	/// </summary>
	[TaskElement("ZenExportSnapshot", typeof(ZenExportSnapshotTaskParameters))]
	public class ZenExportSnapshotTask : BgTaskImpl
	{
		/// <summary>
		/// Metadata about a snapshot
		/// </summary>
		class SnapshotDescriptor
		{
			/// <summary>
			/// Name of the snapshot
			/// </summary>
			public string Name { get; set; }

			/// <summary>
			/// Storage type used for the snapshot
			/// </summary>
			public SnapshotStorageType Type { get; set; }

			/// <summary>
			/// Target platform for this snapshot
			/// </summary>
			public string TargetPlatform { get; set; }

			/// <summary>
			/// For cloud snapshots, the host they are stored on.
			/// </summary>
			public string Host { get; set; }

			/// <summary>
			/// For cloud snapshots, the namespace they are stored in.
			/// </summary>
			public string Namespace { get; set; }

			/// <summary>
			/// For cloud snapshots, the bucket they are stored in.
			/// </summary>
			public string Bucket { get; set; }

			/// <summary>
			/// For cloud snapshots, the key they are stored in.
			/// </summary>
			public string Key { get; set; }

			/// <summary>
			/// For file snapshots, the directory it is stored in.
			/// </summary>
			public string Directory { get; set; }

			/// <summary>
			/// For file snapshots, the filename (not including path) that they are stored in.
			/// </summary>
			public string Filename { get; set; }
		}
		/// <summary>
		/// A collection of one or more snapshot descriptors
		/// </summary>
		class SnapshotDescriptorCollection
		{
			/// <summary>
			/// The list of snapshots contained within this collection.
			/// </summary>
			public List<SnapshotDescriptor> Snapshots { get; set; }
		}

		private class ExportSourceData
		{
			public bool _isLocalHost;
			public string _hostName;
			public int _hostPort;
			public string _projectId;
			public string _oplogId;
			public string _targetPlatform;
			public SnapshotDescriptor _snapshotBaseDescriptor;
		}

		/// <summary>
		/// Parameters for the task
		/// </summary>
		readonly ZenExportSnapshotTaskParameters _parameters;

		/// <summary>
		/// Constructor.
		/// </summary>
		/// <param name="parameters">Parameters for this task</param>
		public ZenExportSnapshotTask(ZenExportSnapshotTaskParameters parameters)
		{
			_parameters = parameters;
		}

		/// <summary>
		/// Gets the assumed path to where Zen should exist
		/// </summary>
		/// <returns></returns>
		public static FileReference ZenExeFileReference()
		{
			return ResolveFile(String.Format("Engine/Binaries/{0}/zen{1}", HostPlatform.Current.HostEditorPlatform.ToString(), RuntimeInformation.IsOSPlatform(OSPlatform.Windows) ? ".exe" : ""));
		}

		/// <summary>
		/// Ensures that ZenServer is running on this current machine. This is needed before running any oplog commands
		/// This passes the sponsor'd process Id to launch zen.
		/// This ensures that zen does not live longer than the lifetime of a particular a process that needs Zen to be running
		/// </summary>
		/// <param name="projectFile"></param>
		public static void ZenLaunch(FileReference projectFile)
		{
			// Get the ZenLaunch executable path
			FileReference zenLaunchExe = ResolveFile(String.Format("Engine/Binaries/{0}/ZenLaunch{1}", HostPlatform.Current.HostEditorPlatform.ToString(), RuntimeInformation.IsOSPlatform(OSPlatform.Windows) ? ".exe" : ""));

			StringBuilder zenLaunchCommandline = new StringBuilder();
			zenLaunchCommandline.AppendFormat("{0} -SponsorProcessID={1}", CommandUtils.MakePathSafeToUseWithCommandLine(projectFile.FullName), Environment.ProcessId);

			CommandUtils.RunAndLog(CommandUtils.CmdEnv, zenLaunchExe.FullName, zenLaunchCommandline.ToString(), Options: CommandUtils.ERunOptions.Default);
		}

		static JsonSerializerOptions GetDefaultJsonSerializerOptions()
		{
			JsonSerializerOptions options = new JsonSerializerOptions();
			options.AllowTrailingCommas = true;
			options.ReadCommentHandling = JsonCommentHandling.Skip;
			options.PropertyNameCaseInsensitive = true;
			options.Converters.Add(new JsonStringEnumConverter());
			return options;
		}

#nullable enable
		static bool TryLoadJson<T>(FileReference file, [NotNullWhen(true)] out T? obj) where T : class
		{
			if (!FileReference.Exists(file))
			{
				obj = null;
				return false;
			}

			try
			{
				obj = LoadJson<T>(file);
				return true;
			}
			catch (Exception)
			{
				obj = null;
				return false;
			}
		}

		static T LoadJson<T>(FileReference file)
		{
			byte[] data = FileReference.ReadAllBytes(file);
			return JsonSerializer.Deserialize<T>(data, GetDefaultJsonSerializerOptions())!;
		}

		static string SanitizeOplogName(string name)
		{
			return name.Replace('/', '_').Replace(' ', '_').Replace('+', '_').Replace('-', '_');
		}

		private string GetCloudBucketName()
		{
			string bucketName = _parameters.DestinationCloudBucket;
			string projectNameAsBucketName = _parameters.Project.GetFileNameWithoutAnyExtensions().ToLowerInvariant();
			if (String.IsNullOrEmpty(bucketName))
			{
				bucketName = projectNameAsBucketName;
			}
			bucketName = SanitizeBucketName(bucketName);
			return bucketName;
		}

		private void WriteExportSource(JsonWriter writer, SnapshotStorageType destinationStorageType, ExportSourceData exportSource, string name, string destinationId)
		{
			string targetPlatform = _parameters.SnapshotDescriptorPlatform;
			if (String.IsNullOrEmpty(targetPlatform))
			{
				targetPlatform = exportSource._targetPlatform;
			}
			writer.WriteObjectStart();
			switch (destinationStorageType)
			{
				case SnapshotStorageType.Builds:
				case SnapshotStorageType.Cloud:
					string bucketName = GetCloudBucketName();

					string hostName = _parameters.SnapshotDescriptorCloudHost;
					if (String.IsNullOrEmpty(hostName))
					{
						hostName = _parameters.DestinationCloudHost;
					}

					string httpVersion = _parameters.SnapshotDescriptorCloudHttpVersion;
					if (String.IsNullOrEmpty(httpVersion))
					{
						httpVersion = _parameters.DestinationCloudHttpVersion;
					}

					string storageTypeName = "cloud";
					string storageIdentifierName = "key";
					if (destinationStorageType == SnapshotStorageType.Builds)
					{
						storageTypeName = "builds";
						storageIdentifierName = "builds-id";
					}

					writer.WriteValue("name", name);
					writer.WriteValue("type", storageTypeName);
					writer.WriteValue("targetplatform", targetPlatform);
					writer.WriteValue("host", hostName);
					if (!String.IsNullOrEmpty(httpVersion) && !httpVersion.Equals("None", StringComparison.OrdinalIgnoreCase))
					{
						writer.WriteValue("httpversion", httpVersion);
					}
					writer.WriteValue("namespace", _parameters.DestinationCloudNamespace);
					writer.WriteValue("bucket", bucketName);
					writer.WriteValue(storageIdentifierName, destinationId);
					break;
				case SnapshotStorageType.Zen:
					string projectName = _parameters.Project.GetFileNameWithoutAnyExtensions().ToLowerInvariant() + ".oplog";

					writer.WriteValue("name", name);
					writer.WriteValue("type", "zen");
					writer.WriteValue("targetplatform", targetPlatform);
					writer.WriteValue("host", _parameters.DestinationZenHost);
					writer.WriteValue("projectid", projectName);
					writer.WriteValue("oplogid", SanitizeOplogName(name));
					break;
				case SnapshotStorageType.File:
					writer.WriteValue("name", name);
					writer.WriteValue("type", "file");
					writer.WriteValue("targetplatform", targetPlatform);
					writer.WriteValue("directory", _parameters.DestinationFileDir.FullName);
					writer.WriteValue("filename", _parameters.DestinationFileName);
					break;
			}
			writer.WriteObjectEnd();
		}

		private static bool TryRunAndLogWithoutSpew(string app, string commandLine, bool ignoreFailure, out int exitCode)
		{
			ProcessResult.SpewFilterCallbackType silentOutputFilter = new ProcessResult.SpewFilterCallbackType(line =>
				{
					return null;
				});
			try
			{
				CommandUtils.RunAndLog(CommandUtils.CmdEnv, app, commandLine, MaxSuccessCode: 0, Options: CommandUtils.ERunOptions.Default, SpewFilterCallback: silentOutputFilter);
			}
			catch (CommandUtils.CommandFailedException e)
			{
				if (!ignoreFailure)
				{
					Logger.LogWarning("{Text}", e.ToString());
				}

				exitCode = (int)e.ErrorCode;
				return false;
			}

			exitCode = 0;
			return true;
		}

		private static bool TryExportOplogCommand(string app, string commandLine)
		{
			int attempt = 0;
			// increasingly long time to wait before retrying
			int[] waitDurationWebSeconds = [60, 300, 900, 3600];
			int attemptLimit = waitDurationWebSeconds.Length;

			while (attempt < attemptLimit)
			{
				if (TryRunAndLogWithoutSpew(app, commandLine, false, out int exitCode))
				{
					return true;
				}

				// 502 and 504 are the old exit codes for retries the new exit codes for retries that also work on mac and linux are 69, 92 and 93 see zen cli documentation of exit codes for more details
				bool isGatewayError = exitCode is 502 or 504 or 69 or 92 or 93 or 84;
				int waitDuration = isGatewayError ? waitDurationWebSeconds[attempt] : 0;
				bool willRetry = attempt < (attemptLimit - 1);
				Logger.LogWarning("Attempt {AttemptNum} of exporting the oplog failed, {Action}...", attempt + 1, willRetry ? $"retrying after {waitDuration}s" : "abandoning");
				if (willRetry)
				{
					Thread.Sleep(TimeSpan.FromSeconds(waitDuration));
				}
				attempt = attempt + 1;
			}
			return false;
		}

		private static bool IsValidCharacter(char character) => character <= 0x7f && IsValidCharacter((byte)character);

		private static bool IsValidCharacter(byte character)
		{
			if (character >= 'a' && character <= 'z')
			{
				return true;
			}
			if (character >= '0' && character <= '9')
			{
				return true;
			}
			if (character == '-' || character == '_' || character == '.')
			{
				return true;
			}
			return false;
		}

		private static string SanitizeBucketName(string inString)
		{
			// Originally matched StringId.Sanitize, but with a MaxLength of 256 instead of 64
			const int MaxLength = 256;
			StringBuilder result = new StringBuilder();
			for (int idx = 0; idx < inString.Length && result.Length < MaxLength; idx++)
			{
				char character = (char)inString[idx];
				if (character >= 'A' && character <= 'Z')
				{
					result.Append((char)('a' + (character - 'A')));
				}
				else if (character == '.')
				{
					if (result.Length > 0)
					{
						if (result[^1] == '-')
						{
							result[^1] = character;
						}
						else
						{
							result.Append(character);
						}
					}
				}
				else if (IsValidCharacter(character))
				{
					result.Append(character);
				}
				else if (result.Length > 0 && result[^1] != '-' && result[^1] != '.')
				{
					result.Append('-');
				}
			}
			while (result.Length > 0 && (result[^1] == '-' || result[^1] == '.'))
			{
				result.Remove(result.Length - 1, 1);
			}
			return result.ToString();
		}

		/// <summary>
		/// ExecuteAsync the task.
		/// </summary>
		/// <param name="job">Information about the current job</param>
		/// <param name="buildProducts">Set of build products produced by this node.</param>
		/// <param name="tagNameToFileSet">Mapping from tag names to the set of files they include</param>
		public override Task ExecuteAsync(JobContext job, HashSet<FileReference> buildProducts, Dictionary<string, HashSet<FileReference>> tagNameToFileSet)
		{
			SnapshotStorageType destinationStorageType = SnapshotStorageType.Invalid;
			if (!String.IsNullOrEmpty(_parameters.DestinationStorageType))
			{
				destinationStorageType = (SnapshotStorageType)Enum.Parse(typeof(SnapshotStorageType), _parameters.DestinationStorageType);
			}

			FileReference projectFile = _parameters.Project;
			if (!FileReference.Exists(projectFile))
			{
				throw new AutomationException("Missing project file - {0}", projectFile.FullName);
			}

			ZenLaunch(projectFile);

			List<ExportSourceData> exportSources = new List<ExportSourceData>();
			foreach (string platform in _parameters.Platform.Split('+'))
			{
				DirectoryReference platformCookedDirectory;
				if (String.IsNullOrEmpty(_parameters.OverridePlatformCookedDir))
				{
					platformCookedDirectory = DirectoryReference.Combine(projectFile.Directory, "Saved", "Cooked", platform);
				}
				else
				{
					platformCookedDirectory = new DirectoryReference(_parameters.OverridePlatformCookedDir.Replace("{Platform}", platform, StringComparison.InvariantCultureIgnoreCase));
				}
				if (!DirectoryReference.Exists(platformCookedDirectory))
				{
					throw new AutomationException("Cook output directory not found ({0})", platformCookedDirectory.FullName);
				}

				FileReference projectStoreFile = FileReference.Combine(platformCookedDirectory, "ue.projectstore");
				ProjectStoreData? parsedProjectStore = null;
				if (TryLoadJson(projectStoreFile, out parsedProjectStore) && (parsedProjectStore != null) && (parsedProjectStore.ZenServer != null))
				{
					ExportSourceData newExportSource = new ExportSourceData();
					newExportSource._isLocalHost = parsedProjectStore.ZenServer.IsLocalHost;
					newExportSource._hostName = parsedProjectStore.ZenServer.HostName;
					newExportSource._hostPort = parsedProjectStore.ZenServer.HostPort;
					newExportSource._projectId = parsedProjectStore.ZenServer.ProjectId;
					newExportSource._oplogId = parsedProjectStore.ZenServer.OplogId;
					newExportSource._targetPlatform = platform;
					newExportSource._snapshotBaseDescriptor = null;

					if (_parameters.SnapshotBaseDescriptorFile != null)
					{
						FileReference platformSnapshotBase = new FileReference(_parameters.SnapshotBaseDescriptorFile.FullName.Replace("{Platform}", platform, StringComparison.InvariantCultureIgnoreCase));

						SnapshotDescriptorCollection? parsedDescriptorCollection = null;
						if (TryLoadJson(platformSnapshotBase, out parsedDescriptorCollection) && (parsedDescriptorCollection != null) && (parsedDescriptorCollection.Snapshots != null))
						{
							foreach (SnapshotDescriptor parsedDescriptor in parsedDescriptorCollection.Snapshots)
							{
								if (parsedDescriptor.TargetPlatform == platform)
								{
									newExportSource._snapshotBaseDescriptor = parsedDescriptor;
									break;
								}
							}
						}
					}

					exportSources.Add(newExportSource);
				}
			}
			int exportIndex;
			string[] exportNames = new string[exportSources.Count];
			string[] exportIds = new string[exportSources.Count];
			List<ExportSourceData> successfullyExportedSources = new List<ExportSourceData>();

			// Get the Zen executable path
			FileReference zenExe = ZenExeFileReference();

			// Format the command line
			StringBuilder oplogExportCommandline = new StringBuilder();
			oplogExportCommandline.Append("oplog-export --embedloosefiles");
			if (_parameters.ForceExport)
			{
				oplogExportCommandline.Append(" --force");
			}

			switch (destinationStorageType)
			{
				case SnapshotStorageType.Builds:
				case SnapshotStorageType.Cloud:
					ProjectProperties properties = ProjectUtils.GetProjectProperties(_parameters.Project);
					ConfigHierarchy config = properties.EngineConfigs[HostPlatform.Current.HostEditorPlatform];
					bool foundConfig = config.TryGetValueGeneric("StorageServers", "Cloud", out CloudConfiguration cloudConfig);

					if (String.IsNullOrEmpty(_parameters.DestinationCloudHost))
					{
						if (!foundConfig || String.IsNullOrEmpty(cloudConfig.Host))
						{
							throw new AutomationException("Missing destination cloud host");
						}

						_parameters.DestinationCloudHost = cloudConfig.Host.Split(";")[0];
					}
					if (String.IsNullOrEmpty(_parameters.DestinationCloudNamespace))
					{
						if (!foundConfig || String.IsNullOrEmpty(cloudConfig.BuildsNamespace))
						{
							throw new AutomationException(String.Format("Missing destination cloud namespace {0}", cloudConfig.BuildsNamespace));
						}

						_parameters.DestinationCloudNamespace = cloudConfig.BuildsNamespace;
					}
					if (String.IsNullOrEmpty(_parameters.DestinationIdentifier))
					{
						throw new AutomationException("Missing destination identifier when exporting to cloud");
					}

					string bucketName = GetCloudBucketName();

					string storageTypeName = "cloud";
					string storageIdentifierName = "key";
					if (destinationStorageType == SnapshotStorageType.Builds)
					{
						storageTypeName = "builds";
						storageIdentifierName = "builds-id";
					}
					oplogExportCommandline.AppendFormat(" --{0} {1} --namespace {2} --bucket {3}", storageTypeName, _parameters.DestinationCloudHost, _parameters.DestinationCloudNamespace, bucketName);

					if (!String.IsNullOrEmpty(_parameters.DestinationCloudHttpVersion))
					{
						if (_parameters.DestinationCloudHttpVersion.Equals("http2-only", StringComparison.OrdinalIgnoreCase))
						{
							oplogExportCommandline.Append(" --assume-http2");
						}
						else
						{
							throw new AutomationException("Unexpected destination cloud http version");
						}
					}

					exportIndex = 0;
					foreach (ExportSourceData exportSource in exportSources)
					{
						string hostUrlArg = String.Format("--hosturl http://{0}:{1}", exportSource._isLocalHost ? "localhost" : exportSource._hostName, exportSource._hostPort);

						string baseKeyArg = String.Empty;
						if ((destinationStorageType == SnapshotStorageType.Cloud) && (exportSource._snapshotBaseDescriptor != null) && !String.IsNullOrEmpty(exportSource._snapshotBaseDescriptor.Key))
						{
							if (exportSource._snapshotBaseDescriptor.Type == SnapshotStorageType.Cloud)
							{
								baseKeyArg = " --basekey " + exportSource._snapshotBaseDescriptor.Key;
							}
							else
							{
								Logger.LogWarning("Base snapshot descriptor was for a snapshot storage type {Type}, but we're producing a snapshot of type cloud.  Skipping use of base snapshot.", exportSource._snapshotBaseDescriptor.Type);
							}
						}

						StringBuilder exportSingleSourceCommandline = new StringBuilder(oplogExportCommandline.Length);
						exportSingleSourceCommandline.Append(oplogExportCommandline);

						StringBuilder destinationNameBuilder = new StringBuilder();
						destinationNameBuilder.AppendFormat("{0}.{1}.{2}", bucketName, _parameters.DestinationIdentifier, exportSource._oplogId);
						exportNames[exportIndex] = destinationNameBuilder.ToString().ToLowerInvariant();

						string destinationId;
						if (destinationStorageType == SnapshotStorageType.Builds)
						{
							StringBuilder metadata = new StringBuilder();
							metadata.AppendFormat("type=oplog;createdAt={0}", DateTime.UtcNow.ToString("O"));

							// Add keys for the job that's executing
							string? stepId = null;
							string? jobId = Environment.GetEnvironmentVariable("UE_HORDE_JOBID");
							if (!String.IsNullOrEmpty(jobId))
							{
								metadata.AppendFormat(";job={0}", jobId);

								stepId = Environment.GetEnvironmentVariable("UE_HORDE_STEPID");
								if (!String.IsNullOrEmpty(stepId))
								{
									metadata.AppendFormat(";step={0}", stepId);
								}
							}

							string? hordeUrl = Environment.GetEnvironmentVariable("UE_HORDE_URL");
							// if we are running in horde and have the required environment variables we append a link back to the horde job into metadata
							if (!String.IsNullOrEmpty(hordeUrl) && !String.IsNullOrEmpty(jobId) && !String.IsNullOrEmpty(stepId))
							{
								metadata.Append($";buildurl={hordeUrl}job/{jobId}?step={stepId}");
							}

							if (!String.IsNullOrEmpty(_parameters.Metadata))
							{
								metadata.AppendFormat(";{0}", _parameters.Metadata);
							}

							exportSingleSourceCommandline.AppendFormat(" --builds-metadata \"{0}\"", metadata.ToString());
							CbObjectId objectId = CbObjectId.NewObjectId();
							destinationId = objectId.ToString().ToLowerInvariant();
						}
						else
						{
							IoHash destinationKeyHash = IoHash.Compute(Encoding.UTF8.GetBytes(exportNames[exportIndex]));
							destinationId = destinationKeyHash.ToString().ToLowerInvariant();
						}

						exportIds[exportIndex] = destinationId;
						exportSingleSourceCommandline.AppendFormat(" {0} --{1} {2} {3} {4} {5}", hostUrlArg, storageIdentifierName, destinationId, baseKeyArg, exportSource._projectId, exportSource._oplogId);
						if (_parameters.SkipExport || TryExportOplogCommand(zenExe.FullName, exportSingleSourceCommandline.ToString()))
						{
							successfullyExportedSources.Add(exportSource);
						}

						exportIndex = exportIndex + 1;
					}

					break;
				case SnapshotStorageType.Zen:
					if (String.IsNullOrEmpty(_parameters.DestinationZenHost))
					{
						throw new AutomationException("Missing destination zen host");
					}
					if (String.IsNullOrEmpty(_parameters.DestinationIdentifier))
					{
						throw new AutomationException("Missing destination identifier when exporting to zen");
					}

					string projectName = projectFile.GetFileNameWithoutAnyExtensions().ToLowerInvariant() + ".oplog";

					StringBuilder createProjectCommandline = new StringBuilder();
					createProjectCommandline.AppendFormat("project-create --hosturl {0} {1}", _parameters.DestinationZenHost, projectName);
					TryRunAndLogWithoutSpew(zenExe.FullName, createProjectCommandline.ToString(), true, out int _);

					oplogExportCommandline.AppendFormat(" --zen {0}", _parameters.DestinationZenHost);

					exportIndex = 0;
					foreach (ExportSourceData exportSource in exportSources)
					{
						string hostUrlArg = String.Format("--hosturl http://{0}:{1}", exportSource._isLocalHost ? "localhost" : exportSource._hostName, exportSource._hostPort);

						StringBuilder exportSingleSourceCommandline = new StringBuilder(oplogExportCommandline.Length);
						exportSingleSourceCommandline.Append(oplogExportCommandline);

						StringBuilder destinationKeyBuilder = new StringBuilder();
						destinationKeyBuilder.AppendFormat("{0}.{1}", _parameters.DestinationIdentifier, exportSource._oplogId);
						exportNames[exportIndex] = destinationKeyBuilder.ToString().ToLowerInvariant();
						string destinationOplog = SanitizeOplogName(exportNames[exportIndex]);

						exportSingleSourceCommandline.AppendFormat(" {0} --target-project {1} --target-oplog {2} {3} {4}", hostUrlArg, projectName, destinationOplog, exportSource._projectId, exportSource._oplogId);
						if (_parameters.SkipExport || TryExportOplogCommand(zenExe.FullName, exportSingleSourceCommandline.ToString()))
						{
							successfullyExportedSources.Add(exportSource);
						}

						exportIndex = exportIndex + 1;
					}

					break;
				case SnapshotStorageType.File:
					string defaultProjectId = ProjectUtils.GetProjectPathId(projectFile);
					exportIndex = 0;
					foreach (ExportSourceData exportSource in exportSources)
					{
						StringBuilder exportNameBuilder = new StringBuilder();
						exportNameBuilder.AppendFormat("{0}.{1}.{2}", projectFile.GetFileNameWithoutAnyExtensions().ToLowerInvariant(), _parameters.DestinationIdentifier, exportSource._oplogId);
						exportNames[exportIndex] = exportNameBuilder.ToString().ToLowerInvariant();

						StringBuilder exportSingleSourceCommandline = new StringBuilder(oplogExportCommandline.Length);
						exportSingleSourceCommandline.Append(oplogExportCommandline);

						string destinationFileName = exportSource._oplogId;
						if (!String.IsNullOrEmpty(_parameters.DestinationFileName))
						{
							destinationFileName = _parameters.DestinationFileName.Replace("{Platform}", exportSource._targetPlatform, StringComparison.InvariantCultureIgnoreCase);
						}

						string projectId = String.IsNullOrEmpty(exportSource._projectId) ? defaultProjectId : exportSource._projectId;
						string baseNameArg = String.Empty;
						DirectoryReference platformDestinationFileDir = new DirectoryReference(_parameters.DestinationFileDir.FullName.Replace("{Platform}", exportSource._targetPlatform, StringComparison.InvariantCultureIgnoreCase));
						if ((exportSource._snapshotBaseDescriptor != null) && !String.IsNullOrEmpty(exportSource._snapshotBaseDescriptor.Directory) && !String.IsNullOrEmpty(exportSource._snapshotBaseDescriptor.Filename))
						{
							if (exportSource._snapshotBaseDescriptor.Type == SnapshotStorageType.File)
							{
								FileReference baseSnapshotFile = new FileReference(Path.Combine(exportSource._snapshotBaseDescriptor.Directory, exportSource._snapshotBaseDescriptor.Filename));
								if (FileReference.Exists(baseSnapshotFile))
								{
									baseNameArg = " --basename " + CommandUtils.MakePathSafeToUseWithCommandLine(baseSnapshotFile.FullName);
								}
								else
								{
									Logger.LogWarning("Base snapshot descriptor missing.  Skipping use of base snapshot.");
								}
							}
							else
							{
								Logger.LogWarning("Base snapshot descriptor was for a snapshot storage type {Type}, but we're producing a snapshot of type file.  Skipping use of base snapshot.", exportSource._snapshotBaseDescriptor.Type);
							}
						}
						exportSingleSourceCommandline.AppendFormat(" --file {0} --name {1} {2} {3} {4}", CommandUtils.MakePathSafeToUseWithCommandLine(platformDestinationFileDir.FullName), destinationFileName, baseNameArg, projectId, exportSource._oplogId);

						if (_parameters.SkipExport || TryExportOplogCommand(zenExe.FullName, exportSingleSourceCommandline.ToString()))
						{
							successfullyExportedSources.Add(exportSource);
						}

						exportIndex = exportIndex + 1;
					}
					break;
				default:
					throw new AutomationException("Unknown/invalid/unimplemented destination storage type - {0}", _parameters.DestinationStorageType);
			}

			if ((_parameters.SnapshotDescriptorFile != null) && successfullyExportedSources.Any())
			{
				if (_parameters.SnapshotDescriptorFile.FullName.Contains("{Platform}", StringComparison.OrdinalIgnoreCase))
				{
					// Separate descriptor file per platform
					exportIndex = 0;
					foreach (ExportSourceData exportSource in successfullyExportedSources)
					{
						FileReference platformSnapshotDescriptorFile = new FileReference(_parameters.SnapshotDescriptorFile.FullName.Replace("{Platform}", exportSource._targetPlatform, StringComparison.InvariantCultureIgnoreCase));
						DirectoryReference.CreateDirectory(platformSnapshotDescriptorFile.Directory);
						using (JsonWriter writer = new JsonWriter(platformSnapshotDescriptorFile))
						{
							writer.WriteObjectStart();
							writer.WriteArrayStart("snapshots");
							WriteExportSource(writer, destinationStorageType, exportSource, exportNames[exportIndex], exportIds[exportIndex]);
							writer.WriteArrayEnd();
							writer.WriteObjectEnd();
						}
						exportIndex = exportIndex + 1;
					}
				}
				else
				{
					// Write out a single snapshot descriptor with info about all snapshots
					DirectoryReference.CreateDirectory(_parameters.SnapshotDescriptorFile.Directory);
					using (JsonWriter writer = new JsonWriter(_parameters.SnapshotDescriptorFile))
					{
						writer.WriteObjectStart();
						writer.WriteArrayStart("snapshots");

						exportIndex = 0;
						foreach (ExportSourceData exportSource in successfullyExportedSources)
						{
							WriteExportSource(writer, destinationStorageType, exportSource, exportNames[exportIndex], exportIds[exportIndex]);
							exportIndex = exportIndex + 1;
						}

						writer.WriteArrayEnd();
						writer.WriteObjectEnd();
					}
				}
			}

			return Task.CompletedTask;
		}

		/// <summary>
		/// Output this task out to an XML writer.
		/// </summary>
		public override void Write(XmlWriter writer)
		{
			Write(writer, _parameters);
		}

		/// <summary>
		/// Find all the tags which are used as inputs to this task
		/// </summary>
		/// <returns>The tag names which are read by this task</returns>
		public override IEnumerable<string> FindConsumedTagNames()
		{
			yield break;
		}

		/// <summary>
		/// Find all the tags which are modified by this task
		/// </summary>
		/// <returns>The tag names which are modified by this task</returns>
		public override IEnumerable<string> FindProducedTagNames()
		{
			yield break;
		}
	}

	struct CloudConfiguration
	{
#pragma warning disable IDE1006 // Static analysis wants these to be named differently, but they must be named the same as the config file properties
		public string Host = "";
		public string BuildsNamespace = "";
		public string BuildsBaselineBranch = "";
#pragma warning restore IDE1006

		public CloudConfiguration() {}
	}
}
