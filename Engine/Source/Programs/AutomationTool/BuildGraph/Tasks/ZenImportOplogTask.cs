// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.IO;
using System.Text;
using System.Text.Json;
using System.Text.Json.Serialization;
using System.Threading.Tasks;
using System.Xml;
using EpicGames.Core;
using EpicGames.ProjectStore;
using Microsoft.Extensions.Logging;

namespace AutomationTool.Tasks
{

	/// <summary>
	/// Parameters for a task that exports an snapshot from ZenServer
	/// </summary>
	public class ZenImportOplogTaskParameters
	{
		/// <summary>
		/// The type of destination to import from to (cloud, file...)
		/// </summary>
		[TaskParameter]
		public string ImportType { get; set; }

		/// <summary>
		/// comma separated full path to the oplog dir to import into the local zen server
		/// Files="Path1,Path2"
		/// </summary>
		[TaskParameter(Optional = true)]
		public string Files { get; set; }

		/// <summary>
		/// The project from which to import for
		/// </summary>
		[TaskParameter(Optional = true)]
		public FileReference Project { get; set; }

		/// <summary>
		/// The name of the newly created Zen Project we will be importing into
		/// </summary>
		[TaskParameter(Optional = true)]
		public string ProjectName { get; set; }

		/// <summary>
		/// The target platform to import the snapshot for
		/// </summary>
		[TaskParameter(Optional = true)]
		public string Platform { get; set; }

		/// <summary>
		/// Root dir for the UE project. Used to derive the Enging folder and the Project folder
		/// </summary>
		[TaskParameter(Optional = true)]
		public string RootDir { get; set; }

		/// <summary>
		/// The name of the imported oplog
		/// </summary>
		[TaskParameter(Optional = true)]
		public string OplogName { get; set; }

		/// <summary>
		/// The host URL for the zen server we are importing from
		/// </summary>
		[TaskParameter(Optional = true)]
		public string HostName { get; set; } = "localhost";

		/// <summary>
		/// The host port for the zen server we are importing from
		/// </summary>
		[TaskParameter(Optional = true)]
		public string HostPort { get; set; } = "8558";

		/// <summary>
		/// The cloud URL to import from
		/// </summary>
		[TaskParameter(Optional = true)]
		public string CloudURL { get; set; }

		/// <summary>
		/// what namespace to use when importing from cloud
		/// </summary>
		[TaskParameter(Optional = true)]
		public string Namespace { get; set; }

		/// <summary>
		/// what bucket to use when importing from cloud
		/// </summary>
		[TaskParameter(Optional = true)]
		public string Bucket { get; set; }

		/// <summary>
		/// What key to use when importing from cloud
		/// </summary>
		[TaskParameter(Optional = true)]
		public string Key { get; set; }
	}

	/// <summary>
	/// Imports an oplog from Zen to a specified destination.
	/// </summary>
	[TaskElement("ZenImportOplog", typeof(ZenImportOplogTaskParameters))]
	public class ZenImportOplogTask : BgTaskImpl
	{
		readonly ZenImportOplogTaskParameters _parameters;

		FileReference _projectFile;

		/// <summary>
		/// Constructor.
		/// </summary>
		/// <param name="parameters">Parameters for this task</param>
		public ZenImportOplogTask(ZenImportOplogTaskParameters parameters)
		{
			_parameters = parameters;
		}

		/// <summary>
		/// ExecuteAsync the task.
		/// </summary>
		/// <param name="job">Information about the current job</param>
		/// <param name="buildProducts">Set of build products produced by this node.</param>
		/// <param name="tagNameToFileSet">Mapping from tag names to the set of files they include</param>
		public override Task ExecuteAsync(JobContext job, HashSet<FileReference> buildProducts, Dictionary<string, HashSet<FileReference>> tagNameToFileSet)
		{
			SnapshotStorageType importMethod = SnapshotStorageType.Invalid;
			if (!String.IsNullOrEmpty(_parameters.ImportType))
			{
				importMethod = (SnapshotStorageType)Enum.Parse(typeof(SnapshotStorageType), _parameters.ImportType);
			}

			_projectFile = _parameters.Project;
			if (!FileReference.Exists(_projectFile))
			{
				throw new AutomationException("Missing project file - {0}", _projectFile.FullName);
			}

			ZenExportSnapshotTask.ZenLaunch(_projectFile);

			// Get the Zen executable path
			FileReference zenExe = ZenExportSnapshotTask.ZenExeFileReference();
			{
				if (String.IsNullOrEmpty(_parameters.RootDir))
				{
					throw new AutomationException("RootDir was not specified");
				}
				if (String.IsNullOrEmpty(_parameters.ProjectName))
				{
					throw new AutomationException("ProjectName was not specified");
				}

				// Create a new project to import everything into.
				string rootDir = _parameters.RootDir;
				string engineDir = System.IO.Path.Combine(_parameters.RootDir, "Engine");
				string projectDir = System.IO.Path.Combine(_parameters.RootDir, _projectFile.GetFileNameWithoutAnyExtensions());
				string hostUrlArg = String.Format("--hosturl http://{0}:{1}", _parameters.HostName, _parameters.HostPort);
				StringBuilder oplogProjectCreateCommandline = new StringBuilder();
				oplogProjectCreateCommandline.AppendFormat("project-create -p {0} --rootdir {1} --enginedir {2} --projectdir {3} --projectfile {4} {5}",
					_parameters.ProjectName,
					rootDir,
					engineDir,
					projectDir,
					_projectFile.FullName,
					hostUrlArg);

				Logger.LogInformation("Running '{Arg0} {Arg1}'", CommandUtils.MakePathSafeToUseWithCommandLine(zenExe.FullName), oplogProjectCreateCommandline.ToString());
				CommandUtils.RunAndLog(CommandUtils.CmdEnv, zenExe.FullName, oplogProjectCreateCommandline.ToString(), Options: CommandUtils.ERunOptions.Default);
			}

			switch (importMethod)
			{
				case SnapshotStorageType.File:
					ImportFromFile(zenExe);
					break;
				case SnapshotStorageType.Cloud:
					ImportFromCloud(zenExe);
					break;
				default:
					throw new AutomationException("Unknown/invalid/unimplemented import type - {0}", _parameters.ImportType);
			}

			WriteProjectStoreFile();
			return Task.CompletedTask;
		}

		private void ImportFromFile(FileReference zenExe)
		{
			if (String.IsNullOrEmpty(_parameters.OplogName))
			{
				throw new AutomationException("OplogName was not specified");
			}

			foreach (string fileToImport in _parameters.Files.Split(','))
			{
				if (DirectoryReference.Exists(new DirectoryReference(fileToImport)))
				{
					StringBuilder oplogImportCommandline = new StringBuilder();
					oplogImportCommandline.AppendFormat("oplog-import --file {0} --oplog {1} -p {2}", fileToImport, _parameters.OplogName, _parameters.ProjectName);

					Logger.LogInformation("Running '{Arg0} {Arg1}'", CommandUtils.MakePathSafeToUseWithCommandLine(zenExe.FullName), oplogImportCommandline.ToString());
					CommandUtils.RunAndLog(CommandUtils.CmdEnv, zenExe.FullName, oplogImportCommandline.ToString(), Options: CommandUtils.ERunOptions.Default);
				}
			}
		}

		private void WriteProjectStoreFile()
		{
			DirectoryReference platformCookedDirectory = DirectoryReference.Combine(_projectFile.Directory, "Saved", "Cooked", _parameters.Platform);
			if (!DirectoryReference.Exists(platformCookedDirectory))
			{
				DirectoryReference.CreateDirectory(platformCookedDirectory);
			}
			ProjectStoreData projectStore = new ProjectStoreData();
			projectStore.ZenServer = new ZenServerStoreData
			{
				ProjectId = _parameters.ProjectName,
				OplogId = _parameters.OplogName
			};

			JsonSerializerOptions serializerOptions = new JsonSerializerOptions
			{
				AllowTrailingCommas = true,
				ReadCommentHandling = JsonCommentHandling.Skip,
				PropertyNameCaseInsensitive = true
			};
			serializerOptions.Converters.Add(new JsonStringEnumConverter());

			FileReference projectStoreFile = FileReference.Combine(platformCookedDirectory, "ue.projectstore");
			File.WriteAllText(projectStoreFile.FullName, JsonSerializer.Serialize(projectStore, serializerOptions), new UTF8Encoding(encoderShouldEmitUTF8Identifier: false));
		}

		private void ImportFromCloud(FileReference zenExe)
		{
			if (String.IsNullOrEmpty(_parameters.CloudURL))
			{
				throw new AutomationException("Missing destination cloud host");
			}
			if (String.IsNullOrEmpty(_parameters.Namespace))
			{
				throw new AutomationException("Missing destination cloud namespace");
			}
			if (String.IsNullOrEmpty(_parameters.Key))
			{
				throw new AutomationException("Missing destination cloud storage key");
			}

			string bucketName = _parameters.Bucket;
			string projectNameAsBucketName = _projectFile.GetFileNameWithoutAnyExtensions().ToLowerInvariant();
			if (String.IsNullOrEmpty(bucketName))
			{
				bucketName = projectNameAsBucketName;
			}

			string hostUrlArg = String.Format("--hosturl http://{0}:{1}", _parameters.HostName, _parameters.HostPort);
			StringBuilder oplogImportCommandline = new StringBuilder();
			oplogImportCommandline.AppendFormat("oplog-import {0} --cloud {1} --namespace {2} --bucket {3}", hostUrlArg, _parameters.CloudURL, _parameters.Namespace, bucketName);
			oplogImportCommandline.AppendFormat(" {0}", _parameters.Key);

			Logger.LogInformation("Running '{Arg0} {Arg1}'", CommandUtils.MakePathSafeToUseWithCommandLine(zenExe.FullName), oplogImportCommandline.ToString());
			CommandUtils.RunAndLog(CommandUtils.CmdEnv, zenExe.FullName, oplogImportCommandline.ToString(), Options: CommandUtils.ERunOptions.Default);
		}

		/// <summary>
		/// Output this task out to an XML writer.
		/// </summary>
		public override void Write(XmlWriter writer)
		{
			Write(writer, _parameters);
		}

		/// <summary>
		/// Find all the tags which are used as inputs to this task
		/// </summary>
		/// <returns>The tag names which are read by this task</returns>
		public override IEnumerable<string> FindConsumedTagNames()
		{
			yield break;
		}

		/// <summary>
		/// Find all the tags which are modified by this task
		/// </summary>
		/// <returns>The tag names which are modified by this task</returns>
		public override IEnumerable<string> FindProducedTagNames()
		{
			yield break;
		}
	}
}
