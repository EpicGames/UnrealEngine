// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.IO;
using System.Linq;
using System.Net;
using System.Runtime.InteropServices;
using System.Text;
using System.Text.Json;
using System.Text.Json.Serialization;
using System.Threading.Tasks;
using System.Xml;

using EpicGames.Core;
using EpicGames.ProjectStore;

using UnrealBuildBase;

namespace AutomationTool.Tasks
{
	/// <summary>
	/// Parameters for a task that imports snapshots into ZenServer
	/// </summary>
	public class ZenImportSnapshotTaskParameters
	{
		/// <summary>
		/// The project into which snapshots should be imported
		/// </summary>
		[TaskParameter]
		public FileReference Project { get; set; }

		/// <summary>
		/// A file to read with information about the snapshots to be imported
		/// </summary>
		[TaskParameter(Optional = true)]
		public FileReference SnapshotDescriptorFile { get; set; }

		/// <summary>
		/// A JSON blob with information about the snapshots to be imported
		/// </summary>
		[TaskParameter(Optional = true)]
		public string SnapshotDescriptorJSON { get; set; }

		/// <summary>
		/// Optional. Where to look for the ue.projectstore file.
		/// The pattern {Platform} can be used for importing multiple platforms at once.
		/// </summary>
		[TaskParameter(Optional = true)]
		public string OverridePlatformCookedDir { get; set; }

		/// <summary>
		/// Optional. If true, force import of the oplog (corresponds to --force on the Zen command line).
		/// </summary>
		[TaskParameter(Optional = true)]
		public bool ForceImport { get; set; } = false;

		/// <summary>
		/// Optional. If true, import oplogs asynchronously (corresponds to --async on the Zen command line).
		/// </summary>
		[TaskParameter(Optional = true)]
		public bool AsyncImport { get; set; } = false;

		/// <summary>
		/// Optional. Remote zenserver host to import snapshots into.
		/// </summary>
		[TaskParameter(Optional = true)]
		public string RemoteZenHost { get; set; }

		/// <summary>
		/// Optional. Port on remote host which zenserver is listening on.
		/// </summary>
		[TaskParameter(Optional = true)]
		public int RemoteZenPort { get; set; } = 8558;

		/// <summary>
		/// Optional. Destination project ID to import snapshots into.
		/// </summary>
		[TaskParameter(Optional = true)]
		public string DestinationProjectId { get; set; }
	}

	/// <summary>
	/// Imports a snapshot from a specified destination to Zen.
	/// </summary>
	[TaskElement("ZenImportSnapshot", typeof(ZenImportSnapshotTaskParameters))]
	public class ZenImportSnapshotTask : BgTaskImpl
	{
		/// <summary>
		/// Enumeration of different storage options for snapshots.
		/// </summary>
		public enum SnapshotStorageType
		{
			/// <summary>
			/// A reserved non-valid storage type for snapshots.
			/// </summary>
			Invalid,
			/// <summary>
			/// Snapshot stored in cloud repositories such as Unreal Cloud DDC.
			/// </summary>
			Cloud,
			/// <summary>
			/// Snapshot stored in a zenserver.
			/// </summary>
			Zen,
			/// <summary>
			/// Snapshot stored as a file on disk.
			/// </summary>
			File,
			/// <summary>
			/// Snapshot stored in Unreal Cloud builds API.
			/// </summary>
			Builds,
		}

		/// <summary>
		/// Metadata about a snapshot
		/// </summary>
		class SnapshotDescriptor
		{
			/// <summary>
			/// Name of the snapshot
			/// </summary>
			public string Name { get; set; }

			/// <summary>
			/// Storage type used for the snapshot
			/// </summary>
			public SnapshotStorageType Type { get; set; }

			/// <summary>
			/// Target platform for this snapshot
			/// </summary>
			public string TargetPlatform { get; set; }

			/// <summary>
			/// For cloud or Zen snapshots, the host they are stored on.
			/// </summary>
			public string Host { get; set; }

			/// <summary>
			/// For Zen snapshots, the project ID to import from.
			/// </summary>
			public string ProjectId { get; set; }

			/// <summary>
			/// For Zen snapshots, the oplog ID to import from.
			/// </summary>
			public string OplogId { get; set; }

			/// <summary>
			/// For cloud snapshots, the namespace they are stored in.
			/// </summary>
			public string Namespace { get; set; }

			/// <summary>
			/// For cloud snapshots, the bucket they are stored in.
			/// </summary>
			public string Bucket { get; set; }

			/// <summary>
			/// For cloud snapshots, the key they are stored in.
			/// </summary>
			public string Key { get; set; }

			/// <summary>
			/// For builds snapshots, the builds ID that identifies them.
			/// </summary>
			[JsonPropertyName("builds-id")]
			public string BuildsId { get; set; }

			/// <summary>
			/// For file snapshots, the directory it is stored in.
			/// </summary>
			public string Directory { get; set; }

			/// <summary>
			/// For file snapshots, the filename (not including path) that they are stored in.
			/// </summary>
			public string Filename { get; set; }
		}

		/// <summary>
		/// A collection of one or more snapshot descriptors
		/// </summary>
		class SnapshotDescriptorCollection
		{
			/// <summary>
			/// The list of snapshots contained within this collection.
			/// </summary>
			public List<SnapshotDescriptor> Snapshots { get; set; }
		}

		/// <summary>
		/// Parameters for the task
		/// </summary>
		readonly ZenImportSnapshotTaskParameters _parameters;

		/// <summary>
		/// Constructor.
		/// </summary>
		/// <param name="parameters">Parameters for this task.</param>
		public ZenImportSnapshotTask(ZenImportSnapshotTaskParameters parameters)
		{
			_parameters = parameters;	
		}

		/// <summary>
		/// Output this task out to an XML writer.
		/// </summary>
		public override void Write(XmlWriter writer)
		{
			Write(writer, _parameters);
		}

		/// <summary>
		/// Find all the tags which are used as inputs to this task
		/// </summary>
		/// <returns>The tag names which are read by this task</returns>
		public override IEnumerable<string> FindConsumedTagNames()
		{
			yield break;
		}

		/// <summary>
		/// Find all the tags which are modified by this task
		/// </summary>
		/// <returns>The tag names which are modified by this task</returns>
		public override IEnumerable<string> FindProducedTagNames()
		{
			yield break;
		}

		private static void RunAndLogWithoutSpew(string app, string commandLine)
		{
			ProcessResult.SpewFilterCallbackType silentOutputFilter = new ProcessResult.SpewFilterCallbackType(line =>
				{
					return null;
				});
			try
			{
				CommandUtils.RunAndLog(CommandUtils.CmdEnv, app, commandLine, MaxSuccessCode: 0, Options: CommandUtils.ERunOptions.Default, SpewFilterCallback: silentOutputFilter);
			}
			catch (CommandUtils.CommandFailedException e)
			{
				throw new AutomationException("Zen command failed: {0}", e.ToString());
			}
		}

		private static FileReference ZenExeFileReference()
		{
			return ResolveFile(String.Format("Engine/Binaries/{0}/zen{1}", HostPlatform.Current.HostEditorPlatform.ToString(), RuntimeInformation.IsOSPlatform(OSPlatform.Windows) ? ".exe" : ""));
		}

		private static void ZenLaunch(FileReference projectFile)
		{
			FileReference zenLaunchExe = ResolveFile(String.Format("Engine/Binaries/{0}/ZenLaunch{1}", HostPlatform.Current.HostEditorPlatform.ToString(), RuntimeInformation.IsOSPlatform(OSPlatform.Windows) ? ".exe" : ""));

			StringBuilder zenLaunchCommandline = new StringBuilder();
			zenLaunchCommandline.AppendFormat("{0} -SponsorProcessID={1}", CommandUtils.MakePathSafeToUseWithCommandLine(projectFile.FullName), Environment.ProcessId);

			CommandUtils.RunAndLog(CommandUtils.CmdEnv, zenLaunchExe.FullName, zenLaunchCommandline.ToString(), Options: CommandUtils.ERunOptions.Default);
		}

		private class LowerCaseNamingPolicy : JsonNamingPolicy
		{
			public override string ConvertName(string name) => name.ToLower();
		}

		private static JsonSerializerOptions GetDefaultJsonSerializerOptions()
		{
			JsonSerializerOptions options = new JsonSerializerOptions();
			options.AllowTrailingCommas = true;
			options.ReadCommentHandling = JsonCommentHandling.Skip;
			options.PropertyNameCaseInsensitive = true;
			options.PropertyNamingPolicy = new LowerCaseNamingPolicy();
			options.Converters.Add(new JsonStringEnumConverter());
			return options;
		}

		private static SnapshotDescriptorCollection GetDescriptors(ZenImportSnapshotTaskParameters parameters)
		{
			SnapshotDescriptorCollection descriptors = new SnapshotDescriptorCollection();
			descriptors.Snapshots = new List<SnapshotDescriptor>();

			if (parameters.SnapshotDescriptorFile == null && String.IsNullOrEmpty(parameters.SnapshotDescriptorJSON))
			{
				throw new AutomationException("Must provide one of either SnapshotDescriptorFile or SnapshotDescriptorJSON");
			}

			if (parameters.SnapshotDescriptorFile != null)
			{
				if (!FileReference.Exists(parameters.SnapshotDescriptorFile))
				{
					throw new AutomationException("Snapshot descriptor file {0} does not exist", parameters.SnapshotDescriptorFile.FullName);
				}

				try
				{
					byte[] data = FileReference.ReadAllBytes(parameters.SnapshotDescriptorFile);
					SnapshotDescriptorCollection fileDescriptors = JsonSerializer.Deserialize<SnapshotDescriptorCollection>(data, GetDefaultJsonSerializerOptions());
					descriptors.Snapshots.AddRange(fileDescriptors.Snapshots);
				}
				catch (Exception)
				{
					throw new AutomationException("Failed to deserialize snapshot descriptors from file {0}", parameters.SnapshotDescriptorFile.FullName);
				}
			}

			if (!String.IsNullOrEmpty(parameters.SnapshotDescriptorJSON))
			{
				try 
				{
					SnapshotDescriptorCollection jsonDescriptors = JsonSerializer.Deserialize<SnapshotDescriptorCollection>(parameters.SnapshotDescriptorJSON, GetDefaultJsonSerializerOptions());
					descriptors.Snapshots.AddRange(jsonDescriptors.Snapshots);
				}
				catch (Exception)
				{
					throw new AutomationException("Failed to deserialize snapshot descriptors from property SnapshotDescriptorJSON");
				}
			}

			return descriptors;
		}

		private FileReference WriteProjectStoreFile(string projectId, string platform, string host, int port)
		{
			DirectoryReference platformCookedDirectory;

			if (String.IsNullOrEmpty(_parameters.OverridePlatformCookedDir))
			{
				platformCookedDirectory = DirectoryReference.Combine(_parameters.Project.Directory, "Saved", "Cooked", platform);
			}
			else
			{
				platformCookedDirectory = new DirectoryReference(_parameters.OverridePlatformCookedDir.Replace("{Platform}", platform, StringComparison.InvariantCultureIgnoreCase));
			}

			if (!DirectoryReference.Exists(platformCookedDirectory))
			{
				DirectoryReference.CreateDirectory(platformCookedDirectory);
			}

			FileReference projectStoreFile = new FileReference(Path.Combine(platformCookedDirectory.FullName, "ue.projectstore"));

			ProjectStoreData data = new ProjectStoreData();
			data.ZenServer = new ZenServerStoreData();

			data.ZenServer.IsLocalHost = String.IsNullOrEmpty(host);
			data.ZenServer.HostName = "[::1]";
			data.ZenServer.HostPort = port;
			data.ZenServer.ProjectId = projectId;
			data.ZenServer.OplogId = platform;

			if (!String.IsNullOrEmpty(host))
			{
				data.ZenServer.RemoteHostNames.Add(host);
			}
			else
			{
				data.ZenServer.RemoteHostNames.AddRange(GetHostAddresses());
			}

			string json = JsonSerializer.Serialize(data, GetDefaultJsonSerializerOptions());
			FileReference.WriteAllText(projectStoreFile, json);

			return projectStoreFile;
		}

		[System.Diagnostics.CodeAnalysis.SuppressMessage("Usage", "CA5351", Justification = "Not using MD5 for cryptographically secure use case")]
		private string GetProjectID()
		{
			using (System.Security.Cryptography.MD5 hasher = System.Security.Cryptography.MD5.Create())
			{
				byte[] bytes = System.Text.Encoding.UTF8.GetBytes(_parameters.Project.FullName.Replace("\\", "/", StringComparison.InvariantCulture));
				byte[] hashed = hasher.ComputeHash(bytes);

				string hexString = Convert.ToHexString(hashed).ToLowerInvariant();

				return _parameters.Project.GetFileNameWithoutAnyExtensions() + "." + hexString.Substring(0, 8);
			}
		}

		private static string GetHostNameFQDN()
		{
			string hostName = Dns.GetHostName();

			// take first hostname that contains '.'
			if (!hostName.Contains('.', StringComparison.InvariantCulture))
			{
				IPAddress[] ipAddresses = Dns.GetHostAddresses("");

				foreach (IPAddress ipAddress in ipAddresses)
				{
					string hostNameTemp = Dns.GetHostEntry(ipAddress).HostName;
					if (hostNameTemp.Contains('.', StringComparison.InvariantCulture))
					{
						hostName = hostNameTemp;
						break;
					}
				}
			}

			return hostName;
		}

		private static string[] GetHostAddresses()
		{
			return Dns.GetHostAddresses("", System.Net.Sockets.AddressFamily.InterNetwork)
				.Select(addr => addr.ToString())
				.Append($"hostname://{GetHostNameFQDN()}")
				.ToArray();
		}

		/// <summary>
		/// ExecuteAsync the task.
		/// </summary>
		/// <param name="job">Information about the current job</param>
		/// <param name="buildProducts">Set of build products produced by this node.</param>
		/// <param name="tagNameToFileSet">Mapping from tag names to the set of files they include</param>
		public override Task ExecuteAsync(JobContext job, HashSet<FileReference> buildProducts, Dictionary<string, HashSet<FileReference>> tagNameToFileSet)
		{
			// Correct the casing of the case-insensitive path passed in to match the actual casing of the project file on disk.
			_parameters.Project = FileReference.FindCorrectCase(_parameters.Project);
			SnapshotDescriptorCollection descriptors = GetDescriptors(_parameters);

			string host;
			if (String.IsNullOrEmpty(_parameters.RemoteZenHost))
			{
				ZenLaunch(_parameters.Project);
				host = "localhost:8558";
			}
			else
			{
				host = _parameters.RemoteZenHost + ":" + _parameters.RemoteZenPort.ToString();
			}

			string projectId = String.IsNullOrEmpty(_parameters.DestinationProjectId) ? GetProjectID() : _parameters.DestinationProjectId;

			DirectoryReference rootDir = Unreal.RootDirectory;
			DirectoryReference engineDir = new DirectoryReference(Path.Combine(Unreal.RootDirectory.FullName, "Engine"));
			DirectoryReference projectDir = _parameters.Project.Directory;

			StringBuilder projectCreateCommandLine = new StringBuilder();
			projectCreateCommandLine.AppendFormat("project-create --force-update {0} {1} {2} {3} {4}", projectId, rootDir.FullName, engineDir.FullName, projectDir.FullName, _parameters.Project.FullName);
			projectCreateCommandLine.AppendFormat(" --hosturl {0}", host);

			FileReference zenExe = ZenExeFileReference();
			RunAndLogWithoutSpew(zenExe.FullName, projectCreateCommandLine.ToString());

			foreach (SnapshotDescriptor descriptor in descriptors.Snapshots)
			{
				FileReference projectStoreFile = WriteProjectStoreFile(projectId, descriptor.TargetPlatform, _parameters.RemoteZenHost, _parameters.RemoteZenPort);

				string oplogName = descriptor.TargetPlatform;

				StringBuilder oplogCreateCommandLine = new StringBuilder();
				oplogCreateCommandLine.AppendFormat("oplog-create --force-update {0} {1}", projectId, oplogName);
				oplogCreateCommandLine.AppendFormat(" --hosturl {0}", host);

				StringBuilder oplogImportCommandLine = new StringBuilder();
				oplogImportCommandLine.AppendFormat("oplog-import {0} {1} {2} --clean", projectId, oplogName, projectStoreFile.FullName);
				oplogImportCommandLine.AppendFormat(" --hosturl {0}", host);

				if (_parameters.ForceImport)
				{
					oplogImportCommandLine.AppendFormat(" --force");
				}

				if (_parameters.AsyncImport)
				{
					oplogImportCommandLine.AppendFormat(" --async");
				}

				switch (descriptor.Type)
				{
				case SnapshotStorageType.Cloud:
					oplogImportCommandLine.AppendFormat(" --cloud {0}", descriptor.Host);
					oplogImportCommandLine.AppendFormat(" --namespace {0}", descriptor.Namespace);
					oplogImportCommandLine.AppendFormat(" --bucket {0}", descriptor.Bucket);
					oplogImportCommandLine.AppendFormat(" --key {0}", descriptor.Key);
					oplogImportCommandLine.AppendFormat(" --assume-http2");
					break;
				case SnapshotStorageType.Zen:
					oplogImportCommandLine.AppendFormat(" --zen {0}", descriptor.Host);
					oplogImportCommandLine.AppendFormat(" --source-project {0}", descriptor.ProjectId);
					oplogImportCommandLine.AppendFormat(" --source-oplog {0}", descriptor.OplogId);
					break;
				case SnapshotStorageType.File:
					oplogImportCommandLine.AppendFormat(" --file {0}", descriptor.Directory);
					oplogImportCommandLine.AppendFormat(" --name {0}", descriptor.Filename);
					break;
				case SnapshotStorageType.Builds:
					oplogImportCommandLine.AppendFormat(" --builds {0}", descriptor.Host);
					oplogImportCommandLine.AppendFormat(" --builds-id {0}", descriptor.BuildsId);
					oplogImportCommandLine.AppendFormat(" --namespace {0}", descriptor.Namespace);
					oplogImportCommandLine.AppendFormat(" --bucket {0}", descriptor.Bucket);
					break;
				default:
					break;
				}

				RunAndLogWithoutSpew(zenExe.FullName, oplogCreateCommandLine.ToString());
				RunAndLogWithoutSpew(zenExe.FullName, oplogImportCommandLine.ToString());
			}

			return Task.CompletedTask;
		}
	}
}