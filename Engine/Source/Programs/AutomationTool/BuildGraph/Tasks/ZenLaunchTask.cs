// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Runtime.InteropServices;
using System.Text;
using System.Threading.Tasks;
using System.Xml;
using EpicGames.Core;

namespace AutomationTool.Tasks
{
	/// <summary>
	/// Parameters for a task that launches ZenServer
	/// </summary>
	public class ZenLaunchTaskParameters
	{
		/// <summary>
		/// The project for which to launch ZenServer
		/// </summary>
		[TaskParameter]
		public FileReference Project { get; set; }
	}

	/// <summary>
	/// Launches ZenServer
	/// </summary>
	[TaskElement("ZenLaunch", typeof(ZenLaunchTaskParameters))]
	public class ZenLaunchTask : BgTaskImpl
	{

		/// <summary>
		/// Ensures that ZenServer is running on this current machine. This is needed before running any oplog commands
		/// This passes the sponsor'd process Id to launch zen.
		/// This ensures that zen does not live longer than the lifetime of a particular a process that needs Zen to be running
		/// </summary>
		/// <param name="projectFile"></param>
		public static void ZenLaunch(FileReference projectFile)
		{
			// Get the ZenLaunch executable path
			FileReference zenLaunchExe = ResolveFile(String.Format("Engine/Binaries/{0}/ZenLaunch{1}", HostPlatform.Current.HostEditorPlatform.ToString(), RuntimeInformation.IsOSPlatform(OSPlatform.Windows) ? ".exe" : ""));

			StringBuilder zenLaunchCommandline = new StringBuilder();
			zenLaunchCommandline.AppendFormat("{0} -SponsorProcessID={1}", CommandUtils.MakePathSafeToUseWithCommandLine(projectFile.FullName), Environment.ProcessId);

			CommandUtils.RunAndLog(CommandUtils.CmdEnv, zenLaunchExe.FullName, zenLaunchCommandline.ToString(), Options: CommandUtils.ERunOptions.Default);
		}

		/// <summary>
		/// Parameters for the task
		/// </summary>
		readonly ZenLaunchTaskParameters _parameters;

		/// <summary>
		/// Constructor.
		/// </summary>
		/// <param name="parameters">Parameters for this task</param>
		public ZenLaunchTask(ZenLaunchTaskParameters parameters)
		{
			_parameters = parameters;
		}

		/// <summary>
		/// ExecuteAsync the task.
		/// </summary>
		/// <param name="job">Information about the current job</param>
		/// <param name="buildProducts">Set of build products produced by this node.</param>
		/// <param name="tagNameToFileSet">Mapping from tag names to the set of files they include</param>
		public override Task ExecuteAsync(JobContext job, HashSet<FileReference> buildProducts, Dictionary<string, HashSet<FileReference>> tagNameToFileSet)
		{
			ZenLaunch(_parameters.Project);

			return Task.CompletedTask;
		}

		/// <summary>
		/// Output this task out to an XML writer.
		/// </summary>
		public override void Write(XmlWriter writer)
		{
			Write(writer, _parameters);
		}

		/// <summary>
		/// Find all the tags which are used as inputs to this task
		/// </summary>
		/// <returns>The tag names which are read by this task</returns>
		public override IEnumerable<string> FindConsumedTagNames()
		{
			yield break;
		}

		/// <summary>
		/// Find all the tags which are modified by this task
		/// </summary>
		/// <returns>The tag names which are modified by this task</returns>
		public override IEnumerable<string> FindProducedTagNames()
		{
			yield break;
		}
	}
}
