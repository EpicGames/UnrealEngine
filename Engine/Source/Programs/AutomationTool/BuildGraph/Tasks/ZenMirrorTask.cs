// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Runtime.InteropServices;
using System.Text;
using System.Threading.Tasks;
using System.Xml;
using EpicGames.Core;
using Microsoft.Extensions.Logging;

namespace AutomationTool.Tasks
{
	/// <summary>
	/// Parameters for a task that calls zen mirror to write the Zen oplog onto disk
	/// </summary>
	public class ZenMirrorTaskParameters
	{
		/// <summary>
		/// The project from which to export the snapshot
		/// </summary>
		[TaskParameter(Optional = true)]
		public FileReference Project { get; set; }

		/// <summary>
		/// The target platform to mirror the snapshot for
		/// </summary>
		[TaskParameter(Optional = true)]
		public string Platform { get; set; }

		/// <summary>
		/// The path on the local disk to which the data will be mirrored
		/// If empty then the path will be set to the %Project%\Saved\Cooked\%Platform% directory.
		/// </summary>
		[TaskParameter(Optional = true)]
		public DirectoryReference DestinationFileDir { get; set; }
	}

	/// <summary>
	/// Exports an snapshot from Zen to a specified destination.
	/// </summary>
	[TaskElement("ZenMirror", typeof(ZenMirrorTaskParameters))]
	public class ZenMirrorTask : BgTaskImpl
	{
		/// <summary>
		/// Parameters for the task
		/// </summary>
		readonly ZenMirrorTaskParameters _parameters;

		/// <summary>
		/// Constructor.
		/// </summary>
		/// <param name="parameters">Parameters for this task</param>
		public ZenMirrorTask(ZenMirrorTaskParameters parameters)
		{
			_parameters = parameters;
		}

		/// <summary>
		/// Gets the assumed path to where Zen should exist
		/// </summary>
		/// <returns></returns>
		public static FileReference ZenExeFileReference()
		{
			return ResolveFile(String.Format("Engine/Binaries/{0}/zen{1}",
				HostPlatform.Current.HostEditorPlatform.ToString(),
				RuntimeInformation.IsOSPlatform(OSPlatform.Windows) ? ".exe" : ""));
		}

		/// <summary>
		/// Ensures that ZenServer is running on this current machine. This is needed before running any oplog commands
		/// This passes the sponsor'd process Id to launch zen.
		/// This ensures that zen does not live longer than the lifetime of the a process that needs Zen.
		/// </summary>
		/// <param name="projectFile"></param>
		public static void ZenLaunch(FileReference projectFile)
		{
			// Get the ZenLaunch executable path
			FileReference zenLaunchExe = ResolveFile(String.Format("Engine/Binaries/{0}/ZenLaunch{1}",
				HostPlatform.Current.HostEditorPlatform.ToString(),
				RuntimeInformation.IsOSPlatform(OSPlatform.Windows) ? ".exe" : ""));

			StringBuilder zenLaunchCommandline = new StringBuilder();
			zenLaunchCommandline.AppendFormat("{0} -SponsorProcessID={1}",
				CommandUtils.MakePathSafeToUseWithCommandLine(projectFile.FullName), Environment.ProcessId);

			CommandUtils.RunAndLog(CommandUtils.CmdEnv, zenLaunchExe.FullName,
				zenLaunchCommandline.ToString(), Options: CommandUtils.ERunOptions.Default);
		}

		private static bool TryRunAndLogWithoutSpew(string app, string commandLine, bool ignoreFailure)
		{
			ProcessResult.SpewFilterCallbackType silentOutputFilter = new ProcessResult.SpewFilterCallbackType(line =>
				{
					return null;
				});
			try
			{
				CommandUtils.RunAndLog(CommandUtils.CmdEnv, app, commandLine, MaxSuccessCode: 0,
					Options: CommandUtils.ERunOptions.Default, SpewFilterCallback: silentOutputFilter);
			}
			catch (CommandUtils.CommandFailedException e)
			{
				if (!ignoreFailure)
				{
					Logger.LogWarning("{Text}", e.ToString());
				}
				return false;
			}
			return true;
		}

		private static bool TryRunMirrorCommand(string app, string commandLine)
		{
			int attemptLimit = 2;
			int attempt = 0;
			while (attempt < attemptLimit)
			{
				if (TryRunAndLogWithoutSpew(app, commandLine, false))
				{
					return true;
				}
				Logger.LogWarning("Attempt {AttemptNum} of mirroring the oplog failed, {Action}...", attempt + 1, attempt < (attemptLimit - 1) ? "retrying" : "abandoning");

				attempt = attempt + 1;
			}
			return false;
		}

		/// <summary>
		/// ExecuteAsync the task.
		/// </summary>
		/// <param name="job">Information about the current job</param>
		/// <param name="buildProducts">Set of build products produced by this node.</param>
		/// <param name="tagNameToFileSet">Mapping from tag names to the set of files they include</param>
		public override Task ExecuteAsync(JobContext job, HashSet<FileReference> buildProducts,
			Dictionary<string, HashSet<FileReference>> tagNameToFileSet)
		{
			FileReference projectFile = _parameters.Project;
			if (!FileReference.Exists(projectFile))
			{
				throw new AutomationException("Missing project file - {0}", projectFile.FullName);
			}
			if (String.IsNullOrEmpty(_parameters.Platform))
			{
				throw new AutomationException("Missing platform");
			}

			ZenLaunch(projectFile);

			DirectoryReference destinationFileDir = _parameters.DestinationFileDir;
			if (destinationFileDir == null || destinationFileDir.FullName.Length == 0)
			{
				destinationFileDir
					= DirectoryReference.Combine(projectFile.Directory, "Saved", "Cooked", _parameters.Platform);
			}

			// Get the Zen executable path
			FileReference zenExe = ZenExeFileReference();

			// Format the command line
			StringBuilder commandLine = new StringBuilder();
			commandLine.Append("oplog-mirror");

			commandLine.Append(" --project ");
			commandLine.Append(ProjectUtils.GetProjectPathId(projectFile));

			commandLine.Append(" --oplog ");
			commandLine.Append(_parameters.Platform);

			commandLine.Append(" --target ");
			commandLine.Append(destinationFileDir.FullName);

			TryRunMirrorCommand(zenExe.FullName, commandLine.ToString());

			return Task.CompletedTask;
		}

		/// <summary>
		/// Output this task out to an XML writer.
		/// </summary>
		public override void Write(XmlWriter writer)
		{
			Write(writer, _parameters);
		}

		/// <summary>
		/// Find all the tags which are used as inputs to this task
		/// </summary>
		/// <returns>The tag names which are read by this task</returns>
		public override IEnumerable<string> FindConsumedTagNames()
		{
			yield break;
		}

		/// <summary>
		/// Find all the tags which are modified by this task
		/// </summary>
		/// <returns>The tag names which are modified by this task</returns>
		public override IEnumerable<string> FindProducedTagNames()
		{
			yield break;
		}
	}
}
