// Copyright Epic Games, Inc. All Rights Reserved.

using System.Collections.Generic;
using System.Linq;
using System.Threading.Tasks;
using System.Xml;
using EpicGames.Core;
using Microsoft.Extensions.Logging;

namespace AutomationTool.Tasks
{
	/// <summary>
	/// Parameters for a zip task
	/// </summary>
	public class ZipTaskParameters
	{
		/// <summary>
		/// The directory to read compressed files from.
		/// </summary>
		[TaskParameter]
		public DirectoryReference FromDir { get; set; }

		/// <summary>
		/// List of file specifications separated by semicolons (for example, *.cpp;Engine/.../*.bat), or the name of a tag set. Relative paths are taken from FromDir.
		/// </summary>
		[TaskParameter(Optional = true, ValidationType = TaskParameterValidationType.FileSpec)]
		public string Files { get; set; }

		/// <summary>
		/// List of files that should have an executable bit set.
		/// </summary>
		[TaskParameter(Optional = true, ValidationType = TaskParameterValidationType.FileSpec)]
		public string ExecutableFiles { get; set; }

		/// <summary>
		/// The zip file to create.
		/// </summary>
		[TaskParameter]
		public FileReference ZipFile { get; set; }

		/// <summary>
		/// Skips adding the resulting zip to the default tagset.
		/// </summary>
		[TaskParameter(Optional = true, ValidationType = TaskParameterValidationType.Default)]
		public bool SkipDefaultTag { get; set; }

		/// <summary>
		/// Tag to be applied to the created zip file.
		/// </summary>
		[TaskParameter(Optional = true, ValidationType = TaskParameterValidationType.TagList)]
		public string Tag { get; set; }
	}

	/// <summary>
	/// Compresses files into a zip archive.
	/// </summary>
	[TaskElement("Zip", typeof(ZipTaskParameters))]
	public class ZipTask : BgTaskImpl
	{
		/// <summary>
		/// Parameters for this task
		/// </summary>
		readonly ZipTaskParameters _parameters;

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="parameters">Parameters for this task</param>
		public ZipTask(ZipTaskParameters parameters)
		{
			_parameters = parameters;
		}

		/// <summary>
		/// ExecuteAsync the task.
		/// </summary>
		/// <param name="job">Information about the current job</param>
		/// <param name="buildProducts">Set of build products produced by this node.</param>
		/// <param name="tagNameToFileSet">Mapping from tag names to the set of files they include</param>
		public override Task ExecuteAsync(JobContext job, HashSet<FileReference> buildProducts, Dictionary<string, HashSet<FileReference>> tagNameToFileSet)
		{
			// Find all the input files
			List<FileReference> files;
			if (_parameters.Files == null)
			{
				files = DirectoryReference.EnumerateFiles(_parameters.FromDir, "*", System.IO.SearchOption.AllDirectories).ToList();
			}
			else
			{
				files = ResolveFilespec(_parameters.FromDir, _parameters.Files, tagNameToFileSet).ToList();
			}

			// Create the zip file
			Logger.LogInformation("Adding {NumFiles} files to {ZipFile}...", files.Count, _parameters.ZipFile);

			HashSet<FileReference> executableFiles = null;
			if (_parameters.ExecutableFiles != null)
			{
				executableFiles = ResolveFilespec(_parameters.FromDir, _parameters.ExecutableFiles, tagNameToFileSet);
				foreach (FileReference executableFile in files.Intersect(executableFiles))
				{
					Logger.LogInformation("  Executable file: {File}", executableFile);
				}
			}

			CommandUtils.ZipFiles(_parameters.ZipFile, _parameters.FromDir, files, executableFiles);

			// Apply the optional tag to the produced archive
			foreach (string tagName in FindTagNamesFromList(_parameters.Tag))
			{
				FindOrAddTagSet(tagNameToFileSet, tagName).Add(_parameters.ZipFile);
			}

			if(!_parameters.SkipDefaultTag)
			{
				// Add the archive to the set of build products
				buildProducts.Add(_parameters.ZipFile);
			}
			return Task.CompletedTask;
		}

		/// <summary>
		/// Output this task out to an XML writer.
		/// </summary>
		public override void Write(XmlWriter writer)
		{
			Write(writer, _parameters);
		}

		/// <summary>
		/// Find all the tags which are used as inputs to this task
		/// </summary>
		/// <returns>The tag names which are read by this task</returns>
		public override IEnumerable<string> FindConsumedTagNames()
		{
			return FindTagNamesFromFilespec(_parameters.Files);
		}

		/// <summary>
		/// Find all the tags which are modified by this task
		/// </summary>
		/// <returns>The tag names which are modified by this task</returns>
		public override IEnumerable<string> FindProducedTagNames()
		{
			return FindTagNamesFromList(_parameters.Tag);
		}
	}
}
