// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Concurrent;
using System.Collections.Generic;
using System.Diagnostics;
using System.Diagnostics.CodeAnalysis;
using System.IO;
using System.IO.Compression;
using System.Linq;
using System.Text.RegularExpressions;
using System.Threading;
using System.Threading.Tasks;
using System.Xml;
using System.Xml.Serialization;
using EpicGames.Core;
using Microsoft.Extensions.Logging;
using OpenTracing;
using OpenTracing.Util;
using UnrealBuildTool;
using static AutomationTool.CommandUtils;

#nullable enable

namespace AutomationTool
{
	/// <summary>
	/// Stores the name of a temp storage block
	/// </summary>
	public class TempStorageBlockRef
	{
		/// <summary>
		/// Name of the node
		/// </summary>
		[XmlAttribute]
		public string NodeName { get; set; }

		/// <summary>
		/// Name of the output from this node
		/// </summary>
		[XmlAttribute]
		public string OutputName { get; set; }

		/// <summary>
		/// Default constructor, for XML serialization.
		/// </summary>
		private TempStorageBlockRef()
		{
			NodeName = String.Empty;
			OutputName = String.Empty;
		}

		/// <summary>
		/// Construct a temp storage block
		/// </summary>
		/// <param name="nodeName">Name of the node</param>
		/// <param name="outputName">Name of the node's output</param>
		public TempStorageBlockRef(string nodeName, string outputName)
		{
			NodeName = nodeName;
			OutputName = outputName;
		}

		/// <summary>
		/// Tests whether two temp storage blocks are equal
		/// </summary>
		/// <param name="other">The object to compare against</param>
		/// <returns>True if the blocks are equivalent</returns>
		public override bool Equals(object? other)
		{
			TempStorageBlockRef? otherBlock = other as TempStorageBlockRef;
			return otherBlock != null && NodeName == otherBlock.NodeName && OutputName == otherBlock.OutputName;
		}

		/// <summary>
		/// Returns a hash code for this block name
		/// </summary>
		/// <returns>Hash code for the block</returns>
		public override int GetHashCode()
		{
			return ToString().GetHashCode(StringComparison.OrdinalIgnoreCase);
		}

		/// <summary>
		/// Returns the name of this block for debugging purposes
		/// </summary>
		/// <returns>Name of this block as a string</returns>
		public override string ToString()
		{
			return String.Format("{0}/{1}", NodeName, OutputName);
		}
	}

	/// <summary>
	/// Information about a single file in temp storage
	/// </summary>
	[DebuggerDisplay("{RelativePath}")]
	public class TempStorageFile
	{
		/// <summary>
		/// The path of the file, relative to the engine root. Stored using forward slashes.
		/// </summary>
		[XmlAttribute]
		public string RelativePath { get; set; }

		/// <summary>
		/// The last modified time of the file, in UTC ticks since the Epoch.
		/// </summary>
		[XmlAttribute]
		public long LastWriteTimeUtcTicks { get; set; }

		/// <summary>
		/// Length of the file
		/// </summary>
		[XmlAttribute]
		public long Length { get; set; }

		/// <summary>
		/// Digest for the file. Not all files are hashed.
		/// </summary>
		[XmlAttribute]
		public string? Digest { get; set; }

		/// <summary>
		/// Default constructor, for XML serialization.
		/// </summary>
		private TempStorageFile()
		{
			RelativePath = String.Empty;
		}

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="fileInfo">File to be added</param>
		/// <param name="rootDir">Root directory to store paths relative to</param>
		public TempStorageFile(FileInfo fileInfo, DirectoryReference rootDir)
		{
			// Check the file exists and is in the right location
			FileReference file = new FileReference(fileInfo);
			if (!file.IsUnderDirectory(rootDir))
			{
				throw new AutomationException("Attempt to add file to temp storage manifest that is outside the root directory ({0})", file.FullName);
			}
			if (!fileInfo.Exists)
			{
				throw new AutomationException("Attempt to add file to temp storage manifest that does not exist ({0})", file.FullName);
			}

			RelativePath = file.MakeRelativeTo(rootDir).Replace(Path.DirectorySeparatorChar, '/');
			LastWriteTimeUtcTicks = fileInfo.LastWriteTimeUtc.Ticks;
			Length = fileInfo.Length;

			if (GenerateDigest())
			{
				Digest = ContentHash.MD5(file).ToString();
			}
		}

		/// <summary>
		/// Compare stored for this file with the one on disk, and output an error if they differ.
		/// </summary>
		/// <param name="rootDir">Root directory for this branch</param>
		/// <returns>True if the files are identical, false otherwise</returns>
		public bool Compare(DirectoryReference rootDir)
		{
			string? message;
			if (Compare(rootDir, out message))
			{
				if (message != null)
				{
					Logger.LogInformation("{Text}", message);
				}
				return true;
			}
			else
			{
				if (message != null)
				{
					Logger.LogError("{Text}", message);
				}
				return false;
			}
		}

		/// <summary>
		/// Compare stored for this file with the one on disk, and output an error if they differ.
		/// </summary>
		/// <param name="rootDir">Root directory for this branch</param>
		/// <param name="message">Message describing the difference</param>
		/// <returns>True if the files are identical, false otherwise</returns>
		public bool Compare(DirectoryReference rootDir, [NotNullWhen(false)] out string? message)
		{
			FileReference localFile = ToFileReference(rootDir);

			// Get the local file info, and check it exists
			FileInfo info = new FileInfo(localFile.FullName);
			if (!info.Exists)
			{
				message = String.Format("Missing file from manifest - {0}", RelativePath);
				return false;
			}

			// Check the size matches
			if (info.Length != Length)
			{
				if (TempStorage.IsDuplicateBuildProduct(localFile))
				{
					message = String.Format("Ignored file size mismatch for {0} - was {1} bytes, expected {2} bytes", RelativePath, info.Length, Length);
					return true;
				}
				else
				{
					message = String.Format("File size differs from manifest - {0} is {1} bytes, expected {2} bytes", RelativePath, info.Length, Length);
					return false;
				}
			}

			// Check the timestamp of the file matches. On FAT filesystems writetime has a two seconds resolution (see http://msdn.microsoft.com/en-us/library/windows/desktop/ms724290%28v=vs.85%29.aspx)
			TimeSpan timeDifference = new TimeSpan(info.LastWriteTimeUtc.Ticks - LastWriteTimeUtcTicks);
			if (timeDifference.TotalSeconds >= -2 && timeDifference.TotalSeconds <= +2)
			{
				message = null;
				return true;
			}

			// Check if the files have been modified
			DateTime expectedLocal = new DateTime(LastWriteTimeUtcTicks, DateTimeKind.Utc).ToLocalTime();
			if (Digest != null)
			{
				string localDigest = ContentHash.MD5(localFile).ToString();
				if (Digest.Equals(localDigest, StringComparison.Ordinal))
				{
					message = null;
					return true;
				}
				else
				{
					message = String.Format("Digest mismatch for {0} - was {1} ({2}), expected {3} ({4}), TimeDifference {5}", RelativePath, localDigest, info.LastWriteTime, Digest, expectedLocal, timeDifference);
					return false;
				}
			}
			else
			{
				if (RequireMatchingTimestamps() && !TempStorage.IsDuplicateBuildProduct(localFile))
				{
					message = String.Format("File date/time mismatch for {0} - was {1}, expected {2}, TimeDifference {3}", RelativePath, info.LastWriteTime, expectedLocal, timeDifference);
					return false;
				}
				else
				{
					message = String.Format("Ignored file date/time mismatch for {0} - was {1}, expected {2}, TimeDifference {3}", RelativePath, info.LastWriteTime, expectedLocal, timeDifference);
					return true;
				}
			}
		}

		/// <summary>
		/// Whether we should compare timestamps for this file. Some build products are harmlessly overwritten as part of the build process, so we flag those here.
		/// </summary>
		/// <returns>True if we should compare the file's timestamp, false otherwise</returns>
		bool RequireMatchingTimestamps()
		{
			return !RelativePath.Contains("/Binaries/DotNET/", StringComparison.OrdinalIgnoreCase) && !RelativePath.Contains("/Binaries/Mac/", StringComparison.OrdinalIgnoreCase);
		}

		/// <summary>
		/// Determines whether to generate a digest for the current file
		/// </summary>
		/// <returns>True to generate a digest for this file, rather than relying on timestamps</returns>
		bool GenerateDigest()
		{
			return RelativePath.EndsWith(".version", StringComparison.OrdinalIgnoreCase) || RelativePath.EndsWith(".modules", StringComparison.OrdinalIgnoreCase);
		}

		/// <summary>
		/// Determine whether to serialize the digest property
		/// </summary>
		/// <returns></returns>
		public bool ShouldSerializeDigest()
		{
			return Digest != null;
		}

		/// <summary>
		/// Gets a local file reference for this file, given a root directory to base it from.
		/// </summary>
		/// <param name="rootDir">The local root directory</param>
		/// <returns>Reference to the file</returns>
		public FileReference ToFileReference(DirectoryReference rootDir)
		{
			return FileReference.Combine(rootDir, RelativePath.Replace('/', Path.DirectorySeparatorChar));
		}
	}

	/// <summary>
	/// Information about a single file in temp storage
	/// </summary>
	[DebuggerDisplay("{Name}")]
	public class TempStorageZipFile
	{
		/// <summary>
		/// Name of this file, including extension
		/// </summary>
		[XmlAttribute]
		public string Name { get; set; }

		/// <summary>
		/// Length of the file in bytes
		/// </summary>
		[XmlAttribute]
		public long Length { get; set; }

		/// <summary>
		/// Default constructor, for XML serialization
		/// </summary>
		private TempStorageZipFile()
		{
			Name = String.Empty;
		}

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="info">FileInfo for the zip file</param>
		public TempStorageZipFile(FileInfo info)
		{
			Name = info.Name;
			Length = info.Length;
		}
	}

	/// <summary>
	/// A manifest storing information about build products for a node's output
	/// </summary>
	[XmlRoot(ElementName = "TempStorageManifest")]
	public class TempStorageBlockManifest
	{
		/// <summary>
		/// List of output files
		/// </summary>
		[XmlArray]
		[XmlArrayItem("File")]
		public TempStorageFile[] Files { get; set; }

		/// <summary>
		/// List of compressed archives containing the given files
		/// </summary>
		[XmlArray]
		[XmlArrayItem("ZipFile")]
		public TempStorageZipFile[] ZipFiles { get; set; }

		/// <summary>
		/// Construct a static Xml serializer to avoid throwing an exception searching for the reflection info at runtime
		/// </summary>
		static readonly XmlSerializer s_serializer = XmlSerializer.FromTypes(new Type[] { typeof(TempStorageBlockManifest) })[0]!;

		/// <summary>
		/// Construct an empty temp storage manifest
		/// </summary>
		private TempStorageBlockManifest()
		{
			Files = Array.Empty<TempStorageFile>();
			ZipFiles = Array.Empty<TempStorageZipFile>();
		}

		/// <summary>
		/// Creates a manifest from a flat list of files (in many folders) and a BaseFolder from which they are rooted.
		/// </summary>
		/// <param name="files">List of full file paths</param>
		/// <param name="rootDir">Root folder for all the files. All files must be relative to this RootDir.</param>
		public TempStorageBlockManifest(FileInfo[] files, DirectoryReference rootDir)
		{
			Files = files.Select(x => new TempStorageFile(x, rootDir)).ToArray();
			ZipFiles = Array.Empty<TempStorageZipFile>();
		}

		/// <summary>
		/// Gets the total size of the files stored in this manifest
		/// </summary>
		/// <returns>The total size of all files</returns>
		public long GetTotalSize()
		{
			long result = 0;
			foreach (TempStorageFile file in Files)
			{
				result += file.Length;
			}
			return result;
		}

		/// <summary>
		/// Load a manifest from disk
		/// </summary>
		/// <param name="file">File to load</param>
		public static TempStorageBlockManifest Load(FileReference file)
		{
			using (StreamReader reader = new(file.FullName))
			{
				return (TempStorageBlockManifest)(s_serializer.Deserialize(reader) ?? throw new InvalidOperationException());
			}
		}

		/// <summary>
		/// Saves a manifest to disk
		/// </summary>
		/// <param name="file">File to save</param>
		public void Save(FileReference file)
		{
			using (StreamWriter writer = new StreamWriter(file.FullName))
			{
				XmlWriterSettings writerSettings = new() { Indent = true };
				using (XmlWriter xmlWriter = XmlWriter.Create(writer, writerSettings))
				{
					s_serializer.Serialize(xmlWriter, this);
				}
			}
		}
	}

	/// <summary>
	/// Stores the contents of a tagged file set
	/// </summary>
	[XmlRoot(ElementName = "TempStorageFileList")]
	public class TempStorageTagManifest
	{
		/// <summary>
		/// List of files that are in this tag set, relative to the root directory
		/// </summary>
		[XmlArray]
		[XmlArrayItem("LocalFile")]
		public string[] LocalFiles { get; set; }

		/// <summary>
		/// List of files that are in this tag set, but not relative to the root directory
		/// </summary>
		[XmlArray]
		[XmlArrayItem("LocalFile")]
		public string[] ExternalFiles { get; set; }

		/// <summary>
		/// List of referenced storage blocks
		/// </summary>
		[XmlArray]
		[XmlArrayItem("Block")]
		public TempStorageBlockRef[] Blocks { get; set; }

		/// <summary>
		/// List of keys for published artifacts
		/// </summary>
		[XmlArray]
		[XmlArrayItem("ArtifactKey")]
		public string[] ArtifactKeys { get; set; }

		/// <summary>
		/// Construct a static Xml serializer to avoid throwing an exception searching for the reflection info at runtime
		/// </summary>
		static readonly XmlSerializer s_serializer = XmlSerializer.FromTypes(new Type[] { typeof(TempStorageTagManifest) })[0]!;

		/// <summary>
		/// Construct an empty file list for deserialization
		/// </summary>
		private TempStorageTagManifest()
		{
			LocalFiles = Array.Empty<string>();
			ExternalFiles = Array.Empty<string>();
			Blocks = Array.Empty<TempStorageBlockRef>();
			ArtifactKeys = Array.Empty<string>();
		}

		/// <summary>
		/// Creates a manifest from a flat list of files (in many folders) and a BaseFolder from which they are rooted.
		/// </summary>
		/// <param name="files">List of full file paths</param>
		/// <param name="rootDir">Root folder for all the files. All files must be relative to this RootDir.</param>
		/// <param name="blocks">Referenced storage blocks required for these files</param>
		/// <param name="artifactKeys">Keys for published artifacts</param>
		public TempStorageTagManifest(IEnumerable<FileReference> files, DirectoryReference rootDir, IEnumerable<TempStorageBlockRef> blocks, IEnumerable<string> artifactKeys)
		{
			List<string> newLocalFiles = new List<string>();
			List<string> newExternalFiles = new List<string>();
			foreach (FileReference file in files)
			{
				if (file.IsUnderDirectory(rootDir))
				{
					newLocalFiles.Add(file.MakeRelativeTo(rootDir).Replace(Path.DirectorySeparatorChar, '/'));
				}
				else
				{
					newExternalFiles.Add(file.FullName.Replace(Path.DirectorySeparatorChar, '/'));
				}
			}
			LocalFiles = newLocalFiles.ToArray();
			ExternalFiles = newExternalFiles.ToArray();

			Blocks = blocks.ToArray();
			ArtifactKeys = artifactKeys.ToArray();
		}

		/// <summary>
		/// Load this list of files from disk
		/// </summary>
		/// <param name="file">File to load</param>
		public static TempStorageTagManifest Load(FileReference file)
		{
			using (StreamReader reader = new StreamReader(file.FullName))
			{
				return (TempStorageTagManifest)s_serializer.Deserialize(reader)!;
			}
		}

		/// <summary>
		/// Saves this list of files to disk
		/// </summary>
		/// <param name="file">File to save</param>
		public void Save(FileReference file)
		{
			using (StreamWriter writer = new StreamWriter(file.FullName))
			{
				XmlWriterSettings writerSettings = new() { Indent = true };
				using (XmlWriter xmlWriter = XmlWriter.Create(writer, writerSettings))
				{
					s_serializer.Serialize(xmlWriter, this);
				}
			}
		}

		/// <summary>
		/// Converts this file list into a set of FileReference objects
		/// </summary>
		/// <param name="rootDir">The root directory to rebase local files</param>
		/// <returns>Set of files</returns>
		public HashSet<FileReference> ToFileSet(DirectoryReference rootDir)
		{
			HashSet<FileReference> files = new HashSet<FileReference>();
			files.UnionWith(LocalFiles.Select(x => FileReference.Combine(rootDir, x)));
			files.UnionWith(ExternalFiles.Select(x => new FileReference(x)));
			return files;
		}
	}

	/// <summary>
	/// Tracks the state of the current build job using the filesystem, allowing jobs to be restarted after a failure or expanded to include larger targets, and 
	/// providing a proxy for different machines executing parts of the build in parallel to transfer build products and share state as part of a build system.
	/// 
	/// If a shared temp storage directory is provided - typically a mounted path on a network share - all build products potentially needed as inputs by another node
	/// are compressed and copied over, along with metadata for them (see TempStorageFile) and flags for build events that have occurred (see TempStorageEvent).
	/// 
	/// The local temp storage directory contains the same information, with the exception of the archived build products. Metadata is still kept to detect modified 
	/// build products between runs. If data is not present in local temp storage, it's retrieved from shared temp storage and cached in local storage.
	/// </summary>
	class TempStorage
	{
		/// <summary>
		/// Root directory for this branch.
		/// </summary>
		readonly DirectoryReference _rootDir;

		/// <summary>
		/// The local temp storage directory (typically somewhere under /Engine/Saved directory).
		/// </summary>
		readonly DirectoryReference _localDir;

		/// <summary>
		/// The shared temp storage directory; typically a network location. May be null.
		/// </summary>
		readonly DirectoryReference? _sharedDir;

		/// <summary>
		/// Whether to allow writes to shared storage
		/// </summary>
		readonly bool _writeToSharedStorage;

		/// <summary>
		/// Constructor
		/// </summary>
		/// <param name="rootDir">Root directory for this branch</param>
		/// <param name="localDir">The local temp storage directory.</param>
		/// <param name="sharedDir">The shared temp storage directory. May be null.</param>
		/// <param name="writeToSharedStorage">Whether to write to shared storage, or only permit reads from it</param>
		public TempStorage(DirectoryReference rootDir, DirectoryReference localDir, DirectoryReference? sharedDir, bool writeToSharedStorage)
		{
			_rootDir = rootDir;
			_localDir = localDir;
			_sharedDir = sharedDir;
			_writeToSharedStorage = writeToSharedStorage;
		}

		/// <summary>
		/// Cleans all cached local state. We never remove shared storage.
		/// </summary>
		public void CleanLocal()
		{
			CommandUtils.DeleteDirectoryContents(_localDir.FullName);
		}

		/// <summary>
		/// Cleans local build products for a given node. Does not modify shared storage.
		/// </summary>
		/// <param name="nodeName">Name of the node</param>
		public void CleanLocalNode(string nodeName)
		{
			DirectoryReference nodeDir = GetDirectoryForNode(_localDir, nodeName);
			if (DirectoryReference.Exists(nodeDir))
			{
				CommandUtils.DeleteDirectoryContents(nodeDir.FullName);
				CommandUtils.DeleteDirectory_NoExceptions(nodeDir.FullName);
			}
		}

		/// <summary>
		/// Check whether the given node is complete
		/// </summary>
		/// <param name="nodeName">Name of the node</param>
		/// <returns>True if the node is complete</returns>
		public bool IsComplete(string nodeName)
		{
			// Check if it already exists locally
			FileReference localFile = GetCompleteMarkerFile(_localDir, nodeName);
			if (FileReference.Exists(localFile))
			{
				return true;
			}
			
			// Check if it exists in shared storage
			if (_sharedDir != null)
			{
				FileReference sharedFile = GetCompleteMarkerFile(_sharedDir, nodeName);
				if (FileReference.Exists(sharedFile))
				{
					return true;
				}
			}

			// Otherwise we don't have any data
			return false;
		}

		/// <summary>
		/// Mark the given node as complete
		/// </summary>
		/// <param name="nodeName">Name of the node</param>
		public void MarkAsComplete(string nodeName)
		{
			// Create the marker locally
			FileReference localFile = GetCompleteMarkerFile(_localDir, nodeName);
			DirectoryReference.CreateDirectory(localFile.Directory);
			File.OpenWrite(localFile.FullName).Close();

			// Create the marker in the shared directory
			if (_sharedDir != null && _writeToSharedStorage)
			{
				FileReference sharedFile = GetCompleteMarkerFile(_sharedDir, nodeName);
				DirectoryReference.CreateDirectory(sharedFile.Directory);
				File.OpenWrite(sharedFile.FullName).Close();
			}
		}

		/// <summary>
		/// Checks the integrity of the give node's local build products.
		/// </summary>
		/// <param name="nodeName">The node to retrieve build products for</param>
		/// <param name="tagNames">List of tag names from this node.</param>
		/// <param name="ignoreModified">Filter for files to ignore when performing integrity check. Specified per node. </param>
		/// <returns>True if the node is complete and valid, false if not (and typically followed by a call to CleanNode()).</returns>
		public bool CheckLocalIntegrity(string nodeName, IEnumerable<string> tagNames, FileFilter ignoreModified)
		{
			// If the node is not locally complete, fail immediately.
			FileReference completeMarkerFile = GetCompleteMarkerFile(_localDir, nodeName);
			if (!FileReference.Exists(completeMarkerFile))
			{
				return false;
			}

			// Check that each of the tags exist
			HashSet<TempStorageBlockRef> blocks = new HashSet<TempStorageBlockRef>();
			foreach (string tagName in tagNames)
			{
				// Check the local manifest exists
				FileReference localFileListLocation = GetTaggedFileListLocation(_localDir, nodeName, tagName);
				if (!FileReference.Exists(localFileListLocation))
				{
					return false;
				}

				// Check the local manifest matches the shared manifest
				if (_sharedDir != null)
				{
					// Check the shared manifest exists
					FileReference sharedFileListLocation = GetManifestLocation(_sharedDir, nodeName, tagName);
					if (!FileReference.Exists(sharedFileListLocation))
					{
						return false;
					}

					// Check the manifests are identical, byte by byte
					byte[] localManifestBytes = File.ReadAllBytes(localFileListLocation.FullName);
					byte[] sharedManifestBytes = Array.Empty<byte>();
					PerformActionWithRetries(() => sharedManifestBytes = File.ReadAllBytes(sharedFileListLocation.FullName), 3, TimeSpan.FromSeconds(1));
					if (!localManifestBytes.SequenceEqual(sharedManifestBytes))
					{
						return false;
					}
				}

				// Read the manifest and add the referenced blocks to be checked
				TempStorageTagManifest localFileList = TempStorageTagManifest.Load(localFileListLocation);
				blocks.UnionWith(localFileList.Blocks);
			}

			// Check that each of the outputs match
			foreach (TempStorageBlockRef block in blocks)
			{
				// Check the local manifest exists
				FileReference localManifestFile = GetManifestLocation(_localDir, block.NodeName, block.OutputName);
				if (!FileReference.Exists(localManifestFile))
				{
					return false;
				}

				// Check the local manifest matches the shared manifest
				if (_sharedDir != null)
				{
					// Check the shared manifest exists
					FileReference sharedManifestFile = GetManifestLocation(_sharedDir, block.NodeName, block.OutputName);
					if (!FileReference.Exists(sharedManifestFile))
					{
						return false;
					}

					// Check the manifests are identical, byte by byte
					byte[] localManifestBytes = File.ReadAllBytes(localManifestFile.FullName);
					byte[] sharedManifestBytes = Array.Empty<byte>();
					PerformActionWithRetries(() => sharedManifestBytes = File.ReadAllBytes(sharedManifestFile.FullName), 3, TimeSpan.FromSeconds(1));
					if (!localManifestBytes.SequenceEqual(sharedManifestBytes))
					{
						return false;
					}
				}

				// Read the manifest and check the files
				TempStorageBlockManifest localManifest = TempStorageBlockManifest.Load(localManifestFile);
				if (localManifest.Files.Any(x => !ignoreModified.Matches(x.ToFileReference(_rootDir).FullName) && !x.Compare(_rootDir)))
				{
					return false;
				}
			}
			return true;
		}

		/// <summary>
		/// Reads a set of tagged files from disk
		/// </summary>
		/// <param name="nodeName">Name of the node which produced the tag set</param>
		/// <param name="tagName">Name of the tag, with a '#' prefix</param>
		/// <returns>The set of files</returns>
		public TempStorageTagManifest? ReadTagFileList(string nodeName, string tagName)
		{
#pragma warning disable CA1508 // False positive; FileList is always null
			TempStorageTagManifest? fileList = null;

			// Try to read the tag set from the local directory
			FileReference localFileListLocation = GetTaggedFileListLocation(_localDir, nodeName, tagName);
			if (FileReference.Exists(localFileListLocation))
			{
				Logger.LogInformation("Reading tag \"{NodeName}\":\"{TagName}\" from local file {File}", nodeName, tagName, localFileListLocation.FullName);
				fileList = TempStorageTagManifest.Load(localFileListLocation);
			}
			else
			{
				// Check we have shared storage
				if (_sharedDir == null)
				{
					throw new AutomationException("Missing local file list - {0}", localFileListLocation.FullName);
				}

				// Make sure the manifest exists. Try up to 5 times with a 5s wait between to harden against network hiccups.
				int attempts = 5;
				FileReference sharedFileListLocation;
				sharedFileListLocation = GetTaggedFileListLocation(_sharedDir, nodeName, tagName);
				PerformActionWithRetries(() =>
				{
					if (!FileReference.Exists(sharedFileListLocation))
					{
						throw new AutomationException("Missing local or shared file list - {0}", sharedFileListLocation.FullName);
					}
				}, attempts, TimeSpan.FromSeconds(5));

				try
				{
					PerformActionWithRetries(() =>
					{
						// Read the shared manifest
						Logger.LogInformation("Copying tag \"{NodeName}\":\"{TagName}\" from {SourceFile} to {TargetFile}", nodeName, tagName, sharedFileListLocation.FullName, localFileListLocation.FullName);
						fileList = TempStorageTagManifest.Load(sharedFileListLocation);
					}, attempts, TimeSpan.FromSeconds(5));
				}
				catch
				{
					throw new AutomationException("Local or shared file list {0} was found but failed to be read", sharedFileListLocation.FullName);
				}

				// Save the manifest locally
				DirectoryReference.CreateDirectory(localFileListLocation.Directory);
				fileList?.Save(localFileListLocation);
			}
			return fileList;
#pragma warning restore CA1508
		}

		/// <summary>
		/// Writes a list of tagged files to disk
		/// </summary>
		/// <param name="nodeName">Name of the node which produced the tag set</param>
		/// <param name="tagName">Name of the tag, with a '#' prefix</param>
		/// <param name="files">List of files in this set</param>
		/// <param name="blocks">List of referenced storage blocks</param>
		/// <param name="artifactKeys">Keys for published artifacts</param>
		/// <returns>The set of files</returns>
		public void WriteFileList(string nodeName, string tagName, IEnumerable<FileReference> files, IEnumerable<TempStorageBlockRef> blocks, IEnumerable<string> artifactKeys)
		{
			// Create the file list
			TempStorageTagManifest fileList = new TempStorageTagManifest(files, _rootDir, blocks, artifactKeys);

			// Save the set of files to the local and shared locations
			FileReference localFileListLocation = GetTaggedFileListLocation(_localDir, nodeName, tagName);
			if (_sharedDir != null && _writeToSharedStorage)
			{
				FileReference sharedFileListLocation = GetTaggedFileListLocation(_sharedDir, nodeName, tagName);

				try
				{
					PerformActionWithRetries(() =>
					{
						Logger.LogInformation("Saving file list to {Arg0} and {Arg1}", localFileListLocation.FullName, sharedFileListLocation.FullName);
						DirectoryReference.CreateDirectory(sharedFileListLocation.Directory);
						fileList.Save(sharedFileListLocation);
					}, 3, TimeSpan.FromSeconds(5));
				}
				catch (Exception ex)
				{
					throw new AutomationException("Failed to save file list {0} to {1}, exception: {2}", localFileListLocation, sharedFileListLocation, ex);
				}
			}
			else
			{
				Logger.LogInformation("Saving file list to {Arg0}", localFileListLocation.FullName);
			}

			// Save the local file list
			DirectoryReference.CreateDirectory(localFileListLocation.Directory);
			fileList.Save(localFileListLocation);
		}

		/// <summary>
		/// Saves the given files (that should be rooted at the branch root) to a shared temp storage manifest with the given temp storage node and game.
		/// </summary>
		/// <param name="nodeName">The node which created the storage block</param>
		/// <param name="blockName">Name of the block to retrieve. May be null or empty.</param>
		/// <param name="buildProducts">Array of build products to be archived</param>
		/// <param name="pushToRemote">Allow skipping the copying of this manifest to shared storage, because it's not required by any other agent</param>
		/// <returns>The created manifest instance (which has already been saved to disk).</returns>
		public TempStorageBlockManifest Archive(string nodeName, string? blockName, FileReference[] buildProducts, bool pushToRemote = true)
		{
			using (IScope scope = GlobalTracer.Instance.BuildSpan("StoreToTempStorage").StartActive())
			{
				// Create a manifest for the given build products
				FileInfo[] files = buildProducts.Select(x => new FileInfo(x.FullName)).ToArray();
				TempStorageBlockManifest manifest = new TempStorageBlockManifest(files, _rootDir);

				// Create the local directory for this node
				DirectoryReference localNodeDir = GetDirectoryForNode(_localDir, nodeName);
				DirectoryReference.CreateDirectory(localNodeDir);

				// Compress the files and copy to shared storage if necessary
				bool remote = _sharedDir != null && pushToRemote && _writeToSharedStorage;
				if (remote)
				{
					// Create the shared directory for this node
					FileReference sharedManifestFile = GetManifestLocation(_sharedDir!, nodeName, blockName);
					DirectoryReference.CreateDirectory(sharedManifestFile.Directory);

					// Zip all the build products
					FileInfo[] zipFiles = ParallelZipFiles(files, _rootDir, sharedManifestFile.Directory, localNodeDir, sharedManifestFile.GetFileNameWithoutExtension());
					manifest.ZipFiles = zipFiles.Select(x => new TempStorageZipFile(x)).ToArray();

					// Save the shared manifest
					Logger.LogInformation("Saving block \"{NodeName}\":\"{BlockName}\" manifest to {File}", nodeName, blockName, sharedManifestFile.FullName);
					PerformActionWithRetries(() => manifest.Save(sharedManifestFile), 3, TimeSpan.FromSeconds(5));
				}

				// Save the local manifest
				FileReference localManifestFile = GetManifestLocation(_localDir, nodeName, blockName);
				Logger.LogInformation("Saving block \"{NodeName}\":\"{BlockName}\" manifest to {File}", nodeName, blockName, localManifestFile.FullName);
				manifest.Save(localManifestFile);

				// Update the stats
				long zipFilesTotalSize = (manifest.ZipFiles == null) ? 0 : manifest.ZipFiles.Sum(x => x.Length);
				scope.Span.SetTag("numFiles", files.Length);
				scope.Span.SetTag("manifestSize", manifest.GetTotalSize());
				scope.Span.SetTag("manifestZipFilesSize", zipFilesTotalSize);
				scope.Span.SetTag("isRemote", remote);
				scope.Span.SetTag("blockName", blockName);
				return manifest;
			}
		}

		/// <summary>
		/// Retrieve an output of the given node. Fetches and decompresses the files from shared storage if necessary, or validates the local files.
		/// </summary>
		/// <param name="nodeName">The node which created the storage block</param>
		/// <param name="outputName">Name of the block to retrieve. May be null or empty.</param>
		/// <param name="ignoreModified">Filter for files to ignore</param>
		/// <returns>Manifest of the files retrieved</returns>
		public TempStorageBlockManifest Retrieve(string nodeName, string? outputName, FileFilter ignoreModified)
		{
			using (IScope scope = GlobalTracer.Instance.BuildSpan("RetrieveFromTempStorage").StartActive())
			{
				// Get the path to the local manifest
				FileReference localManifestFile = GetManifestLocation(_localDir, nodeName, outputName);
				bool local = FileReference.Exists(localManifestFile);

				// Read the manifest, either from local storage or shared storage
				TempStorageBlockManifest? manifest = null;
				if (local)
				{
					Logger.LogInformation("Reading tag \"{NodeName}\":\"{TagName}\" manifest from {File}", nodeName, outputName, localManifestFile.FullName);
					manifest = TempStorageBlockManifest.Load(localManifestFile);
				}
				else
				{
					// Check we have shared storage
					if (_sharedDir == null)
					{
						throw new AutomationException("Missing local manifest for node - {0}", localManifestFile.FullName);
					}

					// Get the shared directory for this node
					FileReference sharedManifestFile = GetManifestLocation(_sharedDir, nodeName, outputName);

					// Make sure the manifest exists
					if (!FileReference.Exists(sharedManifestFile))
					{
						throw new AutomationException("Missing local or shared manifest for node - {0}", sharedManifestFile.FullName);
					}

					// Read the shared manifest
					Logger.LogInformation("Copying tag \"{NodeName}\":\"{TagName}\" from {SourceFile} to {TargetFile}", nodeName, outputName, sharedManifestFile.FullName, localManifestFile.FullName);
					PerformActionWithRetries(() => manifest = TempStorageBlockManifest.Load(sharedManifestFile), 3, TimeSpan.FromSeconds(5));

					// Unzip all the build products
					DirectoryReference sharedNodeDir = GetDirectoryForNode(_sharedDir, nodeName);
					FileInfo[] zipFiles = manifest!.ZipFiles.Select(x => new FileInfo(FileReference.Combine(sharedNodeDir, x.Name).FullName)).ToArray();
					string result = ParallelUnzipFiles(zipFiles, _rootDir);
					if (!String.IsNullOrWhiteSpace(result))
					{
						string logPath = CommandUtils.CombinePaths(CmdEnv.LogFolder, $"Copy Manifest - {nodeName}.log");
						Logger.LogInformation("Saving copy log to {LogPath}", logPath);
						Directory.CreateDirectory(Path.GetDirectoryName(logPath)!);
						File.WriteAllText(logPath, result);
					}

					// Update the timestamps to match the manifest. Zip files only use local time, and there's no guarantee it matches the local clock.
					foreach (TempStorageFile manifestFile in manifest.Files)
					{
						FileReference file = manifestFile.ToFileReference(_rootDir);
						System.IO.File.SetLastWriteTimeUtc(file.FullName, new DateTime(manifestFile.LastWriteTimeUtcTicks, DateTimeKind.Utc));
					}

					// Save the manifest locally
					DirectoryReference.CreateDirectory(localManifestFile.Directory);
					manifest.Save(localManifestFile);
				}

				// Check all the local files are as expected
				List<string> modifiedFileMessages = new List<string>();
				foreach (TempStorageFile file in manifest.Files)
				{
					string? message;
					if (!ignoreModified.Matches(file.ToFileReference(_rootDir).FullName) && !file.Compare(_rootDir, out message))
					{
						modifiedFileMessages.Add(message);
					}
				}
				if (modifiedFileMessages.Count > 0)
				{
					string modifiedFileList = "";
					if (modifiedFileMessages.Count < 100)
					{
						modifiedFileList = String.Join("\n", modifiedFileMessages.Select(x => $"  {x}"));
					}
					else
					{
						modifiedFileList = String.Join("\n", modifiedFileMessages.Take(100).Select(x => $"  {x}"));
						modifiedFileList += $"\n  ...and {modifiedFileMessages.Count - 100} more.";
					}
					throw new AutomationException($"Files have been modified:\n{modifiedFileList}");
				}

				// Update the stats and return
				scope.Span.SetTag("numFiles", manifest.Files.Length);
				scope.Span.SetTag("manifestSize", manifest.Files.Sum(x => x.Length));
				scope.Span.SetTag("manifestZipFilesSize", local ? 0 : manifest.ZipFiles.Sum(x => x.Length));
				scope.Span.SetTag("isRemote", !local);
				scope.Span.SetTag("outputName", outputName);
				return manifest;
			}
		}

		static void PerformActionWithRetries(Action retryAction, int retryCount, TimeSpan waitTime)
		{
			while (retryCount-- > 0)
			{
				try
				{
					retryAction();
					break;
				}
				catch
				{
					if (retryCount == 0)
					{
						throw;
					}

					Thread.Sleep(waitTime);
				}
			}
		}

		/// <summary>
		/// Zips a set of files (that must be rooted at the given RootDir) to a set of zip files in the given OutputDir. The files will be prefixed with the given basename.
		/// </summary>
		/// <param name="inputFiles">Fully qualified list of files to zip (must be rooted at RootDir).</param>
		/// <param name="rootDir">Root Directory where all files will be extracted.</param>
		/// <param name="outputDir">Location to place the set of zip files created.</param>
		/// <param name="stagingDir">Location to create zip files before copying them to the OutputDir. If the OutputDir is on a remote file share, staging may be more efficient. Use null to avoid using a staging copy.</param>
		/// <param name="zipBaseName">The basename of the set of zip files.</param>
		/// <returns>Some metrics about the zip process.</returns>
		/// <remarks>
		/// This function tries to zip the files in parallel as fast as it can. It makes no guarantees about how many zip files will be created or which files will be in which zip,
		/// but it does try to reasonably balance the file sizes.
		/// </remarks>
		static FileInfo[] ParallelZipFiles(FileInfo[] inputFiles, DirectoryReference rootDir, DirectoryReference outputDir, DirectoryReference stagingDir, string zipBaseName)
		{
			// First get the sizes of all the files. We won't parallelize if there isn't enough data to keep the number of zips down.
			var filesInfo = inputFiles
				.Select(inputFile => new { File = new FileReference(inputFile), FileSize = inputFile.Length })
				.ToList();

			// Profiling results show that we can zip 100MB quite fast and it is not worth parallelizing that case and creating a bunch of zips that are relatively small.
			const long MinFileSizeToZipInParallel = 1024 * 1024 * 100L;
			bool zipInParallel = filesInfo.Sum(fileInfo => fileInfo.FileSize) >= MinFileSizeToZipInParallel;

			// order the files in descending order so our threads pick up the biggest ones first.
			// We want to end with the smaller files to more effectively fill in the gaps
			ConcurrentQueue<FileReference> filesToZip = new(filesInfo.OrderByDescending(fileInfo => fileInfo.FileSize).Select(fileInfo => fileInfo.File));

			ConcurrentBag<FileInfo> zipFiles = new ConcurrentBag<FileInfo>();

			DirectoryReference zipDir = stagingDir ?? outputDir;

			// We deliberately avoid Parallel.ForEach here because profiles have shown that dynamic partitioning creates
			// too many zip files, and they can be of too varying size, creating uneven work when unzipping later,
			// as ZipFile cannot unzip files in parallel from a single archive.
			// We can safely assume the build system will not be doing more important things at the same time, so we simply use all our logical cores,
			// which has shown to be optimal via profiling, and limits the number of resulting zip files to the number of logical cores.
			List<Thread> zipThreads = (
				from CoreNum in Enumerable.Range(0, zipInParallel ? Environment.ProcessorCount : 1)
				select new Thread((object? indexObject) =>
				{
					int index = (int)indexObject!;
					FileReference zipFileName = FileReference.Combine(zipDir, String.Format("{0}{1}.zip", zipBaseName, zipInParallel ? "-" + index.ToString("00") : ""));
					// don't create the zip unless we have at least one file to add
					FileReference? file;
					if (filesToZip.TryDequeue(out file))
					{
						try
						{
							// Create one zip per thread using the given basename
							using (ZipArchive zipArchive = ZipFile.Open(zipFileName.FullName, ZipArchiveMode.Create))
							{
								// pull from the queue until we are out of files.
								do
								{
									// use fastest compression. In our best case we are CPU bound, so this is a good tradeoff,
									// cutting overall time by 2/3 while only modestly increasing the compression ratio (22.7% -> 23.8% for RootEditor PDBs).
									// This is in cases of a super hot cache, so the operation was largely CPU bound.
									ZipArchiveExtensions.CreateEntryFromFile_CrossPlatform(zipArchive, file.FullName, CommandUtils.ConvertSeparators(PathSeparator.Slash, file.MakeRelativeTo(rootDir)), CompressionLevel.Fastest);
								} while (filesToZip.TryDequeue(out file));
							}
						}
						catch (IOException)
						{
							Logger.LogError("Unable to open file for TempStorage zip: \"{Arg0}\"", zipFileName.FullName);
							throw new AutomationException("Unable to open file {0}", zipFileName.FullName);
						}

						zipFiles.Add(new FileInfo(zipFileName.FullName));
					}
				})).ToList();

			for (int index = 0; index < zipThreads.Count; index++)
			{
				Thread thread = zipThreads[index];
				thread.Start(index);
			}

			zipThreads.ForEach(thread => thread.Join());

			if (zipFiles.Any() && !String.IsNullOrWhiteSpace(stagingDir?.FullName))
			{
				try
				{
					string copyResult = CopyDirectory(zipDir, outputDir);
					string logPath = CommandUtils.CombinePaths(CmdEnv.LogFolder, $"Copy Files to Temp Storage.log");
					Logger.LogInformation("Saving copy log to {LogPath}", logPath);
					Directory.CreateDirectory(Path.GetDirectoryName(logPath)!);
					File.WriteAllText(logPath, String.Join(Environment.NewLine, copyResult));
					Parallel.ForEach(zipFiles, (z) => z.Delete());
					zipFiles = new ConcurrentBag<FileInfo>(zipFiles.Select(z => new FileInfo(CommandUtils.MakeRerootedFilePath(z.FullName, stagingDir.FullName, outputDir.FullName))));
				}
				catch (IOException ex)
				{
					Logger.LogError("Unable to copy staging directory {Arg0} to {Arg1}, Ex: {Ex}", zipDir.FullName, outputDir.FullName, ex);
					throw new AutomationException(ex, "Unable to copy staging directory {0} to {1}", zipDir.FullName, outputDir.FullName);
				}
			}

			return zipFiles.OrderBy(x => x.Name).ToArray();
		}

		/// <summary>
		/// Unzips a set of zip files with a given basename in a given folder to a given RootDir.
		/// </summary>
		/// <param name="zipFiles">Files to extract</param>
		/// <param name="rootDir">Root Directory where all files will be extracted.</param>
		/// <returns>Some metrics about the unzip process.</returns>
		/// <remarks>
		/// The code is expected to be the used as the symmetrical inverse of <see cref="ParallelZipFiles"/>, but could be used independently, as long as the files in the zip do not overlap.
		/// </remarks>
		private static string ParallelUnzipFiles(FileInfo[] zipFiles, DirectoryReference rootDir)
		{
			ConcurrentBag<string> copyResults = new ConcurrentBag<string>();
			Parallel.ForEach(zipFiles,
				(zipFile) =>
				{
					// Copy the ZIP to the local drive before unzipping to harden against network issues.
					copyResults.Add(CopyFile(zipFile, rootDir));
					string localZipFile = CommandUtils.CombinePaths(rootDir.FullName, zipFile.Name);

					// unzip the files manually instead of caling ZipFile.ExtractToDirectory() because we need to overwrite readonly files. Because of this, creating the directories is up to us as well.
					List<string> extractedPaths = new List<string>();
					int unzipFileAttempts = 3;
					while (unzipFileAttempts-- > 0)
					{
						try
						{
							using (ZipArchive zipArchive = System.IO.Compression.ZipFile.OpenRead(localZipFile))
							{
								foreach (ZipArchiveEntry entry in zipArchive.Entries)
								{
									// Use CommandUtils.CombinePaths to ensure directory separators get converted correctly.
									string extractedFilename = CommandUtils.CombinePaths(rootDir.FullName, entry.FullName);

									// Skip this if it's already been extracted.
									if (extractedPaths.Contains(extractedFilename))
									{
										continue;
									}

									// Zips can contain empty dirs. Ours usually don't have them, but we should support it.
									if (Path.GetFileName(extractedFilename).Length == 0)
									{
										Directory.CreateDirectory(extractedFilename);
										extractedPaths.Add(extractedFilename);
									}
									else
									{
										// We must delete any existing file, even if it's readonly. .Net does not do this by default.
										if (File.Exists(extractedFilename))
										{
											InternalUtils.SafeDeleteFile(extractedFilename, true);
										}
										else
										{
											Directory.CreateDirectory(Path.GetDirectoryName(extractedFilename)!);
										}

										int unzipEntryAttempts = 3;
										while (unzipEntryAttempts-- > 0)
										{
											try
											{
												entry.ExtractToFile_CrossPlatform(extractedFilename, true);
												extractedPaths.Add(extractedFilename);
												break;
											}
											catch (IOException ioEx)
											{
												if (unzipEntryAttempts == 0)
												{
													throw;
												}

												Log.Logger.LogWarning(ioEx, "Failed to unzip '{File}' from '{LocalZipFile}' to '{ExtractedFilename}', retrying.. (Error: {Message})", entry.FullName, localZipFile, extractedFilename, ioEx.Message);
											}
										}
									}
								}
							}

							break;
						}
						catch (Exception ex)
						{
							if (unzipFileAttempts == 0)
							{
								Log.Logger.LogError(ex, "All retries exhausted attempting to unzip entries from '{LocalZipFile}'. Terminating.", localZipFile);
								string logPath = CommandUtils.CombinePaths(CmdEnv.LogFolder, $"Copy Manifest - {zipFile.Name}.log");
								Logger.LogInformation("Saving copy log to {LogPath}", logPath);
								Directory.CreateDirectory(Path.GetDirectoryName(logPath)!);
								File.WriteAllText(logPath, String.Join(Environment.NewLine, copyResults));
								throw;
							}

							// Some exceptions may be caused by networking hiccups. We want to retry in those cases.
							if ((ex is IOException || ex is InvalidDataException))
							{
								Log.Logger.LogWarning(ex, "Failed to unzip entries from '{LocalZipFile}' to '{TargetDir}', retrying.. (Error: {Message})", localZipFile, rootDir.FullName, ex.Message);
							}
						}
						finally
						{
							if (File.Exists(localZipFile))
							{
								File.Delete(localZipFile);
							}
						}
					}
				});

			return String.Join(Environment.NewLine, copyResults);
		}

		/// <summary>
		/// Gets the directory used to store data for the given node
		/// </summary>
		/// <param name="baseDir">A local or shared temp storage root directory.</param>
		/// <param name="nodeName">Name of the node</param>
		/// <returns>Directory to contain a node's data</returns>
		static DirectoryReference GetDirectoryForNode(DirectoryReference baseDir, string nodeName)
		{
			return DirectoryReference.Combine(baseDir, nodeName);
		}

		/// <summary>
		/// Gets the path to the manifest created for a node's output.
		/// </summary>
		/// <param name="baseDir">A local or shared temp storage root directory.</param>
		/// <param name="nodeName">Name of the node to get the file for</param>
		/// <param name="blockName">Name of the output block to get the manifest for</param>
		static FileReference GetManifestLocation(DirectoryReference baseDir, string nodeName, string? blockName)
		{
			return FileReference.Combine(baseDir, nodeName, String.IsNullOrEmpty(blockName) ? "Manifest.xml" : String.Format("Manifest-{0}.xml", blockName));
		}

		/// <summary>
		/// Gets the path to the file created to store a tag manifest for a node
		/// </summary>
		/// <param name="baseDir">A local or shared temp storage root directory.</param>
		/// <param name="nodeName">Name of the node to get the file for</param>
		/// <param name="tagName">Name of the tag to get the manifest for</param>
		static FileReference GetTaggedFileListLocation(DirectoryReference baseDir, string nodeName, string tagName)
		{
			Debug.Assert(tagName.StartsWith("#", StringComparison.Ordinal));
			return FileReference.Combine(baseDir, nodeName, String.Format("Tag-{0}.xml", tagName.Substring(1)));
		}

		/// <summary>
		/// Gets the path to a file created to indicate that a node is complete, under the given base directory.
		/// </summary>
		/// <param name="baseDir">A local or shared temp storage root directory.</param>
		/// <param name="nodeName">Name of the node to get the file for</param>
		static FileReference GetCompleteMarkerFile(DirectoryReference baseDir, string nodeName)
		{
			return FileReference.Combine(GetDirectoryForNode(baseDir, nodeName), "Complete");
		}

		/// <summary>
		/// Checks whether the given path is allowed as a build product that can be produced by more than one node (timestamps may be modified, etc..). Used to suppress
		/// warnings about build products being overwritten.
		/// </summary>
		/// <param name="localFile">File name to check</param>
		/// <returns>True if the given path may be output by multiple build products</returns>
		public static bool IsDuplicateBuildProduct(FileReference localFile)
		{
			string fileName = localFile.GetFileName();
			if (fileName.Equals("AgentInterface.dll", StringComparison.OrdinalIgnoreCase) || fileName.Equals("AgentInterface.pdb", StringComparison.OrdinalIgnoreCase))
			{
				return true;
			}
			if (fileName.Equals("dxil.dll", StringComparison.OrdinalIgnoreCase))
			{
				return true;
			}
			if (fileName.Equals("dxcompiler.dll", StringComparison.OrdinalIgnoreCase))
			{
				return true;
			}
			if (fileName.StartsWith("tbb", StringComparison.OrdinalIgnoreCase) && fileName.EndsWith(".pdb", StringComparison.OrdinalIgnoreCase))
			{
				return true;
			}
			if (fileName.EndsWith(".dll", StringComparison.OrdinalIgnoreCase))
			{
				return true;
			}
			if (fileName.StartsWith("lib", StringComparison.OrdinalIgnoreCase) && fileName.EndsWith(".dylib", StringComparison.OrdinalIgnoreCase))
			{
				return true;
			}
			if (fileName.StartsWith("lib", StringComparison.OrdinalIgnoreCase) && fileName.EndsWith(".so", StringComparison.OrdinalIgnoreCase))
			{
				return true;
			}
			if (fileName.StartsWith("lib", StringComparison.OrdinalIgnoreCase) && fileName.Contains(".so.", StringComparison.OrdinalIgnoreCase))
			{
				// e.g. a Unix shared library with a version number suffix.
				return true;
			}
			if (fileName.Equals("plugInfo.json", StringComparison.OrdinalIgnoreCase))
			{
				return true;
			}
			if ((fileName.Equals("info.plist", StringComparison.OrdinalIgnoreCase) || fileName.Equals("coderesources", StringComparison.OrdinalIgnoreCase)) && localFile.FullName.Contains(".app/", StringComparison.OrdinalIgnoreCase))
			{
				// xcode can generate plist files and coderesources differently in different stages of compile/cook/stage/package/etc. only allow ones inside a .app bundle
				return true;
			}
			return false;
		}
		
		/// <summary>
		/// Copy a temp storage .zip file to directory.
		/// Uses Robocopy on Windows, rsync on Linux/macOS and native .NET API when under Wine.
		/// </summary>
		/// <param name="zipFile">.zip file to copy</param>
		/// <param name="rootDir">Destination dir</param>
		/// <returns>Output from the copy operation</returns>
		static string CopyFile(FileInfo zipFile, DirectoryReference rootDir)
		{
			if (RuntimePlatform.IsRunningOnWine())
			{
				string sourceFile = zipFile.FullName;
				string destFile = Path.Join(rootDir.FullName, zipFile.Name);
				File.Copy(sourceFile, destFile, true);
				return $".NET file copy. Source='{sourceFile}' Dest='{destFile}'";
			}
			else if (HostPlatform.Current.HostEditorPlatform == UnrealTargetPlatform.Win64)
			{
				return CommandUtils.RunAndLog(GetRoboCopyExe(), $"\"{zipFile.DirectoryName}\" \"{rootDir}\" \"{zipFile.Name}\" /w:5 /r:10", MaxSuccessCode: 3, Options: CommandUtils.ERunOptions.AppMustExist | CommandUtils.ERunOptions.NoLoggingOfRunCommand);
			}
			else
			{
				return CommandUtils.RunAndLog("rsync", $"-v \"{zipFile.DirectoryName}/{zipFile.Name}\" \"{rootDir}/{zipFile.Name}\"", Options: CommandUtils.ERunOptions.AppMustExist | CommandUtils.ERunOptions.NoLoggingOfRunCommand);
			}
		}

		/// <summary>
		/// Copy a directory using the most suitable option
		/// </summary>
		/// <param name="sourceDir">Source directory</param>
		/// <param name="destinationDir">Directory directory</param>
		/// <returns>Output from directory copy operation</returns>
		/// <exception cref="DirectoryNotFoundException">If source directory wasn't found</exception>
		static string CopyDirectory(DirectoryReference sourceDir, DirectoryReference destinationDir)
		{
			if (RuntimePlatform.IsRunningOnWine())
			{
				return CopyDirectoryDotNet(sourceDir, destinationDir);
			}
			else
			{
				return CopyDirectoryExternalTool(sourceDir, destinationDir);
			}
		}

		/// <summary>
		/// Copy a directory using an external tool native to OS
		/// Robocopy Windows and rsync for Linux and macOS.
		/// </summary>
		/// <param name="sourceDir">Source directory</param>
		/// <param name="destinationDir">Directory directory</param>
		/// <returns>Output from directory copy operation</returns>
		/// <exception cref="DirectoryNotFoundException">If source directory wasn't found</exception>
		static string CopyDirectoryExternalTool(DirectoryReference sourceDir, DirectoryReference destinationDir)
		{
			if (HostPlatform.Current.HostEditorPlatform == UnrealTargetPlatform.Win64)
			{
				return CommandUtils.RunAndLog(GetRoboCopyExe(), $"\"{sourceDir}\" \"{destinationDir}\" * /S /w:5 /r:10", MaxSuccessCode: 3, Options: CommandUtils.ERunOptions.AppMustExist | CommandUtils.ERunOptions.NoLoggingOfRunCommand);
			}
			else
			{
				return CommandUtils.RunAndLog("rsync", $"-vam --include=\"**\" \"{sourceDir}/\" \"{destinationDir}/\"", Options: CommandUtils.ERunOptions.AppMustExist | CommandUtils.ERunOptions.NoLoggingOfRunCommand);
			}
		}
		
		/// <summary>
		/// Copy a directory using .NET SDK directory and file copy API
		/// </summary>
		/// <param name="sourceDir">Source directory</param>
		/// <param name="destinationDir">Directory directory</param>
		/// <returns>Output from directory copy operation</returns>
		/// <exception cref="DirectoryNotFoundException">If source directory wasn't found</exception>
		static string CopyDirectoryDotNet(DirectoryReference sourceDir, DirectoryReference destinationDir)
		{
			DirectoryInfo dir = new(sourceDir.FullName);

			if (!dir.Exists)
			{
				throw new DirectoryNotFoundException($"Source directory not found: {dir.FullName}");
			}

			DirectoryInfo[] dirs = dir.GetDirectories();
			Directory.CreateDirectory(destinationDir.FullName);

			foreach (FileInfo file in dir.GetFiles())
			{
				string targetFilePath = Path.Combine(destinationDir.FullName, file.Name);
				file.CopyTo(targetFilePath, true);
			}

			foreach (DirectoryInfo subDir in dirs)
			{
				string newDestinationDir = Path.Combine(destinationDir.FullName, subDir.Name);
				CopyDirectoryDotNet(new DirectoryReference(subDir.FullName), new DirectoryReference(newDestinationDir));
			}

			return ".NET directory copy";
		}

		static string GetRoboCopyExe()
		{
			string result = CommandUtils.CombinePaths(Environment.SystemDirectory, "robocopy.exe");
			if (!CommandUtils.FileExists(result))
			{
				// Use Regex.Replace so we can do a case-insensitive replacement of System32
				string sysNativeDirectory = Regex.Replace(Environment.SystemDirectory, "System32", "Sysnative", RegexOptions.IgnoreCase);
				string sysNativeExe = CommandUtils.CombinePaths(sysNativeDirectory, "robocopy.exe");
				if (CommandUtils.FileExists(sysNativeExe))
				{
					result = sysNativeExe;
				}
			}
			return result;
		}
	}

	/// <summary>
	/// Automated tests for temp storage
	/// </summary>
	class TempStorageTests : BuildCommand
	{
		/// <summary>
		/// Run the automated tests
		/// </summary>
		public override void ExecuteBuild()
		{
			// Get all the shared directories
			DirectoryReference rootDir = new DirectoryReference(CommandUtils.CmdEnv.LocalRoot);

			DirectoryReference localDir = DirectoryReference.Combine(rootDir, "Engine", "Saved", "TestTempStorage-Local");
			CommandUtils.CreateDirectory(localDir);
			CommandUtils.DeleteDirectoryContents(localDir.FullName);

			DirectoryReference sharedDir = DirectoryReference.Combine(rootDir, "Engine", "Saved", "TestTempStorage-Shared");
			CommandUtils.CreateDirectory(sharedDir);
			CommandUtils.DeleteDirectoryContents(sharedDir.FullName);

			DirectoryReference workingDir = DirectoryReference.Combine(rootDir, "Engine", "Saved", "TestTempStorage-Working");
			CommandUtils.CreateDirectory(workingDir);
			CommandUtils.DeleteDirectoryContents(workingDir.FullName);

			// Create the temp storage object
			TempStorage tempStore = new TempStorage(workingDir, localDir, sharedDir, true);

			// Create a working directory, and copy some source files into it
			DirectoryReference sourceDir = DirectoryReference.Combine(rootDir, "Engine", "Source", "Runtime");
			if (!CommandUtils.CopyDirectory_NoExceptions(sourceDir.FullName, workingDir.FullName, true))
			{
				throw new AutomationException("Couldn't copy {0} to {1}", sourceDir.FullName, workingDir.FullName);
			}

			// Save the default output
			Dictionary<FileReference, DateTime> defaultOutput = SelectFiles(workingDir, 'a', 'f');
			tempStore.Archive("TestNode", null, defaultOutput.Keys.ToArray(), false);

			Dictionary<FileReference, DateTime> namedOutput = SelectFiles(workingDir, 'g', 'i');
			tempStore.Archive("TestNode", "NamedOutput", namedOutput.Keys.ToArray(), true);
			
			// Check both outputs are still ok
			TempStorageBlockManifest defaultManifest = tempStore.Retrieve("TestNode", null, new FileFilter());
			CheckManifest(workingDir, defaultManifest, defaultOutput);

			TempStorageBlockManifest namedManifest = tempStore.Retrieve("TestNode", "NamedOutput", new FileFilter());
			CheckManifest(workingDir, namedManifest, namedOutput);

			// Delete local temp storage and the working directory and try again
			Logger.LogInformation("Clearing local folders...");
			CommandUtils.DeleteDirectoryContents(workingDir.FullName);
			CommandUtils.DeleteDirectoryContents(localDir.FullName);

			// First output should fail
			Logger.LogInformation("Checking default manifest is now unavailable...");
			bool gotManifest;
			try
			{
				tempStore.Retrieve("TestNode", null, new FileFilter());
				gotManifest = true;
			}
			catch
			{
				gotManifest = false;
			}
			if (gotManifest)
			{
				throw new AutomationException("Did not expect shared temp storage manifest to exist");
			}

			// Second one should be fine
			TempStorageBlockManifest namedManifestFromShared = tempStore.Retrieve("TestNode", "NamedOutput", new FileFilter());
			CheckManifest(workingDir, namedManifestFromShared, namedOutput);
		}

		/// <summary>
		/// Enumerate all the files beginning with a letter within a certain range
		/// </summary>
		/// <param name="sourceDir">The directory to read from</param>
		/// <param name="charRangeBegin">First character in the range to files to return</param>
		/// <param name="charRangeEnd">Last character (inclusive) in the range of files to return</param>
		/// <returns>Mapping from filename to timestamp</returns>
		static Dictionary<FileReference, DateTime> SelectFiles(DirectoryReference sourceDir, char charRangeBegin, char charRangeEnd)
		{
			Dictionary<FileReference, DateTime> archiveFileToTime = new Dictionary<FileReference, DateTime>();
			foreach (FileInfo fileInfo in new DirectoryInfo(sourceDir.FullName).EnumerateFiles("*", SearchOption.AllDirectories))
			{
				char firstCharacter = Char.ToLower(fileInfo.Name[0]);
				if (firstCharacter >= charRangeBegin && firstCharacter <= charRangeEnd)
				{
					archiveFileToTime.Add(new FileReference(fileInfo), fileInfo.LastWriteTimeUtc);
				}
			}
			return archiveFileToTime;
		}

		/// <summary>
		/// Checks that a manifest matches the files on disk
		/// </summary>
		/// <param name="rootDir">Root directory for relative paths in the manifest</param>
		/// <param name="manifest">Manifest to check</param>
		/// <param name="files">Mapping of filename to timestamp as expected in the manifest</param>
		static void CheckManifest(DirectoryReference rootDir, TempStorageBlockManifest manifest, Dictionary<FileReference, DateTime> files)
		{
			if (files.Count != manifest.Files.Length)
			{
				throw new AutomationException("Number of files in manifest does not match");
			}
			foreach (TempStorageFile manifestFile in manifest.Files)
			{
				FileReference file = manifestFile.ToFileReference(rootDir);
				if (!FileReference.Exists(file))
				{
					throw new AutomationException("File in manifest does not exist");
				}

				DateTime originalTime;
				if (!files.TryGetValue(file, out originalTime))
				{
					throw new AutomationException("File in manifest did not exist previously");
				}

				double diffSeconds = (new FileInfo(file.FullName).LastWriteTimeUtc - originalTime).TotalSeconds;
				if (Math.Abs(diffSeconds) > 2)
				{
					throw new AutomationException("Incorrect timestamp for {0}", manifestFile.RelativePath);
				}
			}
		}
	}
	/// <summary>
	/// Commandlet to clean up all folders under a temp storage root that are older than a given number of days
	/// </summary>
	[Help("Removes folders in a given temp storage directory that are older than a certain time.")]
	[Help("TempStorageDir=<Directory>", "Path to the root temp storage directory")]
	[Help("Days=<N>", "Number of days to keep in temp storage")]
	class CleanTempStorage : BuildCommand
	{
		/// <summary>
		/// Entry point for the commandlet
		/// </summary>
		public override void ExecuteBuild()
		{
			string? tempStorageDir = ParseParamValue("TempStorageDir", null);
			if (tempStorageDir == null)
			{
				throw new AutomationException("Missing -TempStorageDir parameter");
			}

			if (!Directory.Exists(tempStorageDir))
			{
				Logger.LogInformation("Temp Storage folder '{TempStorageDir}' does not exist, no work to do.", tempStorageDir);
				return;
			}

			string? days = ParseParamValue("Days", null);
			if (days == null)
			{
				throw new AutomationException("Missing -Days parameter");
			}

			double daysValue;
			if (!Double.TryParse(days, out daysValue))
			{
				throw new AutomationException("'{0}' is not a valid value for the -Days parameter", days);
			}

			DateTime retainTime = DateTime.UtcNow - TimeSpan.FromDays(daysValue);

			// Enumerate all the build directories
			Logger.LogInformation("Scanning {TempStorageDir}...", tempStorageDir);
			int numBuilds = 0;
			List<DirectoryInfo> buildsToDelete = new List<DirectoryInfo>();
			foreach (DirectoryInfo streamDirectory in new DirectoryInfo(tempStorageDir).EnumerateDirectories().OrderBy(x => x.Name))
			{
				Logger.LogInformation("Scanning {Path}...", streamDirectory.FullName);
				try
				{
					foreach (DirectoryInfo buildDirectory in streamDirectory.EnumerateDirectories())
					{
						try
						{
							if (!buildDirectory.EnumerateFiles("*", SearchOption.AllDirectories).Any(x => x.LastWriteTimeUtc > retainTime) &&
								!buildDirectory.EnumerateDirectories("*", SearchOption.AllDirectories).Any(x => x.LastWriteTimeUtc > retainTime))
							{
								buildsToDelete.Add(buildDirectory);
							}
							numBuilds++;
						}
						catch (Exception ex)
						{
							Logger.LogError(ex, "Exception while trying to scan files under {BuildDirectory}: {Ex}", buildDirectory, ex);
						}
					}
				}
				catch (Exception ex)
				{
					Logger.LogError(ex, "Exception while trying to scan {StreamDirectory}: {Ex}", streamDirectory, ex);
				}
			}
			Logger.LogInformation("Found {NumBuilds} builds; {Count} to delete.", numBuilds, buildsToDelete.Count);

			// Loop through them all, checking for files older than the delete time
			int idx = buildsToDelete.Count;
			while (idx-- > 0)
			{
				try
				{
					// Done if something already cleaned up this folder.
					if (!buildsToDelete[idx].Exists)
					{
						continue;
					}

					// Check if there is a marker file, if so skip this folder unless it has been twenty minutes.
					FileInfo? deleteInProgressFile = buildsToDelete[idx].GetFiles("DeleteInProgress.tmp").FirstOrDefault();
					if (deleteInProgressFile != null && deleteInProgressFile.LastWriteTimeUtc < (DateTimeOffset.UtcNow - TimeSpan.FromMinutes(20)))
					{
						Logger.LogInformation("[{Index}/{Total}] {Path} flagged as delete in progress, skipping...", buildsToDelete.Count - idx, buildsToDelete.Count, buildsToDelete[idx].FullName);
						continue;
					}

					File.WriteAllBytes(Path.Combine(buildsToDelete[idx].FullName, "DeleteInProgress.tmp"), Array.Empty<byte>());
					Logger.LogInformation("[{Index}/{Total}] Deleting {Path}...", buildsToDelete.Count - idx, buildsToDelete.Count, buildsToDelete[idx].FullName);
					buildsToDelete[idx].Delete(true);
				}
				catch (Exception ex)
				{
					Logger.LogWarning("Failed to delete old manifest folder; will try one file at a time: {Ex}", ex);
					CommandUtils.DeleteDirectory_NoExceptions(true, buildsToDelete[idx].FullName);
				}
			}

			// Try to delete any empty branch folders
			foreach (DirectoryInfo streamDirectory in new DirectoryInfo(tempStorageDir).EnumerateDirectories())
			{
				try
				{
					if (!streamDirectory.EnumerateDirectories().Any() && !streamDirectory.EnumerateFiles().Any())
					{
						try
						{
							streamDirectory.Delete();
						}
						catch (IOException)
						{
							// only catch "directory is not empty type exceptions, if possible. Best we can do is check for IOException.
						}
						catch (Exception ex)
						{
							Logger.LogWarning("Unexpected failure trying to delete (potentially empty) stream directory {Path}: {Ex}", streamDirectory.FullName, ex);
						}
					}
				}
				catch (Exception ex)
				{
					Logger.LogError(ex, "Exception while trying to delete {StreamDirectory}: {Ex}", streamDirectory, ex);
				}
			}
		}
	}
}
