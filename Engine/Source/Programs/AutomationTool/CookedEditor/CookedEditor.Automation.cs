// Copyright Epic Games, Inc. All Rights Reserved.
using System;
using System.Collections.Generic;
using System.Linq;
using System.IO;
using AutomationTool;
using UnrealBuildTool;
using AutomationScripts;
using EpicGames.Core;
using UnrealBuildBase;
using System.Text.Json;
using Microsoft.Extensions.Logging;

using static AutomationTool.CommandUtils;

public class ConfigHelper
{
	private string SpecificConfigSection;
	private string SharedConfigSection;
	public ConfigHierarchy GameConfig { get; }

	public ConfigHelper(UnrealTargetPlatform Platform, FileReference ProjectFile, bool bIsCookedCooker)
	{
		SharedConfigSection = "CookedEditorSettings";
		SpecificConfigSection = SharedConfigSection + (bIsCookedCooker ? "_CookedCooker" : "_CookedEditor");

		GameConfig = ConfigCache.ReadHierarchy(ConfigHierarchyType.Game, ProjectFile.Directory, Platform);
	}

	public bool GetBool(string Key)
	{
		bool Value;
		// GetBool will set Value to false if it's not found, which is what we want
		if (!GameConfig.GetBool(SpecificConfigSection, Key, out Value))
		{
			GameConfig.GetBool(SharedConfigSection, Key, out Value);
		}
		return Value;
	}

	public string GetString(string Key)
	{
		string Value;
		// GetBool will set Value to "" if it's not found, so, set it to null if not found
		if (!GameConfig.GetString(SpecificConfigSection, Key, out Value))
		{
			if (!GameConfig.GetString(SharedConfigSection, Key, out Value))
			{
				Value = null;
			}
		}
		return Value;
	}

	public List<string> GetArray(string Key)
	{
		List<string> Value = new List<string>();
		List<string> Temp;

		// merge both sections into one array (probably don't depend on order)
		if (GameConfig.GetArray(SpecificConfigSection, Key, out Temp))
		{
			Value.AddRange(Temp);
		}
		if (GameConfig.GetArray(SharedConfigSection, Key, out Temp))
		{
			Value.AddRange(Temp);
		}

		return Value;
	}
}

public class ModifyStageContext
{
	// any assets that end up in this list that are already in the DeploymentContext will be removed during Apply
	public List<FileReference> UFSFilesToStage = new List<FileReference>();
	// files in this list will remove the matching cooked package from the DeploymentContext and these uncooked assets will replace them
	public List<FileReference> FilesToUncook = new List<FileReference>();
	// these files will just be staged
	public List<FileReference> NonUFSFilesToStage = new List<FileReference>();

	public List<FileReference> DebugNonUFSFilesToStage = new List<FileReference>();

	public bool bStageShaderDirs = true;
	public bool bStagePlatformBuildDirs = true;
	public bool bStageExtrasDirs = false;
	public bool bStagePlatformDirs = true;
	public bool bStageUAT = false;
	public bool bIsForExternalDistribution = false;
	public bool bStageCollections = false;
	public bool bStagePython = false;
	public bool bStageTargetFiles = false;

	public ConfigHelper ConfigHelper;

	public DirectoryReference EngineDirectory;
	public DirectoryReference ProjectDirectory;
	public string ProjectName;
	public string IniPlatformName;
	public bool bIsDLC;

	// when creating a cooked editor against a premade client, this is the sub-directory in the Releases directory to compare against
	public DirectoryReference ReleaseMetadataLocation = null;

	// where to find files like CachedEditorThumbnails.bin or EditorClientAssetRegistry.bin
	public DirectoryReference CachedEditorDataLocation = null;

	// commandline etc helper
	private BuildCommand Command;

	public ModifyStageContext(ConfigHelper ConfigHelper, DirectoryReference EngineDirectory, ProjectParams Params, DeploymentContext SC, BuildCommand Command)
	{
		this.EngineDirectory = EngineDirectory;
		this.Command = Command;
		this.ConfigHelper = ConfigHelper;

		bStageShaderDirs = ConfigHelper.GetBool("bStageShaderDirs");
		bStagePlatformBuildDirs = ConfigHelper.GetBool("bStagePlatformBuildDirs");
		bStageExtrasDirs = ConfigHelper.GetBool("bStageExtrasDirs");
		bStagePlatformDirs = ConfigHelper.GetBool("bStagePlatformDirs");
		bStageUAT = ConfigHelper.GetBool("bStageUAT");
		bIsForExternalDistribution = ConfigHelper.GetBool("bIsForExternalDistribution");
		bStageCollections = ConfigHelper.GetBool("bStageCollections");
		bStagePython = ConfigHelper.GetBool("bStagePython");

		// cache some useful properties
		ProjectDirectory = Params.RawProjectPath.Directory;
		ProjectName = Params.RawProjectPath.GetFileNameWithoutAnyExtensions();
		IniPlatformName = ConfigHierarchy.GetIniPlatformName(SC.StageTargetPlatform.IniPlatformType);
		bIsDLC = Params.DLCFile != null && SC.MetadataDir != null; // MetadataDir needs to be set for DLC

		Logger.LogInformation("---> ReleaseOverrideDir = {Arg0}, MetadataDir = {Arg1}", Params.BasedOnReleaseVersionPathOverride, SC.MetadataDir);


		// cache info for DLC against a release
		if (Params.BasedOnReleaseVersionPathOverride != null)
		{
			// look in Metadata and the override locations
			ReleaseMetadataLocation = DirectoryReference.Combine(new DirectoryReference(Params.BasedOnReleaseVersionPathOverride), "Metadata");
		}
		else
		{
			ReleaseMetadataLocation = SC.MetadataDir;
		}

		// by default, the files are in the cooked data location (which is SC.Metadatadir)
		CachedEditorDataLocation = SC.MetadataDir;
	}

	public void Apply(DeploymentContext SC)
	{
		if (bIsDLC)
		{
			// remove files that we are about to stage that were already in the shipped client
			RemoveReleasedFiles(SC);
		}

		// maps can't be cooked and loaded by the editor, so make sure no cooked ones exist
		UncookMaps(SC);
		UnUFSFiles(SC);

		// anything we want to be NonUFS make sure is not alreay UFS
		UFSFilesToStage.RemoveAll(x => NonUFSFilesToStage.Contains(x) || DebugNonUFSFilesToStage.Contains(x));

		Dictionary<StagedFileReference, FileReference> StagedUFSFiles = MimicStageFiles(SC, UFSFilesToStage);
		Dictionary<StagedFileReference, FileReference> StagedNonUFSFiles = MimicStageFiles(SC, NonUFSFilesToStage);
		Dictionary<StagedFileReference, FileReference> StagedDebugNonUFSFiles = MimicStageFiles(SC, DebugNonUFSFilesToStage);
		Dictionary<StagedFileReference, FileReference> StagedUncookFiles = MimicStageFiles(SC, FilesToUncook);

		// filter out already-cooked assets
		foreach (var CookedFile in SC.FilesToStage.UFSFiles)
		{
			// remove any of the entries in the "staged" UFSFilesToStage that match already staged files
			// we don't check extension here because the UFSFilesToStage should only contain .uasset/.umap files, and not .uexp, etc, 
			// and .uasset/.umap files are going to be in SC.FilesToStage
			StagedUFSFiles.Remove(CookedFile.Key);
		}

		// remove already-cooked assets to be replaced with 
		List<StagedFileReference> UncookedFilesThatDoNotExist = new List<StagedFileReference>();
		string[] CookedExtensions = { ".uasset", ".umap", ".ubulk", ".uexp", ".uptnl" };
		foreach (var UncookedFile in StagedUncookFiles)
		{
			string PathWithNoExtension = Path.ChangeExtension(UncookedFile.Key.Name, null);
			// we need to remove cooked files that match the files to Uncook, and there can be several extensions
			// for each source asset, so remove them all
			foreach (string CookedExtension in CookedExtensions)
			{
				StagedFileReference PathWithExtension = new StagedFileReference(PathWithNoExtension + CookedExtension);
				SC.FilesToStage.UFSFiles.Remove(PathWithExtension);
				StagedUFSFiles.Remove(PathWithExtension);
			}

			// Some uncooked packages are generated at cook time and do not exist in the uncooked depot.
			// We removed the cooked version of these files from staging above, but we should not add an entry for their
			// non-existent uncooked file. Add them to a list for removal after the loop.
			FileReference FullPathToUncooked = UncookedFile.Value;
			if (!FullPathToUncooked.ToFileInfo().Exists)
			{
				UncookedFilesThatDoNotExist.Add(UncookedFile.Key);
			}
		}
		foreach (StagedFileReference UncookedFile in UncookedFilesThatDoNotExist)
		{
			StagedUncookFiles.Remove(UncookedFile);
		}

		// stage the filtered UFSFiles
		SC.StageFiles(StagedFileType.UFS, StagedUFSFiles.Values);

		// stage the Uncooked files now that any cooked ones are removed from SC
		SC.StageFiles(StagedFileType.UFS, StagedUncookFiles.Values);

		// stage the processed NonUFSFiles
		SC.StageFiles(StagedFileType.NonUFS, StagedNonUFSFiles.Values);

		// stage the processed DebugNonUFSFiles
		SC.StageFiles(StagedFileType.DebugNonUFS, StagedDebugNonUFSFiles.Values);

		// now remove or allow restricted files
		HandleRestrictedFiles(SC, ref SC.FilesToStage.UFSFiles);
		HandleRestrictedFiles(SC, ref SC.FilesToStage.NonUFSFiles);
		HandleRestrictedFiles(SC, ref SC.FilesToStage.NonUFSDebugFiles);

		// remove UFS files if they are also in NonUFS - no need to duplicate
		SC.FilesToStage.UFSFiles = SC.FilesToStage.UFSFiles.Where(x => !SC.FilesToStage.NonUFSFiles.ContainsKey(x.Key) && !SC.FilesToStage.NonUFSDebugFiles.ContainsKey(x.Key)).ToDictionary(x => x.Key, x => x.Value);

	}
	#region Private implementation
	
	private void RemoveReleasedFiles(DeploymentContext SC)
	{
		HashSet<StagedFileReference> ShippedFiles = new HashSet<StagedFileReference>();
		Action<string, string> FindShippedFiles = (string ParamName, string FileNamePortion) =>
		{
			FileReference UFSManifestFile = Command.ParseOptionalFileReferenceParam(ParamName);
			if (UFSManifestFile == null)
			{
				UFSManifestFile = FileReference.Combine(ReleaseMetadataLocation, $"Manifest_{FileNamePortion}_{SC.StageTargetPlatform.PlatformType}.txt");
			}
			if (FileReference.Exists(UFSManifestFile))
			{
				foreach (string Line in File.ReadAllLines(UFSManifestFile.FullName))
				{
					string[] Tokens = Line.Split("\t".ToCharArray());
					if (Tokens?.Length > 1)
					{
						ShippedFiles.Add(new StagedFileReference(Tokens[0]));
					}
				}
			}
		};

		FindShippedFiles("ClientUFSManifest", "UFSFiles");
		FindShippedFiles("ClientNonUFSManifest", "NonUFSFiles");
		FindShippedFiles("ClientDebugManifest", "DebugFiles");

		ShippedFiles.RemoveWhere(x => x.HasExtension(".ttf") && !x.Name.Contains("LastResort"));

		UFSFilesToStage.RemoveAll(x => ShippedFiles.Contains(DeploymentContext.MakeRelativeStagedReference(SC, x)));
		NonUFSFilesToStage.RemoveAll(x => ShippedFiles.Contains(DeploymentContext.MakeRelativeStagedReference(SC, x)));
		DebugNonUFSFilesToStage.RemoveAll(x => ShippedFiles.Contains(DeploymentContext.MakeRelativeStagedReference(SC, x)));
	}
	private Dictionary<StagedFileReference, FileReference> MimicStageFiles(DeploymentContext SC, List<FileReference> SourceFiles)
	{
		Dictionary<StagedFileReference, FileReference> Mapping = new Dictionary<StagedFileReference, FileReference>();

		foreach (FileReference FileRef in new HashSet<FileReference>(SourceFiles))
		{
			DirectoryReference RootDir;
			StagedFileReference StagedFile = DeploymentContext.MakeRelativeStagedReference(SC, FileRef, out RootDir);

			// add the mapping
			Mapping.Add(StagedFile, FileRef);
		}

		return Mapping;
	}

	private void HandleRestrictedFiles(DeploymentContext SC, ref Dictionary<StagedFileReference, FileReference> Files)
	{
		// If a directory has been specifically allowed, do not omit the files wihin it from the staging process.
		if (bIsForExternalDistribution)
		{
			Int32 OrigNumFiles = Files.Count();
			// remove entries where any restricted folder names are in the name remapped path (if we remap from NFL to non-NFL, then we don't remove it)
			// If a configuration file has been explicitly allowed, do not omit it either.
			Files = Files.Where(x => SC.ConfigFilesAllowList.Contains(x.Key) || SC.ExtraFilesAllowList.Contains(x.Key) || SC.DirectoriesAllowList.Contains(x.Key.Directory) || !SC.RestrictedFolderNames.Any(y => DeploymentContext.ApplyDirectoryRemap(SC, x.Key).ContainsName(y))).ToDictionary(x => x.Key, x => x.Value);
			if (OrigNumFiles != Files.Count())
			{
				Log.TraceInformationOnce("Some files were not staged since they have restricted folder names in the remapped path.");
			}
		}
		else
		{
			Log.TraceInformationOnce("Allowing restricted directories to be staged...");
		}
	}

	private void AddCookedUFSFilesToList(List<FileReference> FileList, string Extension, DeploymentContext SC)
	{
		// look in SC and UFSFiles
		FileList.AddRange(SC.FilesToStage.UFSFiles.Keys.Where(x => x.Name.EndsWith(Extension, StringComparison.OrdinalIgnoreCase)).Select(y => DeploymentContext.UnmakeRelativeStagedReference(SC, y)));
		FileList.AddRange(UFSFilesToStage.Where(x => x.FullName.EndsWith(Extension, StringComparison.OrdinalIgnoreCase)));
	}

	private void AddUFSFilesToList(List<FileReference> FileList, string Extension, DeploymentContext SC)
	{
		// look in SC and UFSFiles
		foreach (var Pair in SC.FilesToStage.UFSFiles)
		{
			if (Pair.Key.Name.EndsWith(Extension, StringComparison.OrdinalIgnoreCase))
			{
				FileList.Add(Pair.Value);
			}
		}
		FileList.AddRange(UFSFilesToStage.Where(x => x.FullName.EndsWith(Extension, StringComparison.OrdinalIgnoreCase)));
	}

	private void UncookMaps(DeploymentContext SC)
	{
		string CookedMapMode = ConfigHelper.GetString("MapMode").ToLower();
		if (CookedMapMode == "cooked")
		{
			// nothing to do, they are already staged as cooked as normal
		}
		else if (CookedMapMode == "uncooked")
		{
			// remove maps from SC and Context (SC has path to the cooked map, so we have to come back from Staged reference that doesn't have the Cooked dir in it)
			AddCookedUFSFilesToList(FilesToUncook, ".umap", SC);
		}
		else if (CookedMapMode == "none")
		{
			// remove umaps and their sidecar files so they won't be staged (remove extension so that we remove Foo.umap and Foo.uexp)
			// also remove Foo_BuiltData.*
			HashSet<string> Maps = UFSFilesToStage.Where(x => x.HasExtension("umap")).Select(x => Path.ChangeExtension(x.FullName, null)).ToHashSet();
			UFSFilesToStage = UFSFilesToStage.Where(x => !Maps.Contains(Path.ChangeExtension(x.FullName.Replace("_BuiltData", ""), null))).ToList();

			Maps = FilesToUncook.Where(x => x.HasExtension("umap")).Select(x => Path.ChangeExtension(x.FullName, null)).ToHashSet();
			FilesToUncook = FilesToUncook.Where(x => !Maps.Contains(Path.ChangeExtension(x.FullName.Replace("_BuiltData", ""), null))).ToList();

			Maps = SC.FilesToStage.UFSFiles.Keys.Where(x => x.HasExtension("umap")).Select(x => Path.ChangeExtension(x.Name, null)).ToHashSet();
			Console.WriteLine($"Found {Maps.Count()} maps");
			SC.FilesToStage.UFSFiles = SC.FilesToStage.UFSFiles.Where(x => !Maps.Contains(Path.ChangeExtension(x.Key.Name.Replace("_BuiltData", ""), null))).ToDictionary(x => x.Key, x => x.Value);

		}
	}

	private void UnUFSFiles(DeploymentContext SC)
	{
		if (bStageUAT)
		{
			// UAT needs uplugin and ini files, so make sure they are not in the .pak
			AddUFSFilesToList(NonUFSFilesToStage, ".uplugin", SC);
			AddUFSFilesToList(NonUFSFilesToStage, ".ini", SC);
			AddUFSFilesToList(NonUFSFilesToStage, "SDK.json", SC);
			NonUFSFilesToStage = NonUFSFilesToStage.Where(x => x.GetFileName().IndexOf("BinaryConfig", StringComparison.OrdinalIgnoreCase) == -1).ToList();
		}
	}

	#endregion
}


public class MakeCookedEditor : BuildCommand
{
	protected bool bIsCookedCooker;
	protected FileReference ProjectFile;

	protected ConfigHelper ConfigHelper;

	// used to remember the locations of stating output exactly as calculated by the staging code
	protected DirectoryReference CookedEditorStageDirectory = null;
	protected DirectoryReference ReleaseStageDirectory = null;

	// with -makerelease, this will have the location of optional editor only files, but a subclass can just set this if the optional files
	// were made and saved off somewhere, it can point to this and the optional files will be automatically staged into Content/Paks
	protected DirectoryReference ReleaseOptionalFileStageDirectory = null;

	public override void ExecuteBuild()
	{
		Logger.LogInformation("************************* MakeCookedEditor");

		bIsCookedCooker = ParseParam("cookedcooker");
		ProjectFile = ParseProjectParam();

		// set up config sections and the like
		ConfigHelper = new ConfigHelper(BuildHostPlatform.Current.Platform, ProjectFile, bIsCookedCooker);

		ProjectParams BuildParams = GetParams();


		Project.Build(this, BuildParams);

		// after the editor is built, if we are building against a release, and if desired, make that release first to make sure we have it to build against
		ProjectParams ReleaseParams = null;

		string MakeReleaseOptions = ParseParamValue("makerelease", "");
		if (MakeReleaseOptions != "")
		{
			if (string.IsNullOrEmpty(BuildParams.BasedOnReleaseVersion))
			{
				throw new AutomationException("-makerelease was specified but the project doesn't have bBuildAgainstRelease set to true");
			}

			ReleaseParams = GetReleaseParams(BuildParams, MakeReleaseOptions.Split(','));
			Project.Build(this, ReleaseParams);
			Project.Cook(ReleaseParams);
			Project.CopyBuildToStagingDirectory(ReleaseParams);

			FinalizeRelease(ReleaseParams);
		}

		Project.Cook(BuildParams);
		Project.CopyBuildToStagingDirectory(BuildParams);

		//this will do packaging if requested, and also symbol upload if requested.
		Project.Package(BuildParams);

		Project.Archive(BuildParams);
		PrintRunTime();
//		Project.Deploy(BuildParams);

		if (ReleaseParams != null)
		{
			string CombinedPath = ParseParamValue("CombineBuilds", "");
			if (CombinedPath != "")
			{
				if (CookedEditorStageDirectory == null || ReleaseStageDirectory == null)
				{
					Logger.LogError("Combining Release and CookedEditor together currently requires that both are staged this run (-stage -makerelease=stage)");
					return;
				}

				Logger.LogInformation("Combinging {ReleaseStageDirectory} + {CookedEditorStageDirectory} -> {CombinedPath}", ReleaseStageDirectory, CookedEditorStageDirectory, CombinedPath);
				DirectoryReference Combined = new DirectoryReference(CombinedPath);
				CopyDirectory_NoExceptions(ReleaseStageDirectory.FullName, Combined.FullName, CopyDirectoryOptions.Default);
				CopyDirectory_NoExceptions(CookedEditorStageDirectory.FullName, Combined.FullName, CopyDirectoryOptions.Merge);
			}
		}

	}

	protected virtual void StageEngineEditorFiles(ProjectParams Params, DeploymentContext SC, ModifyStageContext Context)
	{
		StagePlatformExtensionFiles(Params, SC, Context, Unreal.EngineDirectory);
		StagePluginFiles(Params, SC, Context, true);

		// engine shaders
		if (Context.bStageShaderDirs)
		{
			IEnumerable<FileReference> ShaderFiles = DirectoryReference.EnumerateFiles(DirectoryReference.Combine(Unreal.EngineDirectory, "Shaders"), "*", SearchOption.AllDirectories)
				.Where(x => !x.GetExtension().Equals(".cs", StringComparison.OrdinalIgnoreCase));
			Context.NonUFSFilesToStage.AddRange(ShaderFiles);
			GatherTargetDependencies(Params, SC, Context, "ShaderCompileWorker");
		}
		if (bIsCookedCooker)
		{
			GatherTargetDependencies(Params, SC, Context, "UnrealPak");
		}

		// Stage the editor localization targets
		if (!bIsCookedCooker)
		{
			List<string> CulturesToStage = Project.GetCulturesToStage(Params, ConfigHelper.GameConfig);

			string[] EditorLocalizationTargetsToStage = { "Category", "Engine", "Editor", "Keywords", "PropertyNames", "ToolTips" };
			foreach (string EditorLocalizationTargetToStage in EditorLocalizationTargetsToStage)
			{
				// Note: We skip "ShouldStageLocalizationTarget" below as games may disable certain targets that they don't need at runtime (eg, Engine), but all of these targets are still needed for an editor!
				//if (Project.ShouldStageLocalizationTarget(SC, null, EditorLocalizationTargetToStage))
				{
					Project.StageLocalizationDataForTarget(SC, CulturesToStage, DirectoryReference.Combine(Unreal.EngineDirectory, "Content", "Localization", EditorLocalizationTargetToStage));
				}
			}
		}

		StageIniPathArray(Params, SC, "EngineExtraStageFiles", Unreal.EngineDirectory, Context);

		Context.FilesToUncook.Add(FileReference.Combine(Context.EngineDirectory, "Content/EngineMaterials/DefaultMaterial.uasset"));
		Context.FilesToUncook.Add(FileReference.Combine(Context.EngineDirectory, "Content/EditorLandscapeResources/DefaultAlphaTexture.uasset"));
	}

	protected virtual void StageProjectEditorFiles(ProjectParams Params, DeploymentContext SC, ModifyStageContext Context)
	{
		// always stage the main exe, in case DLC mode is on, then it won't by default
		if (SC.StageExecutables.Count > 0)
		{
			GatherTargetDependencies(Params, SC, Context, SC.StageExecutables[0]);
		}

		// project shaders
		if (Context.bStageShaderDirs)
		{
			DirectoryReference ProjectShaders = DirectoryReference.Combine(Context.ProjectDirectory, "Shaders");
			if (DirectoryReference.Exists(ProjectShaders))
			{
				Context.NonUFSFilesToStage.AddRange(DirectoryReference.EnumerateFiles(ProjectShaders, "*", SearchOption.AllDirectories));
			}		
		}

		// .collection files
		if (!bIsCookedCooker && Context.bStageCollections)
		{
			DirectoryReference ProjectCollectionsDirectory = DirectoryReference.Combine(Context.ProjectDirectory, "Content", "Collections");
			if (DirectoryReference.Exists(ProjectCollectionsDirectory))
			{
				Context.DebugNonUFSFilesToStage.AddRange(DirectoryReference.EnumerateFiles(ProjectCollectionsDirectory, "*.collection", SearchOption.TopDirectoryOnly));
			}

			DirectoryReference ProjectDevelopersDirectory = DirectoryReference.Combine(Context.ProjectDirectory, "Content", "Developers");
			if (DirectoryReference.Exists(ProjectDevelopersDirectory))
			{
				foreach (DirectoryReference DeveloperDirectory in DirectoryReference.EnumerateDirectories(ProjectDevelopersDirectory))
				{
					DirectoryReference DeveloperCollectionsDirectory = DirectoryReference.Combine(DeveloperDirectory, "Collections");
					if (DirectoryReference.Exists(DeveloperCollectionsDirectory))
					{
						Context.DebugNonUFSFilesToStage.AddRange(DirectoryReference.EnumerateFiles(DeveloperCollectionsDirectory, "*.collection", SearchOption.TopDirectoryOnly));
					}
				}
			}
		}

		// Python scripts
		if (Context.bStagePython)
		{
			DirectoryReference ProjectPythonDirectory = DirectoryReference.Combine(Context.ProjectDirectory, "Content", "Python");
			if (DirectoryReference.Exists(ProjectPythonDirectory))
			{
				Context.NonUFSFilesToStage.AddRange(DirectoryReference.EnumerateFiles(ProjectPythonDirectory, "*", SearchOption.AllDirectories));
			}
		}

		StagePlatformExtensionFiles(Params, SC, Context, Context.ProjectDirectory);
		StagePluginFiles(Params, SC, Context, false);

		// add stripped out editor .ini files back in
		Context.UFSFilesToStage.AddRange(DirectoryReference.EnumerateFiles(DirectoryReference.Combine(Context.ProjectDirectory, "Config"), "*Editor*", SearchOption.AllDirectories));

		StageIniPathArray(Params, SC, "ProjectExtraStageFiles", Context.ProjectDirectory, Context);

		if (!bIsCookedCooker)
		{
			// the editor AR may be named EditorClientAssetRegistry.bin already, but probably is DevelopmentAssetRegistry.bin, so look for both, and name it EditorClientAssetRegistry
			FileReference EditorAR = FileReference.Combine(Context.CachedEditorDataLocation, "EditorClientAssetRegistry.bin");
			if (!FileReference.Exists(EditorAR))
			{
				EditorAR = FileReference.Combine(Context.CachedEditorDataLocation, "DevelopmentAssetRegistry.bin");
			}
			SC.StageFile(StagedFileType.UFS, EditorAR, new StagedFileReference($"{Context.ProjectName}/EditorClientAssetRegistry.bin"));

			// this file is optional
			FileReference EditorThumbnails = FileReference.Combine(Context.CachedEditorDataLocation, "CachedEditorThumbnails.bin");
			if (FileReference.Exists(EditorThumbnails))
			{
				SC.StageFile(StagedFileType.UFS, EditorThumbnails, new StagedFileReference($"{Context.ProjectName}/CachedEditorThumbnails.bin"));
			}
		}

		// if a subclass or -makerelease didn't set ReleaseOptionalFileStageDirectory, then look in the Params for the commandline option
		if (ReleaseOptionalFileStageDirectory == null && !string.IsNullOrEmpty(Params.OptionalFileInputDirectory) &&
			Directory.Exists(Params.OptionalFileInputDirectory))
		{
			ReleaseOptionalFileStageDirectory = new DirectoryReference(Params.OptionalFileInputDirectory);
		}

		if (ReleaseOptionalFileStageDirectory != null)
		{
			// these files were already staged by the client/release build, so we stage them as NonUFS
			// these could be just copied, but StageFiles handles copying easily
			SC.StageFiles(StagedFileType.NonUFS, ReleaseOptionalFileStageDirectory, StageFilesSearch.AllDirectories, new StagedDirectoryReference($"{Context.ProjectName}/Content/Paks"));

			Logger.LogInformation("Staging optional files from {Arg0}:", ReleaseOptionalFileStageDirectory.FullName);
			foreach (var OptionalFile in DirectoryReference.EnumerateFiles(ReleaseOptionalFileStageDirectory, "*"))
			{
				Logger.LogInformation("  '{Arg0}'", OptionalFile.FullName);
			}
		}
	}

	protected virtual void StageUAT(ProjectParams Params, DeploymentContext SC, ModifyStageContext Context)
	{
		// now look in the .json file that UAT made that lists its script files
		DirectoryReference AutomationToolBinaryDir = DirectoryReference.Combine(Context.EngineDirectory, "Binaries", "DotNET", "AutomationTool");
		DirectoryReference UnrealBuildToolBinaryDir = DirectoryReference.Combine(Context.EngineDirectory, "Binaries", "DotNET", "UnrealBuildTool");
		DirectoryReference ProjectAutomationToolBinaryDir = DirectoryReference.Combine(Context.ProjectDirectory, "Binaries", "DotNET", "AutomationTool");

		// some netcore dependencies in the tool directories can't be discovered with CsProjectInfo, so just stage the enture UBT and UAT directories
		Context.NonUFSFilesToStage.AddRange(DirectoryReference.EnumerateFiles(UnrealBuildToolBinaryDir, "*", SearchOption.AllDirectories));
		Context.NonUFSFilesToStage.AddRange(DirectoryReference.EnumerateFiles(AutomationToolBinaryDir, "*", SearchOption.AllDirectories));

		StagedDirectoryReference StagedBinariesDir = new StagedDirectoryReference("Engine/Binaries/DotNET/AutomationTool");

		// look in Engine/Intermediate/ScriptModules and Project/Intermediate/ScriptModules
		DirectoryReference EngineScriptModulesDir = DirectoryReference.Combine(Context.EngineDirectory, "Intermediate", "ScriptModules");
		DirectoryReference ProjectScriptModulesDir = DirectoryReference.Combine(Context.ProjectDirectory, "Intermediate", "ScriptModules");
		IEnumerable<FileReference> JsonFiles = DirectoryReference.EnumerateFiles(EngineScriptModulesDir);
		if (DirectoryReference.Exists(ProjectScriptModulesDir))
		{
			JsonFiles = JsonFiles.Concat(DirectoryReference.EnumerateFiles(ProjectScriptModulesDir));
		}
		foreach (FileReference JsonFile in JsonFiles)
		{
			try
			{
				// load build info 
				CsProjBuildRecord BuildRecord = JsonSerializer.Deserialize<CsProjBuildRecord>(FileReference.ReadAllText(JsonFile));

				Context.NonUFSFilesToStage.Add(JsonFile);

				// get location of the project where the other paths are relative to
				DirectoryReference RecordRoot = FileReference.Combine(JsonFile.Directory, BuildRecord.ProjectPath).Directory;
				string FullPath = RecordRoot.FullName;

				// stage the output and everything it pulled in next to it
				foreach (FileReference TargetDirFile in DirectoryReference.EnumerateFiles(FileReference.Combine(RecordRoot, BuildRecord.TargetPath).Directory, "*", SearchOption.AllDirectories))
				{
					Context.NonUFSFilesToStage.Add(TargetDirFile);
				}

				// now pull in any dependencies in case something loads it by path
				foreach (string Dep in BuildRecord.Dependencies)
				{
					if (Path.GetExtension(Dep).ToLower() == ".dll")
					{
						FileReference DepFile = FileReference.Combine(RecordRoot, Dep);
						// if ht's not in the engine or the project, we will just stage it next to the .exe, in a last ditch effort
						if (!DepFile.IsUnderDirectory(Context.EngineDirectory) && !DepFile.IsUnderDirectory(Context.ProjectDirectory))
						{
							SC.StageFile(StagedFileType.NonUFS, DepFile, StagedFileReference.Combine(StagedBinariesDir, DepFile.GetFileName()));
						}
						else
						{
							Context.NonUFSFilesToStage.Add(DepFile);
						}
					}
				}
			}
			catch(Exception)
			{
				// skip json files that fail
			}
		}

		if (Context.IniPlatformName == "Linux")
		{
			// linux needs dotnet runtime
			SC.StageFiles(StagedFileType.NonUFS, Unreal.FindDotnetDirectoryForPlatform(RuntimePlatform.Type.Linux), StageFilesSearch.AllDirectories);
		}

		// not sure if we need this or not now

		// ask each platform if they need extra files
		//foreach (UnrealTargetPlatform Platform in UnrealTargetPlatform.GetValidPlatforms())
		//{
		//	List<FileReference> Files = new List<FileReference>();
		//	AutomationTool.Platform.GetPlatform(Platform).GetPlatformUATDependencies(Context.ProjectDirectory, Files);
		//	Context.NonUFSFilesToStage.AddRange(Files.Where(x => FileReference.Exists(x)));
		//}
	}

	protected virtual void StagePluginDirectory(DirectoryReference PluginDir, ModifyStageContext Context, bool bStageUncookedContent)
	{
		foreach (DirectoryReference Subdir in DirectoryReference.EnumerateDirectories(PluginDir))
		{
			StagePluginSubdirectory(Subdir, Context, bStageUncookedContent);
		}
	}

	protected virtual void StagePluginSubdirectory(DirectoryReference PluginSubdir, ModifyStageContext Context, bool bStageUncookedContent)
	{
		string DirNameLower = PluginSubdir.GetDirectoryName().ToLower();

		if (DirNameLower == "content")
		{
			if (bStageUncookedContent)
			{
				Context.FilesToUncook.AddRange(DirectoryReference.EnumerateFiles(PluginSubdir, "*", SearchOption.AllDirectories));
			}
			else
			{
				Context.UFSFilesToStage.AddRange(DirectoryReference.EnumerateFiles(PluginSubdir, "*", SearchOption.AllDirectories));
			}
		}
		else if (DirNameLower == "resources" || DirNameLower == "config" || DirNameLower == "scripttemplates")
		{
			Context.UFSFilesToStage.AddRange(DirectoryReference.EnumerateFiles(PluginSubdir, "*", SearchOption.AllDirectories));
		}
		else if (DirNameLower == "documentation" && Directory.Exists(PluginSubdir.FullName + "/Source/Shared"))
		{
			Context.UFSFilesToStage.AddRange(DirectoryReference.EnumerateFiles(PluginSubdir, "Source/Shared/*", SearchOption.AllDirectories));
		}
		else if (DirNameLower == "shaders" && Context.bStageShaderDirs)
		{
			Context.NonUFSFilesToStage.AddRange(DirectoryReference.EnumerateFiles(PluginSubdir, "*", SearchOption.AllDirectories));
		}
	}

	protected virtual ModifyStageContext CreateContext(ProjectParams Params, DeploymentContext SC)
	{
		return new ModifyStageContext(ConfigHelper, Unreal.EngineDirectory, Params, SC, this);
	}

	protected virtual void ModifyParams(ProjectParams BuildParams)
	{
	}

	protected virtual void ModifyReleaseParams(ProjectParams ReleaseParams)
	{
	}

	protected virtual void FinalizeRelease(ProjectParams ReleaseParams)
	{
	}

	protected virtual void PreModifyDeploymentContext(ProjectParams Params, DeploymentContext SC)
	{
		ModifyStageContext Context = CreateContext(Params, SC);

		DefaultPreModifyDeploymentContext(Params, SC, Context);

		Context.Apply(SC);
	}

	protected virtual void ModifyDeploymentContext(ProjectParams Params, DeploymentContext SC)
	{
		ModifyStageContext Context = CreateContext(Params, SC);

		DefaultModifyDeploymentContext(Params, SC, Context);

		Context.Apply(SC);

		// we do this after the apply to make sure we get any SC and Context based staging
		if (bIsCookedCooker)
		{
			// cooker can run with just the -Cmd, so we reduce the size byt removing the non-Cmd executable and debug info (this is sizeable for monolithic editors)
			string MainCookedTarget = Params.ServerCookedTargets[0];
			// @todo mac
			SC.FilesToStage.NonUFSFiles.Remove(new StagedFileReference(Path.Combine(Context.ProjectName, "Binaries", "Win64", MainCookedTarget + ".exe")));
			SC.FilesToStage.NonUFSFiles.Remove(new StagedFileReference(Path.Combine(Context.ProjectName, "Binaries", "Linux", MainCookedTarget)));
			SC.FilesToStage.NonUFSDebugFiles.Remove(new StagedFileReference(Path.Combine(Context.ProjectName, "Binaries", "Win64", MainCookedTarget + ".pdb")));
			SC.FilesToStage.NonUFSDebugFiles.Remove(new StagedFileReference(Path.Combine(Context.ProjectName, "Binaries", "Linux", MainCookedTarget + ".sym")));
			// todo: find out why pdbs are in the non-debug file list and get rid of them
			SC.FilesToStage.NonUFSFiles.Remove(new StagedFileReference(Path.Combine(Context.ProjectName, "Binaries", "Win64", MainCookedTarget + ".pdb")));
			SC.FilesToStage.NonUFSFiles.Remove(new StagedFileReference(Path.Combine(Context.ProjectName, "Binaries", "Linux", MainCookedTarget + ".sym")));
		}

		CookedEditorStageDirectory = SC.StageDirectory;
	}

	protected virtual void FinalizeDeploymentContext(ProjectParams Params, DeploymentContext SC)
	{
	}

	protected virtual void SetupDLCMode(FileReference ProjectFile, out string DLCName, out string ReleaseVersion, out TargetType Type)
	{
		bool bBuildAgainstRelease = ConfigHelper.GetBool("bBuildAgainstRelease");

		if (ParseParamValue("MakeRelease", null) != null && !bBuildAgainstRelease)
		{
			Logger.LogWarning("-makerelease is meant for projects that have bBuildAgainstRelease set. Will force it on, with default settings for [DLCPluginName, ReleaseName, ReleaseTargetType]");
			bBuildAgainstRelease = true;
		}

		if (bBuildAgainstRelease)
		{
			DLCName = ConfigHelper.GetString("DLCPluginName");
			ReleaseVersion = ConfigHelper.GetString("ReleaseName");

			// if not set, default to gamename
			if (string.IsNullOrEmpty(ReleaseVersion))
			{
				ReleaseVersion = ProjectFile.GetFileNameWithoutAnyExtensions();
			}

			string TargetTypeString;
			TargetTypeString = ConfigHelper.GetString("ReleaseTargetType");
			Type = (TargetType)Enum.Parse(typeof(TargetType), TargetTypeString);
		}
		else
		{
			DLCName = null;
			ReleaseVersion = null;
			Type = TargetType.Game;
		}
	}







	protected void StagePlatformExtensionFiles(ProjectParams Params, DeploymentContext SC, ModifyStageContext Context, DirectoryReference RootDir)
	{
		if (!Context.bStagePlatformDirs)
		{
			return;
		}


		// plugins are already handled in the Plugins staging code
		List<string> RootFoldersToStrip = new List<string> { "source", "plugins" };//, "binaries" };
		List<string> SubFoldersToStrip = new List<string> { "source", "intermediate", "tests", "binaries" + Path.DirectorySeparatorChar + HostPlatform.Current.HostEditorPlatform.ToString().ToLower() };
		List<string> RootNonUFSFolders = new List<string> { "shaders", "binaries", "build", "extras" };


		if (!Context.bStageShaderDirs)
		{
			RootFoldersToStrip.Add("shaders");
		}
		if (!Context.bStagePlatformBuildDirs)
		{
			RootFoldersToStrip.Add("build");
		}
		if (!Context.bStageExtrasDirs)
		{
			RootFoldersToStrip.Add("extras");
		}

		foreach (DirectoryReference PlatformDir in Unreal.GetExtensionDirs(RootDir, true, false, false))
		{
			foreach (DirectoryReference Subdir in DirectoryReference.EnumerateDirectories(PlatformDir, "*", SearchOption.TopDirectoryOnly))
			{
				string SubdirName = Subdir.GetDirectoryName().ToLower();

				// Remvoe some unnecessary folders that can be large
				List<FileReference> ContextFileList = Context.UFSFilesToStage;

				// some files need to be NonUFS for C# etc to access
				if (RootNonUFSFolders.Contains(SubdirName))
				{
					ContextFileList = Context.NonUFSFilesToStage;
				}

				List<FileReference> FilesToStage = new List<FileReference>();
				// if we aren't in a bad subdir, add files
				if (!RootFoldersToStrip.Contains(SubdirName))
				{
					FilesToStage.AddRange(DirectoryReference.EnumerateFiles(Subdir, "*", SearchOption.AllDirectories));

					// now remove files in subdirs we want to skip
					FilesToStage.RemoveAll(x => x.ContainsAnyNames(SubFoldersToStrip, Subdir));
					ContextFileList.AddRange(FilesToStage);
				}

				if (SubdirName == "config")
				{
					Context.NonUFSFilesToStage.AddRange(DirectoryReference.EnumerateFiles(Subdir, "*SDK.json", SearchOption.AllDirectories));
				}
			}
		}
	}

	protected void StagePluginFiles(ProjectParams Params, DeploymentContext SC, ModifyStageContext Context, bool bEnginePlugins)
	{
		List<FileReference> ActivePlugins = new List<FileReference>();
		foreach (StageTarget Target in SC.StageTargets)
		{
			if (Target.Receipt.TargetType == TargetType.Editor)
			{
				IEnumerable<RuntimeDependency> TargetPlugins = Target.Receipt.RuntimeDependencies.Where(x => x.Path.GetExtension().ToLower() == ".uplugin");
				// grab just engine plugins, or non-engine plugins depending
				TargetPlugins = TargetPlugins.Where(x => (bEnginePlugins ? x.Path.IsUnderDirectory(Unreal.EngineDirectory) : !x.Path.IsUnderDirectory(Unreal.EngineDirectory)));

				// convert to paths
				ActivePlugins.AddRange(TargetPlugins.Select(x => x.Path));
			}
		}

		foreach (FileReference ActivePlugin in ActivePlugins)
		{
			PluginInfo Plugin = new PluginInfo(ActivePlugin, bEnginePlugins ? PluginType.Engine : PluginType.Project);
			// we don't cook for unsupported target platforms, but the plugin may still need to be used in the editor, so
			// stage uncooked assets for these plugins
			bool bStageUncookedContent = (!Plugin.Descriptor.SupportsTargetPlatform(SC.StageTargetPlatform.PlatformType));

			StagePluginDirectory(ActivePlugin.Directory, Context, bStageUncookedContent);
		}

	}

	protected void StageIniPathArray(ProjectParams Params, DeploymentContext SC, string IniKey, DirectoryReference BaseDirectory, ModifyStageContext Context)
	{
		HashSet<string> Entries = new HashSet<string>();

		// read the ini for all platforms, and merge together to remove duplicates
		foreach (UnrealTargetPlatform Platform in UnrealTargetPlatform.GetValidPlatforms())
		{
			ConfigHelper PlatformHelper = new ConfigHelper(Platform, ProjectFile, bIsCookedCooker);
			Entries.UnionWith(ConfigHelper.GetArray(IniKey));
		}

		foreach (string Entry in Entries)
		{
			Dictionary<string, string> Props = ConfigHierarchy.GetStructKeyValuePairs(Entry);

			string SubPath = Props["Path"];
			string FileWildcard = "*";
			List<FileReference> FileList = Context.UFSFilesToStage;
			SearchOption SearchMode = SearchOption.AllDirectories;
			if (Props.ContainsKey("Files"))
			{
				FileWildcard = Props["Files"];
			}
			if (Props.ContainsKey("NonUFS") && bool.Parse(Props["NonUFS"]) == true)
			{
				FileList = Context.NonUFSFilesToStage;
			}
			if (Props.ContainsKey("Recursive") && bool.Parse(Props["Recursive"]) == false)
			{
				SearchMode = SearchOption.TopDirectoryOnly;
			}

			bool bHasDest = Props.ContainsKey("Dest");
			bool bHasIniSections = Props.ContainsKey("IniSections");

			// settings with a Dest or IniSections are special, and have to go right to SC
			if (bHasDest || bHasIniSections)
			{
				FileReference SourceFile = FileReference.Combine(BaseDirectory, SubPath, FileWildcard);

				// allow a blank Path to mean empty file
				if (SubPath == "")
				{
					SourceFile = FileReference.Combine(Context.EngineDirectory, "Intermediate/blankfile.txt");
					FileReference.WriteAllText(SourceFile, "");
					FileWildcard = "";
				}

				if (SearchMode == SearchOption.AllDirectories || FileWildcard.Contains("*"))
				{
					throw new AutomationException($"Unable to stage directories with \"Dest\" or \"IniSections\" setting for CookedEditor: '{Entry}'");
				}

				// setup Dest
				StagedFileReference DestFile;
				if (bHasDest)
				{
					DestFile = new StagedFileReference(Props["Dest"]);
				}
				else
				{
					DestFile = DeploymentContext.MakeRelativeStagedReference(SC, SourceFile);
				}

				if (bHasIniSections)
				{
					if (!SourceFile.HasExtension(".ini"))
					{
						throw new AutomationException($"Unable to stage non-ini file '{Entry}' using \"IniSections\" setting for CookedEditor: ");
					}

					FileReference IntermediateFile = FileReference.Combine(Context.ProjectDirectory, "Intermediate", "StagedIniSections", DeploymentContext.MakeRelativeStagedReference(SC, SourceFile).ToString());
					InternalUtils.SafeCopyFile(SourceFile.ToString(), IntermediateFile.ToString(), IniSectionAllowList : Props["IniSections"].Split(',').ToList(), bSafeCreateDirectory : true);
					SourceFile = IntermediateFile;
				}

				// now stage it to a different location as specified in the params
				StagedFileType FileType = (FileList == Context.NonUFSFilesToStage) ? StagedFileType.NonUFS : StagedFileType.UFS;
				if (Props.ContainsKey("Force") && bool.Parse(Props["Force"]) == true)
				{
					SC.ExtraFilesAllowList.Add(DestFile);
				}
				SC.StageFile(FileType, SourceFile, DestFile);
				continue;
			}

			// now enumerate files based on the settings
			DirectoryReference Dir = DirectoryReference.Combine(BaseDirectory, SubPath);
			if (DirectoryReference.Exists(Dir))
			{
				if (Props.ContainsKey("Force") && bool.Parse(Props["Force"]) == true)
				{
					foreach (FileReference File in DirectoryReference.EnumerateFiles(Dir, FileWildcard, SearchMode))
					{
						SC.ExtraFilesAllowList.Add(DeploymentContext.MakeRelativeStagedReference(SC, File));
						FileList.Add(File);
					}
				}
				else
				{
					FileList.AddRange(DirectoryReference.EnumerateFiles(Dir, FileWildcard, SearchMode));
				}
			}
		}
	}


	protected void DefaultPreModifyDeploymentContext(ProjectParams Params, DeploymentContext SC, ModifyStageContext Context)
	{

	}
	protected void DefaultModifyDeploymentContext(ProjectParams Params, DeploymentContext SC, ModifyStageContext Context)
	{
		// this will make sure that uncooked packages (maps, etc) go into the .pak, NOT the IOStore, which will fail to package them from a different location
		SC.OnlyAllowPackagesFromStdCookPathInIoStore = true;

		// if this is for internal use, then we allow all restricted  directories and ini settings
		if (!Context.bIsForExternalDistribution)
		{
			SC.RestrictedFolderNames.Clear();

			if (SC.IniKeyDenyList != null)
			{
				SC.IniKeyDenyList.Clear();
			}
			if (SC.IniSectionDenyList != null)
			{
				SC.IniSectionDenyList.Clear();
			}
		}

		StageEngineEditorFiles(Params, SC, Context);
		StageProjectEditorFiles(Params, SC, Context);

		// we need a better decision for this
		if (Context.bStageUAT)
		{
			StageUAT(Params, SC, Context);
		}


		// final filtering

		// we already cooked assets, so remove assets we may have found, except for the Uncook ones
		Context.UFSFilesToStage.RemoveAll(x => x.GetExtension() == ".uasset");

		if (!Context.bStageTargetFiles)
		{
			// don't need the .target files
			Context.NonUFSFilesToStage.RemoveAll(x => x.GetExtension() == ".target");
		}

		if (!Context.bStageShaderDirs)
		{
			// don't need standalone shaders
			Context.UFSFilesToStage.RemoveAll(x => x.GetExtension() == ".glsl");
			Context.UFSFilesToStage.RemoveAll(x => x.GetExtension() == ".hlsl");
		}

		// move some files from UFS to NonUFS if they ended up there
		List<string> UFSIncompatibleExtensions = new List<string> { ".py", ".pyc", ".pyd" };
		Context.NonUFSFilesToStage.AddRange(Context.UFSFilesToStage.Where(x => UFSIncompatibleExtensions.Contains(x.GetExtension())));
		Context.UFSFilesToStage.RemoveAll(x => UFSIncompatibleExtensions.Contains(x.GetExtension()));
		Context.NonUFSFilesToStage.AddRange(Context.UFSFilesToStage.Where(x => x.ContainsName("Python", 0) || x.ContainsName("__pycache__", 0)));
		Context.UFSFilesToStage.RemoveAll(x => x.ContainsName("Python", 0) || x.ContainsName("__pycache__", 0));

		// strip Python data now that it's all been moved to NonUFSFilesToStage
		bool bShouldStagePython = !bIsCookedCooker && Context.bStagePython;
		if (bShouldStagePython)
		{
			// strip out any leftover static library (.lib or .a) or source code (.c, .cpp, .h, .hpp, .inl) files within any Python script directories,
			// as pre-compiled modules sometimes leave these in their build alongside the resultant .dll or .pyd files
			List<string> NonPythonFilesToRemove = new List<string> { ".lib", ".a", ".c", ".cpp", ".h", ".hpp", ".inl" };
			Context.NonUFSFilesToStage.RemoveAll(x => x.ContainsName("Python", 0) && NonPythonFilesToRemove.Contains(x.GetExtension()));
		}
		else
		{
			// strip out any Python script directories and .py or .pyd files if we're not staging Python
			int PreviousCount = Context.NonUFSFilesToStage.Count;
			Context.NonUFSFilesToStage.RemoveAll(x => x.ContainsName("Python", 0));
			Context.NonUFSFilesToStage.RemoveAll(x => x.GetExtension() == ".py" || x.GetExtension() == ".pyd");
			Logger.LogInformation("Removed {0} Python script file(s) from NonUFSFilesToStage", PreviousCount - Context.NonUFSFilesToStage.Count);
		}
		{
			// strip out any __pycache__ directories and .pyc files regardless of whether we're staging Python (cache files shouldn't be staged)
			int PreviousCount = Context.NonUFSFilesToStage.Count;
			Context.NonUFSFilesToStage.RemoveAll(x => x.ContainsName("__pycache__", 0));
			Context.NonUFSFilesToStage.RemoveAll(x => x.GetExtension() == ".pyc");
			Logger.LogInformation("Removed {0} Python cache file(s) from NonUFSFilesToStage", PreviousCount - Context.NonUFSFilesToStage.Count);
		}

		// .tps files can all live in UFS
		// we do this after stripping Python, as the Python content folders may have .tps files within them
		Context.UFSFilesToStage.AddRange(Context.NonUFSFilesToStage.Where(x => x.GetExtension() == ".tps"));
		Context.NonUFSFilesToStage.RemoveAll(x => x.GetExtension() == ".tps");
	}

	protected virtual string GetReleaseTargetName(UnrealTargetPlatform Platform, TargetType ReleaseType)
	{
		// make the platform name, like "WindowsClient", or "LinuxGame", of the premade build we are cooking/staging against
		string IniPlatformName = ConfigHierarchy.GetIniPlatformName(Platform);
		string ReleaseTargetName = IniPlatformName + (ReleaseType == TargetType.Game ? "" : ReleaseType.ToString());

		return ReleaseTargetName;
	}

	protected virtual string GetReleaseVersionPath(string ReleaseVersionName, string ReleaseTargetName)
	{
		return CommandUtils.CombinePaths(ProjectFile.Directory.FullName, "Releases", ReleaseVersionName, ReleaseTargetName);
	}

	protected virtual ProjectParams MakeParams(string DLCName, string BasedOnReleaseVersion)
	{
		return new ProjectParams(
			Command: this
			, RawProjectPath: ProjectFile

			, NoBootstrapExe: true
			, DLCName: DLCName
			, BasedOnReleaseVersion: BasedOnReleaseVersion
			, DedicatedServer: bIsCookedCooker
			, NoClient: bIsCookedCooker
			, OptionalContent: true
		);
	}

	private ProjectParams GetParams()
	{
		// setup DLC defaults, then ask project if it should 
		string DLCName;
		string BasedOnReleaseVersion;
		TargetType ReleaseType;
		SetupDLCMode(ProjectFile, out DLCName, out BasedOnReleaseVersion, out ReleaseType);
		bool bIsDLC = DLCName != null;

		ProjectParams Params = MakeParams(DLCName, BasedOnReleaseVersion);

		// cook the cooked editor targetplatorm as the "client"
		//Params.ClientCookedTargets.Add("CrashReportClientEditor");

		string TargetPlatformType = bIsCookedCooker ? "CookedCooker" : "CookedEditor";
		string TargetName = ConfigHelper.GetString(bIsCookedCooker ? "CookedCookerTargetName" : "CookedEditorTargetName");
		UnrealTargetPlatform Platform = bIsCookedCooker ? Params.ServerTargetPlatforms[0].Type : Params.ClientTargetPlatforms[0].Type;


		// look to see if ini didn't override target name
		if (string.IsNullOrEmpty(TargetName))
		{
			// if not, then use ProjectCookedEditor
			TargetName = ProjectFile.GetFileNameWithoutAnyExtensions() + TargetPlatformType;
		}

		string OverrideBasedOnReleaseVersion = ParseParamValue("BasedOnReleaseVersion", null);
		if (!string.IsNullOrEmpty(OverrideBasedOnReleaseVersion))
		{
			Params.BasedOnReleaseVersion = OverrideBasedOnReleaseVersion;
		}

		// control the server/client taregts
		Params.ServerCookedTargets.Clear();
		Params.ClientCookedTargets.Clear();
		List<TargetPlatformDescriptor> TargetPlatformList = new List<TargetPlatformDescriptor>() { new TargetPlatformDescriptor(Platform, TargetPlatformType) };
		if (bIsCookedCooker)
		{
			Params.EditorTargets.Add(TargetName);
			Params.ServerCookedTargets.Add(TargetName);
			Params.ServerTargetPlatforms = TargetPlatformList;
		}
		else
		{
			Params.ClientCookedTargets.Add(TargetName);
			Params.ClientTargetPlatforms = TargetPlatformList;
		}


		// when making cooked editors, we some special commandline options to override some assumptions about editor data
		Params.AdditionalCookerOptions += " -ini:Engine:[Core.System]:CanStripEditorOnlyExportsAndImports=False";

		// when making cooked editors, we need to generate package data in our asset registry so that the cooker can actually find various things on disc
		Params.AdditionalCookerOptions += " -ini:Engine:[AssetRegistry]:bSerializePackageData=True";

		// We tend to "over-cook" packages to get everything we might need, so some non-editor BPs that are referencing editor BPs may
		// get cooked. This is okay, because the editor stuff should exist. We may want to revist this, and not cook anything that would
		// cause the issues
		Params.AdditionalCookerOptions += " -AllowUnsafeBlueprintCalls";

		// if we are cooking the editor in Dlc mode, then we want to attempt to cook assets that the base game may have skipped, but still
		// put into the AssetRegistry, so by default the DLC cooker will skip them. this will make the DLC cooker reevaluate these pacakges
		// and choose to cook them or not (generally for editoronly assets)
		Params.AdditionalCookerOptions += " -DlcReevaluateUncookedAssets";

		string AssetRegistryCacheRootFolder = ParseParamValue("AssetRegistryCacheRootFolder", "");
		if (!string.IsNullOrEmpty(AssetRegistryCacheRootFolder))
		{
			Params.AdditionalCookerOptions += string.Format(" -AssetRegistryCacheRootFolder={0}", AssetRegistryCacheRootFolder);
		}

		// set up cooking against a client, as DLC
		if (bIsDLC)
		{
			// cook and stage into our project, instead of the Engine's plugins
			DirectoryReference BaseOutputDirectory = DirectoryReference.Combine(ProjectFile.Directory, "Saved", "CookedEditor");
			string TargetPlatformName = ConfigHierarchy.GetIniPlatformName(Platform) + TargetPlatformList[0].CookFlavor;
			// Only override the defaults if something hasn't already overridden the potentially from the commandline.
			if (string.IsNullOrEmpty(Params.CookOutputDir))
			{
				Params.CookOutputDir = DirectoryReference.Combine(BaseOutputDirectory, "Cooked", TargetPlatformName).FullName;
			}
			if (string.IsNullOrEmpty(Params.StageDirectoryParam))
			{
				Params.StageDirectoryParam = DirectoryReference.Combine(BaseOutputDirectory, "Staged").FullName;
			}

			// make WindowsClient or LinuxGame, etc
			string ReleaseTargetName = GetReleaseTargetName(Platform, ReleaseType);

			Params.AdditionalCookerOptions += " -CookAgainstFixedBase";

			string DevelopmentAssetRegistryPlatformOverride = ParseParamValue("DevelopmentAssetRegistryPlatformOverride", ReleaseTargetName);
			Params.AdditionalCookerOptions += $" -DevelopmentAssetRegistryPlatformOverride={DevelopmentAssetRegistryPlatformOverride}";
			Params.AdditionalIoStoreOptions += $" -DevelopmentAssetRegistryPlatformOverride={DevelopmentAssetRegistryPlatformOverride}";

			// point to where the premade asset registry can be found
			Params.BasedOnReleaseVersionPathOverride = GetReleaseVersionPath(BasedOnReleaseVersion, ReleaseTargetName);

			Params.DLCOverrideStagedSubDir = "";
			Params.DLCIncludeEngineContent = true;
		}

		// set up override functions
		Params.PreModifyDeploymentContextCallback = (P, SC) => PreModifyDeploymentContext(P, SC);
		Params.ModifyDeploymentContextCallback = (P, SC) => ModifyDeploymentContext(P, SC);
		Params.FinalizeDeploymentContextCallback = (P, SC) => FinalizeDeploymentContext(P, SC);

		// this will make all of the files that are specified in BUild.cs files with "AdditionalPropertiesForReceipt.Add("CookerSupportFiles", ...);"  be copied into this
		// subdirectory along with a batch file that can be used to set platform SDK environment variables during cooking
		Params.CookerSupportFilesSubdirectory = "SDK";

		ModifyParams(Params);

		return Params;
	}

	private ProjectParams GetReleaseParams(ProjectParams MainParams, string[] Options)
	{
		ProjectParams ReleaseParams = new ProjectParams(
			Command: this
			, RawProjectPath: MainParams.RawProjectPath

			, Client: MainParams.Client
			, CreateReleaseVersion: MainParams.BasedOnReleaseVersion
			// tell the Params that we want cooked data
			, Build: true
			, Cook: true
			, Stage: true
			// MainParams already builds the editor
			, SkipBuildEditor: true
			, NoBootstrapExe: true
			// if the param to build/cook is specified, then actually build/cook, otherwise assume the Release is already built/cooked
			, SkipBuildClient: !Options.Contains("build", StringComparer.InvariantCultureIgnoreCase)
			, SkipCook: !Options.Contains("cook", StringComparer.InvariantCultureIgnoreCase)
			, SkipStage: !Options.Contains("stage", StringComparer.InvariantCultureIgnoreCase)
		);

		// if the MainParams override the ReleaseVersion path, use it directly
		ReleaseParams.BasedOnReleaseVersionPathOverride = MainParams.BasedOnReleaseVersionPathOverride;
		// if the MainParams have specified a base location to read the release info from, use that as the location to write 
		// to when creating the Release
		ReleaseParams.CreateReleaseVersionBasePath = MainParams.BasedOnReleaseVersionBasePath;

		// copy off the staging dir
		ReleaseParams.PreModifyDeploymentContextCallback = new Action<ProjectParams, DeploymentContext>((ProjectParams P, DeploymentContext SC) => { ReleaseStageDirectory = SC.StageDirectory; });

		// cooked editor doesn't work without OptionalContent now, so always generated it, and save it somewhere that staging of the cookededitor will get
		ReleaseOptionalFileStageDirectory = DirectoryReference.Combine(MainParams.RawProjectPath.Directory, "Saved", "CookedEditor", "OptionalData");
		ReleaseParams.OptionalContent = true;
		ReleaseParams.OptionalFileStagingDirectory = ReleaseOptionalFileStageDirectory.FullName;

		ModifyReleaseParams(ReleaseParams);

		return ReleaseParams;
	}


	protected static void GatherTargetDependencies(ProjectParams Params, DeploymentContext SC, ModifyStageContext Context, string ReceiptName)
	{
		GatherTargetDependencies(Params, SC, Context, ReceiptName, UnrealTargetConfiguration.Development);
	}

	protected static void GatherTargetDependencies(ProjectParams Params, DeploymentContext SC, ModifyStageContext Context, string ReceiptName, UnrealTargetConfiguration Configuration)
	{
		UnrealArchitectures Architecture = Params.EditorArchitecture;
		if (Architecture == null)
		{
			if (PlatformExports.IsPlatformAvailable(SC.StageTargetPlatform.IniPlatformType))
			{
				Architecture = UnrealArchitectureConfig.ForPlatform(SC.StageTargetPlatform.IniPlatformType).ActiveArchitectures(Params.RawProjectPath, null);
			}
		}

		FileReference ReceiptFilename = TargetReceipt.GetDefaultPath(Params.RawProjectPath.Directory, ReceiptName, SC.StageTargetPlatform.IniPlatformType, Configuration, Architecture);
		if (!FileReference.Exists(ReceiptFilename))
		{
			ReceiptFilename = TargetReceipt.GetDefaultPath(Unreal.EngineDirectory, ReceiptName, SC.StageTargetPlatform.IniPlatformType, Configuration, Architecture);
		}

		TargetReceipt Receipt;
		if (!TargetReceipt.TryRead(ReceiptFilename, out Receipt))
		{
			throw new AutomationException("Missing or invalid target receipt ({0})", ReceiptFilename);
		}

		foreach (BuildProduct BuildProduct in Receipt.BuildProducts)
		{
			Context.NonUFSFilesToStage.Add(BuildProduct.Path);
		}

		foreach (RuntimeDependency RuntimeDependency in Receipt.RuntimeDependencies)
		{
			if (RuntimeDependency.Type == StagedFileType.UFS)
			{
				Context.UFSFilesToStage.Add(RuntimeDependency.Path);
			}
			else// if (RuntimeDependency.Type == StagedFileType.NonUFS)
			{
				Context.NonUFSFilesToStage.Add(RuntimeDependency.Path);
			}
			//else
			//{
			//	// otherwise, just stage it directly
			//	// @todo: add a FilesToStage type to context like SC has?
			//	SC.StageFile(RuntimeDependency.Type, RuntimeDependency.Path);
			//}
		}

		Context.NonUFSFilesToStage.Add(ReceiptFilename);
	}
}
