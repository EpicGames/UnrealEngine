// Copyright Epic Games, Inc. All Rights Reserved.
using System;
using System.Collections.Generic;
using System.Linq;
using AutomationTool;

public class WindowsCookedEditor : Win64Platform
{
	public override string GetCookPlatform(bool bDedicatedServer, bool bIsClientOnly)
	{
		return "WindowsCookedEditor";
	}
	public override TargetPlatformDescriptor GetTargetPlatformDescriptor()
	{
		return new TargetPlatformDescriptor(TargetPlatformType, "CookedEditor");
	}
}

public class LinuxCookedEditor : GenericLinuxPlatform
{
	public override string GetCookPlatform(bool bDedicatedServer, bool bIsClientOnly)
	{
		return "LinuxCookedEditor";
	}
	public override TargetPlatformDescriptor GetTargetPlatformDescriptor()
	{
		return new TargetPlatformDescriptor(TargetPlatformType, "CookedEditor");
	}

}

public class MacCookedEditor : MacPlatform
{
	public override string GetCookPlatform(bool bDedicatedServer, bool bIsClientOnly)
	{
		return "MacCookedEditor";
	}
	public override TargetPlatformDescriptor GetTargetPlatformDescriptor()
	{
		return new TargetPlatformDescriptor(TargetPlatformType, "CookedEditor");
	}

}


public class WindowsCookedCooker : Win64Platform
{
	public override string GetCookPlatform(bool bDedicatedServer, bool bIsClientOnly)
	{
		return "WindowsCookedCooker";
	}
	public override TargetPlatformDescriptor GetTargetPlatformDescriptor()
	{
		return new TargetPlatformDescriptor(TargetPlatformType, "CookedCooker");
	}
}

public class LinuxCookedCooker : GenericLinuxPlatform
{
	public override string GetCookPlatform(bool bDedicatedServer, bool bIsClientOnly)
	{
		return "LinuxCookedCooker";
	}
	public override TargetPlatformDescriptor GetTargetPlatformDescriptor()
	{
		return new TargetPlatformDescriptor(TargetPlatformType, "CookedCooker");
	}

}

public class MacCookedCooker : MacPlatform
{
	public override string GetCookPlatform(bool bDedicatedServer, bool bIsClientOnly)
	{
		return "MacCookedCooker";
	}
	public override TargetPlatformDescriptor GetTargetPlatformDescriptor()
	{
		return new TargetPlatformDescriptor(TargetPlatformType, "CookedCooker");
	}

}