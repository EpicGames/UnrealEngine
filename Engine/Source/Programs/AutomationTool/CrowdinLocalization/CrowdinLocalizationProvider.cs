// Copyright Epic Games, Inc. All Rights Reserved.
using System;
using System.Collections.Generic;
using System.IO;
using System.Text;
using System.Linq;
using System.Net;
using System.Net.Http;
using System.Xml;
using System.ServiceModel;
using System.Security.Cryptography;
using AutomationTool;
using UnrealBuildTool;
using EpicGames.Localization;
using System.Diagnostics;
using System.Threading.Tasks;
using System.Text.Json;
using System.Net.Http.Headers;
using System.Text.Json.Serialization;

#pragma warning disable SYSLIB0014

namespace EpicGames.CrowdinLocalization
{
	struct CrowdInUrlData
	{
		public string Url { get; set; }
		public DateTime ExpireIn { get; set; }

		public string ETag { get; set; }
	}

	class DataEnvelope<T>
	{
		public T Data { get; set; }
	}

	struct Pagination
	{
		public int Offset { get; set; }
		public int Limit { get; set; }
	}

	public class FileInfoData
	{
		public int Id { get; set; }
		public int? projectId { get; set; }
		public int? branchId { get; set; }
		public int? directoryId { get; set; }
		public string name { get; set; }
		public string title { get; set; }
		public string type { get; set; }
		public string path { get; set; }
		public string status { get; set; }
	}

	class PagedData<T>
	{
		public DataEnvelope<T>[] Data { get; set; }
		public Pagination Pagination { get; set; }
	}

	public class StorageResponse
	{
		public int Id { get; set; }
		public string FileName { get; set; }
	}

	public class TranslationUploadResponse
	{
		public int projectId { get; set; }
		public int storageId { get; set; }
		public string languageId { get; set; }
		public int fileId { get; set; }
	}

	public class RequestErrors
	{
		public RequestErrorEnvelope[] Errors { get; set; }
	}

	public class RequestErrorEnvelope
	{
		public RequestError Error { get; set; }
	}

	public class RequestError
	{
		public string key { get; set; }
		public RequestKeyError[] Errors { get; set; }
	}

	public class RequestKeyError
	{
		public string code { get; set; }
		public string message { get; set; }
	}

	public class BranchInfo
	{
		public int id { get; set; }
		public int projectId { get; set; }
		public string name { get; set; }
		public string title { get; set; }
	}

	public class CrowdinConfig
	{
		public string ProjectId;
		public string AccessToken;
	};

	public abstract class CrowdinLocalizationProvider : LocalizationProvider
	{
		public CrowdinLocalizationProvider(LocalizationProviderArgs InArgs)
			: base(InArgs)
		{
			Config = new CrowdinConfig();
			Client = new HttpClient();
			MissingFiles = new HashSet<string>();

			BranchId = null;
			
			JsonOptions = new JsonSerializerOptions();
			JsonOptions.PropertyNameCaseInsensitive = true;
			JsonOptions.DefaultIgnoreCondition = JsonIgnoreCondition.WhenWritingNull;
		}

		public async override Task InitializeProjectWithLocalizationProvider(string ProjectName, ProjectImportExportInfo ProjectImportInfo)
		{
			List<string> ProjectLanguages = new List<string>();
			// Get the latest files for each culture.
			foreach (var Culture in ProjectImportInfo.CulturesToGenerate)
			{
				// Skip the native culture, as Crowdin doesn't have an entry for it
				if (Culture == ProjectImportInfo.NativeCulture)
				{
					continue;
				}

				var LanguageId = ConvertEpicCultureToCowdinLanguageCode(Culture);
				ProjectLanguages.Add(LanguageId);
			}

			string UpdateProjectUrl = $"https://api.crowdin.com/api/v2/projects/{Config.ProjectId}";
			string UpdateTargetLanguagesResult = await CrowdInRequest<string>(HttpMethod.Patch, UpdateProjectUrl,
				new[] { new { op = "replace", path = "/targetLanguageIds", value = ProjectLanguages.ToArray() } });

			Console.WriteLine(UpdateTargetLanguagesResult);

			if (!String.IsNullOrEmpty(LocalizationBranchName))
			{
				string ListBranchesUrl = $"https://api.crowdin.com/api/v2/projects/{Config.ProjectId}/branches";
				PagedData<BranchInfo> BranchList = await CrowdInRequest<PagedData<BranchInfo>>(HttpMethod.Get, ListBranchesUrl,
					new { name = SlugBranchName(LocalizationBranchName) });

				DataEnvelope<BranchInfo> ExistingBranch = BranchList.Data.FirstOrDefault();
				if (ExistingBranch == null)
				{
					string AddBranchUrl = $"https://api.crowdin.com/api/v2/projects/{Config.ProjectId}/branches";
					BranchInfo AddBranchResult = await CrowdInRequest<BranchInfo>(HttpMethod.Post, AddBranchUrl,
						new { name = SlugBranchName(LocalizationBranchName), title = LocalizationBranchName });

					BranchId = AddBranchResult.id;
				}
				else
				{
					BranchId = ExistingBranch.Data.id;
				}
			}
		}

		public async override Task DownloadProjectFromLocalizationProvider(string ProjectName, ProjectImportExportInfo ProjectImportInfo)
		{
			// Get the latest files for each culture.
			foreach (var Culture in ProjectImportInfo.CulturesToGenerate)
			{
				// Skip the native culture, as Crowdin doesn't have an entry for it
				if (Culture == ProjectImportInfo.NativeCulture)
				{
					continue;
				}

				await DownloadLatestPOFile(Culture, null, ProjectImportInfo);
				foreach (var Platform in ProjectImportInfo.SplitPlatformNames)
				{
					await DownloadLatestPOFile(Culture, Platform, ProjectImportInfo);
				}
			}
		}

		private async Task DownloadLatestPOFile(string Culture, string Platform, ProjectImportExportInfo ProjectImportInfo)
		{
			var DestinationDirectory = String.IsNullOrEmpty(Platform)
				? new DirectoryInfo(CommandUtils.CombinePaths(RootWorkingDirectory, ProjectImportInfo.DestinationPath))
				: new DirectoryInfo(CommandUtils.CombinePaths(RootWorkingDirectory, ProjectImportInfo.DestinationPath, ProjectImportExportInfo.PlatformLocalizationFolderName, Platform));
			var CultureDirectory = (ProjectImportInfo.bUseCultureDirectory) ? new DirectoryInfo(Path.Combine(DestinationDirectory.FullName, Culture)) : DestinationDirectory;

			var ExportFile = new FileInfo(Path.Combine(CultureDirectory.FullName, ProjectImportInfo.PortableObjectName));
			var CrowdinFilename = GetCrowdinFilename(ProjectImportInfo.PortableObjectName, Platform);

			Console.WriteLine("Exporting: '{0}' as '{1}' ({2})", CrowdinFilename, ExportFile.FullName, Culture);

			try
			{
				var LanguageId = ConvertEpicCultureToCowdinLanguageCode(Culture);

				string Url = $"https://api.crowdin.com/api/v2/projects/{Config.ProjectId}/files";
				PagedData<FileInfoData> FileList = await CrowdInRequest<PagedData<FileInfoData>>(HttpMethod.Get, Url,
					new { filter = CrowdinFilename, BranchId = BranchId });

				int? fileId = GetFileId(FileList, CrowdinFilename);

				DataEnvelope<CrowdInUrlData> DownloadUrl = null;

				string RemoteUrl = $"https://api.crowdin.com/api/v2/projects/{Config.ProjectId}/translations/builds/files/{fileId}";

				if (fileId.HasValue)
				{
					DownloadUrl = await CrowdInRequest<DataEnvelope<CrowdInUrlData>>(HttpMethod.Post, RemoteUrl,
						new { targetLanguageId = LanguageId });
				}
				else
				{
					// This means a 404 was encountered and we'll need to add (rather than 
					// update) this file later when uploading the source data to Crowdin
					MissingFiles.Add(CrowdinFilename);
					throw new Exception(String.Format("HTTP Request to '{0}' failed. 404 (Not Found)", RemoteUrl));
				}

				if (!CultureDirectory.Exists)
				{
					CultureDirectory.Create();
				}

				// Write out the updated PO file so that the gather commandlet will import the new data from it
				{
					var ExportFileWasReadOnly = false;
					if (ExportFile.Exists)
					{
						// We're going to clobber the existing PO file, so make sure it's writable (it may be read-only if in Perforce)
						ExportFileWasReadOnly = ExportFile.IsReadOnly;
						ExportFile.IsReadOnly = false;
					}

					using (var client = new WebClient())
					{
						client.DownloadFile(DownloadUrl.Data.Url, ExportFile.FullName);
					}

					Console.WriteLine("[SUCCESS] Exporting: '{0}' as '{1}' ({2})", CrowdinFilename, ExportFile.FullName, Culture);

					if (ExportFileWasReadOnly)
					{
						ExportFile.IsReadOnly = true;
					}
				}

				// Also update the back-up copy so we can diff against what we got from crowdin, and what the gather commandlet produced
				{
					var ExportFileCopy = new FileInfo(Path.Combine(ExportFile.DirectoryName, String.Format("{0}_FromCrowdin{1}", Path.GetFileNameWithoutExtension(ExportFile.Name), ExportFile.Extension)));

					var ExportFileCopyWasReadOnly = false;
					if (ExportFileCopy.Exists)
					{
						// We're going to clobber the existing PO file, so make sure it's writable (it may be read-only if in Perforce)
						ExportFileCopyWasReadOnly = ExportFileCopy.IsReadOnly;
						ExportFileCopy.IsReadOnly = false;
					}

					ExportFile.CopyTo(ExportFileCopy.FullName, true);

					if (ExportFileCopyWasReadOnly)
					{
						ExportFileCopy.IsReadOnly = true;
					}

					// Add/check out backed up POs from Crowdin.
					if (CommandUtils.P4Enabled)
					{
						UnrealBuild.AddBuildProductsToChangelist(PendingChangeList, new List<string>() { ExportFileCopy.FullName });
					}
				}
			}
			catch (Exception Ex)
			{
				Console.WriteLine("[FAILED] Exporting: '{0}' ({1}) - {2}", ExportFile.FullName, Culture, Ex);
			}

			await Task.CompletedTask;
		}

		int? GetFileId(PagedData<FileInfoData> FileList, string File)
		{
			DataEnvelope<FileInfoData> Entry = FileList.Data.FirstOrDefault(Envelope =>
			{
				return Envelope.Data.name == File && Envelope.Data.branchId == BranchId;
			});

			if (Entry != null)
			{
				return Entry.Data.Id;
			}

			return null;
		}

		public async override Task UploadProjectToLocalizationProvider(string ProjectName, ProjectImportExportInfo ProjectExportInfo)
		{
			// Upload the .po file for the native culture first
			await UploadLatestPOFile(ProjectExportInfo.NativeCulture, null, ProjectExportInfo);
			foreach (var Platform in ProjectExportInfo.SplitPlatformNames)
			{
				await UploadLatestPOFile(ProjectExportInfo.NativeCulture, Platform, ProjectExportInfo);
			}

			if (bUploadAllCultures)
			{
				// Upload the remaining .po files for the other cultures
				foreach (var Culture in ProjectExportInfo.CulturesToGenerate)
				{
					// Skip native culture as we uploaded it above
					if (Culture != ProjectExportInfo.NativeCulture)
					{
						await UploadLatestPOFile(Culture, null, ProjectExportInfo);
						foreach (var Platform in ProjectExportInfo.SplitPlatformNames)
						{
							await UploadLatestPOFile(Culture, Platform, ProjectExportInfo);
						}
					}
				}
			}
		}

		private async Task UploadLatestPOFile(string Culture, string Platform, ProjectImportExportInfo ProjectExportInfo)
		{
			var SourceDirectory = String.IsNullOrEmpty(Platform)
					? new DirectoryInfo(CommandUtils.CombinePaths(RootWorkingDirectory, ProjectExportInfo.DestinationPath))
					: new DirectoryInfo(CommandUtils.CombinePaths(RootWorkingDirectory, ProjectExportInfo.DestinationPath, ProjectImportExportInfo.PlatformLocalizationFolderName, Platform));
			var CultureDirectory = (ProjectExportInfo.bUseCultureDirectory) ? new DirectoryInfo(Path.Combine(SourceDirectory.FullName, Culture)) : SourceDirectory;

			var FileToUpload = new FileInfo(Path.Combine(CultureDirectory.FullName, ProjectExportInfo.PortableObjectName));
			var CrowdinFilename = GetCrowdinFilename(ProjectExportInfo.PortableObjectName, Platform);

			bool bIsNative = Culture == ProjectExportInfo.NativeCulture;

			Console.WriteLine("Uploading: '{0}' as '{1}' ({2})", FileToUpload.FullName, CrowdinFilename, Culture);

			try
			{
				string Url = $"https://api.crowdin.com/api/v2/projects/{Config.ProjectId}/files";
				PagedData<FileInfoData> FileList = await CrowdInRequest<PagedData<FileInfoData>>(HttpMethod.Get, Url,
					new { branchId = BranchId, filter = CrowdinFilename });

				if (bIsNative)
				{
					string StorageUrl = $"https://api.crowdin.com/api/v2/storages";
					StorageResponse StorageResponse = await CrowdInStorage(StorageUrl, CrowdinFilename, FileToUpload.FullName);

					int? fileId = GetFileId(FileList, CrowdinFilename);

					if (fileId.HasValue)
					{
						string ReplaceFileUrl = $"https://api.crowdin.com/api/v2/projects/{Config.ProjectId}/files/{fileId.Value}";
						string AddFileResponse = await CrowdInRequest<string>(HttpMethod.Put, ReplaceFileUrl,
							new { storageId = StorageResponse.Id });
					}
					else
					{
						string AddFileUrl = $"https://api.crowdin.com/api/v2/projects/{Config.ProjectId}/files";
						string AddFileResponse = await CrowdInRequest<string>(HttpMethod.Post, AddFileUrl,
							new { storageId = StorageResponse.Id, name = CrowdinFilename, branchId = BranchId });
					}

					Console.WriteLine("[SUCCESS] Uploading: '{0}' ({1})", FileToUpload.FullName, Culture);
				}
				else
				{
					var LanguageId = ConvertEpicCultureToCowdinLanguageCode(Culture);

					int? fileId = GetFileId(FileList, CrowdinFilename);
					if (fileId.HasValue)
					{
						string StorageUrl = $"https://api.crowdin.com/api/v2/storages";
						StorageResponse StorageResponse = await CrowdInStorage(StorageUrl, CrowdinFilename, FileToUpload.FullName);

						string UploadTranslationUrl = $"https://api.crowdin.com/api/v2/projects/{Config.ProjectId}/translations/{LanguageId}";
						DataEnvelope<TranslationUploadResponse> UploadResponse = await CrowdInRequest<DataEnvelope<TranslationUploadResponse>>(HttpMethod.Post, UploadTranslationUrl,
							new { storageId = StorageResponse.Id, fileId = fileId.Value });

						Console.WriteLine("[SUCCESS] Uploading: '{0}' ({1})", FileToUpload.FullName, Culture);
					}
					else
					{
						Console.WriteLine("Unable to upload: '{0}' ({1})", FileToUpload.FullName, Culture);
					}
				}
			}
			catch (Exception Ex)
			{
				Console.WriteLine("[FAILED] Uploading: '{0}' ({1}) - {2}", FileToUpload.FullName, Culture, Ex);
			}
		}

		private string GetCrowdinFilename(string BaseFilename, string Platform)
		{
			var CrowdinFilename = BaseFilename;
			if (!String.IsNullOrEmpty(Platform))
			{
				// Apply the platform suffix.
				var CrowdinFilenameWithSuffix = Path.GetFileNameWithoutExtension(CrowdinFilename) + "_" + Platform + Path.GetExtension(CrowdinFilename);
				CrowdinFilename = CrowdinFilenameWithSuffix;
			}
			if (!String.IsNullOrEmpty(RemoteFilenamePrefix))
			{
				// Apply the prefix (this is used to avoid collisions with plugins that use the same name for their PO files)
				CrowdinFilename = RemoteFilenamePrefix + "_" + CrowdinFilename;
			}

			return CrowdinFilename;
		}


		private async Task<T> CrowdInRequest<T>(HttpMethod Method, string Url, object Content) where T : class
		{
			using (var Request = new HttpRequestMessage(Method, Url))
			{
				Request.Headers.Add("Authorization", "Bearer " + Config.AccessToken);

				Request.Content = new StringContent(
				   JsonSerializer.Serialize(Content),
				   Encoding.UTF8,
				   "application/json"
				);

				// NOTE: Using await here silently crashes the application.
				var Response = Client.SendAsync(Request).Result;
				if (!Response.IsSuccessStatusCode)
				{
					var errorJsonString = await Response.Content.ReadAsStringAsync();
					var ErrorResult = JsonSerializer.Deserialize<RequestErrors>(errorJsonString, JsonOptions);

					Console.WriteLine(ErrorResult.ToString());

					throw new Exception(errorJsonString);
				}

				var jsonString = await Response.Content.ReadAsStringAsync();

				if (typeof(string).IsAssignableFrom(typeof(T)))
				{
					return jsonString as T;
				}

				var Result = JsonSerializer.Deserialize<T>(jsonString, JsonOptions);

				return Result;
			}
		}

		private async Task<StorageResponse> CrowdInStorage(string Url, string CrowdinFilename, string LocalFilePath)
		{
			using (var Request = new HttpRequestMessage(HttpMethod.Post, Url))
			{
				Request.Headers.Add("Authorization", "Bearer " + Config.AccessToken);
				Request.Headers.Add("Crowdin-API-FileName", CrowdinFilename);

				Request.Content = new StreamContent(new FileStream(LocalFilePath, FileMode.Open));
				Request.Content.Headers.ContentType = new MediaTypeHeaderValue("application/octet-stream");

				// NOTE: Using await here silently crashes the application.
				var Response = Client.SendAsync(Request).Result;
				if (!Response.IsSuccessStatusCode)
				{
					var errorJsonString = await Response.Content.ReadAsStringAsync();
					var ErrorResult = JsonSerializer.Deserialize<RequestErrors>(errorJsonString, JsonOptions);

					Console.WriteLine(ErrorResult.ToString());

					throw new Exception(errorJsonString);
				}

				var jsonString = await Response.Content.ReadAsStringAsync();
				var Result = JsonSerializer.Deserialize<DataEnvelope<StorageResponse>>(jsonString, JsonOptions);

				return Result.Data;
			}
		}

		virtual protected string ConvertEpicCultureToCowdinLanguageCode(string Culture)
		{
			// Anything not mapped is assumed to match what Unreal uses
			string MappedCulture = null;
			if (CultureMappings.TryGetValue(Culture, out MappedCulture))
			{
				Culture = MappedCulture;
			}
			return Culture;
		}

		string SlugBranchName(string BranchName)
		{
			var charsToRemove = new string[] { "\\", "/", ":", "*", "?", "\"", "<", ">", "|" };
			foreach (var c in charsToRemove)
			{
				BranchName = BranchName.Replace(c, string.Empty);
			}

			return BranchName;
		}

		protected CrowdinConfig Config;
		protected HttpClient Client;
		protected HashSet<string> MissingFiles;

		private JsonSerializerOptions JsonOptions;
		private int? BranchId;

		// Crowdin mostly uses the same language codes as Unreal, however there are some differences
		// See: https://support.crowdin.com/api/language-codes/
		private static Dictionary<string, string> CultureMappings =
			new Dictionary<string, string>
			{
				{ "es-419",  "es-US" },	// LatAm Spanish
				{ "zh-Hans", "zh-CN" },	// Simp Chinese
				{ "zh-Hant", "zh-TW" },	// Trad Chinese
			};
	}
}