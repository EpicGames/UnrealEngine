// Copyright Epic Games, Inc. All Rights Reserved.

using Gauntlet;
using System;
using System.Collections.Generic;
using System.Linq;
using System.Text;
using System.Threading.Tasks;

namespace EditorTest
{
	class BootTest : UnrealGame.DefaultTest
	{
		public BootTest(Gauntlet.UnrealTestContext InContext) 
			: base(InContext)
		{
		}

		public override UnrealGame.UnrealTestConfig GetConfiguration()
		{
			UnrealGame.UnrealTestConfig Config = base.GetConfiguration();

			Config.RequireRole(UnrealTargetRole.Editor);

			return Config;
		}
	}
}
