// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using UnrealBuildTool;
using Gauntlet;
using AutomationTool;
using System.Reflection;
using UnrealBuildBase;

namespace EditorTest
{
	/// <summary>
	/// Default set of options that are available for all "EditorTest" derived notes to configure as
	/// appropriate. Configurable options are public, external command-line driven options should 
	/// be protected/private and set via AutoParam's
	/// </summary>
	public class EditorTestConfig : UnrealTestConfiguration, IAutoParamNotifiable
	{
		// Global options set via the command line

		/// <summary>
		/// Name of the EngineTest test to run
		/// </summary>
		[AutoParam("")]
		protected string TestName { get; set; }

		// Global options set via the command line
		[AutoParam(false)]
		protected bool Report { get; set; }

		/// <summary>
		/// Validate external arguments
		/// </summary>
		/// <param name="Params"></param>
		/// <returns></returns>
		public void ParametersWereApplied(string[] Params)
		{
			if (string.IsNullOrEmpty(TestName))
			{
				throw new Exception("No testname was specified. Use -testname=<EngineTest>");
			}
		}

		/// <summary>
		/// Applies the selected options to the provided app config
		/// </summary>
		/// <param name="AppConfig"></param>
		/// <param name="ConfigRole"></param>
		/// <param name="OtherRoles"></param>
		public override void ApplyToConfig(UnrealAppConfig AppConfig, UnrealSessionRole ConfigRole, IEnumerable<UnrealSessionRole> OtherRoles)
		{
			base.ApplyToConfig(AppConfig, ConfigRole, OtherRoles);

			if (AppConfig.ProcessType.IsEditor())
			{
				AppConfig.CommandLine += string.Format(" -ExecCmds=\"Automation RunTests Project.Functional Tests.{0}\"", TestName);

				if (Report)
				{
					AppConfig.CommandLine += "-DeveloperReportOutputPath=\"\\rdu-automation-01\\reports\\\" -DeveloperReportUrl=\"http://automation.epicgames.net/reports/\"";
				}
			}
		}		
	}

	/// <summary>
	/// Base class for common editor test functionality
	/// </summary>
	public class EditorTestNode : UnrealTestNode<EditorTestConfig>
	{
		public EditorTestNode(UnrealTestContext InContext) : base(InContext)
		{
		}

		/// <summary>
		/// Returns the configuration for this test
		/// </summary>
		/// <returns></returns>
		public override EditorTestConfig GetConfiguration()
		{
			var Config = base.GetConfiguration();

			// create an editor role
			var EditorRole = Config.RequireRole(UnrealTargetRole.Editor);

			return Config;
		}
	}

	public class RunEditorTests : Gauntlet.RunUnreal
	{
		public override ExitCode Execute()
		{
			// Cheat - if we can base everything off a single note, don't require users to specify it by appending it
			// to our global argument list
			List<string> ExpandedParams = new List<string>(this.Params);
			ExpandedParams.Add("test=EditorTestNode");

			// save off these params
			Globals.Params = new Gauntlet.Params(ExpandedParams.ToArray());

			// create test options and apply any params
			UnrealTestOptions ContextOptions = new UnrealTestOptions();
			AutoParam.ApplyParamsAndDefaults(ContextOptions, Globals.Params.AllArguments);

			// These are fixed for this prohect
			ContextOptions.Project = "EngineTest";
			ContextOptions.Namespaces = "EditorTest";
			ContextOptions.Build = "Editor";
			ContextOptions.UsesSharedBuildType = true;

			return RunTests(ContextOptions);
		}
	}
}
