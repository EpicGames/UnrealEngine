// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;

namespace Gauntlet
{
	/// <summary>
	/// Defines an account that can be used in tests by appending credential information
	/// to the command line
	/// </summary>
	public abstract class Account : IConfigOption<UnrealAppConfig>
	{
		/// <summary>
		/// Called to apply our data to the provided config
		/// </summary>
		/// <param name="AppConfig"></param>
		public abstract void ApplyToConfig(UnrealAppConfig AppConfig);

		/// <summary>
		/// Username for this account.
		/// </summary>
		public abstract string Username { get; protected set; }
	}

	/// <summary>
	/// Epic account implementation, all of our accounts can be used on a client by providing the type, username, and
	/// password/token on the command line
	/// </summary>
	public class EpicAccount : Account
	{
		public override string Username { get; protected set; }

		public string Password { get; protected set; }

		public string EpicAccountId { get; protected set; }

		public EpicAccount(string InUsername, string InPassword)
		{
			Username = InUsername;
			Password = InPassword;
		}

		public EpicAccount(string InUserName, string InPassword, string InAccountId) : this(InUserName, InPassword)
		{
			EpicAccountId = InAccountId;
		}

		public override void ApplyToConfig(UnrealAppConfig AppConfig)
		{
			// add our credentials to the command line
			AppConfig.CommandLine += string.Format(" -AUTH_TYPE=Epic -AUTH_LOGIN={0} -AUTH_PASSWORD={1}", Username, Password);
		}
	}
}