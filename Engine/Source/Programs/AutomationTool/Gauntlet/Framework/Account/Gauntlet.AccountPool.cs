// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Linq;
using System.Reflection;

namespace Gauntlet
{

	public class AccountPool
	{

		/// <summary>
		/// Singleton
		/// </summary>
		private static IManagedAccountPool _Instance;

		public static void Initialize()
		{
			Initialize <ManagedAccountPool> ();
		}

		~AccountPool()
		{
			if (_Instance != null)
			{
				_Instance.Dispose();
			}
		}

		public static void Initialize<T>()
		{
			object PossiblePool = Activator.CreateInstance(typeof(T));
			if (PossiblePool is IManagedAccountPool)
			{
				if (_Instance != null)
				{
					_Instance.Dispose();
				}
				_Instance = (IManagedAccountPool)PossiblePool;
			}
		}

		public static IManagedAccountPool Instance
		{
			get
			{
				if (_Instance == null)
				{
					Initialize();
				}
				return _Instance;
			}
		}

		public static void Shutdown()
		{
			if (_Instance != null)
			{
				_Instance.Dispose();
				_Instance = null;
			}
		}

	}

	public interface IManagedAccountPool: IDisposable
	{
		void RegisterAccount(Account InAccount);

		void ReleaseAccount(Account InAccount);

		Account ReserveAccount();
	}

	/// <summary>
	/// A pool of accounts for tests to use. The pool should be filled by the project-specific script that derives from
	/// RunUnrealTests
	/// </summary>
	public class ManagedAccountPool : IManagedAccountPool
	{

		/// <summary>
		/// Object used for locking access to internal data
		/// </summary>
		protected Object LockObject = new Object();

		/// <summary>
		/// List of all registered accounts
		/// </summary>
		protected List<Account> AllAccounts = new List<Account>();

		/// <summary>
		/// Accounts that have been reserved
		/// </summary>
		protected Dictionary<Account, bool> ReservedAccounts = new Dictionary<Account, bool>();

		/// <summary>
		/// A random ordering of AllAccounts
		/// </summary>
		protected List<Account> RandomlyOrderedList = new List<Account>();


		~ManagedAccountPool()
		{
			Dispose(false);
		}



		public void Dispose()
		{
			Dispose(true);
			GC.SuppressFinalize(this);
		}


		/// <summary>
		/// Perform actual dispose behavior
		/// </summary>
		/// <param name="disposing"></param>
		protected void Dispose(bool disposing)
		{
			lock (LockObject)
			{
				if (disposing)
				{
					IEnumerable<Account> StillReserved = ReservedAccounts.Keys.Where(K => ReservedAccounts[K] == true);

					StillReserved.ToList().ForEach(A => Log.Warning("Account {0} was still reserved on dispose!", A));

					ReservedAccounts.Clear();
					RandomlyOrderedList.Clear();
					ReservedAccounts.Clear();
				}
			}
		}

		public void RegisterAccount(Account InAccount)
		{
			AllAccounts.Add(InAccount);
		}

		public void ClearReservatios()
		{
			lock (LockObject)
			{
				RandomlyOrderedList.Clear();
				ReservedAccounts.Clear();
			}
		}

		public void ReleaseAccount(Account InAccount)
		{
			lock (LockObject)
			{
				ReservedAccounts[InAccount] = false;
			}
		}

		virtual public Account ReserveAccount()
		{
			lock (LockObject)
			{
				// if number of avail accounts has changed...
				if (RandomlyOrderedList.Count + ReservedAccounts.Keys.Count != AllAccounts.Count)
				{
					// list of all available accounts
					List<Account> SelectionList = AllAccounts.Where(A => ReservedAccounts.ContainsKey(A) == false).ToList();

					Random Rand = new Random();

					while (SelectionList.Count > 0)
					{
						int Index = Rand.Next(SelectionList.Count);

						RandomlyOrderedList.Add(SelectionList[Index]);

						SelectionList.RemoveAt(Index);
					}
				}

				if (RandomlyOrderedList.Count == 0)
				{
					throw new Exception("Unable to find free account!");
				}

				Account SelectedAccount = RandomlyOrderedList.First();
				RandomlyOrderedList.RemoveAt(0);

				ReservedAccounts[SelectedAccount] = true;

				return SelectedAccount;
			}
		}
	}
}