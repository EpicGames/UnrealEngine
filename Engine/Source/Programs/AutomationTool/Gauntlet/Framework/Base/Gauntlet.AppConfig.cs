// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Text;

namespace Gauntlet
{
	/// <summary>
	/// IAppConfig defines an interface that can express the configuration required to install
	/// and run an instance of a given app
	/// </summary>
	public interface IAppConfig
	{
		/// <summary>
		/// Name of the app
		/// </summary>
		string Name { get; }

		/// <summary>
		/// Arguments to execute the app with
		/// </summary>
		string CommandLine { get; }

		/// <summary>
		/// Sandbox that we'd like to install this instance in
		/// </summary>
		string Sandbox { get; }

		/// <summary>
		/// Build that the application uses
		/// </summary>
		IBuild Build { get; }
	}
}
