// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Text;

namespace Gauntlet
{
	/// <summary>
	/// IAppInstall represents an instance of an application that has been installed on a device and may now be run
	/// </summary>
	public interface IAppInstall
	{
		/// <summary>
		/// Description of this app
		/// </summary>
		string Name { get; }

		/// <summary>
		/// Create a running instance of the app
		/// </summary>
		/// <returns></returns>
		IAppInstance Run();

		/// <summary>
		/// The device that we'll be run on
		/// </summary>
		ITargetDevice Device { get; }



		/// <summary>
		/// IAppInstallDynamicCommandLine can be implemented by IAppInstall classes where the platform
		/// allows the command line to be updated after installation
		/// </summary>
		public interface IDynamicCommandLine
		{
			/// <summary>
			/// Appends the given string to the installed application's command line
			/// </summary>
			/// <param name="AdditionalCommandLine"></param>
			void AppendCommandline(string AdditionalCommandLine);
		}
	}


}
