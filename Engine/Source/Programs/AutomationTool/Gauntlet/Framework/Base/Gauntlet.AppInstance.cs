// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Text;


namespace Gauntlet
{
	/// <summary>
	/// Interface that represents an instance of an app running on a device
	/// </summary>
	public interface IAppInstance
 	{
		/// <summary>
		/// Returns true/false if the process has exited for any reason
		/// </summary>
		bool HasExited { get; }

		/// <summary>
		/// Current StdOut of the process. Not efficient when the process is running. Use GetLogBufferReader() instead.
		/// </summary>
		string StdOut { get; }

		/// <summary>
		/// Return a new log reader with an internal cursor
		/// </summary>
		ILogStreamReader GetLogReader();

		/// <summary>
		/// Return a new log buffer reader with an internal cursor. Might not give access to the full log content (usually the last 1024 lines).
		/// Use GetLogReader() if you need the log from the beginning.
		/// </summary>
		/// <returns></returns>
		ILogStreamReader GetLogBufferReader();

		/// <summary>
		/// Write output to file. Return true if there was output data to write.
		/// </summary>
		/// <param name="FilePath"></param>
		/// <returns></returns>
		bool WriteOutputToFile(string FilePath);

		/// <summary>
		/// Exit code of the process.
		/// </summary>
		int ExitCode { get; }

		/// <summary>
		/// Returns true if the process exited due to Kill() being called
		/// </summary>
		bool WasKilled { get; }

		/// <summary>
		/// Path to commandline used to start the process
		/// </summary>
		string CommandLine { get; }

		/// <summary>
		/// Path to artifacts from the process
		/// </summary>
		string ArtifactPath { get; }

		/// <summary>
		/// Device that the app was run on
		/// </summary>
		ITargetDevice Device { get; }

		/// <summary>
		/// Kills the process if its running (no need to call WaitForExit)
		/// </summary>
		void Kill(bool GenerateDumpOnKill = false);

		/// <summary>
		/// Waits for the process to exit normally
		/// </summary>
		/// <returns></returns>
		int WaitForExit();

	}

	/// <summary>
	/// Interface used by IAppInstance if they support Suspend/Resume
	/// </summary>
	public interface IWithPLMSuspend
	{
		/// <summary>
		/// Attempt to suspend the running application. Correlates to FCoreDelegates::ApplicationWillEnterBackgroundDelegate
		/// </summary>
		bool Suspend();

		/// <summary>
		/// Attempts to resume a suspended application. Correlates to FCoreDelegates::ApplicationHasEnteredForegroundDelegate
		/// </summary>
		bool Resume();
	}

	/// <summary>
	/// Interface used by IAppInstance if they support Constrain/Unconstrain
	/// </summary>
	public interface IWithPLMConstrain
	{
		/// <summary>
		/// Attempts to contrain the running application. Correlates to FCoreDelegates::ApplicationWillDeactivateDelegate
		/// </summary>
		bool Constrain();

		/// <summary>
		/// Attempts to unconstained a constrained application. Correlates to FCoreDelegates::ApplicationHasReactivatedDelegate
		/// </summary>
		bool Unconstrain();
	}
}
