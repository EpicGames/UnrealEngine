// Copyright Epic Games, Inc. All Rights Reserved.

using System.Collections.Generic;

namespace Gauntlet
{
	/// <summary>
	/// Simple node that implements the ITestNode interface and provides some convenience functions for
	/// inherited tests to use
	/// </summary>
	public abstract class BaseTest : ITestNode
	{
		/// <summary>
		/// Override this to set the max running time for this test
		/// </summary>
		public abstract float MaxDuration { get; protected set; }

		/// <summary>
		/// What the test result should be treated as if we reach max duration.
		/// </summary>
		public virtual EMaxDurationReachedResult MaxDurationReachedResult { get; set; }

		/// <summary>
		/// Override this to set the priority of this test
		/// </summary>
		public virtual TestPriority Priority { get { return TestPriority.Normal; } }

		/// <summary>
		/// Return the name of this test
		/// </summary>
		public abstract string Name { get; }

		/// <summary>
		/// Returns true if the test has encountered warnings. Test is expected to list any warnings it considers appropriate in the summary
		/// </summary>
		public virtual bool HasWarnings { get; protected set; }

		/// <summary>
		/// Returns reason for the test cancellation
		/// </summary>
		public virtual string CancellationReason { get; protected set; }

		/// <summary>
		/// Returns true if the test was cancelled
		/// </summary>
		public virtual bool WasCancelled { get; protected set; }

		/// <summary>
		/// Internal status state
		/// </summary>
		private TestStatus InnerStatus;

		/// <summary>
		/// Return true if the warnings and errors needs to log after summary
		/// </summary>
		public virtual bool LogWarningsAndErrorsAfterSummary { get; protected set; } = true;

		/// <summary>
		/// Default BaseTest Constructor
		/// </summary>
		public BaseTest()
		{
			SetTestStatus(TestStatus.NotStarted);
		}

		/// <summary>
		/// OVerride this to return the result of the test
		/// </summary>
		/// <returns></returns>
		public abstract TestResult GetTestResult();

		/// <summary>
		/// Set the test result value of the test.
		/// </summary>
		/// <param name="InTestResult">New result that the Test should have.</param>
		public abstract void SetTestResult(TestResult InTestResult);

		/// <summary>
		/// Add a new test event to be rolled up into the summary at the end of this test.
		/// </summary>
		public virtual void AddTestEvent(UnrealTestEvent InEvent)
		{
		}

		/// <summary>
		/// Summarize the result of the test
		/// </summary>
		/// <returns></returns>
		public virtual string GetTestSummary()
		{
			return string.Format("{0}: {1}", Name, GetTestResult());
		}

		/// <summary>
		/// Return list of warnings. Empty by default
		/// </summary>
		/// <returns></returns>
		public virtual IEnumerable<string> GetWarnings()
		{
			return System.Array.Empty<string>();
		}

		/// <summary>
		/// Return list of errors. Empty by default
		/// </summary>
		/// <returns></returns>
		public virtual IEnumerable<string> GetErrors()
		{
			return System.Array.Empty<string>();
		}

		public virtual string GetRunLocalCommand(string LaunchingBuildCommand)
		{
			string CommandToRunLocally =
				string.Format("RunUAT {0} -Test={1} ", LaunchingBuildCommand, GetType());
			return CommandToRunLocally;
		}

		/// <summary>
		/// Updates the test status to the passed in value
		/// </summary>
		/// <param name="InStatus"></param>
		protected void SetTestStatus(TestStatus InStatus)
		{
			InnerStatus = InStatus;
		}

		/// <summary>
		/// Mark the test as started
		/// </summary>
		/// <returns></returns>
		protected void MarkTestStarted()
		{
			SetTestStatus(TestStatus.InProgress);
			SetTestResult(TestResult.Invalid);
		}

		/// <summary>
		/// Mark the test as complete
		/// </summary>
		/// <returns></returns>
		protected void MarkTestComplete()
		{
			SetTestStatus(TestStatus.Complete);
		}

		/// <summary>
		/// Return our internal status
		/// </summary>
		/// <returns></returns>
		public TestStatus GetTestStatus()
		{
			return InnerStatus;
		}

		public virtual bool IsReadyToStart()
		{
			return true;
		}

		/// <summary>
		/// Called to request the test launch. Should call MarkTestComplete
		/// </summary>
		/// <returns></returns>
		public abstract bool StartTest(int Pass, int NumPasses);

		/// <summary>
		/// Called to request the test is shutdown
		/// </summary>
		/// <returns></returns>
		public abstract void CleanupTest();

		/// <summary>
		/// Called if TestResult returns WantRetry. Can be overridden for something
		/// more elegant than a hard shutdown/launch
		/// </summary>
		/// <returns></returns>
		public virtual bool RestartTest()
		{
			CleanupTest();
			return StartTest(0, 1);
		}

		/// <summary>
		/// Gives the test a chance to perform logic. Should call MarkComplete() when it detects a
		/// success or error state
		/// </summary>
		/// <returns></returns>
		public virtual void TickTest()
		{
		}

		/// <summary>
		/// Sets Cancellation Reason.
		/// </summary>
		/// <param name="InReason"></param>
		/// <returns></returns>
		public virtual void SetCancellationReason(string InReason)
		{
			CancellationReason = InReason;
		}

		/// <summary>
		/// Called after the test is completed and shutdown
		/// </summary>
		/// <param name="InReason"></param>
		/// <returns></returns>
		public virtual void StopTest(StopReason InReason)
		{
			WasCancelled = InReason != StopReason.Completed;
		}

		/// <summary>
		/// Helper to set context
		/// </summary>
		/// <param name="InContext"></param>
		/// <returns></returns>
		public virtual void SetContext(ITestContext InContext)
		{
		}

		/// <summary>
		/// Output all defined commandline information for this test to the gauntlet window and exit test early.
		/// </summary>
		public virtual void DisplayCommandlineHelp()
		{
		}
	}
}
