// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Text;
using UnrealBuildTool;

namespace Gauntlet
{
	/// <summary>
	/// Flags that are used to describe the characteristics or abilitites of a build
	/// </summary>
	[Flags]
	public enum BuildFlags
	{
		None = 0,
		Packaged = (1 << 0),					// build is a package (-package). E.g. APK, IPA, PKG
		Loose = (1 << 1),						// build is a loose collection of files (-staged)
		CanReplaceCommandLine = (1 << 2),		// build can run arbitrary command lines
		CanReplaceExecutable = (1 << 3),		// build can use exes from elsewhere
        Bulk = (1 << 4),						// build is full-content, now startup download
		NotBulk = (1 << 5),						// build is not bulk
		ContentOnlyProject = (1 << 6),          // build is a content-only project, e.g. exe is UnrealGame.exe
	}

	/// <summary>
	/// An interface that represents a build. A build is defined as something that a device is able to install, hence
	/// most if not all implementations will have a strong association with the TargetDevice for that platform
	/// </summary>
    public interface IBuild
	{
		/// <summary>
		/// Preference order for this build type vs. other build types. Lower is more prefered.
		/// </summary>
		int PreferenceOrder { get; }

		/// <summary>
		/// Flags that describe the properties of this build
		/// </summary>
		BuildFlags Flags { get; }

		/// <summary>
		/// Platform that this build is for
		/// </summary>
		UnrealTargetPlatform Platform { get; }

		/// <summary>
		/// Special property of the build (vanilla, asan, uban, etc)
		/// </summary>
		string Flavor { get; }

		/// <summary>
		/// Configuration of this build
		/// </summary>
		UnrealTargetConfiguration Configuration { get; }

		/// <summary>
		/// Does this build support additional loose files be copied to the device
		/// </summary>
		bool SupportsAdditionalFileCopy { get; }

		/// <summary>
		/// Check if this buld is able to support the provided role
		/// </summary>
		/// <param name="Role"></param>
		/// <returns></returns>
		bool CanSupportRole(UnrealTargetRole Role);
	}


	/// <summary>
	/// Represents a class that can discover builds
	/// </summary>
	public interface IBuildSource
	{
		string BuildName { get; }

		bool CanSupportPlatform(UnrealTargetPlatform Platform);
	}

	/// <summary>
	/// Represents a class that can discover builds from a folder path
	/// </summary>
	public interface IFolderBuildSource : IBuildSource
	{
		List<IBuild> GetBuildsAtPath(string InProjectName, string InPath, int MaxRecursion = 3);
	}
}


