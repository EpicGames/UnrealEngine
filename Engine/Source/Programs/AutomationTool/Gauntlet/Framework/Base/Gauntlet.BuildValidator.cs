// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using UnrealBuildTool;

namespace Gauntlet
{
	/// <summary>
	/// Utility for returning the path to a "Latest Good" build
	/// This is used to determine build paths when -build=LKG or -build=LatestGood
	/// </summary>
	public interface IBuildValidator
	{
		string Name { get; }
		bool CanSupportProject(string InProjectName);
		string GetLatestGoodBuild();
	}
}
