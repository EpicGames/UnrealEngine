// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Text;

namespace Gauntlet
{
	/// <summary>
	/// An interface to a set of options that implement the IAppConfig interface
	/// </summary>
	/// <typeparam name="ConfigType"></typeparam>
	public interface IConfigOption<ConfigType>
		where ConfigType : IAppConfig
	{
		void ApplyToConfig(ConfigType AppConfig);
	}
}
