// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Text;

namespace Gauntlet
{
	/// <summary>
	/// POCO class for apps that can run through containers.
	/// </summary>
	public class ContainerInfo
	{
		public string ImageName { get; set; }
		public string ContainerName { get; set; }
		public string RunCommandPrepend { get; set; }
		public string WorkingDir { get; set; }
	}

	public interface IContainerized
	{
		/// <summary>
		/// For apps running through Docker containers.
		/// </summary>
		ContainerInfo ContainerInfo { get; set; }
	}
}
