// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.IO;
using System.Linq;
using System.Data;
using System.Collections.Generic;
using MySql.Data.MySqlClient;
using System.Text.Json;
using System.Text.RegularExpressions;
using AutomationTool;

namespace Gauntlet
{

	/// <summary>
	/// Hold information about the telemetry context
	/// </summary>
	public interface ITelemetryContext
	{
		object GetProperty(string Name);
	}

	/// <summary>
	/// Interface to drive Database submission
	/// </summary>
	public interface IDatabaseDriver<Data> where Data : class
	{
		/// <summary>
		/// Submit collection of object to target Database, use TestContext to complete data modeling.
		/// </summary>
		/// <param name="DataItems"></param>
		/// <param name="Context"></param>
		/// <returns></returns>
		bool SubmitDataItems(IEnumerable<Data> DataItems, ITelemetryContext Context);
		/// <summary>
		/// Execute a Query through the database driver
		/// </summary>
		/// <param name="Query"></param>
		/// <returns></returns>
		DataSet ExecuteQuery(string Query);
		/// <summary>
		/// Fetch collection of object from target Database, uses Conditions to narrow down pool of data.
		/// </summary>
		/// <param name="Conditions"></param>
		/// <returns></returns>
		DataSet FetchData(ITelemetryContext Conditions);
	}

	/// <summary>
	/// Interface for Database Configuration
	/// </summary>
	public interface IDatabaseConfig<Data> where Data : class 
	{
		void LoadConfig(string ConfigFilePath);
		IDatabaseDriver<Data> GetDriver();
	}

	/// <summary>
	/// Interface for data type MySQL Configuration
	/// </summary>
	public interface IMySQLConfig<Data> where Data : class
	{
		/// <summary>
		/// Get Target Table name based on data type
		/// </summary>
		string GetTableName();

		/// <summary>
		/// Get Target Table columns based on data type
		/// </summary>
		IEnumerable<string> GetTableColumns();

		/// <summary>
		/// Format the data for target table based on data type
		/// </summary>
		/// <param name="InData">Data to format</param>
		/// <param name="InContext">ITelemetryContext of the telemetry data</param>
		/// <returns></returns>
		IEnumerable<string> FormatDataForTable(Data InData, ITelemetryContext InContext);
	}

	public class MySQLDriver<Data> : IDatabaseDriver<Data> where Data : class 
	{
		protected MySQLConfig<Data> Config;

		public MySQLDriver(MySQLConfig<Data> InConfig)
		{
			Config = InConfig;
			if (string.IsNullOrEmpty(Config.ConfigString))
			{
				throw new AutomationException(string.Format("Database Driver '{0}' not configured.", this.GetType().FullName));
			}
			if (string.IsNullOrEmpty(Config.DatabaseName))
			{
				throw new AutomationException(string.Format("Database Driver '{0}' not configured properly, missing Database name.", this.GetType().FullName));
			}
		}

		public override string ToString()
		{
			return string.Format("{0} with config '{1}'", Config.GetConfigValue("Server"), Config.GetType().FullName);
		}

		public DataSet ExecuteQuery(string SqlQuery)
		{
			return MySqlHelper.ExecuteDataset(Config.ConfigString, SqlQuery);
		}

		public bool Insert(string Table, IEnumerable<string> Columns, IEnumerable<IEnumerable<string>> Rows)
		{
			foreach (var Chunk in ChunkIt(Rows))
			{
				string SqlQuery = string.Format(
					"INSERT INTO `{0}`.{1} ({2}) VALUES {3};SELECT LAST_INSERT_ID()",
					Config.DatabaseName, Table, string.Join(", ", Columns), string.Join(", ", Chunk.Select(R => string.Format("({0})", string.Join(", ", R.Select(V => string.Format("'{0}'", V))))))
				);

				if (MySqlHelper.ExecuteScalar(Config.ConfigString, SqlQuery) == null)
				{
					return false;
				}
			}
			return true;
		}

		private IEnumerable<IEnumerable<T>> ChunkIt<T>(IEnumerable<T> ToChunk, int ChunkSize = 1000)
		{
			return ToChunk.Select((v, i) => new { Value = v, Index = i }).GroupBy(x => x.Index / ChunkSize).Select(g => g.Select(x => x.Value));
		}

		public bool SubmitDataItems(IEnumerable<Data> DataRows, ITelemetryContext Context)
		{
			if (Config is IMySQLConfig<Data> DataConfig)
			{
				if(!PreSubmitQuery(DataRows, Context))
				{
					Log.Error("Fail MySQL '{0}' pre-submit query.", Config.GetType().FullName);
					return false;
				}

				bool Success = Insert(
					DataConfig.GetTableName(),
					DataConfig.GetTableColumns(),
					DataRows.Select(D => DataConfig.FormatDataForTable(D, Context))
				);

				if(Success && !PostSubmitQuery(DataRows, Context))
				{
					Log.Error("Fail MySQL '{0}' post-submit query.", Config.GetType().FullName);
					return false;
				}

				return Success;
			}
			else
			{
				Log.Error("MySQL configuration '{0}' does not known how to handle {1}.", Config.GetType().FullName, typeof(Data).FullName);
			}

			return false;
		}

		public virtual bool PreSubmitQuery(IEnumerable<Data> DataRows, ITelemetryContext Context)
		{
			return Config.PreSubmitQuery(this, DataRows, Context);
		}
		public virtual bool PostSubmitQuery(IEnumerable<Data> DataRows, ITelemetryContext Context)
		{
			return Config.PostSubmitQuery(this, DataRows, Context);
		}

		public DataSet FetchData(ITelemetryContext Context)
		{
			return MySqlHelper.ExecuteDataset(Config.ConfigString, Config.GetFetchDataQuery(Context));
		}

	}

	public abstract class MySQLConfig<Data> : IDatabaseConfig<Data>, IMySQLConfig<Data> where Data : class
	{
		public string ConfigString { get; protected set; }
		public string DatabaseName { get; protected set; }
		protected Dictionary<string, string> KeyValuePairs = null;

		public virtual void LoadConfig(string ConfigFilePath)
		{
			ConfigString = string.Empty;
			DatabaseName = string.Empty;
			KeyValuePairs = null;

			if (File.Exists(ConfigFilePath))
			{
				using (StreamReader ConnectionReader = new StreamReader(ConfigFilePath))
				{
					ConfigString = ConnectionReader.ReadLine();
				}

				if (string.IsNullOrEmpty(ConfigString))
				{
					Log.Warning("Properly found config file, but couldn't read a valid connection string.");
					return;
				}
				else
				{
					Log.Info("Found MySQL connection string from config file.");
				}
			}
			else
			{
				Log.Error("Could not find connection string config file at '{0}'.", ConfigFilePath);
				return;
			}

			DatabaseName = GetConfigValue("database");
			if (string.IsNullOrEmpty(DatabaseName))
			{
				Log.Warning("Missing MySQL Database name in config file '{0}'.", ConfigFilePath);
				return;
			}
		}

		public string GetConfigValue(string Key)
		{
			if (string.IsNullOrEmpty(ConfigString))
			{
				return null;
			}
			if (KeyValuePairs == null)
			{
				KeyValuePairs = ConfigString.Split(';').Where(KeyValue => KeyValue.Contains('='))
														.Select(KeyValue => KeyValue.Split('=', 2))
														.ToDictionary(
															KeyValue => KeyValue[0].Trim(),
															KeyValue => KeyValue[1].Trim(),
															StringComparer.InvariantCultureIgnoreCase
														);
			}
			string FoundValue;
			if (KeyValuePairs.TryGetValue(Key, out FoundValue))
			{
				return FoundValue;
			}
			return string.Empty;
		}

		public IDatabaseDriver<Data> GetDriver()
		{
			return new MySQLDriver<Data>(this);
		}

		/// <summary>
		/// Override to add pre-submit query.
		/// </summary>
		/// <param name="Driver"></param>
		/// <param name="DataRows"></param>
		/// <param name="Context"></param>
		/// <returns></returns>
		public virtual bool PreSubmitQuery(IDatabaseDriver<Data> Driver, IEnumerable<Data> DataRows, ITelemetryContext Context)
		{
			return true;
		}
		/// <summary>
		/// Override to add post-submit query.
		/// </summary>
		/// <param name="Driver"></param>
		/// <param name="DataRows"></param>
		/// <param name="Context"></param>
		/// <returns></returns>
		public virtual bool PostSubmitQuery(IDatabaseDriver<Data> Driver, IEnumerable<Data> DataRows, ITelemetryContext Context)
		{
			return true;
		}
		/// <summary>
		/// Get Target Table name based on data type
		/// </summary>
		public abstract string GetTableName();
		/// <summary>
		/// Get Target Table columns based on data type
		/// </summary>
		public abstract IEnumerable<string> GetTableColumns();
		/// <summary>
		/// Format the data for target table based on data type
		/// </summary>
		/// <returns></returns>
		public abstract IEnumerable<string> FormatDataForTable(Data InData, ITelemetryContext InContext);
		/// <summary>
		/// Return the query to fetch data
		/// </summary>
		/// <param name="InContext"></param>
		/// <returns></returns>
		public virtual string GetFetchDataQuery(ITelemetryContext InContext)
        {
			return null;
        }
	}

	public class JsonHttpRequestDriver<Data> : IDatabaseDriver<Data> where Data : class
	{
		protected JsonHttpRequestConfig<Data> Config;
		public JsonHttpRequestDriver(JsonHttpRequestConfig<Data> InConfig)
		{
			Config = InConfig;
			if (string.IsNullOrEmpty(Config.Host))
			{
				throw new AutomationException(string.Format("Database Driver '{0}' host not configured.", this.GetType().FullName));
			}
			if (Config.Auth == null)
			{
				throw new AutomationException(string.Format("Database Driver '{0}' authorization header not configured.", this.GetType().FullName));
			}
		}
		public virtual bool SubmitDataItems(IEnumerable<Data> DataItems, ITelemetryContext Context)
		{
			var Rows = Config.FormatData(DataItems, Context);
			foreach (var Chunk in ChunkIt(Rows))
			{
				string JsonString = JsonSerializer.Serialize(Chunk);
				var Result = ExecuteQuery(JsonString);
				if (Result == null)
				{
					return false;
				}
			}

			return true;
		}
		private IEnumerable<IEnumerable<T>> ChunkIt<T>(IEnumerable<T> ToChunk, int ChunkSize = 1000)
		{
			return ToChunk.Select((v, i) => new { Value = v, Index = i }).GroupBy(x => x.Index / ChunkSize).Select(g => g.Select(x => x.Value));
		}
		public virtual DataSet ExecuteQuery(string Query)
		{
			// Send data
			var Auth = HttpRequest.Authentication(Config.Auth.target, Config.Auth.token);
			var Connection = new HttpRequest.Connection(Config.Host, Auth);
			var Response = Connection.PostJson(Config.Path, Query);
			if (!Response.IsSuccessStatusCode)
			{
				Log.Error("Error sending data to host. Status={0}.\n{1}", Response.StatusCode, Response.Content);
				return null;
			}
			// Format response
			var Data = new DataSet();
			DataTable Table = new DataTable();
			Table.Columns.Add("Status", typeof(int));
			Table.Columns.Add("Content", typeof(string));
			Table.Rows.Add(new object[] { Response.StatusCode, Response.Content });
			Table.AcceptChanges();
			Data.Tables.Add(Table);
			return Data;
		}
		public DataSet FetchData(ITelemetryContext Conditions)
		{
			return new DataSet();
		}
		public override string ToString()
		{
			return string.Format("{0}/{1} with config '{2}'", Config.Host, Config.Path ?? "", Config.GetType().FullName);
		}
	}

	public abstract class JsonHttpRequestConfig<Data> : IDatabaseConfig<Data>where Data : class
	{
		public string Host;
		public string Path;
		public JsonHttpRequestConfig.Auth Auth;
		public virtual void LoadConfig(string ConfigFilePath)
		{
			JsonHttpRequestConfig Config = null;
			Host = string.Empty;
			Path = string.Empty;
			Auth = null;

			if (File.Exists(ConfigFilePath))
			{
				try
				{
					Config = JsonHttpRequestConfig.LoadFromFile(ConfigFilePath);
					Log.Info("Found Http Request connection config from file.");
				}
				catch (Exception Ex)
				{
					Log.Warning("Properly found config file, but couldn't read a valid json schema.\n{0}", Ex);
					return;
				}
			}
			else
			{
				Log.Error("Could not find Http Request connection config file at '{0}'.", ConfigFilePath);
				return;
			}

			var Url = new Regex(@"^(https?://[^/]+/?)(.+)?$");
			var Match = Url.Match(Config.endpoint);
			var Success = Match.Success;
			if (!Success)
			{
				Log.Warning("Properly found config file, but couldn't parse http endpoint.");
				return;
			}

			Host = Match.Groups[1].Value;
			Path = Match.Groups[2].Value;
			Auth = Config.authorization;
		}

		/// <summary>
		/// Format the data for target data type
		/// </summary>
		/// <returns></returns>
		public abstract IEnumerable<object> FormatData(IEnumerable<Data> DataItems, ITelemetryContext Context);

		public virtual IDatabaseDriver<Data> GetDriver()
		{
			JsonHttpRequestDriver<Data> Driver = new JsonHttpRequestDriver<Data>(this);
			return Driver;
		}
	}

	public class JsonHttpRequestConfig
	{
		public class Auth
		{
			public string target { get; set; }
			public string token { get; set; }
		}

		public string endpoint { get; set; }
		public Auth authorization { get; set; }

		public static JsonHttpRequestConfig LoadFromFile(string FilePath)
		{
			JsonSerializerOptions Options = new JsonSerializerOptions
			{
				PropertyNameCaseInsensitive = true
			};
			string JsonString = File.ReadAllText(FilePath);
			return JsonSerializer.Deserialize<JsonHttpRequestConfig>(JsonString, Options);
		}
	}

	public class DatabaseConfigManager<Data> where Data : class
	{
		protected static IEnumerable<IDatabaseConfig<Data>> Configs;
		static DatabaseConfigManager()
		{
			Configs = Gauntlet.Utils.InterfaceHelpers.FindImplementations<IDatabaseConfig<Data>>(true);
		}
		public static IDatabaseConfig<Data> GetConfigByName(string Name)
		{
			return Configs.Where(C => string.Equals(C.GetType().Name, Name, StringComparison.OrdinalIgnoreCase)).FirstOrDefault();
		}
	}
}
