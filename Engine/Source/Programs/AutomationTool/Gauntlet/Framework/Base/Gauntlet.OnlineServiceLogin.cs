// Copyright Epic Games, Inc. All Rights Reserved.

namespace Gauntlet
{
	/// <summary>
	/// TargetDevice extension for platforms that contain an online service.
	/// Enables more robust user sign in verification.
	/// </summary>
	public interface IOnlineServiceLogin
	{
		/// <summary>
		/// Verifies the device has connected to its relevant platform network.
		/// </summary>
		/// <returns>True if the device is logged into a platform network account</returns>
		bool VerifyLogin();
	}
}