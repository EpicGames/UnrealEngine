// Copyright Epic Games, Inc. All Rights Reserved.

using AutomationTool;
using EpicGames.Core;
using System;
using System.IO;
using UnrealBuildTool;

namespace Gauntlet
{
	/// <summary>
	/// Manages the overlaying of a development executable over an existing build.
	/// These are useful for testing code changes without needing to re-cook or re-package builds.
	/// To use overlay executables, simply specify -dev on the UAT commandline.
	/// The overlay executable will automatically be applied as long as the following conditions are met
	///		- The build being used supports Overlay executables. See BuildFlags.CanReplaceExecutable
	///		- The development executable's write time is newer than that of the supplied builds write time
	/// </summary>
	public class OverlayExecutable
	{
		private string ProjectName;
		private string Flavor;
		private FileReference ProjectFile;
		private UnrealTargetPlatform Platform;
		private UnrealTargetConfiguration Configuration;
		private UnrealTargetRole Role;

		public OverlayExecutable(UnrealSessionRole SessionRole, string ProjectName, FileReference ProjectFile)
			: this(SessionRole.Platform.Value, SessionRole.Configuration, SessionRole.RoleType, ProjectName, ProjectFile, SessionRole.RequiredFlavor)
		{ }

		public OverlayExecutable(UnrealTargetPlatform Platform, UnrealTargetConfiguration Configuration, UnrealTargetRole RoleType,
			string ProjectName, FileReference ProjectFile, string Flavor = "")
		{
			this.ProjectName = ProjectName;
			this.Configuration = Configuration;
			this.Role = RoleType;
			this.Platform = Platform;
			this.Flavor = Flavor;
			this.ProjectFile = ProjectFile;
		}

		/// <summary>
		/// Attempts to get the path to a local executable if it is newer than the base executable
		/// </summary>
		/// <param name="BaseExecutable">The base executable to use if an overlay does not exist</param>
		/// <param name="OverlayExecutable"></param>
		/// <param name="ExtensionOverride">What extension the overlay file should have</param>
		/// <returns>True if a local, newer executable that matches the role's requirements exists</returns>
		/// <exception cref="AutomationException"></exception>
		public bool GetOverlay(string BaseExecutable, out string OverlayExecutable, string ExtensionOverride = null)
		{
			OverlayExecutable = null;

			// If no base executable is specified we'll skip the "newer" check and just return a local binary that matches the role
			if (string.IsNullOrEmpty(BaseExecutable))
			{
				throw new AutomationException("Overlay Error: No base executable was specified.");
			}

			if (!File.Exists(BaseExecutable) && !Directory.Exists(BaseExecutable)) // mac/ios apps can be directories
			{
				throw new AutomationException("Overlay Error: Could not find base executable {0}.", BaseExecutable);
			}

			// If this is a custom module, search for those executables instead
			string ModuleName = ProjectName;
			string ExecutableModule = Path.GetFileNameWithoutExtension(BaseExecutable);
			if (UnrealHelpers.CustomModules.TryGetValue(ExecutableModule, out _))
			{
				ModuleName = ExecutableModule;
			}

			// Ex. /UnrealGame/Binaries/Win64/UnrealClient.exe
			string PlatformBinariesDirectory = Path.Combine(ProjectFile.Directory.FullName, "Binaries", Platform.ToString());
			string ExecutableExtension = string.IsNullOrEmpty(ExtensionOverride) ? Path.GetExtension(BaseExecutable) : ExtensionOverride;

			string ExecutableName = UnrealHelpers.GetExecutableName(ModuleName, Platform, Configuration, Role, Flavor, string.Empty);
			string PreflightExecutableName = ExecutableName + "-Preflight";

			string LocalExecutable = Path.Combine(PlatformBinariesDirectory, ExecutableName + ExecutableExtension);
			string LocalPreflightExecutable = Path.Combine(PlatformBinariesDirectory, PreflightExecutableName + ExecutableExtension);

			bool bIsNewer = File.GetLastWriteTime(LocalPreflightExecutable) > File.GetLastWriteTime(BaseExecutable);

			// Prioritize using -Preflight executables, if they exist
			if ((File.Exists(LocalPreflightExecutable) || Directory.Exists(LocalPreflightExecutable)) && (bIsNewer || CommandUtils.IsBuildMachine))
			{
				Log.Info("Applying newer preflight executable as overlay {PreflightExecutable}", LocalPreflightExecutable);
				OverlayExecutable = LocalPreflightExecutable;
				return true;
			}

			if (!File.Exists(LocalExecutable) && !Directory.Exists(LocalExecutable))
			{
				if (CommandUtils.IsBuildMachine)
				{
					throw new AutomationException("Overlay Error: No local executable for {0} exists. Skipping overlay for this role", Platform);
				}
				else
				{
					Log.Verbose("No local executable for {Platform} exists. Skipping overlay for this role", Platform);
				}
				return false;
			}

			bIsNewer = File.GetLastWriteTime(LocalExecutable) > File.GetLastWriteTime(BaseExecutable);

			if (!bIsNewer && !CommandUtils.IsBuildMachine)
			{
				Log.Verbose("Local executable for {Platform} is not newer than base executable. Skipping overlay for this role", Platform);
				return false;
			}

			Log.Info("Applying newer local executable as overlay {LocalExecutable}", LocalExecutable);

			OverlayExecutable = LocalExecutable;
			return true;
		}
	}
}