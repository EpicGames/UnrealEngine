// Copyright Epic Games, Inc. All Rights Reserved.

using AutomationTool;
using EpicGames.Core;
using System;
using System.Collections.Generic;
using System.IO;
using System.Linq;
using System.Text.RegularExpressions;
using System.Threading;
using UnrealBuildTool;

namespace Gauntlet
{
	/// <summary>
	/// Base class for a device that is able to run applications
	/// </summary>
	public interface ITargetDevice : IDisposable
	{
		/// <summary>
		/// Name of this device
		/// </summary>
		string Name { get; }

		/// <summary>
		/// Platform of this device
		/// </summary>
		UnrealTargetPlatform? Platform { get; }

		/// <summary>
		/// Options used when running processes
		/// </summary>
		CommandUtils.ERunOptions RunOptions { get; set; }

		/// <summary>
		/// Is the device available for use?
		/// Note: If we are the process holding a connection then this should still return true
		/// </summary>
		bool IsAvailable { get; }

		/// <summary>
		/// Are we connected to this device?
		/// </summary>
		bool IsConnected { get; }

		/// <summary>
		/// Connect and reserve the device
		/// </summary>
		/// <returns></returns>
		bool Connect();

		/// <summary>
		/// Disconnect the device.
		/// </summary>
		/// <param name="bForce">If supported force the device into a disconnected state (e.g. kick other users) </param>
		/// <returns></returns>
		bool Disconnect(bool bForce=false);

		/// <summary>
		/// Is the device powered on?
		/// TODO - Do we need to have a way of expressing whether power changes are supported
		/// </summary>
		bool IsOn { get; }

		/// <summary>
		/// Request the device power on. Should block until the change succeeds (or fails)
		/// </summary>
		/// <returns></returns>
		bool PowerOn();

		/// <summary>
		/// Request the device power on. Should block until the change succeeds (or fails)
		/// </summary>
		/// <returns></returns>
		bool PowerOff();

		/// <summary>
		/// Request the device power on. Should block until the change succeeds (or fails)
		/// </summary>
		/// <returns></returns>
		bool Reboot();

		/// <summary>
		/// Returns a Dictionary of EIntendedBaseCopyDirectory keys and their corresponding file path string values.
		/// If a platform has not set up these mappings, returns an empty Dictionary and warns.
		/// </summary>
		/// <returns></returns>
		Dictionary<EIntendedBaseCopyDirectory, string> GetPlatformDirectoryMappings();

		IAppInstall InstallApplication(UnrealAppConfig AppConfiguration);

		IAppInstance Run(IAppInstall App);

		/// Begin new flow ///

		/// <summary>
		/// Fully cleans the device by deleting
		///	 - Artifacts and other loose files associated with UE processes
		///	 - Staged/Packaged builds
		/// </summary>
		void FullClean();

		/// <summary>
		/// Deletes artifacts and other loose files associated with UE processes
		/// </summary>
		void CleanArtifacts(UnrealAppConfig AppConfiguration = null);

		/// <summary>
		/// Local cache path
		/// </summary>
		string LocalCachePath { get; }

		/// <summary>
		/// Installs a build to the device
		/// </summary>
		/// <param name="AppConfiguration">The configuration containing the build to install</param>
		void InstallBuild(UnrealAppConfig AppConfiguration);

		/// <summary>
		/// Create an IAppInstall that is configured by the provided AppConfiguration
		/// </summary>
		/// <param name="AppConfiguration">The configuration used to create the IAppInstall</param>
		/// <returns>An AppInstall handle which can be used to run the process</returns>
		IAppInstall CreateAppInstall(UnrealAppConfig AppConfiguration);

		/// <summary>
		/// Copies any additional files to the device
		/// </summary>
		/// <param name="FilesToCopy">The collection of files to copy</param>
		void CopyAdditionalFiles(IEnumerable<UnrealFileToCopy> FilesToCopy);

		/// <summary>
		/// Take a screenshot using device soft
		/// Filename is saved in the format of: yy-MM-dd-HH-mm-ss-ff
		/// </summary>
		public void TakeScreenshot(string ScreenshotName) 
		{
			Log.Warning(KnownLogEvents.Gauntlet_TestEvent, "Screenshot capture is not implemented");
		}

		// End new flow //
	};

	/// <summary>
	/// Interface used by TargetDevice* classes to track spawned application running state.
	/// </summary>
	public interface IRunningStateOptions
	{
		/// <summary>
		/// Whether or not to sleep after launching an app before querying for its running state.
		/// </summary>
		bool WaitForRunningState { get; set; }

		/// <summary>
		/// The number of seconds to sleep after launching an app before querying for its running state.
		/// </summary>
		int SecondsToRunningState { get; set; }

		/// <summary>
		/// Interval of time between app running state queries, in seconds.
		/// </summary>
		int CachedStateRefresh { get; set; }
	}

	/// <summary>
	/// Represents a class able to provide devices
	/// </summary>
	public interface IDeviceSource
	{
		bool CanSupportPlatform(UnrealTargetPlatform? Platform);
	}

	/// <summary>
	/// Represents a class that provides locally available devices
	/// </summary>
	public interface IDefaultDeviceSource : IDeviceSource
	{
		ITargetDevice[] GetDefaultDevices();
	}

	/// <summary>
	/// Represents a class capable of creating devices of a specific type
	/// </summary>
	public interface IDeviceFactory : IDeviceSource
	{
		ITargetDevice CreateDevice(string InRef, string InLocalCache, string InParam=null);
	}

	/// <summary>
	/// Represents a class that provides services for available devices
	/// </summary>
	public interface IDeviceService : IDeviceSource
	{
		void CleanupDevices();
	}

	/// <summary>
	/// Represents a class that can provides virtual local devices 
	/// </summary>
	public interface IVirtualLocalDevice : IDeviceSource
	{
		bool CanRunVirtualFromPlatform(UnrealTargetPlatform? Platfrom);
		UnrealTargetPlatform? GetPlatform();
	}

	/// <summary>
	/// Represents a class that tell what build the device support
	/// </summary>
	public interface IDeviceBuildSupport : IDeviceSource
	{
		bool CanSupportBuildType(BuildFlags Flag);
		UnrealTargetPlatform? GetPlatform();
		bool NeedBuildDeployed();
	}
	public abstract class BaseBuildSupport : IDeviceBuildSupport
	{
		protected virtual BuildFlags SupportedBuildTypes => BuildFlags.None;
		protected virtual UnrealTargetPlatform? Platform => null;

		public bool CanSupportBuildType(BuildFlags Flag)
		{
			return (SupportedBuildTypes & Flag) == Flag;
		}

		public bool CanSupportPlatform(UnrealTargetPlatform? InPlatform)
		{
			return Platform == InPlatform;
		}

		public UnrealTargetPlatform? GetPlatform()
		{
			return Platform;
		}

		public virtual bool NeedBuildDeployed() => true;
	}

	/// <summary>
	/// If platform supports host mounting of files or other platform specific
	/// functionality, this interface provides a way to provide that information
	/// without needing an instance of the device.
	/// </summary>
	public interface IPlatformTargetSupport
	{
		/// <summary>
		/// If host mounting is supported, return the equivalent
		/// host mounted path for given platform.
		/// </summary>
		/// <param name="Path">Input path</param>
		/// <returns></returns>
		string GetHostMountedPath(string Path);

		/// <summary>
		/// Returns true if host mounting is supported on given
		/// platform. 
		/// </summary>
		/// <returns></returns>
		bool IsHostMountingSupported();

		/// <summary>
		/// The target platform
		/// </summary>
		UnrealTargetPlatform? Platform { get; }
	}

	/// <summary>
	/// Static helper class for querying platform specific target support
	/// </summary>
	public static class PlatformTargetSupport
	{
		public static IPlatformTargetSupport Get(UnrealTargetPlatform Platform)
		{
			if (PlatformSupportInstances == null || PlatformSupportInstances.Count == 0)
			{
				PlatformSupportInstances = Utils.InterfaceHelpers.FindImplementations<IPlatformTargetSupport>().ToList();
			}

			return PlatformSupportInstances
				.Where(x => x.Platform == Platform)
				.FirstOrDefault();
		}

		public static string GetHostMountedPath(UnrealTargetPlatform Platform, string Path) => Get(Platform).GetHostMountedPath(Path);
		public static bool IsHostMountingSupported(UnrealTargetPlatform Platform) => Get(Platform).IsHostMountingSupported();

		private static List<IPlatformTargetSupport> PlatformSupportInstances;
	}

	/// <summary>
	/// Default abstract implementation of IPlatformTargetSupport
	/// so that target platforms can override only parts they need
	/// </summary>
	public abstract class TargetPlatformSupportBase : IPlatformTargetSupport
	{
		public virtual string GetHostMountedPath(string Path) => Path;
		public virtual bool IsHostMountingSupported() => false;
		public virtual UnrealTargetPlatform? Platform => throw new NotImplementedException();
	}
}