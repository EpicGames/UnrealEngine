// Copyright Epic Games, Inc. All Rights Reserved.

using AutomationTool;
using EpicGames.Core;
using System;
using System.Collections.Generic;
using System.IO;
using System.Threading;
using System.Linq;
using System.Diagnostics;
using UnrealBuildTool;
using System.Text.RegularExpressions;
using Gauntlet.Utils;
using UnrealBuildBase;

namespace Gauntlet
{
	public abstract class TargetDeviceDesktopCommon : ITargetDevice
	{
		#region ITargetDevice
		public string Name { get; protected set; }
		public UnrealTargetPlatform? Platform { get; protected set; }
		public CommandUtils.ERunOptions RunOptions { get; set; }
		public bool IsAvailable => true;
		public bool IsConnected => true;
		public bool IsOn => true;
		public bool Connect() => true;
		public bool Disconnect(bool bForce = false) => true;
		public bool PowerOn() => true;
		public bool PowerOff() => true;
		public bool Reboot() => true;
		public Dictionary<EIntendedBaseCopyDirectory, string> GetPlatformDirectoryMappings() => LocalDirectoryMappings;
		public override string ToString() => Name;
		#endregion

		#region IDisposable Support
		private bool DisposedValue = false; // To detect redundant calls

		protected virtual void Dispose(bool Disposing)
		{
			if (!DisposedValue)
			{
				if (Disposing)
				{
					// TODO: dispose managed state (managed objects).
				}

				// TODO: free unmanaged resources (unmanaged objects) and override a finalizer below.
				// TODO: set large fields to null.

				DisposedValue = true;
			}
		}

		// This code added to correctly implement the disposable pattern.
		public void Dispose()
		{
			// Do not change this code. Put cleanup code in Dispose(bool disposing) above.
			Dispose(true);
			// TODO: uncomment the following line if the finalizer is overridden above.
			// GC.SuppressFinalize(this);
		}
		#endregion

		public string LocalCachePath { get; protected set; }

		public string UserDir { get; protected set; }

		/// <summary>
		/// Optional directory staged builds are installed to if the requested build is not already on the same volume
		/// </summary>
		[AutoParam]
		public string InstallRoot { get; protected set; }

		/// <summary>
		/// Cached reference to the install on this device. Used to maintain a soft handle to desktop artifact directories for cleanup
		/// </summary>
		protected IAppInstall InstallCache;

		protected Dictionary<EIntendedBaseCopyDirectory, string> LocalDirectoryMappings { get; set; }

		public TargetDeviceDesktopCommon(string InName, string InCacheDir)
		{
			AutoParam.ApplyParamsAndDefaults(this, Globals.Params.AllArguments);

			Name = InName;
			LocalCachePath = InCacheDir;
			UserDir = Path.Combine(InCacheDir, "UserDir");

			if (string.IsNullOrEmpty(InstallRoot))
			{
				InstallRoot = InCacheDir;
			}

			LocalDirectoryMappings = new Dictionary<EIntendedBaseCopyDirectory, string>();
		}

		public void FullClean()
		{
			CleanArtifacts();
		}

		public void CleanArtifacts(UnrealAppConfig AppConfig = null)
		{
			if (Directory.Exists(UserDir))
			{
				// make sure the user dir is clean and not locked by anything
				try
				{
					CleanArtifactDirectory(Path.Combine(UserDir, "Saved"));
				}
				catch (IOException)
				{
					// otherwise iterate to get a non existing folder
					int Incr = 0;
					while (Directory.Exists(UserDir))
					{
						UserDir = Path.Combine(LocalCachePath, $"UserDir_{++Incr}");
					}
				}
			}

			if (AppConfig == null)
			{
				Log.Info("{Platform} expects an non-null AppConfig value to reliably determine which artifact path to clear! Skipping clean of build's saved folder", Platform);
				return;
			}

			if (AppConfig.Build is not StagedBuild Build)
			{
				return;
			}

			string BuildDirectory = GetStagedBuildDirectory(AppConfig, Build);
			string ArtifactDirectory = Path.Combine(BuildDirectory, AppConfig.ProjectName, "Saved");
			CleanArtifactDirectory(ArtifactDirectory);
		}

		public virtual void InstallBuild(UnrealAppConfig AppConfig)
		{
			IBuild Build = AppConfig.Build;
			switch (Build)
			{
				case NativeStagedBuild:
				case EditorBuild:
				{
					Log.Info("Skipping installation of {BuildType}", Build.GetType().Name);
					break;
				}

				case StagedBuild:
				{
					StagedBuild Staged = Build as StagedBuild;

					if (SystemHelpers.IsNetworkPath(Staged.BuildPath))
					{
						string SubDir = string.IsNullOrEmpty(AppConfig.Sandbox) ? AppConfig.ProjectName : AppConfig.Sandbox;
						string InstallDir = Path.Combine(InstallRoot, SubDir, AppConfig.ProcessType.ToString());

						InstallDir = StagedBuild.InstallBuildParallel(AppConfig, Staged, Staged.BuildPath, InstallDir, ToString());
						SystemHelpers.MarkDirectoryForCleanup(InstallDir);
					}
					else
					{
						Log.Info("Build exists on local drive, skipping installation.");
					}
					break;
				}

				default:
					throw new AutomationException("{0} is not a valid build type for {1}", Build.GetType().Name, Platform);
			}
		}

		public virtual IAppInstall CreateAppInstall(UnrealAppConfig AppConfig)
		{
			IAppInstall Install;
			IBuild Build = AppConfig.Build;

			switch (AppConfig.Build)
			{
				case NativeStagedBuild:
					Install = CreateNativeStagedInstall(AppConfig, Build as NativeStagedBuild);
					break;
				case StagedBuild:
					Install = CreateStagedInstall(AppConfig, Build as StagedBuild);
					break;
				case EditorBuild:
					Install = CreateEditorInstall(AppConfig, Build as EditorBuild);
					break;
				default:
					throw new AutomationException("{0} is an invalid build type for {1}!", Build.GetType().Name, Platform);
			}

			return Install;
		}

		public void CopyAdditionalFiles(IEnumerable<UnrealFileToCopy> FilesToCopy)
		{
			if (FilesToCopy == null || !FilesToCopy.Any())
			{
				return;
			}

			if (!LocalDirectoryMappings.Any())
			{
				throw new AutomationException("Attempted to copy additional files before LocalDirectoryMappings were populated." +
					"{0} must call PopulateDirectoryMappings before attempting to call CopyAdditionalFiles", this.GetType());
			}

			foreach (UnrealFileToCopy FileToCopy in FilesToCopy)
			{
				string PathToCopyTo = Path.Combine(LocalDirectoryMappings[FileToCopy.TargetBaseDirectory], FileToCopy.TargetRelativeLocation);
				if (File.Exists(FileToCopy.SourceFileLocation))
				{
					FileInfo SrcInfo = new FileInfo(FileToCopy.SourceFileLocation);
					SrcInfo.IsReadOnly = false;
					string DirectoryToCopyTo = Path.GetDirectoryName(PathToCopyTo);
					if (!Directory.Exists(DirectoryToCopyTo))
					{
						Directory.CreateDirectory(DirectoryToCopyTo);
					}
					if (File.Exists(PathToCopyTo))
					{
						FileInfo ExistingFile = new FileInfo(PathToCopyTo);
						ExistingFile.IsReadOnly = false;
					}

					Log.Info("Copying {SourceFile} to {DestinationFile}", FileToCopy.SourceFileLocation, PathToCopyTo);
					SrcInfo.CopyTo(PathToCopyTo, true);
				}
				else
				{
					Log.Warning(KnownLogEvents.Gauntlet_DeviceEvent, "File to copy {File} not found", FileToCopy);
				}
			}
		}

		public virtual void PopulateDirectoryMappings(string BaseDirectory)
		{
			LocalDirectoryMappings.Clear();

			LocalDirectoryMappings.Add(EIntendedBaseCopyDirectory.Build, Path.Combine(BaseDirectory, "Build"));
			LocalDirectoryMappings.Add(EIntendedBaseCopyDirectory.Binaries, Path.Combine(BaseDirectory, "Binaries"));
			LocalDirectoryMappings.Add(EIntendedBaseCopyDirectory.Config, Path.Combine(BaseDirectory, "Saved", "Config"));
			LocalDirectoryMappings.Add(EIntendedBaseCopyDirectory.Content, Path.Combine(BaseDirectory, "Content"));
			LocalDirectoryMappings.Add(EIntendedBaseCopyDirectory.PersistentDownloadDir, Path.Combine(BaseDirectory, "Saved", "PersistentDownloadDir"));
			LocalDirectoryMappings.Add(EIntendedBaseCopyDirectory.Profiling, Path.Combine(BaseDirectory, "Saved", "Profiling"));
			LocalDirectoryMappings.Add(EIntendedBaseCopyDirectory.Saved, Path.Combine(BaseDirectory, "Saved"));
			// Folders that are located in User dir instead of build dir
			LocalDirectoryMappings.Add(EIntendedBaseCopyDirectory.Demos, Path.Combine(UserDir, "Saved", "Demos"));
		}

		public abstract IAppInstance Run(IAppInstall Install);

		protected abstract IAppInstall CreateNativeStagedInstall(UnrealAppConfig AppConfig, NativeStagedBuild Build);

		protected abstract IAppInstall CreateStagedInstall(UnrealAppConfig AppConfig, StagedBuild Build);

		protected abstract IAppInstall CreateEditorInstall(UnrealAppConfig AppConfig, EditorBuild Build);

		protected string GetStagedBuildDirectory(UnrealAppConfig AppConfig, StagedBuild Build)
		{
			string BuildDir = Build.BuildPath;
			if (SystemHelpers.IsNetworkPath(BuildDir))
			{
				string SubDir = string.IsNullOrEmpty(AppConfig.Sandbox) ? AppConfig.ProjectName : AppConfig.Sandbox;
				string InstallDir = Path.Combine(InstallRoot, SubDir, AppConfig.ProcessType.ToString());
				BuildDir = InstallDir;
			}

			return BuildDir;
		}

		private void CleanArtifactDirectory(string ArtifactDirectory)
		{
			if (!string.IsNullOrEmpty(ArtifactDirectory) && Directory.Exists(ArtifactDirectory))
			{
				Log.Info("Cleaning device artifacts path {ArtifactDirectory}", ArtifactDirectory);
				DirectoryInfo Info = new(ArtifactDirectory);
				SystemHelpers.Delete(Info, true, true);
			}
		}

		#region Legacy Implementations
		public virtual IAppInstall InstallApplication(UnrealAppConfig AppConfig)
		{
			switch (AppConfig.Build)
			{
				case NativeStagedBuild:
					return InstallNativeStagedBuild(AppConfig, AppConfig.Build as NativeStagedBuild);

				case StagedBuild:
					return InstallStagedBuild(AppConfig, AppConfig.Build as StagedBuild);

				case EditorBuild:
					return InstallEditorBuild(AppConfig, AppConfig.Build as EditorBuild);

				default:
					throw new AutomationException("{0} is an invalid build type!", AppConfig.Build.ToString());
			}
		}

		protected abstract IAppInstall InstallNativeStagedBuild(UnrealAppConfig AppConfig, NativeStagedBuild Build);
		protected abstract IAppInstall InstallStagedBuild(UnrealAppConfig AppConfig, StagedBuild Build);
		protected abstract IAppInstall InstallEditorBuild(UnrealAppConfig AppConfig, EditorBuild Build);
		#endregion
	}

	public abstract class DesktopCommonAppInstall<DesktopTargetDevice> : IAppInstall where DesktopTargetDevice : TargetDeviceDesktopCommon
	{
		public string Name { get; }

		public ITargetDevice Device => DesktopDevice;

		public DesktopTargetDevice DesktopDevice { get; protected set; }

		public string ArtifactPath;

		public string ExecutablePath;

		public string ProjectName;

		public string WorkingDirectory;

		public string LogFile;

		public bool CanAlterCommandArgs;

		public string CommandArguments
		{
			get { return CommandArgumentsPrivate; }
			set
			{
				if (CanAlterCommandArgs || string.IsNullOrEmpty(CommandArgumentsPrivate))
				{
					CommandArgumentsPrivate = value;
				}
				else
				{
					Log.Info("Skipped setting command AppInstall line when CanAlterCommandArgs = false");
				}
			}
		}

		private string CommandArgumentsPrivate;

		public void AppendCommandline(string AdditionalCommandline)
		{
			CommandArguments += AdditionalCommandline;
		}

		public LongProcessResult.OutputFilterCallbackType FilterLoggingDelegate { get; set; }

		public CommandUtils.ERunOptions RunOptions { get; set; }

		public DesktopCommonAppInstall(string InName, string InProjectName, DesktopTargetDevice InDevice)
		{
			Name = InName;
			ProjectName = InProjectName;
			DesktopDevice = InDevice;
			CommandArguments = string.Empty;
			RunOptions = CommandUtils.ERunOptions.NoWaitForExit;
			CanAlterCommandArgs = true;
		}

		public virtual IAppInstance Run()
		{
			return Device.Run(this);
		}

		/// <summary>
		/// Obsolete! Will be removed in a future release.
		/// Use ITargetDevice.CleanArtifacts instead
		/// </summary>
		public virtual void CleanDeviceArtifacts()
		{
			Device.CleanArtifacts();
		}

		/// <summary>
		/// Obsolete! Will be removed in a future release.
		/// Use ITargetDevice.CleanArtifacts instead
		/// </summary>
		public virtual bool ForceCleanDeviceArtifacts()
		{
			try
			{
				Device.CleanArtifacts();
				return true;
			}
			catch
			{
				return false;
			}
		}

		public virtual void SetDefaultCommandLineArguments(UnrealAppConfig AppConfig, CommandUtils.ERunOptions InRunOptions, string BuildDir)
		{
			RunOptions = InRunOptions;
			if (Log.IsVeryVerbose)
			{
				RunOptions |= CommandUtils.ERunOptions.AllowSpew;
			}

			string UserDir = DesktopDevice.UserDir;
			// Set commandline replace any InstallPath arguments with the path we use
			CommandArguments = Regex.Replace(AppConfig.CommandLine, @"\$\(InstallPath\)", BuildDir, RegexOptions.IgnoreCase);
			CanAlterCommandArgs = AppConfig.CanAlterCommandArgs;
			FilterLoggingDelegate = AppConfig.FilterLoggingDelegate;

			if (CanAlterCommandArgs)
			{
				// Unreal userdir
				if (string.IsNullOrEmpty(UserDir) == false)
				{
					CommandArguments += $" -userdir=\"{UserDir}\"";
					ArtifactPath = Path.Combine(UserDir, "Saved");

					Utils.SystemHelpers.MarkDirectoryForCleanup(UserDir);
				}
				else
				{
					// e.g d:\Unreal\GameName\Saved
					ArtifactPath = Path.Combine(BuildDir, ProjectName, "Saved");
				}

				// Unreal abslog
				// Look in app parameters if abslog is specified, if so use it
				Regex LogRegex = new Regex(@"-abslog[:\s=](?:""([^""]*)""|([^-][^\s]*))?");
				Match M = LogRegex.Match(CommandArguments);
				if (M.Success)
				{
					LogFile = !string.IsNullOrEmpty(M.Groups[1].Value) ? M.Groups[1].Value : M.Groups[2].Value;
				}

				// Explicitly set log file either when stdout processing is disabled, or when not already defined if not build machine
				// -abslog makes sure Unreal dynamically update the log window when using -log
				if (AppConfig.DisableStdoutProcessing ||
					!CommandUtils.IsBuildMachine && (!string.IsNullOrEmpty(LogFile) || AppConfig.CommandLine.Contains("-log")))
				{
					string LogFolder = string.IsNullOrEmpty(LogFile) ?
						(string.IsNullOrEmpty(UserDir) ? Path.Combine(Device.LocalCachePath, "Logs") : Path.Combine(UserDir, "Saved", "Logs")) :
						Path.GetDirectoryName(LogFile);

					if (!Directory.Exists(LogFolder))
					{
						Directory.CreateDirectory(LogFolder);
					}

					if (string.IsNullOrEmpty(LogFile))
					{
						LogFile = Path.Combine(LogFolder, $"{ProjectName}_{AppConfig.ProcessType.ToString()}.log");
						CommandArguments += string.Format(" -abslog=\"{0}\"", LogFile);
					}

					RunOptions |= CommandUtils.ERunOptions.NoStdOutRedirect;
				}
			}

			if(Globals.IsRunningDev && AppConfig.OverlayExecutable.GetOverlay(ExecutablePath, out string OverlayExecutable))
			{
				CommandArguments += string.Format(" -basedir=\"{0}\"", Path.GetDirectoryName(ExecutablePath));
				ExecutablePath = OverlayExecutable;
			}
		}
	}

	public abstract class DesktopCommonAppInstance<DesktopAppInstall, DesktopTargetDevice> : LocalAppProcess
		where DesktopAppInstall : DesktopCommonAppInstall<DesktopTargetDevice>
		where DesktopTargetDevice : TargetDeviceDesktopCommon
	{
		public override string ArtifactPath => Install.ArtifactPath;

		public override ITargetDevice Device => Install.Device;

		protected DesktopAppInstall Install;

		public DesktopCommonAppInstance(DesktopAppInstall InInstall, ILongProcessResult InProcess, string InProcessLogFile = null)
			: base(InProcess, InInstall.CommandArguments, InProcessLogFile)
		{
			Install = InInstall;
		}
	}

	public abstract class LocalAppProcess : IAppInstance
	{
		public ILongProcessResult ProcessResult { get; private set; }

		public bool HasExited { get { return ProcessResult.HasExited; } }

		public bool WasKilled { get; protected set; }

		public string StdOut
		{
			get
			{
				if (string.IsNullOrEmpty(ProcessLogFile))
				{
					return ProcessResult.Output;
				}

				if (File.Exists(ProcessLogFile))
				{
					using (LogFileReader Stream = new LogFileReader(ProcessLogFile))
					{
						return Stream.GetContent();
					}
				}

				return ProcessLogOutput.GetContent();
			}
		}

		public ILogStreamReader GetLogReader()
		{
			if (string.IsNullOrEmpty(ProcessLogFile))
			{
				return ProcessResult.GetLogReader();
			}

			if (File.Exists(ProcessLogFile))
			{
				return new LogFileReader(ProcessLogFile);
			}

			Log.Warning(KnownLogEvents.Horde_BuildHealth_Ignore, "Log file '{filepath}' is missing. The buffer will be used instead but will most likely lack the beginning of the file.", ProcessLogFile);

			return ProcessLogOutput.GetReader();
		}

		public ILogStreamReader GetLogBufferReader()
		{
			if (string.IsNullOrEmpty(ProcessLogFile))
			{
				return ProcessResult.GetLogBufferReader();
			}

			return ProcessLogOutput.GetReader();
		}

		public bool WriteOutputToFile(string FilePath)
		{
			if (string.IsNullOrEmpty(ProcessLogFile))
			{
				return ProcessResult.WriteOutputToFile(FilePath) != null;
			}

			if (File.Exists(ProcessLogFile))
			{
				ProcessUtils.CheckProcessLogReachedSizeLimit(new FileReference(ProcessLogFile));
				File.Copy(ProcessLogFile, FilePath, true);
				if (HasExited)
				{
					// Follow the new location
					// SaveRoleArtifacts may clean device cache folder prior parsing the log file
					ProcessLogFile = FilePath;
				}
			}
			else
			{
				Log.Warning(KnownLogEvents.Horde_BuildHealth_Ignore, "Log file '{filepath}' is missing at the time of making a copy. The buffer will be used instead but will most likely lack the beginning of the file.", ProcessLogFile);
				StreamWriter Writer = ProcessUtils.CreateWriterForProcessLog(FilePath, CommandLine);
				foreach(string Line in ProcessLogOutput)
				{
					Writer.WriteLine(Line);
				}
				Writer.Close();
			}

			return true;
		}

		public int ExitCode { get { return ProcessResult.ExitCode; } }

		public string CommandLine { get; private set; }

		public LocalAppProcess(ILongProcessResult InProcess, string InCommandLine, string InProcessLogFile = null)
		{
			this.CommandLine = InCommandLine;
			this.ProcessResult = InProcess;
			this.ProcessLogFile = InProcessLogFile;

			// start reader thread if logging to a file
			if (!string.IsNullOrEmpty(InProcessLogFile))
			{
				ProcessLogOutput = ProcessUtils.CreateLogBuffer();
				new Thread(LogFileReaderThread).Start();
			}
		}

		public int WaitForExit()
		{
			if (!HasExited)
			{
				ProcessResult.WaitForExit();
			}

			return ExitCode;
		}

		virtual public void Kill(bool bGenerateDump)
		{
			if (!HasExited)
			{
				if (bGenerateDump)
				{
					GenerateDump();
				}
				WasKilled = true;
				ProcessResult.ProcessObject.Kill(true);
			}
		}

		virtual protected void GenerateDump() { }

		/// <summary>
		/// Reader thread when logging to file
		/// </summary>
		void LogFileReaderThread()
		{
			// Wait for the processes log file to be created
			while (!File.Exists(ProcessLogFile) && !HasExited)
			{
				Thread.Sleep(2000);
			}

			// Check whether the process exited before log file was created (this can happen for example if a server role exits and forces client to shutdown)
			if (!File.Exists(ProcessLogFile))
			{
				ProcessLogOutput.AppendLine("Process exited before log file created");
				return;
			}

			Thread.Sleep(1000);

			using (FileStream ProcessLog = File.Open(ProcessLogFile, FileMode.Open, FileAccess.Read, FileShare.ReadWrite))
			{
				StreamReader LogReader = new StreamReader(ProcessLog);
				// Read until the process has exited
				do
				{
					Thread.Sleep(250);

					while (!LogReader.EndOfStream)
					{
						string Output = LogReader.ReadToEnd();

						if (!string.IsNullOrEmpty(Output))
						{
							foreach(string Line in Output.Split('\n'))
							{
								ProcessLogOutput.AppendLine(Line.TrimEnd('\r'));
							}
						}
					}
				}
				while (!HasExited);

				LogReader.Close();
				ProcessLog.Close();
				ProcessLog.Dispose();
			}
		}


		public abstract string ArtifactPath { get; }

		public abstract ITargetDevice Device { get; }

		string ProcessLogFile;
		CircularLogBuffer ProcessLogOutput = null;
	}
}