// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.IO;
using System.Text.RegularExpressions;
using System.Linq;
using AutomationTool;
using UnrealBuildTool;


namespace Gauntlet
{
	/// <summary>
	/// A context that defines that environment that tests will be executed under. The base class has few properties but it is expected
	/// that concrete implementations use their context to store additional global and test-agnostic settings.
	/// </summary>
	public interface ITestContext
	{

	}
}