// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.IO;
using System.Text.RegularExpressions;
using System.Linq;
using AutomationTool;
using UnrealBuildTool;


namespace Gauntlet
{

	/// <summary>
	/// Describe the end result of a test. Until a test is complete
	/// TestResult.Invalid should be returned
	/// </summary>
	public enum TestResult
	{
		Invalid,
		Passed,
		Failed,
		WantRetry,
		Cancelled,
		TimedOut,
		InsufficientDevices,
		OperationalException,	// It's not a test failure, but also not specifically a Cancel, or Timeout, or lack of available devices
	};

	/// <summary>
	/// Describes the current state of a test. 
	/// </summary>
	public enum TestStatus
	{
		NotStarted,
		Pending,			// Not currently used
		InProgress,
		Complete,
	};

	/// <summary>
	/// Describes why a test is stopping
	/// </summary>
	public enum StopReason
	{
		Completed,				// Test reported itself as complete so we're asking it to stop
		MaxDuration,			// Test reached a maximum running time
	};

	/// <summary>
	/// Describes the priority of test. 
	/// </summary>
	public enum TestPriority
	{
		Critical,
		High,
		Normal,
		Low,
		Idle
	};

	/// <summary>
	/// Describes the severity level of an event
	/// </summary>
	public enum EventSeverity
	{
		Info,
		Warning,
		Error,
		Fatal
	}

	/// <summary>
	/// Interface for test event that provides must-have information
	/// </summary>
	public interface ITestEvent
	{
		/// <summary>
		/// Level of severity that describes this event
		/// </summary>
		EventSeverity Severity { get; }

		/// <summary>
		/// Time at which the event was fired
		/// </summary>
		DateTime Time { get; }

		/// <summary>
		/// High level single-line summary of what occurred. Should never be null
		/// </summary>
		string Summary { get; }

		/// <summary>
		/// Details for this event. E.g this could be a list of error messages or warnings for a failed test.
		/// May be empty, should never be null
		/// </summary>
		IEnumerable<string> Details { get; }

		/// <summary>
		/// Callstack for this event if that information is available. May be empty, should never be null
		/// </summary>
		IEnumerable<string> Callstack { get; }
	};

	/// <summary>
	/// The interface that all Gauntlet rests are required to implement. How these are
	/// implemented and whether responsibilities are handed off to other systems (e.g. Launch)
	/// is left to the implementor. It's expected that tests for major systems (e.g. Unreal)
	/// will likely implement their own base node.
	/// </summary>
	public interface ITestNode
	{
		/// <summary>
		/// Name of the test - used for logging / reporting
		/// </summary>
		string Name { get; }

		/// <summary>
		/// Maximum duration that the test is expected to run for. Tests longer than this will be halted.
		/// </summary>
		float MaxDuration { get; }

		/// <summary>
		/// Return true if the warnings and errors needs to log after summary
		/// </summary>
		bool LogWarningsAndErrorsAfterSummary { get; }

		/// <summary>
		/// What the test result should be treated as if we reach max duration.
		/// </summary>
		EMaxDurationReachedResult MaxDurationReachedResult { get; }

		/// <summary>
		/// Priority of this test in relation to any others that are running
		/// </summary>
		TestPriority Priority { get;  }

		/// <summary>
		/// Sets the context that this test will run under. TODO - make this more of a contract that happens before CheckRequirements / LaunchTest
		/// </summary>
		/// <param name="InContext"></param>
		/// <returns></returns>
		void SetContext(ITestContext InContext);
		
		/// <summary>
		/// Checks if the test is ready to be started. If the test is not ready to run (e.g. resources not available) then it should return false.
		/// If it determines that it will never be ready it should throw an exception.
		/// </summary>
		/// <returns></returns>
		bool IsReadyToStart();

		/// <summary>
		/// Begin executing the provided test. At this point .Status should return InProgress and the test will eventually receive
		/// OnComplete and ShutdownTest calls
		/// </summary>
		/// <param name="Pass"></param>
		/// <param name="NumPasses"></param>
		/// <returns>true/false based on whether the test successfully launched</returns>
		bool StartTest(int Pass, int NumPasses);

		/// <summary>
		/// Called regularly to allow the test to check and report status
		/// </summary>
		void TickTest();

		/// <summary>
		/// Sets Cancellation Reason.
		/// </summary>
		/// <param name="Reason"></param>
		/// <returns></returns>
		void SetCancellationReason(string Reason);

		/// <summary>
		/// Called to request that the test stop, either because it's reported that its complete or due to external factors.
		/// Tests should consider whether they passed or succeeded (even a terminated test may have gotten all the data it needs) 
		/// and set their result appropriately.
		/// </summary>
		void StopTest(StopReason InReason);

		/// <summary>
		/// Allows the node to restart with the same assigned devices. Only called if the expresses 
		/// a .Result of TestResult.WantRetry while running.
		/// </summary>
		/// <returns></returns>
		bool RestartTest();

		/// <summary>
		/// Return an enum, that describes the state of the test
		/// </summary>
		TestStatus GetTestStatus();

		/// <summary>
		/// The result of the test. Only called once GetTestStatus() returns complete, but may be called multiple
		/// times.
		/// </summary>
		TestResult GetTestResult();
		
		/// <summary>
		/// Set the Test Result value
		/// </summary>
		void SetTestResult(TestResult testResult);

		/// <summary>
		/// Add a new Test Event to be rolled up and summarized at the end of this test.
		/// </summary>
		void AddTestEvent(UnrealTestEvent InEvent);

		/// <summary>
		/// Summary of the test. Only called once GetTestStatus() returns complete, but may be called multiple
		/// times.
		/// </summary>
		string GetTestSummary();

		/// <summary>
		/// Return a list of all warnings that should be reported. May be different than Summary contents
		/// </summary>
		IEnumerable<string> GetWarnings();

		/// <summary>
		/// Return a list of all errors that should be reported. May be different than Summary contents
		/// </summary>
		IEnumerable<string> GetErrors();

		/// <summary>
		/// returns a string that will be how to run this test locally with RunUAT.bat
		/// </summary>
		string GetRunLocalCommand(string LaunchingBuildCommand);

		/// <summary>
		/// Called to request any that any necessary cleanup be performed. After CleanupTest is called no further calls will be
		/// made to this test and thus all resources should be released.
		/// </summary>
		/// <returns></returns>
		void CleanupTest();

		/// <summary>
		/// Output all defined commandline information for this test to the gauntlet window and exit test early.
		/// </summary>
		void DisplayCommandlineHelp();
	}
}