// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Reflection;
using System.Collections.Generic;
using System.Text.Json;
using System.Text.Json.Serialization;

namespace Gauntlet
{
	/// <summary>
	/// Event Type
	/// </summary>
	/// 
	[JsonConverter(typeof(JsonTryParseEnumConverter))]
	public enum EventType
	{
		Unknown,
		Info,
		Warning,
		Error
	}

	/// <summary>
	/// Test State type
	/// </summary>
	/// 
	[JsonConverter(typeof(JsonTryParseEnumConverter))]
	public enum TestStateType
	{
		Unknown,

        /// <summary>
        /// The test case has not (yet) been executed
        /// </summary>
		NotRun,

        /// <summary>
        /// The test case is currently in progress
        /// </summary>
		InProcess,

        /// <summary>
        /// The behavior that this test case was trying to exercise did not work as expected
        /// </summary>
		Fail,

        /// <summary>
        /// The behavior that this test case was trying to exercise worked as expected
        /// </summary>
		Success,

        /// <summary>
        /// The test case was not executed, and will not be executed in this test run
        /// </summary>
		Skipped,

        /// <summary>
        /// The test case was stopped for some reason other than failure of the behavior being tested
        /// </summary>
        Interrupted,
	}

	/// <summary>
	/// Interface for test report
	/// </summary>
	public interface ITestReport
	{
		/// <summary>
		/// Return report type
		/// </summary>
		string Type { get; }

		/// <summary>
		/// Set a property of the test report
		/// </summary>
		/// <param name="Property"></param>
		/// <param name="Value"></param>
		/// <returns></returns>
		void SetProperty(string Property, object Value);

		/// <summary>
		/// Set a metadata key/value to the test report
		/// </summary>
		/// <param name="Key"></param>
		/// <param name="Value"></param>
		/// <returns></returns>
		void SetMetadata(string Key, string Value);

		/// <summary>
		/// Add event to the test report
		/// </summary>
		/// <param name="Type"></param>
		/// <param name="Message"></param>
		/// <param name="Args"></param>
		/// <returns></returns>
		void AddEvent(EventType Type, string Message, params object[] Args);

		/// <summary>
		/// Attach an Artifact to the ITestReport
		/// </summary>
		/// <param name="ArtifactPath"></param>
		/// <param name="Name"></param>
		/// <returns></returns>
		bool AttachArtifact(string ArtifactPath, string Name = null);

		/// <summary>
		/// Return a dictionary of key/object intended to be a data collection of dependencies that comes with the report but not part of the report. ie: an artifact manifest
		/// </summary>
		/// <returns></returns>
		Dictionary<string, object> GetReportDependencies();

		/// <summary>
		/// Call to finalize report
		/// </summary>
		void FinalizeReport();
	}

	/// <summary>
	/// Hold telemetry data
	/// </summary>
	public class TelemetryData
	{
		public string TestName { get; private set; }
		public string DataPoint { get; private set; }
		public double Measurement { get; private set; }
		public string Context { get; private set; }
		public string Unit { get; private set; }
		public double Baseline { get; private set; }
		public TelemetryData(string InTestName, string InDataPoint, double InMeasurement, string InContext = "", string InUnit = "", double InBaseline = 0)
		{
			TestName = InTestName;
			DataPoint = InDataPoint;
			Measurement = InMeasurement;
			Context = InContext;
			Unit = InUnit;
			Baseline = InBaseline;
		}
	}

	/// <summary>
	/// Interface for telemetry report
	/// </summary>
	public interface ITelemetryReport
	{
		/// <summary>
		/// Attach Telemetry data to the ITestReport
		/// </summary>
		/// <param name="TestName"></param>
		/// <param name="DataPoint"></param>
		/// <param name="Measurement"></param>
		/// <param name="Context"></param>
		/// <param name="Unit"></param>
		/// <param name="Baseline"></param>
		/// <returns></returns>
		void AddTelemetry(string TestName, string DataPoint, double Measurement, string Context = "", string Unit = "", double Baseline = 0);

		/// <summary>
		/// Return the telemetry data accumulated
		/// </summary>
		/// <returns></returns>
		IEnumerable<TelemetryData> GetAllTelemetryData();
	}

	/// <summary>
	/// Test Report Base class
	/// </summary>
	public abstract class BaseTestReport : ITestReport, ITelemetryReport
	{
		public BaseTestReport()
		{
			Metadata = new Dictionary<string, string>();
		}

		/// <summary>
		/// Horde report version
		/// </summary>
		public abstract int Version { get; }

		/// <summary>
		/// Metadata blackboard
		/// </summary>
		public Dictionary<string, string> Metadata { get; private set; }

		/// <summary>
		/// Return report type
		/// </summary>
		public virtual string Type { get { return "Base Test Report"; } }
	
		/// <summary>
		/// Hold the list of telemetry data accumulated
		/// </summary>
		protected List<TelemetryData> TelemetryDataList;

		/// <summary>
		/// Set a property of the test report
		/// </summary>
		/// <param name="Property"></param>
		/// <param name="Value"></param>
		/// <returns></returns>
		public virtual void SetProperty(string Property, object Value)
		{
			PropertyInfo PropertyInstance = GetType().GetProperty(Property);
			PropertyInstance.SetValue(this, Convert.ChangeType(Value, PropertyInstance.PropertyType));
		}

		/// <summary>
		/// Set a metadata key/value to the test report
		/// </summary>
		/// <param name="Key"></param>
		/// <param name="Value"></param>
		/// <returns></returns>
		public virtual void SetMetadata(string Key, string Value)
		{
			Metadata[Key] = Value;
		}

		/// <summary>
		/// Add event to the test report
		/// </summary>
		/// <param name="Type"></param>
		/// <param name="Message"></param>
		/// <param name="Args"></param>
		/// <returns></returns>
		public abstract void AddEvent(EventType Type, string Message, params object[] Args);

		/// <summary>
		/// Add Error event with optional formatting inputs to report
		/// </summary>
		/// <param name="Message"></param>
		/// <param name="Args"></param>
		public void AddError(string Message, params object[] Args)
		{
			AddEvent(EventType.Error, Message, Args);
		}

		/// <summary>
		/// Add Warning event with optional formatting inputs to report
		/// </summary>
		/// <param name="Message"></param>
		/// <param name="Args"></param>
		public void AddWarning(string Message, params object[] Args)
		{
			AddEvent(EventType.Warning, Message, Args);
		}

		/// <summary>
		/// Add Info event with optional formatting inputs to report
		/// </summary>
		/// <param name="Message"></param>
		/// <param name="Args"></param>
		public void AddInfo(string Message, params object[] Args)
		{
			AddEvent(EventType.Info, Message, Args);
		}

		/// <summary>
		/// Attach an Artifact to the ITestReport
		/// </summary>
		/// <param name="ArtifactPath"></param>
		/// <param name="Name"></param>
		/// <returns>return true if the file was successfully attached</returns>
		public abstract bool AttachArtifact(string ArtifactPath, string Name = null);

		/// <summary>
		/// Attach Telemetry data to the ITestReport
		/// </summary>
		/// <param name="TestName"></param>
		/// <param name="DataPoint"></param>
		/// <param name="Measurement"></param>
		/// <param name="Context"></param>
		/// <param name="Unit"></param>
		/// <param name="Baseline"></param>
		/// <returns></returns>
		public virtual void AddTelemetry(string TestName, string DataPoint, double Measurement, string Context = "", string Unit = "", double Baseline = 0)
		{
			if (TelemetryDataList is null)
			{
				TelemetryDataList = new List<TelemetryData>();
			}

			TelemetryDataList.Add(new TelemetryData(TestName, DataPoint, Measurement, Context, Unit, Baseline));
		}

		/// <summary>
		/// Return all the telemetry data stored
		/// </summary>
		/// <returns></returns>
		public virtual IEnumerable<TelemetryData> GetAllTelemetryData()
		{
			return TelemetryDataList;
		}

		/// <summary>
		/// Return a dictionary of key/object intended to be a data collection of dependencies that comes with the report but not part of the report. ie: an artifact manifest
		/// </summary>
		/// <returns></returns>
		public virtual Dictionary<string, object> GetReportDependencies()
		{
			return new Dictionary<string, object>();
		}

		/// <inheritdoc/>
		public virtual void FinalizeReport()
		{ }
	}

	class JsonTryParseEnumConverter : JsonConverterFactory
	{
		public override bool CanConvert(Type TypeToConvert)
		{
			return TypeToConvert.IsEnum;
		}

		public override JsonConverter CreateConverter(Type EnumType, JsonSerializerOptions Options)
		{
			JsonConverter Converter = (JsonConverter)Activator.CreateInstance(typeof(DefaultToFirstEnumConverterInner<>).MakeGenericType(EnumType));

			return Converter;
		}

		private class DefaultToFirstEnumConverterInner<TEnum> :
			JsonConverter<TEnum> where TEnum : struct, Enum
		{
			public override TEnum Read(ref Utf8JsonReader Reader, Type TypeToConvert, JsonSerializerOptions Options)
			{
				if (Reader.TokenType == JsonTokenType.String)
				{
					string PropertyName = Reader.GetString();
					// Try to parse it, assigned it to the default(0) if not known.
					TEnum EnumValue;
					Enum.TryParse(PropertyName, true, out EnumValue);

					return EnumValue;
				}
				else if (Reader.TokenType == JsonTokenType.Number)
				{
					int Value = Reader.GetInt32();
					TEnum EnumValue;
					// Out of bound value get assigned anyway, so instead of try parse, we check if the value is defined else we use default()
					if(Enum.IsDefined(TypeToConvert, Value))
					{
						EnumValue = (TEnum)Enum.ToObject(TypeToConvert, Value);
					}
					else
					{
						EnumValue = default(TEnum);
					}
					return EnumValue;
				}

				throw new JsonException();
			}

			public override void Write(Utf8JsonWriter Writer, TEnum EnumValue, JsonSerializerOptions Options)
			{
				var PropertyName = EnumValue.ToString();
				Writer.WriteStringValue(Options.PropertyNamingPolicy?.ConvertName(PropertyName) ?? PropertyName);
			}
		}
	}

#nullable enable
	namespace Report
	{
		/// <summary>
		/// Interface to retrieve test audit metadata
		/// </summary>
		public interface ITestAuditMetadata
		{
			/* Unique identifier */
			public string TestKey { get; }

			/* Test name for display */
			public string TestName { get; }

			/* TestNode class */
			public string TestNodeClass { get; }

			/* Test version */
			public Version TestVersion { get; }

			/* Duration in minutes after which the test run has timeout */
			public float TestTimeoutMinutes { get; }

			/* Command line used to run the test */
			public string? TestCommandLine { get; }

			/* Test harness */
			public TestHarness? TestHarness { get; }

			/* Test result profile */
			public TestProfile? TestProfile { get; }

			/* Team to which the test was written for */
			public TestTeam? TestTeam { get; }

			/* The purpose of the test */
			public string? TestIntent { get; }

			/* Tags associated with the test */
			public TestTag[]? TestTags { get; }

			/* Params relevant to run this test */
			public Dictionary<string, object>? TestParams { get; }
		}

		/// <summary>
		/// Extend able Property value for report
		/// </summary>
		public partial class ReportProperty
		{
			private string Key;
			public ReportProperty(string InKey)
			{
				Key = InKey;
			}

			public override string ToString()
			{
				return Key;
			}
		}

		/// <summary>
		/// Test profile (unit, functional, performance, etc)
		/// </summary>
		public partial class TestProfile : ReportProperty
		{
			public TestProfile(string Key) : base(Key) { }
		}

		/// <summary>
		/// Test harness
		/// </summary>
		public partial class TestHarness : ReportProperty
		{
			public TestHarness(string Key) : base(Key) { }
		}

		/// <summary>
		/// Test team
		/// </summary>
		public partial class TestTeam : ReportProperty
		{
			public TestTeam(string Key) : base(Key) { }
		}

		/// <summary>
		/// Test tag
		/// </summary>
		public partial class TestTag : ReportProperty
		{
			public TestTag(string Key) : base(Key) { }
		}

	}
}