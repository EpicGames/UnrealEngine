// Copyright Epic Games, Inc. All Rights Reserved.

#nullable enable
using System;
using System.Linq;
using System.Collections;
using System.Collections.Generic;
using System.Collections.Concurrent;
using Microsoft.Extensions.Logging;
using EpicGames.Core;
using static Gauntlet.HordeReport.AutomatedTestSessionData;
using AutomatedTestSessionData = Gauntlet.HordeReport.AutomatedTestSessionData;
using AutomationTool;
using Logging = Microsoft.Extensions.Logging;

namespace Gauntlet
{
	/// <summary>
	/// Test execution tracking objects
	/// </summary>
	namespace TestTracking
	{
		/// <summary>
		/// Handler for associated phase data
		/// </summary>
		public interface IPhaseDataHandler
		{
			/// <summary>
			/// Called when test phase is added to report
			/// </summary>
			/// <param name="ReportedPhase"></param>
			public void OnAddToReport(TestPhase ReportedPhase);
		}

		/// <summary>
		/// Object that represent a test phase
		/// </summary>
		public class Phase
		{
			public string Name;
			public double ElapseTimeInSeconds;
			public DateTime? StartedAt;
			public TestPhaseOutcome Outcome;
			private ConcurrentQueue<LogEvent> Events;

			/// <summary>
			/// Opaque object that can be associated with the phase.
			/// The object class can implement its own handler by implementing IPhaseDataHandler interface.
			/// OnAddToReport method will be called when the phase is attached to the report allowing
			/// the opaque data to make modifications to the reported data.
			/// </summary>
			public object? Data;

			public Phase(string InName, DateTime? InDateTime = null, object? InData = null)
			{
				Name = InName;
				ElapseTimeInSeconds = 0;
				StartedAt = InDateTime;
				Outcome = TestPhaseOutcome.Unknown;
				Events = new ConcurrentQueue<LogEvent>();
				Data = InData;
			}

			/// <summary>
			/// Mark the phase as started
			/// </summary>
			/// <param name="InData"></param>
			/// <exception cref="AutomationException"></exception>
			public void Start(object? InData = null)
			{
				if (StartedAt != null)
				{
					throw new AutomationException($"Phase '{Name}' already started.");
				}
				StartedAt = DateTime.UtcNow;
				if (InData != null) Data = InData;
			}

			/// <summary>
			/// Mark the phase as finished
			/// </summary>
			/// <param name="InOutcome">The optional outcome. By default it will evaluted based on tracked events</param>
			/// <returns></returns>
			public double End(TestPhaseOutcome? InOutcome = null)
			{
				DateTime StartedTime = StartedAt ?? DateTime.UtcNow;
				ElapseTimeInSeconds = (DateTime.UtcNow - StartedTime).TotalSeconds;
				if (InOutcome != null)
				{
					Outcome = InOutcome.Value;
				}
				else
				{
					Outcome = Events.Any(E => E.Level == Logging.LogLevel.Error || E.Level == Logging.LogLevel.Critical) ? TestPhaseOutcome.Failed : TestPhaseOutcome.Success;
				}

				return ElapseTimeInSeconds;
			}

			/// <summary>
			/// Add an event to the phase tracker
			/// </summary>
			/// <param name="Level">The log level of the event</param>
			/// <param name="Message"></param>
			/// <param name="Args"></param>
			public void AddEvent(Logging.LogLevel Level, string Message, params object[] Args)
			{
				string? Format = null;
				Dictionary<string, object?>? Properties = null;
				if (Args.Any())
				{
					Format = Message;
					Properties = new Dictionary<string, object?>();
					MessageTemplate.ParsePropertyValues(Format, Args, Properties);
					Message = MessageTemplate.Render(Format, Properties);
				}
				EventId EventId = Level == Logging.LogLevel.Critical ? KnownLogEvents.Gauntlet_FatalEvent : KnownLogEvents.Gauntlet_TestEvent;
				LogEvent Event = new LogEvent(DateTime.UtcNow, Level, EventId, Message, Format, Properties, null);

				Events.Enqueue(Event);
			}

			/// <summary>
			/// Get the associated data
			/// </summary>
			/// <typeparam name="T"></typeparam>
			/// <returns></returns>
			public T? GetData<T>()
			{
				if (Data is T Value)
				{
					return Value;
				}

				return default(T);
			}

			/// <summary>
			/// Add the information tracked by the phase to the test report
			/// </summary>
			/// <param name="Report"></param>
			/// <returns></returns>
			public TestPhase AddToReport(AutomatedTestSessionData Report)
			{
				if (Outcome == TestPhaseOutcome.Unknown)
				{
					if (StartedAt == null)
					{
						Outcome = TestPhaseOutcome.NotRun;
					}
					else
					{
						End(TestPhaseOutcome.Interrupted);
					}
				}

				TestPhase ReportedPhase = Report.AddPhase(Name);
				if (StartedAt != null)
				{
					ReportedPhase.SetTiming(StartedAt.Value, (float)ElapseTimeInSeconds);
				}
				ReportedPhase.SetOutcome(Outcome);
				TestEventStream Stream = ReportedPhase.GetStream();
				foreach (LogEvent Event in Events)
				{
					Stream.AddEvent(Event);
				}

				if (Data is IPhaseDataHandler DataHandler)
				{
					DataHandler.OnAddToReport(ReportedPhase);
				}

				return ReportedPhase;
			}
		}

		/// <summary>
		/// Object to track queued phases and enumerate through
		/// </summary>
		public class PhaseQueue : IEnumerable<Phase>
		{
			public PhaseQueue()
			{
				_manifest = new();
				_queuedPhases = new();
				_last = null;
				_current = null;
			}

			private ConcurrentDictionary<string, Phase> _manifest;
			private ConcurrentQueue<Phase> _queuedPhases;
			private Phase? _last;
			private Phase? _current;
			private bool _orderByCompletion = false;

			/// <summary>
			/// The last added phase to the queue
			/// </summary>
			public Phase? Last => _last;


			/// <summary>
			/// The current running phase. Use Start to identify the running phase
			/// </summary>
			public Phase? Current => _current;

			/// <summary>
			/// Start a phase, if not queued it will be added to the queue
			/// </summary>
			/// <param name="Name">The name of the phase</param>
			/// <param name="Data">Extra object to associate with the phase</param>
			/// <returns></returns>
			/// <exception cref="AutomationException"></exception>
			public Phase Start(string Name, object? Data = null)
			{
				bool bAlreadyAdded = false;
				Phase StartedPhase = _manifest.AddOrUpdate(Name,
					(Name) => new Phase(Name, DateTime.UtcNow, Data),
					(Name, ExistingPhase) =>
					{
						bAlreadyAdded = true;
						ExistingPhase.Start(Data);
						return ExistingPhase;
					}
				);

				if (!bAlreadyAdded)
				{
					_queuedPhases.Enqueue(StartedPhase);
					_last = StartedPhase;
				}

				_current = StartedPhase;

				return StartedPhase;
			}

			/// <summary>
			/// Add a new phase. Raise an exception if it already exists.
			/// </summary>
			/// <param name="Name">The name of the phase</param>
			/// <param name="Data">Extra object to associate with the phase</param>
			/// <returns></returns>
			/// <exception cref="AutomationException"></exception>
			public Phase Add(string Name, object? Data = null)
			{
				Phase NewPhase = new Phase(Name, null, Data);
				if (!_manifest.TryAdd(Name, NewPhase))
				{
					throw new AutomationException($"Phase '{Name}' already added. Make sure to use unique phase name.");
				}

				_queuedPhases.Enqueue(NewPhase);
				_last = NewPhase;

				return NewPhase;
			}

			/// <summary>
			/// Get the corresponding phase
			/// </summary>
			/// <param name="Name"></param>
			/// <returns></returns>
			public Phase? Get(string Name)
			{
				Phase? FetchedPhase = null;
				_manifest.TryGetValue(Name, out FetchedPhase);

				return FetchedPhase;
			}

			/// <summary>
			/// Set current phase
			/// </summary>
			/// <param name="Name"></param>
			/// <returns></returns>
			public bool SetCurrent(string Name)
			{
				Phase? FetchedPhase = null;
				_manifest.TryGetValue(Name, out FetchedPhase);
				_current = FetchedPhase;

				return FetchedPhase != null;
			}

			IEnumerator IEnumerable.GetEnumerator()
			{
				return (IEnumerator)GetEnumerator();
			}

			/// <inheritdoc/>
			public IEnumerator<Phase> GetEnumerator()
			{
				if (_orderByCompletion)
				{
					return _queuedPhases.OrderBy(P => !P.StartedAt.HasValue).ThenBy(P => P.Outcome == TestPhaseOutcome.Unknown).ThenBy(P => P.StartedAt?.AddSeconds(P.ElapseTimeInSeconds)).GetEnumerator();
				}

				return _queuedPhases.GetEnumerator();
			}

			/// <summary>
			/// Whether or not to order the phases by completion date when iterating the object
			/// </summary>
			public void SetOrderByCompletion(bool Value)
			{
				_orderByCompletion = Value;
			}

			/// <summary>
			/// Clear the queue
			/// </summary>
			public void Clear()
			{
				_queuedPhases.Clear();
				_manifest.Clear();
				_last = null;
				_current = null;
			}
		}
	}
}