// Copyright Epic Games, Inc. All Rights Reserved.

using EpicGames.Core;
using System;
using System.Collections.Generic;
using System.IO;
using System.Linq;
using UnrealBuildTool;

namespace Gauntlet
{
	/// <summary>
	/// ITargetDevice extension for platforms that support configurable settings
	/// </summary>
	public interface IConfigurableDevice
	{
		/// <summary>
		/// Returns a configuration profile with the current device settings
		/// </summary>
		/// <returns></returns>
		PlatformConfigurationBase GetCurrentConfigurationSnapshot();

		/// <summary>
		/// Applies a configuration profile to the device. If required it will reboot the device
		/// </summary>
		/// <param name="Configuration"></param>
		/// <returns>false if applying the config profile fails</returns>
		bool ApplyConfiguration(PlatformConfigurationBase Configuration);
	}

	/// <summary>
	/// Base interface for a platform specific configuration reader
	/// </summary>
	public interface IPlatformConfigurationReader
	{
		bool SupportsPlatform(UnrealTargetPlatform? Platform);

		/// <summary>
		/// Returns the supported platform config file extension
		/// </summary>
		/// <returns></returns>
		string ConfigFileExtension();

		/// <summary>
		/// Reads the configuration from the passed in file location
		/// </summary>
		/// <param name="Location">An absolute path the to a file with platform configuration</param>
		/// <returns></returns>
		PlatformConfigurationBase ReadConfiguration(FileReference Location);
	}

	/// <summary>
	/// Base class for a platform configuration profile.
	/// This class needs to be implemented for each configurable platform
	/// </summary>
	public abstract class PlatformConfigurationBase
	{
		/// <summary>
		/// Name of the config profile.
		/// This name doesn't need to be unique across namespaces.
		/// </summary>
		public string ProfileName { get; set; }

		/// <summary>
		/// Namespace of the profile. Profiles have to have a unique name inside a namespace.
		/// </summary>
		public string Namespace { get; set; }

		/// <summary>
		/// Platform this configuration profile is for
		/// </summary>
		public UnrealTargetPlatform Platform { get; set; }
	}

	/// <summary>
	/// A singleton class that encapsulates a cache of device config profiles
	/// </summary>
	public class DeviceConfigurationCache
	{
		public static DeviceConfigurationCache Instance { get; private set; } = new();

		private Object LockObject = new Object();
		private Dictionary<ConfigurationCacheKey, PlatformConfigurationBase> ConfigurationCache = new();

		protected DeviceConfigurationCache()
		{
			Instance = this;
		}

		/// <summary>
		/// Key by which a configuration profile is identified in the cache.
		/// </summary>
		class ConfigurationCacheKey
		{
			public UnrealTargetPlatform Platform;
			public string Namespace;
			public string ProfileName;

			public ConfigurationCacheKey(UnrealTargetPlatform Platform, string ProjectName, string ProfileName)
			{
				this.Platform = Platform;
				this.Namespace = ProjectName;
				this.ProfileName = ProfileName;
			}

			public ConfigurationCacheKey(PlatformConfigurationBase Configuration)
			{
				this.Platform = Configuration.Platform;
				this.Namespace = Configuration.Namespace;
				this.ProfileName = Configuration.ProfileName;
			}

			public override bool Equals(object Other)
			{
				ConfigurationCacheKey OtherKey = Other as ConfigurationCacheKey;
				return OtherKey != null &&
					OtherKey.Platform == Platform &&
					OtherKey.ProfileName == ProfileName &&
					OtherKey.Namespace == Namespace;

			}

			public override int GetHashCode()
			{
				return (Platform, Namespace, ProfileName).GetHashCode();
			}
		}

		/// <summary>
		/// Scans the SettingDir for configuration profiles
		/// </summary>
		/// <param name="Platform">Platform for which to look for config files</param>
		/// <param name="Namespace">The namespace in which to put the configuration profiles</param>
		/// <param name="SettingDir"></param>
		/// <returns></returns>
		public bool DiscoverConfigurationProfiles(UnrealTargetPlatform? Platform, string Namespace, string SettingDir)
		{
			if (Platform == null || !Directory.Exists(SettingDir))
			{
				return false;
			}

			IPlatformConfigurationReader ConfigReader = Utils.InterfaceHelpers.FindImplementations<IPlatformConfigurationReader>(true)
				.Where(D => D.SupportsPlatform(Platform))
				.FirstOrDefault();

			if (ConfigReader == null)
			{
				Log.Info("Couldn't find a configuration reader for {0}", Platform);
				return false;
			}

			foreach (string File in Directory.EnumerateFiles(SettingDir, ConfigReader.ConfigFileExtension(), SearchOption.AllDirectories))
			{
				FileReference FileRef = new FileReference(File);
				PlatformConfigurationBase DeviceProfile = ConfigReader.ReadConfiguration(FileRef);
				if (DeviceProfile != null)
				{
					DeviceProfile.Platform = Platform.Value;
					DeviceProfile.Namespace = Namespace;
					DeviceProfile.ProfileName = FileRef.GetFileNameWithoutAnyExtensions();

					lock (LockObject)
					{
						ConfigurationCacheKey Key = new ConfigurationCacheKey(Platform.Value, Namespace, DeviceProfile.ProfileName);
						if (ConfigurationCache.ContainsKey(Key))
						{
							Log.Verbose("Device configuration profile {0} already exists", FileRef.FullName);
							continue;
						}

						ConfigurationCache.Add(Key, DeviceProfile);
					}
				}
			}

			return true;
		}

		public void CacheConfigurationSnapshot(PlatformConfigurationBase Configuration, bool Overwrite = false)
		{
			lock (LockObject)
			{
				ConfigurationCacheKey Key = new ConfigurationCacheKey(Configuration.Platform, "Snapshot", Configuration.ProfileName);
				if (ConfigurationCache.ContainsKey(Key))
				{
					if(Overwrite)
					{
						ConfigurationCache[Key] = Configuration;
					}
				}
				else
				{
					ConfigurationCache.Add(Key, Configuration);
				}
			}
		}

		public PlatformConfigurationBase GetConfigurationSnapshot(UnrealTargetPlatform? Platform, string DeviceName)
		{
			return GetConfiguration(Platform, "Snapshot", DeviceName);
		}

		public void ClearSnapshot(PlatformConfigurationBase Snapshot)
		{
			lock (LockObject)
			{
				ConfigurationCacheKey Key = new ConfigurationCacheKey(Snapshot);
				ConfigurationCache.Remove(Key);
			}
		}

		public void RevertDeviceConfiguration(ITargetDevice Device)
		{
			if (Device is IConfigurableDevice ConfigurableDevice)
			{
				var Snapshot = GetConfigurationSnapshot(Device.Platform, Device.Name);
				if (Snapshot == null)
				{
					return;
				}

				// Connect temporarily to be able to revert the device's configuration
				// if the device was disconnected entering here disconnect it after this is over
				bool bNeedsDisconnect = false;
				if (!Device.IsConnected)
				{
					Device.Connect();
					bNeedsDisconnect = true;
				}

				if (ConfigurableDevice.ApplyConfiguration(Snapshot))
				{
					ClearSnapshot(Snapshot);
				}

				if (bNeedsDisconnect)
				{
					Device.Disconnect();
				}
			}
		}

		public PlatformConfigurationBase GetConfiguration(UnrealTargetPlatform? Platform, string Namespace, string ProfileName)
		{
			if (Platform == null)
			{
				return null;
			}

			PlatformConfigurationBase Res = null;

			lock (LockObject)
			{
				ConfigurationCacheKey Key = new ConfigurationCacheKey(Platform.Value, Namespace, ProfileName);
				ConfigurationCache.TryGetValue(Key, out Res);
			}

			return Res;
		}
	}
}
