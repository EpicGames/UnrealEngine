// Copyright Epic Games, Inc. All Rights Reserved.

using EpicGames.Core;
using System;
using System.Linq;
using UnrealBuildTool;

namespace Gauntlet
{
	public interface IPlatformFirmwareHandler
	{
		bool CanSupportPlatform(UnrealTargetPlatform Platform);

		bool CanSupportProject(string ProjectName);

		bool GetDesiredVersion(UnrealTargetPlatform Platform, string ProjectName, out string Version);

		bool GetCurrentVersion(ITargetDevice Device, out string Version);

		bool UpdateDeviceFirmware(ITargetDevice Device, string Version);
	};

	public class FirmwareValidator : IDeviceValidator
	{
		[AutoParam("")]
		public string FirmwareVersion { get; set; }

		[AutoParamWithNames(Default: "", "Project", "ProjectName")]
		public string ProjectName { get; set; }

		[AutoParamWithNames(false, "UpdateFirmware", "TryUpdateFirmware")]
		public bool bEnabled { get; set; }

		public FirmwareValidator()
		{
			AutoParam.ApplyParamsAndDefaults(this, Globals.Params.AllArguments);

			// Only enable this validator if a firmware version or a project name is specified
			bEnabled = bEnabled && (!string.IsNullOrEmpty(FirmwareVersion) || !string.IsNullOrEmpty(ProjectName));
		}

		public bool TryValidateDevice(ITargetDevice Device, ref string Message)
		{
			if(!string.IsNullOrEmpty(FirmwareVersion))
			{
				// Requesting an explicit firmware version
				return TryUpdateFirmware(Device, FirmwareVersion, ref Message);
			}
			else if(!string.IsNullOrEmpty(ProjectName))
			{
				// Requesting the firmware version a project requires
				return TryUpdateFirmwareForProject(Device, ProjectName, ref Message);
			}

			return true;
		}

		/// <summary>
		/// Attempts to update the provided Device's firmware to the requested version
		/// </summary>
		/// <param name="Device">The Device to update</param>
		/// <param name="DesiredVersion">The firmware version to update to</param>
		/// <param name="Message">The output message when failing to validate</param>
		/// <returns></returns>
		public bool TryUpdateFirmware(ITargetDevice Device, string DesiredVersion, ref string Message)
		{
			try
			{
				if (!IDeviceValidator.EnsureConnection(Device, ref Message))
				{
					return false;
				}

				if (Device.Platform == null)
				{
					Log.Warning("Cannot update firmware for device because the platform value is null. Was the device mocked?");
					return false;
				}

				if(string.IsNullOrEmpty(DesiredVersion))
				{
					Log.Warning("Cannot apply a null or empty firmware version to the device");
					return false;
				}

				UnrealTargetPlatform Platform = Device.Platform.Value;

				IPlatformFirmwareHandler Handler = GetRequiredPlatformFirmwareHandler(Platform);
				if (Handler == null)
				{
					Log.Info("Cannot find an IPlatformFirmwareHandler that supports {Platform}. Has one been implemented? Skipping firmware update", Platform);
					return true;
				}

				string CurrentVersion;
				if (!Handler.GetCurrentVersion(Device, out CurrentVersion))
				{
					Log.Warning("Failed to retrieve current firmware version");
					return false;
				}

				if (!CurrentVersion.Equals(DesiredVersion, StringComparison.OrdinalIgnoreCase))
				{
					if (!Handler.UpdateDeviceFirmware(Device, DesiredVersion))
					{
						Message = $"Failed to update device's firmware to version {DesiredVersion}";
						Log.Info(KnownLogEvents.Gauntlet_DeviceEvent, Message);
						return false;
					}
					else
					{
						Log.Info("Successfully updated {DeviceName} to new firmware version {DesiredVersion}", Device.Name, DesiredVersion);
					}
				}
				else
				{
					Log.Info("Current firmware version {CurrentVersion} matches desired version {DesiredVersion}. Skipping firmware update", CurrentVersion, DesiredVersion);
				}

				return true;
			}
			catch(Exception Ex)
			{
				string ErrorMessage = $"Encountered an {Ex.GetType()} while attempting to update firmware. {Ex}";
				if (Ex is DeviceException)
				{
					Message = ErrorMessage;
					Log.Info(KnownLogEvents.Gauntlet_DeviceEvent, Message);
				}
				else
				{
					Log.Warning(ErrorMessage);
				}

				return false;
			}
		}

		/// <summary>
		/// Attempts to update the provided Device's firmware version to the version requested by the specified project
		/// </summary>
		/// <param name="Device">The Device to update</param>
		/// <param name="ProjectName">Name of the project to get firmware version of</param>
		/// <param name="Message">The output message when failing to validate</param>
		/// <returns></returns>
		public bool TryUpdateFirmwareForProject(ITargetDevice Device, string ProjectName, ref string Message)
		{
			try
			{
				if (!IDeviceValidator.EnsureConnection(Device, ref Message))
				{
					return false;
				}

				if (Device.Platform == null)
				{
					Log.Warning("Cannot update firmware for device {Device} because the platform value is null. Was the device mocked?");
					return false;
				}

				if (string.IsNullOrEmpty(ProjectName))
				{
					Log.Warning("Cannot use a null or empty project to determine desired firmware version");
					return false;
				}

				UnrealTargetPlatform Platform = Device.Platform.Value;

				IPlatformFirmwareHandler Handler = GetRequiredPlatformFirmwareHandler(Platform, ProjectName);

				if (Handler == null)
				{
					Log.Info("Cannot find an IPlatformFirmwareHandler that supports {Platform} and {Project}. Has one been implemented? Skipping firmware update", Platform, ProjectName);
					return true;
				}

				string DesiredVersion;
				if (!Handler.GetDesiredVersion(Platform, ProjectName, out DesiredVersion))
				{
					Log.Warning("{Handler} failed to find a desired firmware version for {Platform} and {Project}", Handler.GetType(), Platform, ProjectName);
					return false;
				}

				return TryUpdateFirmware(Device, DesiredVersion, ref Message);
			}
			catch (Exception Ex)
			{
				string ErrorMessage = $"Encountered an {Ex.GetType()} while attempting to update firmware. {Ex}";
				if (Ex is DeviceException)
				{
					Message = ErrorMessage;
					Log.Info(KnownLogEvents.Gauntlet_DeviceEvent, Message);
				}
				else
				{
					Log.Warning(ErrorMessage);
				}

				return false;
			}
		}

		// Gets the required platform handler that supports the provided platform
		private IPlatformFirmwareHandler GetRequiredPlatformFirmwareHandler(UnrealTargetPlatform Platform)
		{
			return Utils.InterfaceHelpers.FindImplementations<IPlatformFirmwareHandler>(true)
				.Where(Handler => Handler.CanSupportPlatform(Platform))
				.FirstOrDefault();
		}

		// Gets the required platform handler that supports the provided platform and project
		private IPlatformFirmwareHandler GetRequiredPlatformFirmwareHandler(UnrealTargetPlatform Platform, string ProjectName)
		{
			return Utils.InterfaceHelpers.FindImplementations<IPlatformFirmwareHandler>(true)
				.Where(Handler => Handler.CanSupportPlatform(Platform) && Handler.CanSupportProject(ProjectName))
				.FirstOrDefault();
		}
	}
};