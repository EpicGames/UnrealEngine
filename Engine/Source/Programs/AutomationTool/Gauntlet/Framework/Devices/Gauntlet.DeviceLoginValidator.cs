// Copyright Epic Games, Inc. All Rights Reserved.

using EpicGames.Core;

namespace Gauntlet
{
	/// <summary>
	/// DeviceLoginValidator works for any ITargetDevice that implements IOnlineServiceLogin (typically consoles)
	/// This validator ensures the device have an account logged into that platforms online service.
	/// </summary>
	public class DeviceLoginValidator : IDeviceValidator
	{
		[AutoParamWithNames(false, "VerifyLogin")]
		public bool bEnabled { get; set; }

		public DeviceLoginValidator()
		{
			AutoParam.ApplyParamsAndDefaults(this, Globals.Params.AllArguments);
		}

		public bool TryValidateDevice(ITargetDevice Device, ref string Message)
		{
			if(!IDeviceValidator.EnsureConnection(Device, ref Message))
			{
				return false;
			}

			if (Device is IOnlineServiceLogin DeviceLogin)
			{
				Log.Info("Verifying device login...");
				if(DeviceLogin.VerifyLogin())
				{
					Log.Info("User signed-in");
					return true;
				}
				else
				{
					Message = "Unable to secure login to an online platform account on device!";
					Log.Info(KnownLogEvents.Gauntlet_DeviceEvent, Message);
					return false;
				}
			}
			else
			{
				Log.Verbose("{Platform} does not implement IOnlineServiceLogin, skipping login validation.", Device.Platform.Value);
				return true;
			}
		}
	}
}
