// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.IO;
using AutomationTool;
using AutomationTool.DeviceReservation;
using UnrealBuildTool;
using System.Text.RegularExpressions;
using System.Linq;
using System.Text.Json;
using System.Text.Json.Serialization;
using EpicGames.Core;
using System.Data;
using Gauntlet.Utils;
using System.Reflection;
using static AutomationTool.DeviceReservation.Reservation;

namespace Gauntlet
{
	/// <summary>
	/// Device performance specification
	/// </summary>
	public enum EPerfSpec
	{
		Unspecified,
		Minimum,
		Recommended,
		High
	};

	/// <summary>
	/// Information that defines a device
	/// </summary>
	public class DeviceDefinition
	{
		public string Name { get; set; }

		public string Address { get; set; }

		public string DeviceData { get; set; }

		// legacy - remove!
		[JsonConverter(typeof(UnrealTargetPlatformConvertor))]
		public UnrealTargetPlatform Type { get; set; }

		[JsonConverter(typeof(UnrealTargetPlatformConvertor))]
		public UnrealTargetPlatform? Platform { get; set; }

		public EPerfSpec PerfSpec { get; set; }

		public string Model { get; set; } = string.Empty;

		public string Available { get; set; }

		public bool RemoveOnShutdown { get; set; }

		public List<string> Tags { get; set; } = new List<string>();

		public override string ToString()
		{
			return string.Format("{0} @ {1}. Platform={2} Model={3}", Name, Address, Platform, string.IsNullOrEmpty(Model) ? "Unspecified" : Model);
		}
	}

	/// <summary>
	/// Device target constraint, can be expanded for specifying installed RAM, OS version, etc
	/// </summary>
	public class UnrealDeviceTargetConstraint : IEquatable<UnrealDeviceTargetConstraint>
	{
		public readonly UnrealTargetPlatform? Platform;
		public readonly EPerfSpec PerfSpec;
		public readonly string Model;
		public readonly string DeviceName;
		public readonly Tags Tags;

		public UnrealDeviceTargetConstraint(UnrealTargetPlatform? Platform, EPerfSpec PerfSpec = EPerfSpec.Unspecified, string Model = null, string DeviceName = null, Tags Tags = null)
		{
			this.Platform = Platform;
			this.PerfSpec = PerfSpec;
			this.Model = Model == null ? string.Empty : Model;
			this.DeviceName = DeviceName == null ? string.Empty : DeviceName;
			this.Tags = Tags;
		}

		/// <summary>
		/// Tests whether the constraint is identity, ie. unconstrained
		/// </summary>
		public bool IsIdentity()
		{
			return (PerfSpec == EPerfSpec.Unspecified)
				&& (Model == string.Empty)
				&& (DeviceName == string.Empty)
				&& (Tags == null || !Tags.Any());
		}


		/// <summary>
		/// Check whether device satisfies the constraint
		/// </summary>
		public bool Check(ITargetDevice Device)
		{
			return Platform == Device.Platform && (IsIdentity() || this == DevicePool.Instance.GetConstraint(Device));
		}

		public bool Check(DeviceDefinition DeviceDef)
		{
			if (Platform != DeviceDef.Platform)
			{
				return false;
			}

			if (IsIdentity())
			{
				return true;
			}

			bool ModelMatch = Model == string.Empty ? true : Model.Equals(DeviceDef.Model, StringComparison.InvariantCultureIgnoreCase);
			bool PerfMatch = (PerfSpec == EPerfSpec.Unspecified) ? true : PerfSpec == DeviceDef.PerfSpec;
			bool NameMatch = DeviceName == string.Empty ? true : DeviceName.Equals(DeviceDef.Name, StringComparison.InvariantCultureIgnoreCase);

			// Tag matches if null or has no required/blocked tags
			bool TagMatch = Tags == null
				|| ((Tags.RequiredTags == null || Tags.RequiredTags.Count == 0)
				&& (Tags.BlockedTags == null || Tags.BlockedTags.Count == 0));

			if (!TagMatch)
			{
				// If there are tags, make sure it has the required tags but none of the blocked tags
				bool bHasRequiredTags = Tags.RequiredTags == null || Tags.RequiredTags.All(Tag => DeviceDef.Tags.Contains(Tag, StringComparer.OrdinalIgnoreCase));
				bool bHasNoBlockedTags = Tags.BlockedTags == null || !DeviceDef.Tags.Any(Tag => Tags.BlockedTags.Contains(Tag, StringComparer.OrdinalIgnoreCase));
				TagMatch = bHasRequiredTags && bHasNoBlockedTags;
			}

			return ModelMatch && PerfMatch && NameMatch && TagMatch;
		}

		public bool Equals(UnrealDeviceTargetConstraint Other)
		{
			if (ReferenceEquals(Other, null))
			{
				throw new AutomationException("Comparing null target constraint");
			}

			if (ReferenceEquals(this, Other))
			{
				return true;
			}

			bool bNullTags = Other.Tags == null && Tags == null;
			bool bMatchingTags = Other.Tags != null && Other.Tags.Equals(Tags);
			bool bNullEmptyTags = (Other.Tags == null && Tags != null && !Tags.Any()) || (Other.Tags != null && !Other.Tags.Any() && Tags == null);

			return Other.Platform == Platform
				&& Other.Model.Equals(Model, StringComparison.InvariantCultureIgnoreCase)
				&& Other.DeviceName.Equals(DeviceName, StringComparison.InvariantCultureIgnoreCase)
				&& Other.PerfSpec == PerfSpec
				&& (bNullTags || bMatchingTags || bNullEmptyTags);
		}

		public override bool Equals(object Obj)
		{
			if (ReferenceEquals(Obj, null))
			{
				throw new AutomationException("Comparing null target constraint");
			}

			if (ReferenceEquals(this, Obj))
			{
				return true;
			}

			if (Obj.GetType() != typeof(UnrealDeviceTargetConstraint))
			{
				return false;
			}

			return Equals((UnrealDeviceTargetConstraint)Obj);
		}

		public static bool operator ==(UnrealDeviceTargetConstraint C1, UnrealDeviceTargetConstraint C2)
		{
			if (ReferenceEquals(C1, null) || ReferenceEquals(C2, null))
			{
				throw new AutomationException("Comparing null target constraint");
			}

			return C1.Equals(C2);
		}

		public static bool operator !=(UnrealDeviceTargetConstraint C1, UnrealDeviceTargetConstraint C2)
		{
			return !(C1 == C2);
		}

		public override string ToString()
		{
			string Value = Platform.ToString();
			if (PerfSpec != EPerfSpec.Unspecified)
			{
				Value = string.Format("{0}:{1}", Value, PerfSpec.ToString());
			}
			if(Model != string.Empty)
			{
				Value = string.Format("{0}:{1}", Value, Model);
			}
			if (DeviceName != string.Empty)
			{
				Value = string.Format("{0}:{1}", Value, DeviceName);
			}
			if (Tags != null && Tags.Any())
			{
				Value = string.Format("{0}:{1}", Value, Tags.ToString());
			}
			return Value;
		}

		/// <summary>
		/// Format to string the device constraint including its identify
		/// </summary>
		/// <returns></returns>
		public string FormatWithIdentifier()
		{
			return string.Format("{0}:{1}", Platform, Model == string.Empty ? PerfSpec.ToString() : Model);
		}

		public override int GetHashCode()
		{
			return ToString().GetHashCode();
		}
	}

	/// <summary>
	/// Device marked as having a problem
	/// </summary>
	public struct ProblemDevice
	{
		public ProblemDevice(string Name, UnrealTargetPlatform Platform)
		{
			this.Name = Name;
			this.Platform = Platform;
		}

		public string Name;
		public UnrealTargetPlatform Platform;
	}

	/// <summary>
	/// Singleton class that's responsible for providing a list of devices and reserving them. Code should call
	/// EnumerateDevices to build a list of desired devices, which must then be reserved by calling ReserveDevices.
	/// Once done ReleaseDevices should be called.
	///
	/// These reservations exist at the process level and we rely on the device implementation to provide arbitrage
	/// between difference processes and machines
	///
	/// </summary>
	public class DevicePool : IDisposable
	{
		/// <summary>
		/// Access to our singleton
		/// </summary>
		public static DevicePool Instance
		{
			get
			{
				if (_Instance == null || _Instance.bDisposed)
				{
					if (_Instance != null)
					{
						Log.Info("DevicePool has been disposed. Reinitilizing with a new instance.");
					}

					_Instance = new DevicePool();
				}

				return _Instance;
			}
			private set
			{
				_Instance = value;
			}
		}

		/// <summary>
		/// Whether or not installations should be skipped
		/// This is primarily used within the context of UnrealSession
		/// </summary>
		public static bool SkipInstall;

		/// <summary>
		/// Whether or not devices should be fully cleaned
		/// This is primarily used within the context of UnrealSession
		/// </summary>
		public static bool FullClean;

		/// <summary>
		/// Whether or a device reservation block is active
		/// This is primarily used within the context of UnrealSession
		/// when a step is executed by Horde that contains a DeviceReserve=Begin annotation
		/// or is a step that proceeds one
		/// </summary>
		public static bool DeviceReservationBlock;

		/// <summary>
		/// Whether or not a device reservation block is in it's final active step
		/// This is used to prevent cleaning of the tempdir where builds may be stored
		/// </summary>
		public static bool DeviceReservationBlockFinalStep;

		/// <summary>
		/// Whether or not this particular step is an installations tep in a device reservation block
		/// This is primarily used within the context of UnrealSession
		/// when a job is executed by Horde that contains a DeviceReserve=Begin/Install annotation
		public static bool? IsInstallStep;

		/// <summary>
		/// Optional index to name local devices
		/// Occasionally useful for retaining device caches from previous tests
		/// </summary>
		[AutoParam(0)]
		public int LocalDeviceIndex { get; set; }

		/// <summary>
		/// Active pool instance
		/// </summary>
		private static DevicePool _Instance;

		/// <summary>
		/// Object used for locking access to internal data
		/// </summary>
		private object LockObject = new object();

		/// <summary>
		/// List of all provisioned devices that are can be claimed
		/// </summary>
		private List<ITargetDevice> AvailableDevices = new List<ITargetDevice>();

		/// <summary>
		/// List of all provisioned devices that have been claimed
		/// </summary>
		private List<ITargetDevice> ClaimedDevices = new List<ITargetDevice>();

		/// <summary>
		/// List of all provisioned devices that were not provided by a reservation service
		/// </summary>
		private List<ITargetDevice> LocalDevices => AvailableDevices
			.Union(ClaimedDevices)
			.Except(ReservationServices.SelectMany(Service => Service.ReservedDevices))
			.ToList();

		/// <summary>
		/// List of all enabled reservation services
		/// </summary>
		private List<IDeviceReservationService> ReservationServices = new List<IDeviceReservationService>();

		/// <summary>
		/// List of platforms we've had devices for
		/// </summary>
		private HashSet<UnrealTargetPlatform?> UsedPlatforms = new HashSet<UnrealTargetPlatform?>();

		/// <summary>
		/// List of device definitions that can be provisioned on demand
		/// </summary>
		private List<DeviceDefinition> UnprovisionedDevices = new List<DeviceDefinition>();

		/// <summary>
		/// List of definitions that failed to provision
		/// </summary>
		private List<DeviceDefinition> FailedProvisions = new List<DeviceDefinition>();

		/// <summary>
		/// Device constraints for performance profiles, etc
		/// </summary>
		private Dictionary<ITargetDevice, UnrealDeviceTargetConstraint> Constraints = new Dictionary<ITargetDevice, UnrealDeviceTargetConstraint>();

		/// <summary>
		/// Directory automation artifacts are saved to
		/// </summary>
		private string LocalTempDir;

		/// <summary>
		/// Uniform cache, useful for some instances of multi-role editor tests. Only used if -UniformCache is enabled
		/// </summary>
		private string UniformCache;

		/// <summary>
		/// Protected constructor - code should use DevicePool.Instance
		/// </summary>
		protected DevicePool()
		{
			AutoParam.ApplyParamsAndDefaults(this, Globals.Params.AllArguments);

			lock (LockObject)
			{
				// Create two local devices by default for ease of running a client and server
				AddLocalDevices(2);
				InitializeReservationServices();
			}

			SkipInstall |= Globals.Params.ParseParams("SkipInstall", "SkipCopy", "SkipDeploy");
			FullClean |= Globals.Params.ParseParam("FullClean");
		}

		#region IDisposable Support
		bool bDisposed = false;
		~DevicePool()
		{
			Dispose(false);
		}

		/// <summary>
		/// Shutdown the pool and release all devices.
		/// </summary>
		public static void Shutdown()
		{
			Instance?.Dispose();
			Instance = null;
		}

		public void Dispose()
		{
			Dispose(true);
			GC.SuppressFinalize(this);
		}

		/// <summary>
		/// Perform actual dispose behavior
		/// </summary>
		/// <param name="disposing"></param>
		protected virtual void Dispose(bool disposing)
		{
			if (bDisposed)
			{
				return;
			}

			lock (LockObject)
			{
				if (disposing)
				{
					// Dispose all local devices
					foreach (ITargetDevice LocalDevice in LocalDevices)
					{
						DeviceConfigurationCache.Instance.RevertDeviceConfiguration(LocalDevice);
						LocalDevice.Dispose();
					}

					// Dispose of any services. These dispose their associated devices
					foreach (IDeviceReservationService Service in ReservationServices)
					{
						Service.Dispose();
					}

					// Cleanup things like platform sdk daemons
					CleanupDevices(UsedPlatforms);

					AvailableDevices.Clear();
					ClaimedDevices.Clear();
					ReservationServices.Clear();
					UnprovisionedDevices.Clear();
				}
			}

			bDisposed = true;
		}
		#endregion

		/// <summary>
		/// Returns the number of available devices of the provided type. This includes unprovisioned devices but not reserved ones.
		/// Note: unprovisioned devices are currently only returned when device is not constrained
		/// </summary>
		public int GetAvailableDeviceCount(UnrealDeviceTargetConstraint Constraint, Func<ITargetDevice, bool> Validate = null)
		{
			lock (LockObject)
			{
				return AvailableDevices.Where(D => Validate == null ? Constraint.Check(D) : Validate(D)).Count() +
					UnprovisionedDevices.Where(D => Constraint.Check(D)).Count();
			}
		}

		/// <summary>
		/// Returns the number of available devices of the provided type. This includes unprovisioned devices but not reserved ones
		/// Note: unprovisioned devices are currently only returned when device is not constrained
		/// </summary>
		public int GetTotalDeviceCount(UnrealDeviceTargetConstraint Constraint, Func<ITargetDevice, bool> Validate = null)
		{
			lock (LockObject)
			{
				return AvailableDevices.Union(ClaimedDevices).Where(D => Validate == null ? Constraint.Check(D) : Validate(D)).Count() +
					UnprovisionedDevices.Where(D => Constraint.Check(D)).Count();
			}
		}

		public UnrealDeviceTargetConstraint GetConstraint(ITargetDevice Device)
		{
			if (!Constraints.ContainsKey(Device))
			{
				throw new AutomationException("Device pool has no contstaint for {0} (device was likely released)", Device);
			}

			return Constraints[Device];
		}

		public void SetLocalOptions(string InLocalTemp, bool InUniqueTemps = false, string InDeviceURL = "")
		{
			LocalTempDir = InLocalTemp;
			Legacy_DeviceURL = InDeviceURL;
		}

		public void AddLocalDevices(int MaxCount)
		{
			AddLocalDevices(MaxCount, BuildHostPlatform.Current.Platform);
		}

		public void AddLocalDevices(int MaxCount, UnrealTargetPlatform LocalPlatform)
		{
			int NumDevices = GetAvailableDeviceCount(new UnrealDeviceTargetConstraint(LocalPlatform));

			for (int i = NumDevices; i < MaxCount; i++)
			{
				DeviceDefinition Def = new DeviceDefinition();
				Def.Name = string.Format("LocalDevice{0}", i + LocalDeviceIndex);
				Def.Platform = LocalPlatform;
				UnprovisionedDevices.Add(Def);
			}
		}

		public void AddVirtualDevices(int MaxCount)
		{
			UnrealTargetPlatform LocalPlat = BuildHostPlatform.Current.Platform;

			IEnumerable<IVirtualLocalDevice> VirtualDevices = Gauntlet.Utils.InterfaceHelpers.FindImplementations<IVirtualLocalDevice>()
					.Where(F => F.CanRunVirtualFromPlatform(LocalPlat));

			foreach (IVirtualLocalDevice Device in VirtualDevices)
			{
				UnrealTargetPlatform? DevicePlatform = Device.GetPlatform();
				if (DevicePlatform != null)
				{
					int NumDevices = GetAvailableDeviceCount(new UnrealDeviceTargetConstraint(DevicePlatform));
					for (int i = NumDevices; i < MaxCount; i++)
					{
						DeviceDefinition Def = new DeviceDefinition();
						Def.Name = string.Format("Virtual{0}{1}", DevicePlatform.ToString(), i);
						Def.Platform = DevicePlatform ?? BuildHostPlatform.Current.Platform;
						UnprovisionedDevices.Add(Def);
					}
				}
			}
		}

		/// <summary>
		/// Created a list of device definitions from the passed in reference. Needs work....
		/// </summary>
		/// <param name="DefaultPlatform"></param>
		/// <param name="InputReference"></param>
		/// <param name="ObeyConstraints"></param>
		public void AddDevices(UnrealTargetPlatform DefaultPlatform, string InputReference, bool ObeyConstraints = true)
		{
			lock (LockObject)
			{
				List<ITargetDevice> NewDevices = new List<ITargetDevice>();

				int SlashIndex = InputReference.IndexOf("\\") >= 0 ? InputReference.IndexOf("\\") : InputReference.IndexOf("/");

				bool PossibleFileName = InputReference.IndexOfAny(Path.GetInvalidPathChars()) < 0 &&
							(InputReference.IndexOf(":") == -1 || (InputReference.IndexOf(":") == SlashIndex - 1));
				// Did they specify a file?
				if (PossibleFileName && File.Exists(InputReference))
				{
					Log.Info("Adding devices from {Reference}", InputReference);
					List<DeviceDefinition> DeviceDefinitions = JsonSerializer.Deserialize<List<DeviceDefinition>>(
						File.ReadAllText(InputReference),
						new JsonSerializerOptions { PropertyNameCaseInsensitive = true }
					);

					foreach (DeviceDefinition Def in DeviceDefinitions)
					{
						Log.Info("Adding {DeviceDetails}", Def);

						// use Legacy field if it exists
						if (Def.Platform == null)
						{
							Def.Platform = Def.Type;
						}

						// check for an availability constraint
						if (string.IsNullOrEmpty(Def.Available) == false && ObeyConstraints)
						{
							// check whether disabled
							if (String.Compare(Def.Available, "disabled", true) == 0)
							{
								Log.Info("Skipping {DeviceName} due to being disabled", Def.Name);
								continue;
							}

							// availability is specified as a range, e.g 21:00-09:00.
							Match M = Regex.Match(Def.Available, @"(\d{1,2}:\d\d)\s*-\s*(\d{1,2}:\d\d)");

							if (M.Success)
							{
								DateTime From, To;

								if (DateTime.TryParse(M.Groups[1].Value, out From) && DateTime.TryParse(M.Groups[2].Value, out To))
								{
									// these are just times so when parsed will have todays date. If the To time is less than
									// From (22:00-06:00) time it spans midnight so move it to the next day
									if (To < From)
									{
										To = To.AddDays(1);
									}

									// if From is in the future (e.g. it's 01:00 and range is 22:00-08:00) we may be in the previous days window,
									// so move them both back a day
									if (From > DateTime.Now)
									{
										From = From.AddDays(-1);
										To = To.AddDays(-1);
									}

									if (DateTime.Now < From || DateTime.Now > To)
									{
										Log.Info("Skipping {DeviceName} due to availability constraint {Constraint}", Def.Name, Def.Available);
										continue;
									}
								}
								else
								{
									Log.Warning("Failed to parse availability {Constraint} for {DeviceName}", Def.Available, Def.Name);
								}
							}
						}

						Def.RemoveOnShutdown = true;

						if (Def.Platform == null)
						{
							Def.Platform = DefaultPlatform;
						}

						UnprovisionedDevices.Add(Def);
					}

					// randomize devices so if there's a bad device st the start so we don't always hit it (or we do if its later)
					UnprovisionedDevices = UnprovisionedDevices.OrderBy(D => Guid.NewGuid()).ToList();
				}
				else
				{
					if (string.IsNullOrEmpty(InputReference) == false)
					{
						string[] DevicesList = InputReference.Split(',');

						int UserDeviceIndex = 0;
						foreach (string DeviceRef in DevicesList)
						{

							// check for <platform>:<address>:<port>|<model>. We pass address:port to device constructor
							Match M = Regex.Match(DeviceRef, @"(.+?):(.+)");

							UnrealTargetPlatform DevicePlatform = DefaultPlatform;
							string DeviceAddress = DeviceRef;
							string Model = string.Empty;

							// When using device services, skip adding non-desktop local devices to pool if any of the services can support that platform
							bool IsDesktop = DevicePlatform.IsInGroup(UnrealPlatformGroup.Desktop);
							bool ReservationsEnabled = ReservationServices.Count > 0;
							bool ServicesCanSupportThisPlatform = ReservationsEnabled
								&& ReservationServices.Where(Service => Service.CanSupportDeviceConstraint(new UnrealDeviceTargetConstraint(DevicePlatform))).Any();
							bool DeviceServiceEnabled = ServicesCanSupportThisPlatform;

							if (!IsDesktop && DeviceRef.Equals("default", StringComparison.OrdinalIgnoreCase) && DeviceServiceEnabled)
							{
								continue;
							}

							if (M.Success)
							{
								if (!UnrealTargetPlatform.TryParse(M.Groups[1].ToString(), out DevicePlatform))
								{
									throw new AutomationException("platform {0} is not a recognized device type", M.Groups[1].ToString());
								}

								DeviceAddress = M.Groups[2].ToString();

								// parse device model
								if (DeviceAddress.Contains("|"))
								{
									string[] Components = DeviceAddress.Split(new char[] { '|' });
									DeviceAddress = Components[0];
									Model = Components[1];
								}

							}

							Log.Info(KnownLogEvents.Gauntlet_DeviceEvent, "Added device {Platform}:{Address} to pool", DevicePlatform, DeviceAddress);
							DeviceDefinition Def = new DeviceDefinition();
							Def.Address = DeviceAddress;
							Def.Name = DeviceAddress;
							Def.Platform = DevicePlatform;
							Def.Model = Model;
							if (Def.Name.Equals("default", StringComparison.OrdinalIgnoreCase))
							{
								UnprovisionedDevices.Add(Def);
							}
							else
							{
								// add user-specified devices at the start of the list so they are given priority
								UnprovisionedDevices.Insert(UserDeviceIndex++, Def);
							}
						}
					}
				}
			}
		}

		/// <summary>
		/// Adds the list of devices to our internal availability list
		/// </summary>
		/// <param name="InDevices"></param>
		public void RegisterDevices(IEnumerable<ITargetDevice> InDevices)
		{
			lock (LockObject)
			{
				AvailableDevices = AvailableDevices.Union(InDevices).ToList();
			}
		}

		/// <summary>
		/// Registers the provided device for availability
		/// </summary>
		/// <param name="Device"></param>
		/// <param name="Constraint"></param>
		public void RegisterDevice(ITargetDevice Device, UnrealDeviceTargetConstraint Constraint = null)
		{
			lock (LockObject)
			{
				if (AvailableDevices.Contains(Device))
				{
					throw new Exception("Device already registered!");
				}

				Constraints[Device] = Constraint ?? new UnrealDeviceTargetConstraint(Device.Platform.Value);
				UsedPlatforms.Add(Device.Platform);
				AvailableDevices.Add(Device);

				if (Log.IsVerbose)
				{
					Device.RunOptions = Device.RunOptions & ~CommandUtils.ERunOptions.NoLoggingOfRunCommand;
				}
			}
		}

		/// <summary>
		/// Run the provided function across all our devices until it returns false. Devices are provisioned on demand (e.g turned from info into an ITargetDevice)
		/// </summary>
		public void EnumerateDevices(UnrealTargetPlatform Platform, Func<ITargetDevice, bool> Predicate)
		{
			EnumerateDevices(new UnrealDeviceTargetConstraint(Platform), Predicate);
		}

		public void EnumerateDevices(UnrealDeviceTargetConstraint Constraint, Func<ITargetDevice, bool> Predicate)
		{
			lock (LockObject)
			{
				List<ITargetDevice> Selection = new List<ITargetDevice>();

				Log.Verbose(KnownLogEvents.Gauntlet_DeviceEvent, $"Enumerating devices for constraint {Constraint}");

				Log.Verbose(KnownLogEvents.Gauntlet_DeviceEvent, $"   Available devices:");
				AvailableDevices.ForEach(D => Log.Verbose(KnownLogEvents.Gauntlet_DeviceEvent, $"      {D.Platform}:{D.Name}"));

				Log.Verbose(KnownLogEvents.Gauntlet_DeviceEvent, $"   Unprovisioned devices:");
				UnprovisionedDevices.ForEach(D => Log.Verbose(KnownLogEvents.Gauntlet_DeviceEvent, $"      {D}"));

				// randomize the order of all devices that are of this platform
				var MatchingProvisionedDevices = AvailableDevices.Where(D => Constraint.Check(D)).ToList();
				var MatchingUnprovisionedDevices = UnprovisionedDevices.Where(D => Constraint.Check(D)).ToList();

				bool OutOfDevices = false;
				bool ContinuePredicate = true;

				do
				{
					// Go through all our provisioned devices to see if these fulfill the predicates
					// requirements

					ITargetDevice NextDevice = MatchingProvisionedDevices.FirstOrDefault();

					while (NextDevice != null && ContinuePredicate)
					{
						Log.Verbose(KnownLogEvents.Gauntlet_DeviceEvent, "Checking {DeviceName} against predicate", NextDevice.Name);
						MatchingProvisionedDevices.Remove(NextDevice);
						ContinuePredicate = Predicate(NextDevice);

						NextDevice = MatchingProvisionedDevices.FirstOrDefault();
					}

					if (ContinuePredicate)
					{
						// add more devices if possible
						OutOfDevices = MatchingUnprovisionedDevices.Count() == 0;

						DeviceDefinition NextDeviceDef = MatchingUnprovisionedDevices.FirstOrDefault();

						if (NextDeviceDef != null)
						{
							Log.Verbose(KnownLogEvents.Gauntlet_DeviceEvent, "Provisioning device {DeviceName} for the pool", NextDeviceDef.Name);

							// try to create a device. This can fail, but if so we'll just end up back here
							// on the next iteration
							ITargetDevice NewDevice = CreateAndRegisterDeviceFromDefinition(NextDeviceDef, Constraint);

							MatchingUnprovisionedDevices.Remove(NextDeviceDef);
							UnprovisionedDevices.Remove(NextDeviceDef);

							if (NewDevice != null)
							{
								MatchingProvisionedDevices.Add(NewDevice);
								Log.Verbose(KnownLogEvents.Gauntlet_DeviceEvent, "Added device {DeviceName} to pool", NewDevice.Name);
							}
							else
							{
								Log.Info(KnownLogEvents.Gauntlet_DeviceEvent, "Failed to provision {DeviceName}", NextDeviceDef.Name);
								// track this
								if (FailedProvisions.Contains(NextDeviceDef) == false)
								{
									FailedProvisions.Add(NextDeviceDef);
								}
							}
						}
						else
						{
							Log.Info(KnownLogEvents.Gauntlet_DeviceEvent, "Pool ran out of devices of type {Constraint}!", Constraint);
							OutOfDevices = true;
						}
					}
				} while (OutOfDevices == false && ContinuePredicate);
			}
		}

		/// <summary>
		/// Claim all devices in the provided list. Once reserved a device will not be seen by any code that
		/// calls EnumerateDevices
		/// </summary>
		/// <param name="DeviceList"></param>
		/// <returns></returns>
		public bool ClaimDevices(IEnumerable<ITargetDevice> DeviceList)
		{
			lock (LockObject)
			{
				// can reserve if not reserved...
				if (ClaimedDevices.Intersect(DeviceList).Count() > 0)
				{
					return false;
				}

				// remove these devices from the available list
				AvailableDevices = AvailableDevices.Where(D => DeviceList.Contains(D) == false).ToList();

				ClaimedDevices.AddRange(DeviceList);
				DeviceList.All(D => UsedPlatforms.Add(D.Platform));
			}
			return true;
		}

		public bool ReserveDevicesFromService(Dictionary<UnrealDeviceTargetConstraint, int> DeviceTypes, IDeviceReservationService.Options Options = null)
		{
			// Flatten the required devices into an enumerable
			List<UnrealDeviceTargetConstraint> RequiredDeviceConstraints = DeviceTypes
				.SelectMany(KVP => Enumerable.Repeat(KVP.Key, KVP.Value))
				.ToList();

			// Project each constraint into a supported service
			Dictionary<IDeviceReservationService, List<UnrealDeviceTargetConstraint>> ServiceMapping = new();
			foreach (UnrealDeviceTargetConstraint Constraint in RequiredDeviceConstraints)
			{
				foreach (IDeviceReservationService Service in ReservationServices)
				{
					if (Service.CanSupportDeviceConstraint(Constraint))
					{
						if (ServiceMapping.ContainsKey(Service))
						{
							ServiceMapping[Service].Add(Constraint);
						}
						else
						{
							ServiceMapping.Add(Service, new List<UnrealDeviceTargetConstraint> { Constraint });
						}
					}
				}
			}

			// Determine if any constraints do not have a service that can supply an appropriate device
			List<UnrealDeviceTargetConstraint> DevicesWithService = ServiceMapping.Values.SelectMany(Constraint => Constraint).ToList();
			List<UnrealDeviceTargetConstraint> DevicesLackingService = RequiredDeviceConstraints.Except(DevicesWithService).ToList();

			if (DevicesLackingService.Any())
			{
				string Message = "No enabled reservation services were capable of supporting the following constraints:\n";
				Message += string.Join("\n\t", DevicesLackingService);
				Message += "\nThe following reservation services were available:\n";
				Message += string.Join("\n\t", ReservationServices.Select(Service => Service.GetType().Name));
				Message += "\nEnsure your environment is configured to enable any necessary services.";
				throw new AutomationException(Message);
			}

			foreach (var ReservationPair in ServiceMapping)
			{
				if (!ReservationPair.Key.ReserveDevicesFromService(ReservationPair.Value, Options))
				{
					Log.Info("Failed to reserve all devices from service. See above log for details");
					return false;
				}
			}

			return true;
		}

		/// <summary>
		/// Report target device issue to service with given error message
		/// </summary>
		public void ReportDeviceError(ITargetDevice Device, string ErrorMessage)
		{
			if (TryGetDevicesReservationService(Device, out IDeviceReservationService Service))
			{
				Service.ReportDeviceError(Device.Name, ErrorMessage);
			}
			else
			{
				Log.Verbose("{Device} was not reserved by a service! Ignoring error report.");
			}
		}

		/// <summary>
		/// Release all devices in the provided list from our reserved list
		/// </summary>
		/// <param name="DeviceList">Devices to release</param>
		/// <param name="ServiceOptions">Options for releasing devices to a reservation service (where appropriate)</param>
		public void ReleaseDevices(IEnumerable<ITargetDevice> DeviceList, IDeviceReservationService.Options ServiceOptions = null)
		{
			if (DeviceList == null || !DeviceList.Any())
			{
				return;
			}

			lock (LockObject)
			{
				List<ITargetDevice> KeepList = new List<ITargetDevice>();
				foreach (ITargetDevice Device in DeviceList)
				{
					if (LocalDevices.Contains(Device))
					{
						DeviceConfigurationCache.Instance.RevertDeviceConfiguration(Device);
						KeepList.Add(Device);
					}
					else if (TryGetDevicesReservationService(Device, out IDeviceReservationService Service))
					{
						Service.ReleaseDevices([Device], ServiceOptions);
					}
					else
					{
						Log.Info("Attempted to release an unregistered device. Ensure the device is registered with DevicePool.Instance.RegisterDevice first.");
					}
				}

				// Remove any provisioned devices
				AvailableDevices = AvailableDevices.Where(Device => !DeviceList.Contains(Device)).ToList();
				ClaimedDevices = ClaimedDevices.Where(Device => !DeviceList.Contains(Device)).ToList();

				// But keep local devices available
				KeepList.ForEach(D => { AvailableDevices.Remove(D); AvailableDevices.Insert(0, D); });
			}
		}

		/// <summary>
		///	Checks whether device pool can accommodate requirements, optionally add service devices to meet demand
		/// </summary>
		public bool CheckAvailableDevices(Dictionary<UnrealDeviceTargetConstraint, int> RequiredDevices, IReadOnlyCollection<ProblemDevice> ProblemDevices = null, bool UseServiceDevices = true)
		{
			Dictionary<UnrealDeviceTargetConstraint, int> AvailableDeviceTypes = new Dictionary<UnrealDeviceTargetConstraint, int>();
			Dictionary<UnrealDeviceTargetConstraint, int> TotalDeviceTypes = new Dictionary<UnrealDeviceTargetConstraint, int>();

			// Do these "how many available" checks every time because the DevicePool provisions on demand so while it may think it has N machines,
			// some of them may fail to be provisioned and we could end up with none!

			// See how many of these types are in device pool (mostly to supply informative info if we can't meet these)

			foreach (var PlatformRequirement in RequiredDevices)
			{
				UnrealDeviceTargetConstraint Constraint = PlatformRequirement.Key;

				Func<ITargetDevice, bool> Validate = (ITargetDevice Device) =>
				{
					if (!Constraint.Check(Device))
					{
						return false;
					}

					if (ProblemDevices == null)
					{
						return true;
					}

					foreach (ProblemDevice PDevice in ProblemDevices)
					{
						if (PDevice.Platform == Device.Platform && PDevice.Name == Device.Name)
						{
							return false;
						}
					}

					return true;
				};

				AvailableDeviceTypes[Constraint] = DevicePool.Instance.GetAvailableDeviceCount(Constraint, Validate);
				TotalDeviceTypes[Constraint] = DevicePool.Instance.GetTotalDeviceCount(Constraint, Validate);


				Log.Verbose(KnownLogEvents.Gauntlet_DeviceEvent, "{Constraint}: {Platform} devices required. Total:{Total}, Available:{Available}",
					Constraint, PlatformRequirement.Value,
					TotalDeviceTypes[PlatformRequirement.Key], AvailableDeviceTypes[PlatformRequirement.Key]);
			}

			// get a list of any platforms where we don't have enough
			var TooFewTotalDevices = RequiredDevices.Where(KP => TotalDeviceTypes[KP.Key] < RequiredDevices[KP.Key]).Select(KP => KP.Key);
			var TooFewCurrentDevices = RequiredDevices.Where(KP => AvailableDeviceTypes[KP.Key] < RequiredDevices[KP.Key]).Select(KP => KP.Key);

			var Devices = TooFewTotalDevices.Concat(TooFewCurrentDevices);

			// Request devices from the service if we need them
			if (UseServiceDevices && (TooFewTotalDevices.Count() > 0 || TooFewCurrentDevices.Count() > 0))
			{
				Dictionary<UnrealDeviceTargetConstraint, int> DeviceCounts = new Dictionary<UnrealDeviceTargetConstraint, int>();
				Devices.ToList().ForEach(Platform => DeviceCounts[Platform] = RequiredDevices[Platform]);

				if (!ReserveDevicesFromService(DeviceCounts))
				{
					return false;
				}
			}
			else
			{
				// if we can't ever run then throw an exception
				if (TooFewTotalDevices.Count() > 0)
				{
					var MissingDeviceStrings = TooFewTotalDevices.Select(D => string.Format("Not enough devices of type {0} exist for test. ({1} required, {2} available)", D, RequiredDevices[D], AvailableDeviceTypes[D]));
					Log.Error(KnownLogEvents.Gauntlet_DeviceEvent, string.Join("\n", MissingDeviceStrings));
					throw new AutomationException("Not enough devices available");
				}

				// if we can't  run now then return false
				if (TooFewCurrentDevices.Count() > 0)
				{
					var MissingDeviceStrings = TooFewCurrentDevices.Select(D => string.Format("Not enough devices of type {0} available for test. ({1} required, {2} available)", D, RequiredDevices[D], AvailableDeviceTypes[D]));
					Log.Verbose(KnownLogEvents.Gauntlet_DeviceEvent, string.Join("\n", MissingDeviceStrings));
					return false;
				}
			}

			return true;
		}

		/// <summary>
		/// Created and registered a device from the provided definition
		/// </summary>
		/// <param name="Def"></param>
		/// <returns></returns>
		public ITargetDevice CreateAndRegisterDeviceFromDefinition(DeviceDefinition Def, UnrealDeviceTargetConstraint Constraint,
			IDeviceReservationService Service = null, IDeviceFactory Factory = null)
		{
			ITargetDevice NewDevice = null;

			if (Factory == null)
			{
				Factory = InterfaceHelpers.FindImplementations<IDeviceFactory>()
					.Where(F => F.CanSupportPlatform(Def.Platform))
					.FirstOrDefault();
			}

			if (Factory == null)
			{
				throw new AutomationException("No IDeviceFactory implementation that supports {0}", Def.Platform);
			}

			try
			{
				bool IsDesktop = Def.Platform != null && UnrealBuildTool.Utils.GetPlatformsInClass(UnrealPlatformClass.Desktop).Contains(Def.Platform!.Value);

				string ClientTempDir = GetCleanCachePath(Def);

				if (IsDesktop)
				{
					NewDevice = Factory.CreateDevice(Def.Name, ClientTempDir);
				}
				else
				{
					NewDevice = Factory.CreateDevice(Def.Address, ClientTempDir, Def.DeviceData);
				}

				if (NewDevice == null)
				{
					Log.Warning(KnownLogEvents.Gauntlet_DeviceEvent, "Failed to create device {DeviceName}. Device could not be connected.", Def.Name);
					return null;
				}

				if (NewDevice.IsAvailable == false)
				{
					Log.Info(KnownLogEvents.Gauntlet_DeviceEvent, "Assigned device {DeviceName} reports unavailable. Requesting a forced disconnect", NewDevice.Name);
					NewDevice.Disconnect(true);

					if (NewDevice.IsAvailable == false)
					{
						Log.Info(KnownLogEvents.Gauntlet_DeviceEvent, "Assigned device {DeviceName} still unavailable. Requesting a reboot", NewDevice.Name);
						NewDevice.Reboot();
					}
				}

				// Now validate if the kit meets the necessary requirements
				string Message = string.Empty;
				if (!TryValidateDeviceRequirements(NewDevice, ref Message))
				{
					if (!string.IsNullOrEmpty(Message))
					{
						if (Service != null)
						{
							Service.ReportDeviceError(Def.Name, Message);
						}
					}
					Log.Info("\nSkipping device.");
					return null;
				}

				lock (LockObject)
				{
					if (NewDevice != null)
					{
						RegisterDevice(NewDevice, Constraint);
					}
				}
			}
			catch (Exception Ex)
			{
				string WarningMessage = $"Failed to create device {Def.Name}. {Ex.Message}\n{Ex.StackTrace}";
				if (Ex is DeviceException)
				{
					if (Service != null)
					{
						Service.ReportDeviceError(Def.Name, WarningMessage);
					}
				}
				else
				{
					Log.Warning(KnownLogEvents.Gauntlet_DeviceEvent, WarningMessage);
				}
			}

			return NewDevice;
		}

		private void InitializeReservationServices()
		{
			// In order to avoid constructing an instance of the service to determine if it's enabled on any number of types we have to do a bit of generics reflection...
			foreach (Type ServiceType in InterfaceHelpers.FindTypes<IDeviceReservationService>(true, true))
			{
				MethodInfo Function = typeof(DevicePool).GetMethod("IsServiceEnabled", BindingFlags.Instance | BindingFlags.NonPublic);
				MethodInfo Generic = Function.MakeGenericMethod(ServiceType);
				object ReturnValue = Generic.Invoke(this, null);
				bool IsEnabled = (bool)ReturnValue;
				if (IsEnabled)
				{
					ReservationServices.Add(Activator.CreateInstance(ServiceType) as IDeviceReservationService);
				}
			}
		}

		private bool IsServiceEnabled<T>() where T : IDeviceReservationService
		{
			return T.Enabled;
		}

		/// <summary>
		/// Verifies if the provided TargetDevice meets requirements such as firmware, login status, settings, etc.
		/// </summary>
		/// <param name="Device">The device to validate</param>
		/// <param name="Message">The output message when failing to validate</param>
		/// <returns>True if the device matches the required specifications</returns>
		protected bool TryValidateDeviceRequirements(ITargetDevice Device, ref string Message)
		{
			IEnumerable<IDeviceValidator> Validators = InterfaceHelpers.FindImplementations<IDeviceValidator>(true).Where(Validator => Validator.bEnabled);
			if (!Validators.Any())
			{
				return true;
			}

			bool bInitiallyConnected = Device.IsConnected;

			Log.Info(KnownLogEvents.Gauntlet_DeviceEvent, "\nValidating requirements for {DeviceName}...", Device.Name);
			bool bSucceeded = true;
			List<string> MessageAggregate = new();

			foreach (IDeviceValidator Validator in Validators)
			{
				Log.Info("\nStarting validation for {Validator}", Validator);

				string ValidationMessage = string.Empty;
				if(!Validator.TryValidateDevice(Device, ref ValidationMessage))
				{
					Log.Info("Failed!");
					bSucceeded = false;
					if (!string.IsNullOrEmpty(ValidationMessage))
					{
						MessageAggregate.Add(ValidationMessage);
					}
				}
				else
				{
					Log.Info("Success!");
				}
			}

			if(!bSucceeded)
			{
				Message = string.Join("\n", MessageAggregate.Where(M => !string.IsNullOrEmpty(M)));
				Log.Info(KnownLogEvents.Gauntlet_DeviceEvent, "\nFailed to validate requirements on device {DeviceName}", Device.Name);
			}
			else
			{
				Log.Info(KnownLogEvents.Gauntlet_DeviceEvent, "\nAll validators passed, selecting device {DeviceName}\n", Device.Name);
			}

			// Most validators require establishing a connection to the device.
			// If we weren't originally connected, disconnect so the initial connection state can be cached during device reservation
			if(!bInitiallyConnected && Device.IsConnected)
			{
				Device.Disconnect();
			}

			return bSucceeded;
		}

		/// <summary>
		/// Explicitly release all device reservations
		/// </summary>
		private void ReleaseReservations()
		{
			foreach (IDeviceReservationService ReservationService in ReservationServices)
			{
				foreach (ITargetDevice Device in ReservationService.ReservedDevices)
				{
					AvailableDevices.Remove(Device);
					ClaimedDevices.Remove(Device);
				}

				ReservationService.ReleaseDevices(ReservationService.ReservedDevices);
			}
		}

		private void CleanupDevices(IEnumerable<UnrealTargetPlatform?> Platforms)
		{
			IEnumerable<IDeviceService> DeviceServices = InterfaceHelpers.FindImplementations<IDeviceService>();
			if (DeviceServices.Any())
			{
				foreach (UnrealTargetPlatform? Platform in Platforms)
				{
					IDeviceService DeviceService = DeviceServices.Where(D => D.CanSupportPlatform(Platform)).FirstOrDefault();
					if (DeviceService != null)
					{
						DeviceService.CleanupDevices();
					}
				}
			}
		}

		public bool TryGetDevicesReservationService(ITargetDevice Device, out IDeviceReservationService OutService)
		{
			OutService = null;

			foreach (IDeviceReservationService Service in ReservationServices)
			{
				if (Service.ReservedDevices.Contains(Device))
				{
					OutService = Service;
					return true;
				}
			}

			return false;
		}

		/// <summary>
		/// Construct a path to hold cache files and make sure it's properly cleaned
		/// </summary>
		private string GetCleanCachePath(DeviceDefinition InDeviceDefiniton)
		{
			if (!string.IsNullOrEmpty(UniformCache))
			{
				return UniformCache;
			}

			// Give the desktop platform a temp folder with its name under the device cache
			string SanitizedDeviceName = FileUtils.SanitizeFilename(InDeviceDefiniton.Name);
			string DeviceCache = Path.Combine(LocalTempDir, "DeviceCache");
			string PlatformCache = Path.Combine(DeviceCache, InDeviceDefiniton.Platform.ToString());
			string ClientCache = Path.Combine(PlatformCache, SanitizedDeviceName);

			// On Desktops, builds are installed in the device cache.
			// When using device reservation blocks, we don't want to fully clean the cache and lose previously installed builds.
			// If bRetainBuilds evaluates to true, it means we are in the second step or beyond in a device reservation block.
			// In this case we'll just delete the left over UserDir which should already have been emptied by UnrealSession.
			bool bRetainCache = Globals.Params.ParseParam("RetainCache");
			bool bRetainBuilds = SkipInstall && !FullClean;

			if(bRetainBuilds || bRetainCache)
			{
				Log.Info("Retaining build cache for device reservation block");

				DirectoryInfo UserDirectory = new(Path.Combine(ClientCache, "UserDir"));
				if(UserDirectory.Exists)
				{
					try
					{
						Log.Info("Cleaning stale user directory...");
						SystemHelpers.Delete(UserDirectory, true, true);
					}
					catch(Exception Ex)
					{
						throw new AutomationException("Failed to clean user directory {0}. This could result in improper artifact reporting. {1}", UserDirectory, Ex);
					}
				}
			}
			else
			{
				int CleanAttempts = 0;

				while (Directory.Exists(ClientCache))
				{
					DirectoryInfo ClientCacheDirectory = new(ClientCache);

					try
					{
						Log.Info("Cleaning stale client device cache...");
						SystemHelpers.Delete(ClientCacheDirectory, true, true);
					}
					catch (Exception Ex)
					{
						// If we fail to acquire the default client cache while using device reservation blocks,
						// we can't ensure future tests will have their cache directories mapped to the correct build location
						if (DeviceReservationBlock)
						{
							throw new AutomationException("Failed to clean default client device cache {0}. {1}", ClientCache, Ex);
						}

						// When not using device reservation blocks, we can just create a newly indexed directory for the client cache
						else
						{
							string Warning = "Failed to clean client device cache {Folder}. A newly indexed directory will be created instead. {Message}";
							Log.Warning(KnownLogEvents.Gauntlet_DeviceEvent, Warning, ClientCache, Ex.Message);

							ClientCache = Path.Combine(PlatformCache, $"{SanitizedDeviceName}_{++CleanAttempts}");
						}
					}
				}

				// create this path
				Log.Info("Client device cache set to {Directory}", ClientCache);
				Directory.CreateDirectory(ClientCache);
			}

			if (Globals.Params.ParseParam("UniformCache"))
			{
				UniformCache = ClientCache;
			}

			return ClientCache;
		}

		#region Legacy Implementation

		[Obsolete("Will be removed in a future release")]
		public string DeviceURL
		{
			get { return Legacy_DeviceURL; }
			set { Legacy_DeviceURL = value; }
		}

		/// <summary>
		/// Device reservation service URL
		/// </summary>
		private string Legacy_DeviceURL;

		[Obsolete("Misnomer, use ClaimDevices instead. Will be removed in a future release")]
		public void ReserveDevices(IEnumerable<ITargetDevice> Devices)
		{
			ClaimDevices(Devices);
		}

		[Obsolete("Will be removed in a future release. Use the single parameter overload instead.")]
		public bool ReserveDevicesFromService(string DeviceURL, Dictionary<UnrealDeviceTargetConstraint, int> DeviceTypes)
		{
			return ReserveDevicesFromService(DeviceTypes);
		}

		#endregion
	}
}