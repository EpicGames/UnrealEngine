// Copyright Epic Games, Inc. All Rights Reserved.

using EpicGames.Core;
using System;

namespace Gauntlet
{
	public class DeviceProfileValidator : IDeviceValidator
	{
		public bool bEnabled { get; }

		[AutoParam("")]
		public string DeviceProfile { get; set; }

		private const string DefaultNamespace = "Engine";

		public DeviceProfileValidator()
		{
			AutoParam.ApplyParamsAndDefaults(this, Globals.Params.AllArguments);
			bEnabled = !string.IsNullOrEmpty(DeviceProfile);
		}

		public bool TryValidateDevice(ITargetDevice Device, ref string Message)
		{
			if(!IDeviceValidator.EnsureConnection(Device, ref Message))
			{
				return false;
			}

			// Device profile comes in the format <namespace>.<profilename>. If no namespace is given, assume DefaultNamespace
			string Namespace;
			string Profile;
			if (DeviceProfile.Contains('.'))
			{
				int DotIndex = DeviceProfile.IndexOf('.');
				Namespace = DeviceProfile.Substring(0, DotIndex);
				Profile = DeviceProfile.Substring(DotIndex + 1);
			}
			else
			{
				Namespace = DefaultNamespace;
				Profile = DeviceProfile;
			}

			if(Device is IConfigurableDevice ConfigurableDevice)
			{
				try
				{
					// Create a configuration based on the device profile we requested
					PlatformConfigurationBase DesiredConfiguration = DeviceConfigurationCache.Instance.GetConfiguration(Device.Platform, Namespace, Profile);
					if (DesiredConfiguration == null)
					{
						Log.Warning("Failed to locate device profile '{Namespace}.{Profile}' for {Platform} doesn't exist", Namespace, Profile, Device.Platform);
						return false;
					}

					// Cache the existing configuration on the device
					PlatformConfigurationBase CurrentConfiguration = ConfigurableDevice.GetCurrentConfigurationSnapshot();
					if (CurrentConfiguration == null)
					{
						Message = "Failed to snapshot existing device configuration on device";
						Log.Info(KnownLogEvents.Gauntlet_DeviceEvent, Message);
						return false;
					}

					// Cache the existing configuration, we'll return to this state when we have finished with the device
					DeviceConfigurationCache.Instance.CacheConfigurationSnapshot(CurrentConfiguration);

					// Now apply the requested configuration
					return ConfigurableDevice.ApplyConfiguration(DesiredConfiguration);
				}
				catch(Exception Ex)
				{
					string ErrorMessage = $"Encountered an {Ex.GetType()} while while trying to configure device. {Ex}\n\t{Ex.StackTrace}";
					if (Ex is DeviceException)
					{
						Message = ErrorMessage;
						Log.Info(KnownLogEvents.Gauntlet_DeviceEvent, Message);
					}
					else
					{
						Log.Warning(ErrorMessage);
					}

					return false;
				}
			}
			else
			{
				Log.Verbose("{Platform} does not implement IConfigurableDevice, skipping device configuration.", Device.Platform.Value);
				return true;
			}
		}
	}
}
