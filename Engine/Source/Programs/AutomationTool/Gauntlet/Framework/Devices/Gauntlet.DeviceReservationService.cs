// Copyright Epic Games, Inc. All Rights Reserved.

using AutomationTool;
using AutomationTool.DeviceReservation;
using EpicGames.Core;
using EpicGames.Horde;
using EpicGames.Horde.Jobs;
using Microsoft.Extensions.DependencyInjection;
using System;
using System.Collections.Generic;
using System.Data;
using System.Linq;
using UnrealBuildTool;

#nullable enable

namespace Gauntlet
{
	/// <summary>
	/// An IDeviceReservationService provides a mechanism to obtain devices from an external service
	/// </summary>
	public interface IDeviceReservationService : IDisposable
	{
		public class Options
		{ }

		static virtual bool Enabled { get; }
		List<ITargetDevice> ReservedDevices { get; }
		bool CanSupportDeviceConstraint(UnrealDeviceTargetConstraint Constraint);
		bool ReserveDevicesFromService(IEnumerable<UnrealDeviceTargetConstraint> RequestedDevices) => ReserveDevicesFromService(RequestedDevices, null);
		bool ReserveDevicesFromService(IEnumerable<UnrealDeviceTargetConstraint> RequestedDevices, Options? Options);
		void ReleaseDevices(IEnumerable<ITargetDevice> Devices) => ReleaseDevices(Devices, null);
		void ReleaseDevices(IEnumerable<ITargetDevice> Devices, Options? Options);
		void ReportDeviceError(string DeviceName, string ErrorMessage);
	}

	/// <summary>
	/// Horde Service. Enabled by providing the base url via -DeviceURL= and pool via -DevicePool=
	/// </summary>
	public class HordeDeviceReservationService : IDeviceReservationService
	{
		public class Options : IDeviceReservationService.Options
		{
			/// <summary>
			/// If true, reservations will be force deleted.
			/// Force deleted reservations will also clean any associated reservation blocks regardless of future steps
			/// </summary>
			public bool bForceDelete { get; set; } = true;
		}

		/// <summary>
		/// Whether or not this reservation service is enabled
		/// </summary>
		public static bool Enabled
			=> (Horde.IsHordeJob || Globals.Params.ParseParam("EnableHordeDeviceReservations"))
			&& CommandUtils.ParseParam(Globals.Params.AllArguments, "DeviceURL")
			&& CommandUtils.ParseParam(Globals.Params.AllArguments, "DevicePool");

		/// <summary>
		/// List of devices reserved from this service
		/// </summary>
		public List<ITargetDevice> ReservedDevices { get; protected set; } = new List<ITargetDevice>();

		/// <summary>
		/// The base URL of the horde service to request devices from
		/// </summary>
		[AutoParam("")]
		public string DeviceURL { get; protected set; } = string.Empty;

		/// <summary>
		/// Name of the horde device pool to request devices from
		/// </summary>
		[AutoParamWithNames(Default: "", "DevicePool")]
		public string DevicePoolID { get; protected set; } = string.Empty;

		/// <summary>
		/// If true, the horde agent will have it's name appended as a required tag
		/// This is useful in instances where you want an agent to use a limited number of devices in a larger pool
		/// </summary>
		[AutoParam(false)]
		public bool UseAgentTag;

		/// <summary>
		/// HTTP client used to collect horde information
		/// </summary>
		private static HordeHttpClient? HordeClient = null;

		/// <summary>
		/// ID of the actively running horde job
		/// </summary>
		private static JobId HordeJobId = default;

		/// <summary>
		/// ID of the actively running horde step
		/// </summary>
		private static JobStepId HordeStepId = default;

		/// <summary>
		/// Endpoint for reservations
		/// </summary>
		private Uri ReservationServerUri;

		/// <summary>
		/// Device to reservation lookup
		/// </summary>
		private Dictionary<ITargetDevice, DeviceReservationAutoRenew> ServiceReservations = new Dictionary<ITargetDevice, DeviceReservationAutoRenew>();

		/// <summary>
		/// Target device info, private for reservation use
		/// </summary>
		private Dictionary<ITargetDevice, DeviceDefinition> ServiceDeviceInfo = new Dictionary<ITargetDevice, DeviceDefinition>();

		private Dictionary<ITargetDevice, bool> InitialConnectionState = new Dictionary<ITargetDevice, bool>();

		private bool? IsInstallStep
		{
			get
			{
				return DevicePool.IsInstallStep;
			}
			set
			{
				DevicePool.IsInstallStep = value;
			}
		}

		static HordeDeviceReservationService()
		{
			if (Enabled)
			{
				HordeClient = CommandUtils.ServiceProvider.GetRequiredService<IHordeClient>().CreateHttpClient();

				try
				{
					// Allow these to throw null
					string JobIdEnvVar = Environment.GetEnvironmentVariable("UE_HORDE_JOBID")!;
					HordeJobId = JobId.Parse(JobIdEnvVar);

					string StepIdEnvVar = Environment.GetEnvironmentVariable("UE_HORDE_STEPID")!;
					HordeStepId = JobStepId.Parse(StepIdEnvVar);
				}
				catch
				{
					Log.Verbose("Failed to parse Job or Step Id environment variable - some device reservation blocks may not be considered.");
					return;
				}

				try
				{
					GetJobBatchResponse Batch = GetBatchInfo(GetJobInfo());
					GetJobStepResponse Step = GetStepInfo(Batch);
					if (Step.Annotations != null && Step.Annotations.TryGetValue("DeviceReserve", out string? Value))
					{
						bool InstallStep = false;
						if (Value.Equals("Begin", StringComparison.InvariantCultureIgnoreCase))
						{
							Log.Verbose("Detected running step as the beginning of a reservation block. Any devices used will be cleaned");
							InstallStep = true;
						}
						else if (Value.Equals("Install", StringComparison.InvariantCultureIgnoreCase))
						{
							Log.Verbose("Detected running step as an explicit install step within a reservation block. Any devices used will be cleaned");
							InstallStep = true;
						}
						else if (Value.Equals("End", StringComparison.OrdinalIgnoreCase))
						{
							Log.Verbose("Detected running step as the final step within a reservation block.");
							DevicePool.DeviceReservationBlockFinalStep = true;
						}

						if (InstallStep)
						{
							DevicePool.IsInstallStep = true;
							DevicePool.DeviceReservationBlock = true;
							DevicePool.SkipInstall = false;
							DevicePool.FullClean = true;
						}
					}
					else
					{
						foreach (GetJobStepResponse PreviousStep in Batch.Steps)
						{
							if (PreviousStep.Id == Step.Id)
							{
								// Ignore steps that occur after this one
								break;
							}
							else if (PreviousStep.Annotations != null && PreviousStep.Annotations.TryGetValue("DeviceReserve", out Value) && Value.Equals("Begin", StringComparison.InvariantCultureIgnoreCase))
							{
								Log.Verbose("Detected running step is within a reservation block. Any devices used will skip installations");
								DevicePool.IsInstallStep = false;
								DevicePool.DeviceReservationBlock = true;
								DevicePool.SkipInstall = true;
								DevicePool.FullClean = false;
								break;
							}
						}

						if (Step.Id == Batch.Steps.Last().Id)
						{
							DevicePool.DeviceReservationBlockFinalStep = true;
						}
					}
				}
				catch (Exception Ex)
				{
					Log.Info("Encountered an {ExceptionType} when trying to determine if a Horde device reservation block is active. - some device reservation blocks may not be considered", Ex.GetType().Name);
					Log.Verbose("{Exception}", Ex.Message);
					return;
				}
			}
		}

		public HordeDeviceReservationService()
		{
			AutoParam.ApplyParamsAndDefaults(this, Globals.Params.AllArguments);

			if (!Uri.TryCreate(DeviceURL, UriKind.Absolute, out ReservationServerUri!))
			{
				throw new AutomationException("Failed to resolve \"{0}\" as a valid URI", DeviceURL);
			}
		}

		#region IDisposable
		private bool bDisposed;
		~HordeDeviceReservationService()
		{
			Dispose(false);
		}

		public void Dispose()
		{
			Dispose(true);
		}

		public void Dispose(bool bDisposing)
		{
			if (bDisposed)
			{
				return;
			}

			if (bDisposing)
			{
				ReleaseDevices(ReservedDevices);
			}

			bDisposed = true;
			GC.SuppressFinalize(this);
		}
		#endregion

		public virtual bool CanSupportDeviceConstraint(UnrealDeviceTargetConstraint Constraint)
		{
			// By default, do not support desktops. Can be overridden in a subtype
			UnrealTargetPlatform[] SupportedPlatforms = UnrealTargetPlatform.GetValidPlatforms()
				.Where(Platform => !Platform.IsInGroup(UnrealPlatformGroup.Desktop))
				.ToArray();

			if (Constraint.Platform == null || !SupportedPlatforms.Contains(Constraint.Platform.Value))
			{
				// If an unsupported device, we can't reserve it
				Log.Info(KnownLogEvents.Gauntlet_DeviceEvent, "Unable to reserve service device of type: {Type}", Constraint.Platform);
				return false;
			}

			if (!string.IsNullOrEmpty(Constraint.Model))
			{
				// if specific device model, we can't currently reserve it from (legacy) service
				if (DeviceURL.ToLower().Contains("deviceservice.epicgames.net"))
				{
					Log.Info(KnownLogEvents.Gauntlet_DeviceEvent, "Unable to reserve service device of model: {Model} on legacy service", Constraint.Model);
					return false;
				}
			}

			return true;
		}

		bool IDeviceReservationService.ReserveDevicesFromService(IEnumerable<UnrealDeviceTargetConstraint> RequestedConstraints, IDeviceReservationService.Options? Options) 
			=> ReserveDevicesFromService(RequestedConstraints, Options as Options);

		public virtual bool ReserveDevicesFromService(IEnumerable<UnrealDeviceTargetConstraint> RequestedConstraints, Options? Options = null)
		{
			// Ensure no duplicate requests of an explicit device
			HashSet<string> DeviceNames = new HashSet<string>();
			foreach (UnrealDeviceTargetConstraint Constraint in RequestedConstraints)
			{
				if (!string.IsNullOrEmpty(Constraint.DeviceName))
				{
					if (DeviceNames.Contains(Constraint.DeviceName))
					{
						Log.Info(KnownLogEvents.Gauntlet_DeviceEvent, "Attempted to make a reservation for multiple devices using the same device name {DeviceName}. This is not supported.", Constraint.DeviceName);
						return false;
					}

					DeviceNames.Add(Constraint.DeviceName);
				}
			}

			List<ITargetDevice> ScopeReservedDevices = new List<ITargetDevice>();
			foreach (UnrealDeviceTargetConstraint Constraint in RequestedConstraints)
			{
				Reservation.Tags Tags = Tags = Constraint.Tags;
				if (UseAgentTag)
				{
					// If someone is trying to repro locally without the proper horde envvars, this will throw
					// We can't guarantee to get the right device without an agent name, so this is preferred over silently reserving a default device
					string? AgentName = GetBatchInfo(GetJobInfo()).AgentId;
					if (!string.IsNullOrEmpty(AgentName))
					{
						Tags = Tags.Clone();
						Tags.AddTag(AgentName, Reservation.Tags.Type.Required);
					}
				}

				Reservation NewReservation = Reservation.Create(ReservationServerUri, [Constraint.FormatWithIdentifier()], TimeSpan.FromMinutes(10), RetryMax: 0, PoolID: DevicePoolID, DeviceName: Constraint.DeviceName, Tags);
				DeviceReservationAutoRenew DeviceReservation = new DeviceReservationAutoRenew(DeviceURL, NewReservation);

				if (DeviceReservation == null || DeviceReservation.Devices.Count != 1)
				{
					Log.Warning(KnownLogEvents.Gauntlet_DeviceEvent, "Unable to make device registration with constraint {Constraint}", Constraint);
					return false;
				}

				if (DevicePool.DeviceReservationBlock = DeviceReservation.InstallRequired != null)
				{
					// InstallRequired is true only for the first reservation attempt of the current step. Cache that value to avoid conflicting value on retry attempt.
					IsInstallStep = IsInstallStep != null ? IsInstallStep : DeviceReservation.InstallRequired == true;
					DevicePool.SkipInstall = DeviceReservation.InstallRequired == false && IsInstallStep == false;
					DevicePool.FullClean = !DevicePool.SkipInstall;
				}

				// Construct a definition from the reservation
				Device Device = DeviceReservation.Devices[0];
				DeviceDefinition DeviceDefinition = new DeviceDefinition()
				{
					Address = Device.IPOrHostName,
					Name = Device.Name,
					Platform = UnrealTargetPlatform.Parse(UnrealTargetPlatform.GetValidPlatformNames().First(Entry => Entry == Device.Type.Replace("-DevKit", "", StringComparison.OrdinalIgnoreCase))),
					DeviceData = Device.DeviceData,
					Model = Device.Model,
					Tags = Device.Tags
				};

				EPerfSpec PerfSpec = EPerfSpec.Unspecified;
				if (!string.IsNullOrEmpty(Device.PerfSpec) && !Enum.TryParse(Device.PerfSpec, true, out PerfSpec))
				{
					throw new AutomationException("Unable to convert perfspec '{0}' into an EPerfSpec", Device.PerfSpec);
				}
				DeviceDefinition.PerfSpec = PerfSpec;

				ITargetDevice TargetDevice = DevicePool.Instance.CreateAndRegisterDeviceFromDefinition(DeviceDefinition, Constraint, this);

				// If a device from service can't be added, fail reservation and cleanup devices
				if (TargetDevice == null)
				{
					// If some devices from reservation have been created, release them which will also dispose of reservation
					if (ScopeReservedDevices.Count > 0)
					{
						ReleaseDevices(ScopeReservedDevices);
					}

					// Cancel this reservation
					DeviceReservation.Dispose();

					Log.Info(KnownLogEvents.Gauntlet_DeviceEvent, "Unable to make device registration: device registration failed for {Platform}:{DeviceName}", DeviceDefinition.Platform, DeviceDefinition.Name);
					return false;
				}
				else
				{
					ScopeReservedDevices.Add(TargetDevice);
					ReservedDevices.Add(TargetDevice);
					ServiceDeviceInfo.Add(TargetDevice, DeviceDefinition);
					ServiceReservations.Add(TargetDevice, DeviceReservation);
					InitialConnectionState.Add(TargetDevice, TargetDevice.IsConnected);
				}
			}

			if (ScopeReservedDevices.Count == RequestedConstraints.Count())
			{
				return true;
			}

			Log.Info(KnownLogEvents.Gauntlet_DeviceEvent, "Unable to reserve all devices from service.");
			return false;
		}

		void IDeviceReservationService.ReleaseDevices(IEnumerable<ITargetDevice> Devices, IDeviceReservationService.Options? Options)
			=> ReleaseDevices(Devices, Options as Options);

		/// <summary>
		/// Release all devices in the provided list from our reserved list
		/// </summary>
		/// <param name="DeviceList"></param>
		public void ReleaseDevices(IEnumerable<ITargetDevice> DeviceList, Options? Options = null)
		{
			if (!DeviceList.Any())
			{
				return;
			}

			// Remove all these devices from our reserved list
			ReservedDevices = ReservedDevices.Except(DeviceList).ToList();

			List<ITargetDevice> ThrowDevices = new List<ITargetDevice>();
			foreach (ITargetDevice Device in DeviceList)
			{
				// Reset any state if necessary
				DeviceConfigurationCache.Instance.RevertDeviceConfiguration(Device);

				if (Device.IsConnected && !InitialConnectionState[Device])
				{
					Device.Disconnect();
				}

				// Unregister device
				if (ServiceReservations.TryGetValue(Device, out DeviceReservationAutoRenew? Reservation))
				{
					bool DisposeReservation = ServiceReservations.Count(Entry => Entry.Value == Reservation) == 1;

					// remove and dispose of device
					// @todo: add support for reservation modification on server (partial device release)
					ServiceReservations.Remove(Device);
					ServiceDeviceInfo.Remove(Device);
					InitialConnectionState.Remove(Device);

					if (Options != null && Options.bForceDelete)
					{
						Reservation.ForceDelete = true;
					}

					Device.Dispose();
					Reservation.Dispose();
				}
				else
				{
					ThrowDevices.Add(Device);
				}
			}

			if (ThrowDevices.Any())
			{
				// If a user explicitly calls a service's release function with an incorrect device, throw this exception
				string ExceptionMessage = "Attempted to release the following devices from a service that did not reserve them!";
				ExceptionMessage += "\n\t" + string.Join("\n\t", ThrowDevices.Select(Device => Device.Name));
				ExceptionMessage += "\nUse DevicePool.Instance.ReleaseDevices to avoid mistakingly releasing devices reserved from a different service";
				throw new AutomationException(ExceptionMessage);
			}
		}

		/// <summary>
		/// Report target device issue to service with given error message
		/// </summary>
		public virtual void ReportDeviceError(string DeviceName, string ErrorMessage)
		{
			// TargetDevice name is not always DeviceData name... need to try to resolve target device names
			ITargetDevice? MatchingDevice = null;
			foreach (ITargetDevice Device in ReservedDevices)
			{
				if (Device.Name.Equals(DeviceName, StringComparison.OrdinalIgnoreCase))
				{
					MatchingDevice = Device;
					break;
				}
			}

			if (MatchingDevice == null)
			{
				// No Target device, assume this is DeviceData name
				Reservation.ReportDeviceError(DeviceURL, DeviceName, ErrorMessage);
			}
			else
			{
				Reservation.ReportDeviceError(DeviceURL, ServiceDeviceInfo[MatchingDevice].Name, ErrorMessage);
			}
		}

		/// <summary>
		/// Returns a list of the tags on the device, if any
		/// </summary>
		/// <param name="Device">The device who's tags will be polled</param>
		/// <returns>An array containg all tags on the device</returns>
		public List<string> GetDeviceTags(ITargetDevice Device)
		{
			if (ServiceDeviceInfo.TryGetValue(Device, out DeviceDefinition? Definition))
			{
				return Definition.Tags ?? [];
			}

			return [];
		}

		/// <summary>
		/// Checks if a device has a tag
		/// </summary>
		/// <param name="Device">The device who's tags will be observed</param>
		/// <param name="Tag">Name of the tag to look for</param>
		/// <returns>True if the device has the provided Tag</returns>
		public bool DeviceHasTag(ITargetDevice Device, string Tag)
		{
			return GetDeviceTags(Device).Contains(Tag, StringComparer.OrdinalIgnoreCase);
		}

		/// <summary>
		/// Fetches information about the actively running job
		/// </summary>
		/// <returns>Information of the actively running job</returns>
		/// <exception cref="AutomationException">Throws when calling without the horde environment configured</exception>
		private static GetJobResponse GetJobInfo()
		{
			if (HordeClient == null)
			{
				throw new AutomationException("Failed to retrieve Job info because the horde client was not initialized." +
					"Ensure you're running in a horde environment before calling this function.");
			}

			return HordeClient.GetJobAsync(HordeJobId).Result;
		}

		/// <summary>
		/// Fetches information about the actively running job batch
		/// </summary>
		/// <param name="Job">Job the batch is contained in</param>
		/// <returns>Information of the actively running job batch</returns>
		/// <exception cref="AutomationException">Throws when the Job doesn't contain batches</exception>
		private static GetJobBatchResponse GetBatchInfo(GetJobResponse Job)
		{
			if (Job.Batches == null)
			{
				throw new AutomationException("Failed to retrieve the job's batch info because the batch info was null." +
					"Ensure the UE_HORDE_JOBID is set to an existing job's id that contains relevant batch info");
			}

			return Job.Batches
				.Where(Batch => Batch.Steps
				.Where(Step => Step.Id == HordeStepId)
				.FirstOrDefault()?.Id == HordeStepId)
				.First();
		}

		/// <summary>
		/// Fetches information about the actively running job step
		/// </summary>
		/// <param name="Batch">Batch the step is contained in</param>
		/// <returns>Information of the actively running job step</returns>
		private static GetJobStepResponse GetStepInfo(GetJobBatchResponse Batch)
		{
			return Batch.Steps.Where(Step => Step.Id == HordeStepId).First();
		}
	}
}