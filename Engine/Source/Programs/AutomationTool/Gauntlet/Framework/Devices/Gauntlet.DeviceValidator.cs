// Copyright Epic Games, Inc. All Rights Reserved.

using EpicGames.Core;

namespace Gauntlet
{
	/// <summary>
	/// IDeviceValidator provides the ability to verify a TargetDevice matches a set of requirements.
	/// They are particularly useful for ensuring development kits are in a desired state prior to a test.
	/// This can include, but is not limited to:
	///		- Proper firmware versions
	///		- Having an account signed in
	///		- Pushing a list of settings onto the device
	///	To add new DeviceValidators, simply create a concrete type that inherits from IDeviceValidator.
	/// Implement ValidateDevice() and determine whether this validator should be enabled within the constructor
	/// It can be especially helpful to toggle whether a validator is enabled via commandline arguments, for ease of opt-in
	/// </summary>
	public interface IDeviceValidator
	{
		/// <summary>
		/// If false, this validator will not be used when reserving devices for automated tests
		/// </summary>
		bool bEnabled { get; }

		/// <summary>
		/// Verifies if the device is valid in the context of the running program
		/// </summary>
		/// <param name="Device">The device to validate</param>
		/// <param name="Message">The output message when failing to validate</param>
		/// <returns>True if the device succeeded validation</returns>
		bool TryValidateDevice(ITargetDevice Device, ref string Message);

		/// <summary>
		/// Helper function that can be used to ensure a device is powered on/connected
		/// before attempting to run any validation checks
		/// </summary>
		/// <param name="Device"></param>
		/// <param name="Message">The output message when failing to validate</param>
		/// <returns></returns>
		protected static bool EnsureConnection(ITargetDevice Device, ref string Message)
		{
			if (Device == null)
			{
				Log.Warning("Cannot ensure connection for null device");
				return false;
			}

			if (!Device.IsOn)
			{
				if (!Device.PowerOn())
				{
					Message = "Failed to power on device";
					Log.Info(KnownLogEvents.Gauntlet_DeviceEvent, Message);
					return false;
				}
				else
				{
					Log.VeryVerbose(KnownLogEvents.Gauntlet_DeviceEvent, "Powered on device {DeviceName} while ensuring connection", Device.Name);
				}
			}

			if (!Device.IsConnected)
			{
				if (!Device.Connect())
				{
					Message = "Failed to connect to device";
					Log.Info(KnownLogEvents.Gauntlet_DeviceEvent, Message);
					return false;
				}
				else
				{
					Log.VeryVerbose(KnownLogEvents.Gauntlet_DeviceEvent, "Connected to device {DeviceName} while ensuring connection", Device.Name);
				}
			}

			if (!Device.IsAvailable)
			{
				Message = "Device not available";
				Log.Info(KnownLogEvents.Gauntlet_DeviceEvent, Message);
				return false;
			}

			return true;
		}
	}
}
