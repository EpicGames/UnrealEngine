// Copyright Epic Games, Inc. All Rights Reserved.

using System.Collections.Generic;
using System.Linq;
using UnrealBuildTool;

namespace Gauntlet
{
	/// <summary>
	/// Device reservation utility class.
	/// </summary>
	public class UnrealDeviceReservation
	{
		public List<ITargetDevice> ReservedDevices { get; protected set; } = new List<ITargetDevice>();
		protected List<ProblemDevice> ProblemDevices { get; private set; } = new List<ProblemDevice>();
		private static object LockObject = new object();

		private static readonly HordeDeviceReservationService.Options ForceReleaseOption = new HordeDeviceReservationService.Options()
		{
			bForceDelete = true
		};

		public bool TryReserveDevices(Dictionary<UnrealDeviceTargetConstraint, int> RequiredDeviceTypes, int ExpectedNumberOfDevices, bool bAllowPartialReservations = false)
		{
			// First, determine if this is a partial reservation. In the event a device is considered a problem device, don't discard all devices.
			// Instead, only discard the devices that have a problem.
			if (bAllowPartialReservations && ReservedDevices.Count > 0)
			{
				RequiredDeviceTypes = GetPartialReservationTypes(RequiredDeviceTypes);
				if (RequiredDeviceTypes.Count == 0)
				{
					// We've already reserved all the requested devices
					return true;
				}

				int NewExpectedCount = 0;
				foreach (var Pair in RequiredDeviceTypes)
				{
					NewExpectedCount += Pair.Value;
				}
				ExpectedNumberOfDevices = NewExpectedCount;
			}
			else
			{
				ReleaseDevices();
				ProblemDevices.Clear();
			}

			lock (LockObject)
			{
				// Ensure the device pool can support the request. This will reserve devices from services if it needs to
				if (!DevicePool.Instance.CheckAvailableDevices(RequiredDeviceTypes, ProblemDevices))
				{
					return false;
				}

				List<ITargetDevice> AcquiredDevices = new List<ITargetDevice>();
				List<ITargetDevice> SkippedDevices = new List<ITargetDevice>();

				// The pool now contains enough devices to support this reservation. 
				// For each platform, enumerate and select from the available devices as long as they match the predicate.
				// This will discard any devices that are considered unavailable or marked as a problem
				foreach (var PlatformReqKP in RequiredDeviceTypes)
				{
					UnrealDeviceTargetConstraint Constraint = PlatformReqKP.Key;
					UnrealTargetPlatform? Platform = Constraint.Platform;

					int NeedOfThisType = RequiredDeviceTypes[Constraint];
					DevicePool.Instance.EnumerateDevices(Constraint, Device =>
					{
						int HaveOfThisType = AcquiredDevices.Where(D => D.Platform == Device.Platform && Constraint.Check(Device)).Count();

						bool WeWant = NeedOfThisType > HaveOfThisType;

						if (WeWant)
						{
							bool Available = Device.IsAvailable;
							bool Have = AcquiredDevices.Contains(Device);

							bool Problem = ProblemDevices.Where(D => D.Name == Device.Name && D.Platform == Device.Platform).Count() > 0;

							Log.Verbose("Device {0}: Available:{1}, Have:{2}, HasProblem:{3}", Device.Name, Available, Have, Problem);

							if (Available
								&& Have == false
								&& Problem == false)
							{
								Log.Info("Acquiring device {0}", Device.Name);
								AcquiredDevices.Add(Device);
								HaveOfThisType++;
							}
							else
							{
								Log.Info("Skipping device {0}", Device.Name);
								SkippedDevices.Add(Device);
							}
						}

						// continue if we need more of this platform type
						return HaveOfThisType < NeedOfThisType;
					});
				}

				// Release any devices that were skipped
				DevicePool.Instance.ReleaseDevices(SkippedDevices, ForceReleaseOption);

				// Obtain connections to any required devices.
				EstablishDeviceConnections(AcquiredDevices);

				// Mark any devices we can't connect to as problems and release them.
				// Could be something grabbed them before us, could be that they are unresponsive in some way
				List<ITargetDevice> LostDevices = AcquiredDevices.Where(Device => !Device.IsConnected).ToList();
				if (LostDevices.Count > 0)
				{
					LostDevices.ForEach(Device => MarkProblemDevice(Device, "Agent lost connection to device."));
					AcquiredDevices = AcquiredDevices.Except(LostDevices).ToList();
					ReleaseProblemDevices();
				}

				// If we failed to acquire every device we needed, but support partial reservations, hold onto any devices we did manage to acquire
				if (AcquiredDevices.Count < ExpectedNumberOfDevices && AcquiredDevices.Count > 0)
				{
					if (bAllowPartialReservations)
					{
						if (ClaimDevices(AcquiredDevices))
						{
							ReservedDevices.AddRange(AcquiredDevices);
						}
					}
					else
					{
						DevicePool.Instance.ReleaseDevices(AcquiredDevices, ForceReleaseOption);
					}

					return false;
				}


				if (!ClaimDevices(AcquiredDevices))
				{
					return false;
				}

				ReservedDevices.AddRange(AcquiredDevices);
			}

			return true;
		}

		public void ReleaseDevices(bool bForceClearServiceReservations = false)
		{
			if ((ReservedDevices != null) && (ReservedDevices.Count() > 0))
			{
				foreach (ITargetDevice Device in ReservedDevices)
				{
					IDeviceUsageReporter.RecordEnd(Device.Name, Device.Platform, IDeviceUsageReporter.EventType.Device);
				}

				IDeviceReservationService.Options ServiceOptions = bForceClearServiceReservations && HordeDeviceReservationService.Enabled
					? ForceReleaseOption
					: null;
				DevicePool.Instance.ReleaseDevices(ReservedDevices, ServiceOptions);

				ReservedDevices.Clear();
			}
		}

		public IEnumerable<ITargetDevice> ReleaseProblemDevices()
		{
			List<ITargetDevice> Devices = new();
			bool bReservedDevices = ReservedDevices != null && ReservedDevices.Any();
			bool bProblemDevices = ProblemDevices != null && ProblemDevices.Any();

			if(bReservedDevices && bProblemDevices)
			{
				foreach(ProblemDevice Problem in ProblemDevices)
				{
					foreach(ITargetDevice Device in ReservedDevices)
					{
						if (Problem.Name == Device.Name && Problem.Platform == Device.Platform)
						{
							Devices.Add(Device);
						}
					}
				}

				DevicePool.Instance.ReleaseDevices(Devices);
				ProblemDevices.Clear();

				foreach (ITargetDevice Device in Devices)
				{
					ReservedDevices.Remove(Device);
					IDeviceUsageReporter.RecordEnd(Device.Name, Device.Platform, IDeviceUsageReporter.EventType.Device);
				}
			}
			return Devices;
		}

		public void MarkProblemDevice(ITargetDevice Device, string ErrorMessage)
		{
			if (ProblemDevices.Where(D => D.Name == Device.Name && D.Platform == Device.Platform).Count() > 0)
			{
				return;
			}

			// report device has a problem to the pool
			DevicePool.Instance.ReportDeviceError(Device, ErrorMessage);

			if (Device.Platform != null)
			{
				ProblemDevices.Add(new ProblemDevice(Device.Name, Device.Platform.Value));
			}
		}

		/// <summary>
		/// Converts a set of required devices into only what is necessary by analyzing the list of already reserved devices
		/// </summary>
		/// <param name="DeviceTypes">Request list</param>
		/// <returns>The difference in constraints/values between the request types and the currently reserved devices</returns>
		private Dictionary<UnrealDeviceTargetConstraint, int> GetPartialReservationTypes(Dictionary<UnrealDeviceTargetConstraint, int> DeviceTypes)
		{
			Dictionary<UnrealDeviceTargetConstraint, int> CurrentlyAvailableTypes = new();
			foreach (ITargetDevice Device in ReservedDevices)
			{
				UnrealDeviceTargetConstraint Constraint = DevicePool.Instance.GetConstraint(Device);
				if (CurrentlyAvailableTypes.ContainsKey(Constraint))
				{
					++CurrentlyAvailableTypes[Constraint];
				}
				else
				{
					CurrentlyAvailableTypes.Add(Constraint, 1);
				}
			}

			Dictionary<UnrealDeviceTargetConstraint, int> NewSet = new();
			foreach (var Pair in DeviceTypes)
			{
				if (CurrentlyAvailableTypes.ContainsKey(Pair.Key))
				{
					// If we have more devices already reserved than what is requested we don't need to reserve devices
					if (CurrentlyAvailableTypes[Pair.Key] < Pair.Value)
					{
						// We have at least 1 device with this constraint, but not enough to fufill the request. Request only what is needed for this constraint
						NewSet.Add(Pair.Key, Pair.Value - CurrentlyAvailableTypes[Pair.Key]);
					}
				}
				else
				{
					// No existing constraint, we don't have these devices
					NewSet.Add(Pair.Key, Pair.Value);
				}
			}

			return NewSet;
		}

		private void EstablishDeviceConnections(IEnumerable<ITargetDevice> Devices)
		{
			foreach (ITargetDevice Device in Devices)
			{
				if (Device.IsOn == false)
				{
					Log.Info("Powering on {0}", Device);
					Device.PowerOn();
				}
				else if (Globals.Params.ParseParam("reboot"))
				{
					Log.Info("Rebooting {0}", Device);
					Device.Reboot();
					// Force a re-login after a reboot to be sure it is ready.
					if (Globals.Params.ParseParam("VerifyLogin") && Device is IOnlineServiceLogin DeviceLogin)
					{
						Log.Info("Revalidate device login after reboot...");
						DeviceLogin.VerifyLogin();
					}
				}

				if (Device.IsConnected == false)
				{
					Log.Verbose("Connecting to {0}", Device);
					Device.Connect();
				}
			}
		}

		private bool ClaimDevices(IEnumerable<ITargetDevice> Devices)
		{
			bool bSuccess = DevicePool.Instance.ClaimDevices(Devices);
			if (!bSuccess)
			{
				Log.Warning("Attempted to claim the following devices which are already marked as claimed:\n{Devices}",
						string.Join("\n\t", Devices.Select(Device => Device.Name)));
			}
			return bSuccess;
		}
	}
}