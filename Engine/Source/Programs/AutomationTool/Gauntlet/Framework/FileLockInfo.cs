// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Diagnostics;
using System.Linq;
using System.Management;
using System.Runtime.InteropServices;

namespace FileLockInfo
{
	public static class Win32Processes
	{
		public static string GetCommandLine(Process process)
		{
			try
			{
				if (OperatingSystem.IsWindows())
				{
					using (ManagementObjectSearcher searcher = new ManagementObjectSearcher("SELECT CommandLine FROM Win32_Process WHERE ProcessId = " + process.Id))
					using (ManagementObjectCollection objects = searcher.Get())
					{
						return objects.Cast<ManagementBaseObject>().SingleOrDefault()?["CommandLine"]?.ToString();
					}
				}
				return string.Format("<error retrieving commandline: non-windows platform>");
			}
			catch (Exception Ex)
			{
				return string.Format("<error retrieving commandline: {0}>", Ex.Message);
			}			
		}

		/// <summary>
		/// Find out what process(es) have a lock on the specified file.
		/// </summary>
		/// <param name="path">Path of the file.</param>
		/// <returns>Processes locking the file</returns>
		/// <remarks>See also:
		/// http://msdn.microsoft.com/en-us/library/windows/desktop/aa373661(v=vs.85).aspx
		/// http://wyupdate.googlecode.com/svn-history/r401/trunk/frmFilesInUse.cs (no copyright in code at time of viewing)
		/// </remarks>
		public static List<Process> GetProcessesLockingFile(string path)
		{
			string[] resources = { path }; // Just checking on one resource.
			return GetProcessesLockingFiles(resources);
		}

		public static List<Process> GetProcessesLockingFiles(string[] FilePaths)
		{
			uint handle;
			string key = Guid.NewGuid().ToString();
			int res = RmStartSession(out handle, 0, key);

			if (res != 0) throw new Exception("Could not begin restart session.  Unable to determine file locker.");

			try
			{
				const int MORE_DATA = 234;
				uint pnProcInfoNeeded, pnProcInfo = 0, lpdwRebootReasons = RmRebootReasonNone;

				res = RmRegisterResources(handle, (uint)FilePaths.Length, FilePaths, 0, null, 0, null);

				if (res != 0) throw new Exception("Could not register resource.");

				//Note: there's a race condition here -- the first call to RmGetList() returns
				//      the total number of process. However, when we call RmGetList() again to get
				//      the actual processes this number may have increased.
				res = RmGetList(handle, out pnProcInfoNeeded, ref pnProcInfo, null, ref lpdwRebootReasons);

				if (res == MORE_DATA)
				{
					return EnumerateProcesses(pnProcInfoNeeded, handle, lpdwRebootReasons);
				}
				else if (res != 0) throw new Exception("Could not list processes locking resource. Failed to get size of result.");
			}
			finally
			{
				RmEndSession(handle);
			}

			return new List<Process>();
		}


		[StructLayout(LayoutKind.Sequential)]
		public struct RM_UNIQUE_PROCESS
		{
			public int dwProcessId;
			public System.Runtime.InteropServices.ComTypes.FILETIME ProcessStartTime;
		}

		const int RmRebootReasonNone = 0;
		const int CCH_RM_MAX_APP_NAME = 255;
		const int CCH_RM_MAX_SVC_NAME = 63;

		public enum RM_APP_TYPE
		{
			RmUnknownApp = 0,
			RmMainWindow = 1,
			RmOtherWindow = 2,
			RmService = 3,
			RmExplorer = 4,
			RmConsole = 5,
			RmCritical = 1000
		}

		[StructLayout(LayoutKind.Sequential, CharSet = CharSet.Unicode)]
		public struct RM_PROCESS_INFO
		{
			public RM_UNIQUE_PROCESS Process;

			[MarshalAs(UnmanagedType.ByValTStr, SizeConst = CCH_RM_MAX_APP_NAME + 1)] public string strAppName;

			[MarshalAs(UnmanagedType.ByValTStr, SizeConst = CCH_RM_MAX_SVC_NAME + 1)] public string strServiceShortName;

			public RM_APP_TYPE ApplicationType;
			public uint AppStatus;
			public uint TSSessionId;
			[MarshalAs(UnmanagedType.Bool)] public bool bRestartable;
		}

		[DllImport("rstrtmgr.dll", CharSet = CharSet.Unicode)]
		static extern int RmRegisterResources(uint pSessionHandle, uint nFiles, string[] rgsFilenames,
			uint nApplications, [In] RM_UNIQUE_PROCESS[] rgApplications, uint nServices,
			string[] rgsServiceNames);

		[DllImport("rstrtmgr.dll", CharSet = CharSet.Auto)]
		static extern int RmStartSession(out uint pSessionHandle, int dwSessionFlags, string strSessionKey);

		[DllImport("rstrtmgr.dll")]
		static extern int RmEndSession(uint pSessionHandle);

		[DllImport("rstrtmgr.dll")]
		static extern int RmGetList(uint dwSessionHandle, out uint pnProcInfoNeeded,
			ref uint pnProcInfo, [In, Out] RM_PROCESS_INFO[] rgAffectedApps,
			ref uint lpdwRebootReasons);

		private static List<Process> EnumerateProcesses(uint pnProcInfoNeeded, uint handle, uint lpdwRebootReasons)
		{
			var processes = new List<Process>(10);
			// Create an array to store the process results
			var processInfo = new RM_PROCESS_INFO[pnProcInfoNeeded];
			var pnProcInfo = pnProcInfoNeeded;

			// Get the list
			var res = RmGetList(handle, out pnProcInfoNeeded, ref pnProcInfo, processInfo, ref lpdwRebootReasons);

			if (res != 0) throw new Exception("Could not list processes locking resource.");
			for (int i = 0; i < pnProcInfo; i++)
			{
				try
				{
					processes.Add(Process.GetProcessById(processInfo[i].Process.dwProcessId));
				}
				catch (ArgumentException) { } // catch the error -- in case the process is no longer running
			}
			return processes;
		}
	}
}