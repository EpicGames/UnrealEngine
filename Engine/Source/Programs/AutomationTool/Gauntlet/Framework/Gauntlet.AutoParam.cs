// Copyright Epic Games, Inc. All Rights Reserved.

using AutomationTool;
using System;
using System.Collections.Generic;
using System.Linq;
using System.Reflection;

namespace Gauntlet
{

	public interface IAutoParamNotifiable
	{
		void ParametersWereApplied(string[] Params);
	};


	/// <summary>
	/// An attribute that can be used to apply commandline options to fields or properties. 
	/// 
	/// Simply tag properties or fields with the CommandLineOption, a name, and a default value, then 
	/// call CommandLineOption.Apply(obj, args) where args is a list of -switches or -key=value pairs
	/// 
	/// The main constraint is that your object type must be convertable from a string
	/// 
	/// </summary>
	[AttributeUsage(AttributeTargets.Field | AttributeTargets.Property)]
	public class AutoParam : System.Attribute
	{
		/// <summary>
		/// Default value
		/// </summary>
		protected object Default;

		/// <summary>
		/// Names that can refer to this param
		/// </summary>
		public string[] OptionNames { get; protected set; }

		/// <summary>
		/// Constructor that takes nothing. Param option should be -MemberName or -MemberName=value.
		/// Members with no matching param will be left as-is.
		/// </summary>
		public AutoParam()
		{
			this.OptionNames = null;
			this.Default = null;
		}

		/// <summary>
		/// Constructor that takes an array of of potential argument names, e.g. {"build","builds"}
		/// Members with no matching param will be left as-is.
		/// </summary>
		/// <param name="OptionNames"></param>
		protected AutoParam(params string[] OptionNames)
		{
			this.OptionNames = OptionNames;
			this.Default = null;
		}

		/// <summary>

		/// <summary>
		/// Constructor that takes a default argument to use if no param is specified. Param option should be -MemberName or -MemberName=value.
		/// Members with no matching param will be set to 'Default'
		/// </summary>
		/// <param name="Default"></param>
		public AutoParam(object Default)
		{
			this.OptionNames = null;
			this.Default = Default;
		}

		/// <summary>
		/// Constructor that takes an array of of potential argument names, e.g. {"build","builds"}
		/// Members with no matching param will be set to 'Default'
		/// </summary>
		/// <param name="Default"></param>
		/// <param name="OptionNames"></param>
		protected AutoParam(object Default, params string[] OptionNames)
		{
			this.OptionNames = OptionNames;
			this.Default = Default;
		}

		

		/// <summary>
		/// Checks whether Args contains a -Param statement, if so returns true else
		/// checks if the passed argument is different from true or 1
		/// </summary>
		/// <param name="Param"></param>
		/// <param name="Args"></param>
		/// <returns></returns>
		static protected object CoerceToBool(string Param, string[] Args)
		{
			foreach (string Arg in Args)
			{
				string StringArg = Arg;

				if (StringArg.StartsWith("-"))
				{
					StringArg = Arg.Substring(1);
				}

				if (StringArg.ToString().Equals(Param, StringComparison.InvariantCultureIgnoreCase))
				{
					return true;
				}

				if (StringArg.StartsWith($"{Param}="))
				{
					StringArg = StringArg.Substring(Param.Length + 1);
					return !StringArg.Equals("false", StringComparison.InvariantCultureIgnoreCase) && !StringArg.Equals("0");
				}
			}
			return null;
		}

		/// <summary>
		/// Checks Args for a -param=value statement and either returns value or the
		/// provided default
		/// </summary>
		/// <param name="Param"></param>
		/// <param name="ParamType"></param>
		/// <param name="Args"></param>
		/// <returns></returns>
		static protected object ParseAndCoerceParam(string Param, Type ParamType, string[] Args)
		{
			if (!Param.EndsWith("="))
			{
				Param += "=";
			}
			foreach (string Arg in Args)
			{
				string StringArg = Arg;

				if (StringArg.StartsWith("-"))
				{
					StringArg = Arg.Substring(1);
				}

				if (StringArg.StartsWith(Param, StringComparison.InvariantCultureIgnoreCase))
				{
					string StringVal = StringArg.Substring(Param.Length);

					if (ParamType.IsEnum)
					{
						var AllValues = Enum.GetValues(ParamType).Cast<object>();
						var Enums = AllValues.Where(P => string.Equals(StringVal, P.ToString(), StringComparison.OrdinalIgnoreCase));

						if (Enums.Count() == 0)
						{
							throw new AutomationException("AutoParam could not convert param {0} to enum of type {1}", StringVal, ParamType);
						}

						return Enums.First();
					}
					else
					{
						return Convert.ChangeType(StringVal, ParamType);
					}
				}
			}

			return null;
		}

		/// <summary>
		/// Returns true if this type is considered a simple primitive (there is IsClass in c# but no IsStruct :()
		/// </summary>
		/// <param name="type"></param>
		/// <returns></returns>
		static bool IsSimple(Type type)
		{
			return type.IsPrimitive
			  || type.IsEnum
			  || type.Equals(typeof(string))
			  || type.Equals(typeof(decimal));
		}

		/// <summary>
		/// Returns true if this type inherits from IEnumerable<>
		/// </summary>
		/// <param name="Type">Underlying type of the object, e.g. IEnumerable<string></param>
		/// <param name="ElementType">Enclosed type of the generic IEnumerable, e.g. string</param>
		/// <returns></returns>
		private static bool IsEnumerableType(Type Type, out Type ElementType)
		{
			ElementType = null;

			if (Type.IsArray)
			{
				ElementType = Type.GetElementType();
				return true;
			}
			else if (Type.IsGenericType)
			{
				Type[] GenericArgs = Type.GetGenericArguments();
				if (GenericArgs.Length == 1)
				{
					Type EnumerableType = typeof(IEnumerable<>).MakeGenericType(GenericArgs[0]);
					if (EnumerableType.IsAssignableFrom(Type))
					{
						ElementType = GenericArgs[0];
						return true;
					}
				}
			}

			return false;
		}

		/// <summary>
		/// Call to process all CommandLineOption attributes on an objects members and set them based on the
		/// provided argument list
		/// </summary>
		/// <param name="Obj"></param>
		/// <param name="Args"></param>
		/// <param name="ApplyDefaults"></param>
		protected static void ApplyParamsAndDefaultsInternal(object Obj, string[] Args, bool ApplyDefaults, char EnumerableSeparator = ',')
		{
			// get all field and property members
			var Fields = Obj.GetType().GetFields(BindingFlags.Public | BindingFlags.NonPublic | BindingFlags.Instance);
			var Properties = Obj.GetType().GetProperties(BindingFlags.Public | BindingFlags.NonPublic | BindingFlags.Instance);

			var AllMembers = Fields.Cast<MemberInfo>().Concat(Properties);

			foreach (var Member in AllMembers)
			{
				Type MemberType = null;

				// Get the type of the member (note - this is not Member.Type!)
				if (Member is PropertyInfo)
				{
					MemberType = ((PropertyInfo)Member).PropertyType;
				}
				else if (Member is FieldInfo)
				{
					MemberType = ((FieldInfo)Member).FieldType;
				}

				// Go through all attributes
				foreach (object Attrib in Member.GetCustomAttributes(true))
				{
					// Locate AutoParam
					if (Attrib is AutoParam)
					{
						AutoParam Opt = Attrib as AutoParam;
						
						// If the attribute had names provided use them, else use the name of the variable
						string[] ParamNames = (Opt.OptionNames != null && Opt.OptionNames.Length > 0) ? Opt.OptionNames : [Member.Name];

						// Save the default
						object DefaultValue = Opt.Default;
						object NewValue = null;

						// Go through all params used to refer to this member
						foreach (string Name in ParamNames)
						{
							if (IsSimple(MemberType))
							{
								if (MemberType == typeof(bool))
								{
									// if default is a bool then try to coerce to a boolean
									NewValue = CoerceToBool(Name, Args);
								}
								else
								{
									// for all other types try to parse out the value
									NewValue = ParseAndCoerceParam(Name, MemberType, Args);
								}
							}
							else if (IsEnumerableType(MemberType, out Type ElementType))
							{
								string StringValue = ParseAndCoerceParam(Name, typeof(string), Args) as string;
								if (!string.IsNullOrEmpty(StringValue))
								{
									NewValue = ConstructEnumerableFromString(Member, MemberType, ElementType, StringValue, EnumerableSeparator);
								}
							}
							else
							{
								NewValue = ConstructComplexObject(Args, Member, MemberType, Obj, ApplyDefaults, EnumerableSeparator);
							}
							
							// Stop as soon as we find something
							// TODO support concatenation of enumerables?
							if (NewValue != null)
							{
								break;
							}
						}
		
						if (NewValue == null && ApplyDefaults)
						{
							// No commandline arguments were found that matched the name(s) of this AutoParam. Apply the defaults
							ApplyDefaultValue(Opt, Member, MemberType, Obj, EnumerableSeparator);
						}
						else if (NewValue != null)
						{
							// Value was overriden by commandline argument. Set the new member value
							SetMemberValue(Member, Obj, NewValue);
						}
					}
				}
			}

			IAutoParamNotifiable ParamNotifable = Obj as IAutoParamNotifiable;

			if (ParamNotifable != null)
			{
				ParamNotifable.ParametersWereApplied(Args);
			}
		}

		public static void ApplyDefaults(object Obj, char EnumerableSeparator = ',')
		{
			ApplyParamsAndDefaultsInternal(Obj, new string[0], true, EnumerableSeparator);
		}

		public static void ApplyParams(object Obj, string[] Args, char EnumerableSeparator = ',')
		{
			ApplyParamsAndDefaultsInternal(Obj, Args, false, EnumerableSeparator);
		}

		public static void ApplyParamsAndDefaults(object Obj, string[] Args, char EnumerableSeparator = ',')
		{
			ApplyParamsAndDefaultsInternal(Obj, Args, true, EnumerableSeparator);
		}

		private static void SetMemberValue(MemberInfo Member, object Obj, object Value)
		{
			if (Member is PropertyInfo)
			{
				((PropertyInfo)Member).SetValue(Obj, Value);
			}
			else if (Member is FieldInfo)
			{
				((FieldInfo)Member).SetValue(Obj, Value);
			}
		}

		private static void ApplyDefaultValue(AutoParam Param, MemberInfo Member, Type MemberType, object Obj, char EnumerableSeparator = ',')
		{
			object DefaultValue = Param.Default;

			if (IsEnumerableType(MemberType, out Type ElementType) && DefaultValue is string DefaultString)
			{
				DefaultValue = ConstructEnumerableFromString(Member, MemberType, ElementType, DefaultString, EnumerableSeparator);
			}

			if (DefaultValue != null && DefaultValue.GetType() != MemberType)
			{
				throw new AutomationException("AutoParam Default Value for member {0} is type {1}, not {2}", Member.Name, DefaultValue.GetType(), MemberType);
			}

			if (DefaultValue != null)
			{
				SetMemberValue(Member, Obj, DefaultValue);
			}
		}

		private static object ConstructComplexObject(string[] Args, MemberInfo Member, Type MemberType, object Obj, bool bApplyDefaults, char EnumerableSeparator = ',')
		{
			object ComplexObject = null;

			// Check if the object has already been constructed
			if (Member is PropertyInfo)
			{
				ComplexObject = ((PropertyInfo)Member).GetValue(Obj);
			}
			else if (Member is FieldInfo)
			{
				ComplexObject = ((FieldInfo)Member).GetValue(Obj);
			}

			// If null create a new one (e.g. a new instance of a struct);
			if (ComplexObject == null)
			{
				try
				{
					ComplexObject = Activator.CreateInstance(MemberType);
				}
				catch
				{
					throw new AutomationException("AutoParam was unable to construct an instance of {0}. Add a default constructor to the class {0}!", MemberType);
				}
			}

			// Recurse into the object applying autoparam values
			ApplyParamsAndDefaultsInternal(ComplexObject, Args, bApplyDefaults, EnumerableSeparator);

			return ComplexObject;
		}

		private static object ConstructEnumerableFromString(MemberInfo Member, Type MemberType, Type ElementType, string Value, char EnumerableSeparator = ',')
		{
			if (string.IsNullOrEmpty(Value))
			{
				return null;
			}
			else
			{
				// Split and trim entries
				string[] SplitValues = Value.Split(EnumerableSeparator, StringSplitOptions.RemoveEmptyEntries | StringSplitOptions.TrimEntries);

				// Construct an array of the element type which will be converted to the underlying enumerable type
				Array ConvertedArray = Array.CreateInstance(ElementType, SplitValues.Length);
				for (int Index = 0; Index < SplitValues.Length; Index++)
				{
					if (IsSimple(ElementType))
					{
						if (ElementType.IsEnum)
						{
							ConvertedArray.SetValue(Enum.Parse(ElementType, SplitValues[Index], true), Index);
						}
						else
						{
							ConvertedArray.SetValue(Convert.ChangeType(SplitValues[Index], ElementType), Index);
						}
					}
					else
					{
						throw new AutomationException("Member {0} of type {1} uses an invalid element type {2}. AutoParam only supports enumerables of simple types (int, string, etc)", Member.Name, MemberType, ElementType);
					}
				}

				object ConstructedEnumerable = null;

				// Convert the array to the appropriate collection type
				if (MemberType.IsArray)
				{
					ConstructedEnumerable = ConvertedArray;
				}
				else if (MemberType.IsGenericType)
				{
					Type GenericTypeDef = MemberType.GetGenericTypeDefinition();

					// Handle common list/collection interfaces and types
					if (GenericTypeDef == typeof(List<>)
						|| MemberType == typeof(IList<>).MakeGenericType(ElementType)
						|| MemberType == typeof(ICollection<>).MakeGenericType(ElementType)
						|| MemberType == typeof(IEnumerable<>).MakeGenericType(ElementType))
					{
						Type EnumerableType = typeof(List<>).MakeGenericType(ElementType);
						ConstructedEnumerable = Activator.CreateInstance(EnumerableType, [ConvertedArray]);
					}
					else
					{
						// Try to create an instance using a constructor that accepts IEnumerable<T>
						try
						{
							ConstructedEnumerable = Activator.CreateInstance(MemberType, [ConvertedArray]);
						}
						catch
						{
							throw new AutomationException("AutoParam is unable to create an instance of {0} from an array. Ensure it has a constructor accepting IEnumerable<{1}> if you wish to use AutoParam.", MemberType, ElementType);
						}
					}
				}

				return ConstructedEnumerable;
			}
		}

		/*public static string[] GetParams(object Obj)
		{
			List<string> CopiedParams = new List<string>();

			var Fields = Obj.GetType().GetFields(BindingFlags.Public | BindingFlags.Instance);

			var Properties = Obj.GetType().GetProperties(BindingFlags.Public | BindingFlags.Instance);

			var AllMembers = Fields.Cast<MemberInfo>().Concat(Properties);

			foreach (var Member in AllMembers)
			{
				foreach (object Attrib in Member.GetCustomAttributes(true))
				{
					if (Attrib is AutoParam)
					{
						AutoParam Opt = Attrib as AutoParam;

						string ParamName = string.IsNullOrEmpty(Opt.Name) ? Member.Name : Opt.Name;

						object ParamValue = null;

						if (Member is PropertyInfo)
						{
							ParamValue = ((PropertyInfo)Member).GetValue(Obj);
						}
						else if (Member is FieldInfo)
						{
							ParamValue = ((FieldInfo)Member).GetValue(Obj);
						}

						if (Opt.Default.GetType() == typeof(bool))
						{
							if ((bool)ParamValue)
							{
								CopiedParams.Add(ParamName);
							}							
						}
						else
						{
							CopiedParams.Add(string.Format("{0}={1}", ParamName, ParamValue));
						}
					}
				}
			}

			return CopiedParams.ToArray();
		}*/
	}

	[AttributeUsage(AttributeTargets.Field | AttributeTargets.Property)]
	public class AutoParamWithNames : AutoParam
	{
		public AutoParamWithNames(object Default, params string[] OptionNames)  :
			base(Default, OptionNames)
		{
		}

		public AutoParamWithNames(params string[] OptionNames) :
			base(OptionNames)
		{
		}
	}
}
