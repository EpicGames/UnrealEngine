// Copyright Epic Games, Inc. All Rights Reserved.

namespace Gauntlet
{
	public class TestException : System.Exception
	{
		public TestException(string Msg)
				: base(Msg)
		{
		}

		public TestException(string Message, System.Exception InnerException)
			: base(Message, InnerException)
		{
		}

		public TestException(string Format, params object[] Args)
				: base(string.Format(Format, Args))
		{
		}
	}

	public class NonCriticalTestException : TestException
	{
		public NonCriticalTestException(string Msg) : base(Msg)
		{
		}

		public NonCriticalTestException(string Message, System.Exception InnerException)
			: base(Message, InnerException)
		{
		}

		public NonCriticalTestException(string Format, params object[] Args)
			: base(string.Format(Format, Args))
		{
		}
	}

	public class DeviceException : TestException
	{
		public DeviceException(string Msg)
				: base(Msg)
		{
		}

		public DeviceException(string Message, System.Exception InnerException)
			: base(Message, InnerException)
		{
		}

		public DeviceException(string Format, params object[] Args)
				: base(Format, Args)
		{
		}
	}

    /// <summary>
	/// Exception class to be used for errors that stop the test case, but are not specifically a
	/// failure of the behavior being tested.  For instance, setup issues or system/environment problems.
	/// </summary>
	public class InterruptTestException : TestException
    {
        public InterruptTestException(string Msg) : base(Msg)
        {
        }

        public InterruptTestException(string Message, System.Exception InnerException)
            : base(Message, InnerException)
        {
        }
    }
}
