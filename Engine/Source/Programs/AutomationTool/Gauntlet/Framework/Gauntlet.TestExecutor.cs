// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.IO;
using System.Linq;
using System.Net;
using System.Text.RegularExpressions;
using System.Threading;
using System.Threading.Tasks;
using EpicGames.Core;
using Gauntlet.Utils;

namespace Gauntlet
{
	public class TestExecutorOptions
	{
		[AutoParamWithNames(1, "iterations", "repeat")]
		public int TestIterations;

		[AutoParam(false)]
		public bool StopOnError;

		[AutoParam(false)]
		public bool NoTimeout;

		[AutoParam(300)]
		public int Wait;

		[AutoParam(1)]
		public int Parallel;

		[AutoParam(true)]
		public bool DeferReports;

		/// <summary>
		/// Optional delay in seconds to wait between test iterations. Primarily used to allow devices to cool down
		/// when benchmarking to mitigate thermal throttling. Default is 0 (no additional delay beyond standard crash processing time).
		/// Users can tune this value based on their specific device thermal characteristics and observed behavior.
		/// </summary>
		[AutoParam(0)]
		public int TestIterationCooldownSeconds;

		/// <summary>
		/// Maximum number of seconds to wait for test start threads to join when a cancel is signalled
		/// If this time elapses, the thread is abandoned and expected to close on it's own.
		/// The most common case of this happening is during an installation to a devkit in UnrealSession
		/// -------
		/// TODO: InstallBuild should be given a cancellation token so we're not forced to wait or rely on platform dependent
		/// implementations to handle cancellations in the middle of a synchronous sdk command call
		/// </summary>
		[AutoParam(3 * 60)]
		public int StartingThreadShutdownLimitSeconds;
	}

	/// <summary>
	/// Class that is manages the creation and execution of one or more tests
	/// </summary>
	public class TestExecutor
	{
		class TestExecutionInfo
		{
			public enum ExecutionResult
			{
				NotStarted,
				TimedOut,
				Passed,
				Failed,
				Skipped,
				Interrupted
			}

			public ITestNode		TestNode;
			public DateTime			FirstReadyCheckTime;
			public DateTime			TimeSetupBegan;
			public DateTime			TimeSetupEnded;
			public DateTime			TimeTestEnded;
			public ExecutionResult	Result;
			public TestResult		FinalResult;
			public string			CancellationReason;
			
			/// <summary>
			/// Time the test had to wait before running
			/// </summary>
			public TimeSpan			WaitDuration { get { return (TimeSetupBegan - FirstReadyCheckTime); } }

			/// <summary>
			/// Time the test had to wait before running
			/// </summary>
			public TimeSpan			SetupDuration { get { return (TimeSetupEnded - TimeSetupBegan); } }

			/// <summary>
			/// Time the test took to run
			/// </summary>
			public TimeSpan			TestDuration { get { return (TimeTestEnded - TimeSetupEnded); } }

			/// <summary>
			/// Creates TestExecutionInfo instance from an ITestNode. Sets FirstReadyCheckTime, copies the node
			/// and sets CancellationReason to empty string.
			/// </summary>
			/// <param name="InNode"></param>
			public TestExecutionInfo(ITestNode InNode)
			{
				FirstReadyCheckTime = TimeSetupBegan = TimeSetupEnded = TimeTestEnded = DateTime.MinValue;
				TestNode = InNode;
				CancellationReason = string.Empty;
			}

			public override string ToString()
			{
				return TestNode.ToString();
			}
		};

		private List<TestExecutionInfo> RunningTests;

		private DateTime CurrentTestStartTime;

		private int CurrentTestPass;

		TestExecutorOptions Options;

		protected string BuildCommandThatLaunchedExecutor;

		public bool IsRunning { get; private set; }
		public bool IsCancelled { get; private set; }
		protected bool HaveReceivedPostAbort { get; private set; }

		/// <summary>
		/// Constructor that fills in some member variables
		/// @Param the BuildCommand that was used to start this Test Executor.
		/// Useful to log out or know where this test executor came from.
		/// </summary>
		public TestExecutor(string InBuildCommand)
		{
			BuildCommandThatLaunchedExecutor = InBuildCommand.Split(".").Last();
			RunningTests = new List<TestExecutionInfo>();
		}

		public void Dispose()
		{
		}

		/// <summary>
		/// Executes the provided tests. Currently tests are executed synchronously
		/// </summary>
		/// <param name="InOptions"></param>
		/// <param name="RequiredTests"></param>
		public bool ExecuteTests(TestExecutorOptions InOptions, IEnumerable<ITestNode> RequiredTests)
		{
			Options = InOptions;

			Log.Info("Preparing to start {Number} automation test(s)", RequiredTests.Count());

			// install a cancel handler so we can stop parallel-for gracefully
			Action CancelHandler = delegate ()
			{
				Log.Info("Cancelling Tests");
				IsCancelled = true;
			};

			Action PostCancelHandler = delegate ()
			{
				HaveReceivedPostAbort = true;
			};

			Globals.AbortHandlers.Add(CancelHandler);
			Globals.PostAbortHandlers.Add(PostCancelHandler);

			CurrentTestPass = 0;

			IsRunning = true;

			int MaxParallelTasks = 0;
			int MaxStartingTasks = 0;

			// sort by priority
			if (Options.Parallel > 1)
			{
				RequiredTests = RequiredTests.OrderBy(Node => Node.Priority);
			}

			Dictionary<string, int> TestIterationsPassed = new Dictionary<string, int>();
			Dictionary<string, int> TestIterationsInterrupted = new Dictionary<string, int>();
            Dictionary<string, int> TestIterationsFailed = new Dictionary<string, int>();
            Dictionary<string, int> TestIterationsPassedWithWarnings = new Dictionary<string, int>();

			// Clean up local log files
			if (Directory.Exists(ProcessUtils.LocalLogsPath))
			{
				SystemHelpers.Delete(new DirectoryInfo(ProcessUtils.LocalLogsPath), true, true);
			}

			for (CurrentTestPass = 0; CurrentTestPass < Options.TestIterations; CurrentTestPass++)
			{
				// do not start a pass if canceled
				if (IsCancelled)
				{
					break;
				}

				if (CurrentTestPass > 0)
				{
					// if repeating tests wait a little bit. If there was a crash CR might still be
					// processing things.
					const int CrashProcessingDelayTimeMs = 10000;

					// Apply optional device cooldown delay to mitigate thermal throttling between benchmark iterations.
					// The delay is user-configurable based on device thermal characteristics.
					// Clamp to valid range [0, 9999] to prevent negative values and overflow.
					const int MaxCooldownSeconds = 9999;
					int CooldownSeconds = Math.Clamp(Options.TestIterationCooldownSeconds, 0, MaxCooldownSeconds);
					int TestIterationCooldownMs = CooldownSeconds * 1000;
					if (CooldownSeconds > 0)
					{
						Log.Info("Cooling down for {CooldownSeconds} seconds", CooldownSeconds);
					}

					Thread.Sleep(Math.Max(CrashProcessingDelayTimeMs, TestIterationCooldownMs));
				}

				DateTime StartPassTime = DateTime.Now;

				Log.Info("Starting test iteration {Index} of {Total}", CurrentTestPass + 1, Options.TestIterations);

				// Tests that we want to run
				List<TestExecutionInfo> PendingTests = RequiredTests.Select(N => new TestExecutionInfo(N)).ToList();

				// Tests that are in the process of starting
				List<TestExecutionInfo> StartingTests = new List<TestExecutionInfo>();

				List<Thread> StartingTestThreads = new List<Thread>();

				// Tests that have been started and we're ticking/checking
				List<TestExecutionInfo> RunningTests = new List<TestExecutionInfo>();

				// Completed tests
				List<TestExecutionInfo> CompletedTests = new List<TestExecutionInfo>();

				DateTime LastUpdateMsg = DateTime.MinValue;
				DateTime LastReadyCheck = DateTime.MinValue;
				DateTime LastStatusUpdateTime = DateTime.MinValue;

				const double ReadyCheckPeriod = 30.0;
				const double StatusUpdatePeriod = 60.0;

				while (CompletedTests.Count() < RequiredTests.Count() && IsCancelled == false)
				{
					Monitor.Enter(Globals.MainLock);

					int SecondsRunning = (int)(DateTime.Now - StartPassTime).TotalSeconds;

					int InProgressCount = RunningTests.Count() + StartingTests.Count();

					double TimeSinceLastReadyCheck = (DateTime.Now - LastReadyCheck).TotalSeconds;

					// Are any tests ready to run?
					if (InProgressCount < Options.Parallel
						&& PendingTests.Count() > 0
						&& TimeSinceLastReadyCheck >= ReadyCheckPeriod)
					{
						TestExecutionInfo TestToStart = null;

						List<ITestNode> TestsFailingToStart = new List<ITestNode>();

						// find a node that can run, and
						// find the first test that can run....
						for (int i = 0; i < PendingTests.Count(); i++)
						{
							TestExecutionInfo NodeInfo = PendingTests[i];
							ITestNode Node = NodeInfo.TestNode;

							bool IsTestReady = false;

							try
							{
								IsTestReady = Node.IsReadyToStart();
							}
							catch (System.Exception ex)
							{
								if (ex.InnerException != null && ex.InnerException is WebException
									&& (ex.InnerException as WebException).Response != null
									&& ((ex.InnerException as WebException).Response as HttpWebResponse).StatusCode == HttpStatusCode.Conflict)
								{
									Log.Info(ex.ToString());
									// In case of conflict server response, let the test execution loop handles it (resolved through timeout option).
								}
								else
								{
									Log.Error(KnownLogEvents.Horde_BuildHealth_Ignore, "Test {Name} threw an exception during ready check. Ex: {Exception}", Node, ex);
									Node.AddTestEvent(new UnrealTestEvent(EventSeverity.Error, "Test Failed to Start", new List<string> { ex.Message }));
									PendingTests[i] = null;
									NodeInfo.TimeSetupBegan = NodeInfo.TimeSetupEnded = NodeInfo.TimeTestEnded = DateTime.Now;
									Horde.ReportInvalidatingIssue(Horde.GauntletHordeStepMessage.FailedToStart, "The test encountered a failure on startup.");
									NodeInfo.Result = TestExecutionInfo.ExecutionResult.Interrupted;
									CompletedTests.Add(NodeInfo);
								}
							}

							if (IsTestReady)
							{
								// if ready then take it and stop looking
								TestToStart = NodeInfo;

								if (NodeInfo.FirstReadyCheckTime == DateTime.MinValue)
								{
									NodeInfo.FirstReadyCheckTime = DateTime.Now;
								}
								break;
							}
							else
							{
								// track the time that this test should have been able to run due to no other tests
								// consuming resources (at least locally...)
								// TODO - what about the situation where no tests can run so all FirstCheck times are set, but 
								// then a test starts and consumes all resources?
								if (RunningTests.Count() == 0 && StartingTests.Count() == 0)
								{
									if (NodeInfo.FirstReadyCheckTime == DateTime.MinValue)
									{
										NodeInfo.FirstReadyCheckTime = DateTime.Now;
									}

									double TimeWaiting = (DateTime.Now - NodeInfo.FirstReadyCheckTime).TotalSeconds;
									if (TimeWaiting >= Options.Wait)
									{
										string Pool = string.IsNullOrEmpty(Globals.DevicePoolId)? string.Empty : $" from {Globals.DevicePoolId} pool";
										Log.Error(KnownLogEvents.Gauntlet_DeviceEvent, "Test {TestName} has been waiting to run resource-free for {Time:00} seconds{Pool}. Removing from wait list", Node, TimeWaiting, Pool);
										Node.AddTestEvent(new UnrealTestEvent(EventSeverity.Error, "Insufficient devices found", new List<string> {string.Format("Test {0} was unable to find enough devices after trying for {1:00} seconds.", Node, TimeWaiting), "This is not a test-related failure."}));
										PendingTests[i] = null;
										NodeInfo.TimeSetupBegan = NodeInfo.TimeSetupEnded = NodeInfo.TimeTestEnded = DateTime.Now;
										NodeInfo.Result = TestExecutionInfo.ExecutionResult.NotStarted;
										Horde.ReportInvalidatingIssue(Horde.GauntletHordeStepMessage.NotEnoughDevices, "The test was unable to secure enough devices to run.");
										CompletedTests.Add(NodeInfo);
									}
								}
							}
						}

						// remove anything we nulled
						PendingTests = PendingTests.Where(T => T != null).ToList();

						if (TestToStart != null)
						{
							Log.Info("Test {Name} is ready to run", TestToStart);

							PendingTests.Remove(TestToStart);
							StartingTests.Add(TestToStart);

							// StartTest is the only thing we do on a thread because it's likely to be the most time consuming
							// as build are copied so will get the most benefit from happening in parallel
							Thread StartThread = new Thread(() =>
							{
								Thread.CurrentThread.IsBackground = true;

								// start the test, this also fills in the pre/post start times
								bool Started = StartTest(TestToStart, CurrentTestPass, Options.TestIterations);

								lock (Globals.MainLock)
								{
									if (Started == false)
									{
										TestToStart.TimeSetupEnded = TestToStart.TimeTestEnded = DateTime.Now;
										TestToStart.Result = TestExecutionInfo.ExecutionResult.Interrupted;
										CompletedTests.Add(TestToStart);
										Log.Error(KnownLogEvents.Horde_BuildHealth_Ignore, "Test {Name} failed to start", TestToStart);
									}
									else
									{
										RunningTests.Add(TestToStart);
									}

									StartingTests.Remove(TestToStart);
									StartingTestThreads.Remove(Thread.CurrentThread);
								}
							});

							if (StartingTests.Count > MaxStartingTasks)
							{
								MaxStartingTasks = StartingTests.Count;
							}

							// track the thread and start it
							StartingTestThreads.Add(StartThread);
							StartThread.Start();
						}
						else
						{
							// don't check for a while as querying kits for availability can be expensive
							LastReadyCheck = DateTime.Now;
						}
					}

					// Tick all running tests
					foreach (TestExecutionInfo TestInfo in RunningTests)
					{
						// TickTest contains logic for determining run time, timeouts, cancellations, and many other
						// parts of the test process. If overriding TickTest in your Test class, be sure to call base.TickTest.
						TestResult Result = TickTest(TestInfo);

						// invalid = no result yet
						if (Result == TestResult.Invalid)
						{
							TimeSpan RunningTime = DateTime.Now - CurrentTestStartTime;

							if ((SecondsRunning % 60) == 0)
							{
								Log.Verbose("Test {Name} is still running. {Elapsed:00} seconds elapsed, will timeout in {Max:00} seconds",
									TestInfo,
									RunningTime.TotalSeconds,
									TestInfo.TestNode.MaxDuration - RunningTime.TotalSeconds);

								LastUpdateMsg = DateTime.Now;
							}
						}
						else
						{
							TestInfo.TimeTestEnded = DateTime.Now;
							switch (Result)
							{
								case TestResult.Passed:
									{
										TestInfo.Result = TestExecutionInfo.ExecutionResult.Passed;
										break;
									}
								case TestResult.Failed:
									{
										TestInfo.Result = TestExecutionInfo.ExecutionResult.Failed;
										break;
									}
								default:
									{
										// Default to "Interrupt"... if the test actually Failed, then the test case should explicitly say so
										TestInfo.Result = TestExecutionInfo.ExecutionResult.Interrupted;
										break;
									}
							}
							CompletedTests.Add(TestInfo);
						}
					}

					// remove any tests that were completed
					RunningTests = RunningTests.Where(R => CompletedTests.Contains(R) == false).ToList();

					if ((DateTime.Now - LastStatusUpdateTime).TotalSeconds >= StatusUpdatePeriod)
					{
						LastStatusUpdateTime = DateTime.Now;
						Log.Info("Status: Completed:{Completed}, Running:{Running}, Starting: {Starting}, Waiting:{Waiting}",
							CompletedTests.Count(), RunningTests.Count(), StartingTests.Count(), PendingTests.Count());
					}

					if (InProgressCount > MaxParallelTasks)
					{
						MaxParallelTasks = RunningTests.Count();
					}

					// Release our global lock before we loop
					Monitor.Exit(Globals.MainLock);

					// sleep a while before we tick our running tasks again
					Thread.Sleep(500);
				}

				if (IsCancelled)
				{
					DateTime StartTime = DateTime.Now;
					Log.Info("Cleaning up pending and running tests.");
					while (HaveReceivedPostAbort == false)
					{
						Thread.Sleep(500);
						double Elapsed = (DateTime.Now - StartTime).TotalSeconds;

						if (Elapsed >= 5)
						{
							Log.Error(KnownLogEvents.Horde_BuildHealth_Ignore, "Giving up waiting for tests after {Elapsed:00} seconds", Elapsed);
							break;
						}
					}

					TimeSpan ThreadTimeout = TimeSpan.FromSeconds(Options.StartingThreadShutdownLimitSeconds);
					Parallel.ForEach(StartingTestThreads, Thread =>
					{
						Log.Info("Attempting to join test start thread: {ThreadName} | {ThreadId}", Thread.Name, Thread.ManagedThreadId);
						if (!Thread.Join(ThreadTimeout))
						{
							Log.Warning(KnownLogEvents.Horde_BuildHealth_Ignore, "Failed to join the thread {Thread} | {ThreadId} after {Timeout} seconds.",
								Thread.Name, Thread.ManagedThreadId, ThreadTimeout.TotalSeconds);
						}
					});

					foreach (TestExecutionInfo TestInfo in StartingTests)
					{
						Log.Info("Forcing pending test {Name} to run CleanupTest", TestInfo.TestNode.Name);
						TestInfo.TestNode.CleanupTest();
						CompletedTests.Add(TestInfo);
					}

					foreach (TestExecutionInfo TestInfo in RunningTests)
					{
						Log.Info("Ticking test {Name} to cancel", TestInfo.TestNode.Name);
						TestResult Res = TickTest(TestInfo);
						CompletedTests.Add(TestInfo);

						if (Res != TestResult.Failed)
						{
							Log.Warning(KnownLogEvents.Horde_BuildHealth_Ignore, "Ticking of cancelled test {Name} returned {Result}", TestInfo.TestNode.Name, Res);
						}
					}
				}
				else
				{
					TimeSpan PassDuration = DateTime.Now - StartPassTime;

					int InterruptCount = 0;
                    int FailedCount = 0;
                    int TestCount = CompletedTests.Count;

					CompletedTests.ForEach(T =>
					{
						TimeSpan TimeWaiting = T.FirstReadyCheckTime - T.TimeSetupBegan;
						TimeSpan SetupTime = T.TimeSetupEnded - T.TimeSetupBegan;
						TimeSpan TestDuration = T.TimeTestEnded - T.TimeSetupEnded;

						// status msg, kept uniform to avoid spam on notifiers (ie. don't include timestamps, etc) 
						string Msg = string.Format("Test {0} {1}", T.TestNode, T.Result);

                        bool TestHadFailures = T.Result == TestExecutionInfo.ExecutionResult.Failed;
                        bool TestHadInterrupts = !TestHadFailures && T.Result != TestExecutionInfo.ExecutionResult.Passed && T.Result != TestExecutionInfo.ExecutionResult.Skipped;
						bool TestHadWarnings = T.TestNode.GetWarnings().Any();

						if (TestHadFailures)
						{
							FailedCount++;
						}
                        if (TestHadInterrupts)
                        {
                            InterruptCount++;
                        }

                        // increment counts for each test
                        if (!TestIterationsPassed.ContainsKey(T.TestNode.Name))
						{
							TestIterationsPassed[T.TestNode.Name] = 0;
							TestIterationsInterrupted[T.TestNode.Name] = 0;
                            TestIterationsFailed[T.TestNode.Name] = 0;
                            TestIterationsPassedWithWarnings[T.TestNode.Name] = 0;
						}

						if (TestHadInterrupts)
						{
							TestIterationsInterrupted[T.TestNode.Name]++;
						}
                        else if (TestHadFailures)
                        {
                            TestIterationsFailed[T.TestNode.Name]++;
                        }
                        else if (TestHadWarnings)
						{
							TestIterationsPassedWithWarnings[T.TestNode.Name]++;
						}
						else
						{
							TestIterationsPassed[T.TestNode.Name]++;
						}

						Log.Info(Msg);

						// log test timing to info
						Log.Info("Test Time: {Duration:mm\\:ss} (Waited:{Waited:mm\\:ss}, Setup:{Setup:mm\\:ss})", TestDuration, TimeWaiting, SetupTime);

					});

					if (Options.Parallel > 1)
					{
						Log.Info("MaxParallelTasks: {Count}", MaxParallelTasks);
						Log.Info("MaxStartingTasks: {Count}", MaxStartingTasks);
					}

					// report all tests
					ReportMainSummary(CurrentTestPass + 1, Options.TestIterations, PassDuration, CompletedTests);

					if ((FailedCount > 0 || InterruptCount > 0) && Options.StopOnError)
					{
						break;
					}
				}
			} // foreach pass

			int TotalTests = RequiredTests.Count() * Options.TestIterations;
            int InterruptTestCount = TestIterationsInterrupted.Values.Sum();
            int FailedTestCount = TestIterationsFailed.Values.Sum();

			// show details for multi passes				
			if (Options.TestIterations > 1)
			{
				MarkdownBuilder MB = new MarkdownBuilder();

				MB.HorizontalLine();					

				MB.Paragraph(string.Format("All iterations completed. {0} of {1} executed tests completed without error", TotalTests - (FailedTestCount + InterruptTestCount), TotalTests));

				List<string> Lines = new List<string>();
				foreach (ITestNode Test in RequiredTests)
				{
					int IterationsWithInterrupts = TestIterationsInterrupted[Test.Name];
                    int IterationsWithFailures = TestIterationsFailed[Test.Name];
                    int IterationsWithWarnings = TestIterationsPassedWithWarnings[Test.Name];
					int IterationsWithPasses = TestIterationsPassed[Test.Name];				

					Lines.Add(string.Format("{0}: {1} Iterations, {2} Passed, {3} Passed with Warnings, {4} Failed, {5} Interrupts", 
						Test.Name, Options.TestIterations, IterationsWithPasses, IterationsWithWarnings, IterationsWithFailures, IterationsWithInterrupts));
				}

				MB.UnorderedList(Lines);

                if (InterruptTestCount > 0)
                {
                    MB.Paragraph(string.Format("Error: {0} of {1} executed tests had Interrupts.", InterruptTestCount, TotalTests));
                }
                if (FailedTestCount > 0)
				{
					MB.Paragraph(string.Format("Error: {0} of {1} executed tests failed.", FailedTestCount, TotalTests));
				}

				// Always report the number of passed tests
				MB.Paragraph(string.Format("{0} total tests passed.", TotalTests - (InterruptTestCount + FailedTestCount)));

				MB.HorizontalLine();

				Log.Info(MB.ToString());
			}			

			IsRunning = false;

			Globals.AbortHandlers.Remove(CancelHandler);
			Globals.PostAbortHandlers.Remove(PostCancelHandler);
			
			return InterruptTestCount == 0 && FailedTestCount == 0 && !IsCancelled;
		}

		/// <summary>
		/// Executes a single test
		/// </summary>
		/// <param name="TestInfo">Test to execute</param>
		/// <param name="Pass"></param>
		/// <param name="NumPasses"></param>
		/// <returns></returns>
		private bool StartTest(TestExecutionInfo TestInfo, int Pass, int NumPasses)
		{
			string Name = TestInfo.TestNode.Name;

			Log.Info("Starting Test {TestName}", TestInfo);

			try
			{
				TestInfo.TimeSetupBegan = DateTime.Now;
				if (TestInfo.TestNode.StartTest(Pass, NumPasses))
				{
					CurrentTestStartTime = TestInfo.TimeSetupEnded = DateTime.Now;
					Log.Info("Launched test {TestName} at {StartTime}", Name, TestInfo.TimeSetupEnded.ToString("h:mm:ss"));
					return true;
				}
			}
			catch (Exception Ex)
			{
				Log.Error(KnownLogEvents.Horde_BuildHealth_Ignore, "Test {Name} threw an exception during launch. Skipping test. Ex: {Exception}", Name, Ex);
			}

			return false;			
		}

		/// <summary>
		/// Report the summary for a single test
		/// </summary>
		/// <param name="TestInfo"></param>
		/// <returns></returns>
		void ReportTestSummary(TestExecutionInfo TestInfo)
		{
			Log.SuspendSanitization();

			string Summary = TestInfo.TestNode.GetTestSummary();

			// Show summary
			Summary.Split('\n').ToList().ForEach(L => {
				if (L.Contains("Error: "))
				{
					Log.Error(KnownLogEvents.Gauntlet_TestEvent, " " + L);
				}
				else if (L.Contains("Warning: "))
				{
					Log.Warning(KnownLogEvents.Gauntlet_TestEvent, " " + L);
				}
				else
				{
					Log.Info(" " + L);
				}
			});

			Log.ResumeSanitization();

			// list warnings/errors if the test wants that
			if (TestInfo.TestNode.LogWarningsAndErrorsAfterSummary)
			{
				TestInfo.TestNode.GetErrors().ToList().ForEach(E => Log.Error(KnownLogEvents.Gauntlet_TestEvent, E));

				TestInfo.TestNode.GetWarnings().ToList().ForEach(E => Log.Warning(KnownLogEvents.Gauntlet_TestEvent, E));
			}

			// display the final result
			if (TestInfo.FinalResult != TestResult.Passed)
			{
				Log.Info("{TestInfo} result={Result}", TestInfo, TestInfo.FinalResult);
				if (string.IsNullOrEmpty(TestInfo.CancellationReason) == false)
				{
					Log.Info("\tReason: {Reason}", TestInfo.CancellationReason);
				}				
			}
			else
			{
				if (TestInfo.TestNode.GetWarnings().Any())
				{
					Log.Info("{TestInfo} result={Result} with warnings", TestInfo, TestInfo.FinalResult);
				}
				else
				{
					Log.Info("{TestInfo} result={Result}", TestInfo, TestInfo.FinalResult);
				}
			}

			// display how the test could be ran locally
			string RunLocalString = TestInfo.TestNode.GetRunLocalCommand(BuildCommandThatLaunchedExecutor);
			Log.Info("How to run locally: ({0})", RunLocalString);
			
			// Also, showing the arguments each in its row, so it is easier to read.
			string FormattedArgs = Regex.Replace(RunLocalString, @" (?=[^-]*(-|$))", "\n\t");
			Log.Info("Arguments used:\n\t{0}", FormattedArgs);
		}

		/// <summary>
		/// Reports the summary of this pass, including a sumamry for each test
		/// </summary>
		/// <param name="CurrentPass"></param>
		/// <param name="NumPasses"></param>
		/// <param name="Duration"></param>
		/// <param name="AllInfo"></param>
		/// <returns></returns>
		void ReportMainSummary(int CurrentPass, int NumPasses, TimeSpan Duration, IEnumerable<TestExecutionInfo> AllInfo)
		{

			MarkdownBuilder MB = new MarkdownBuilder();

			int TestCount = AllInfo.Count();
			int FailedCount = AllInfo.Where(T => T.FinalResult != TestResult.Passed).Count();
			int WarningCount = AllInfo.Where(T => T.TestNode.GetWarnings().Any()).Count();

			var SortedInfo = AllInfo;

			// sort our tests by failed/warning/ok
			if (FailedCount > 0 || WarningCount > 0)
			{
				SortedInfo = AllInfo.OrderByDescending(T =>
				{
					if (T.FinalResult != TestResult.Passed)
					{
						return 10;
					}

					if (T.TestNode.GetWarnings().Any())
					{
						return 5;
					}

					return 0;
				});
			}

			// write all test results at once
			if (Options.DeferReports)
			{
				// write each tests full summary
				foreach (TestExecutionInfo Info in SortedInfo)
				{
					ReportTestSummary(Info);
				}
			}

			// only show pass info for multiple passes
			if (NumPasses > 1)
			{
				Log.Info("Completed test pass {Index} of {Total}.", CurrentPass, NumPasses);
			}

			// only show count of passed/failed etc for multiple test
			if (TestCount > 1)
			{
				MB.H2(string.Format("{0} of {1} Tests Passed in {2:mm\\:ss}. ({3} Failed, {4} Passed with Warnings)",
					TestCount - FailedCount, TestCount, Duration, FailedCount, WarningCount));
			
				List<string> TestResults = new List<string>();
				foreach (TestExecutionInfo Info in SortedInfo)
				{
					string WarningString = Info.TestNode.GetWarnings().Any() ? " With Warnings" : "";
					TestResults.Add(string.Format("\t{0} result={1}{2} (Waited={3:mm\\:ss}, Duration={4:mm\\:ss})", 
						Info, Info.FinalResult, WarningString,
						Info.WaitDuration, Info.TestDuration));
				}

				MB.UnorderedList(TestResults);
			}

			// write the markdown out with each line indented
			MB.ToString().Split('\n').ToList().ForEach(L => Log.Info("  " + L));
		}

		TestResult TickTest(TestExecutionInfo TestInfo)
		{
			// Give the test a chance to update itself
			try
			{
				TestInfo.TestNode.TickTest();
			}
			catch (Exception Ex)
			{
				TestInfo.CancellationReason = string.Format("Test {0} threw an exception. Cancelling. Ex: {1}", TestInfo.TestNode.Name, Ex);
				TestInfo.FinalResult = TestResult.Cancelled;
			}
		
			// Does the test still say it's running?
			bool TestIsRunning = TestInfo.TestNode.GetTestStatus() == TestStatus.InProgress;

			TimeSpan RunningTime = DateTime.Now - CurrentTestStartTime;

			if (TestIsRunning && RunningTime.TotalSeconds > TestInfo.TestNode.MaxDuration && !Options.NoTimeout)
			{
				if (TestInfo.TestNode.MaxDurationReachedResult == EMaxDurationReachedResult.Failure)
				{
					TestInfo.CancellationReason = string.Format("Terminating Test {0} due to maximum duration of {1} seconds. ", TestInfo.TestNode, TestInfo.TestNode.MaxDuration);
					TestInfo.FinalResult = TestResult.TimedOut;
					Log.Verbose("Terminating current test pass {TestName} due to maximum duration of {TestMaxDuration} seconds.", TestInfo.TestNode, TestInfo.TestNode.MaxDuration);
					Horde.UpdateStepMessage(Horde.GauntletHordeStepMessage.TestTimeoutExceeded);
				}
				else if (TestInfo.TestNode.MaxDurationReachedResult == EMaxDurationReachedResult.Success)
				{
					TestInfo.FinalResult = TestResult.Passed;
					TestIsRunning = false;
					Log.Info("Test {Name} successfully reached maximum duration of {Time} seconds. ", TestInfo.TestNode, TestInfo.TestNode.MaxDuration);
				}
			}

			if (IsCancelled)
			{
				TestInfo.CancellationReason = string.Format("Cancelling Test {0} on request", TestInfo.TestNode);
				TestInfo.FinalResult = TestResult.Cancelled;
				Log.Info(TestInfo.CancellationReason);
			}

			if (!string.IsNullOrEmpty(TestInfo.CancellationReason))
			{
				TestInfo.TestNode.SetCancellationReason(TestInfo.CancellationReason);
			}

			// if the test is not running. or we've determined a result for it..
			if (TestIsRunning == false || TestInfo.FinalResult != TestResult.Invalid)
			{
				// Request the test stop
				try
				{
					// Note - we log in this order to try and make it easy to grep the log and find the
					// artifact links
					Log.Info("*");
					Log.Info("****************************************************************");
					Log.Info("Finished Test: {Name} in {Time:mm\\:ss}", TestInfo, DateTime.Now - TestInfo.TimeSetupEnded);

					// Tell the test it's done. If it still thinks its running it was cancelled
					TestInfo.TestNode.StopTest(TestIsRunning && TestInfo.FinalResult != TestResult.Cancelled ? StopReason.MaxDuration : StopReason.Completed);
					TestInfo.TimeTestEnded = DateTime.Now;

					TestResult NodeResult = TestInfo.TestNode.GetTestResult();
					if (NodeResult == TestResult.WantRetry)
					{
						TestInfo.FinalResult = TestResult.WantRetry;
					}
					else
					{
						TestInfo.FinalResult = (TestInfo.FinalResult != TestResult.Invalid) ? TestInfo.FinalResult : NodeResult;
					}

					bool bCanFinalizeTest = true;
					if (TestInfo.FinalResult == TestResult.WantRetry)
					{
						Log.Info("Test {TestName} requested retry. Cleaning up old test and relaunching", TestInfo);

						bool bIsRestarted = TestInfo.TestNode.RestartTest();
						if (bIsRestarted)
						{
							// Mark us as still running
							CurrentTestStartTime = DateTime.Now;
							TestInfo.CancellationReason = "";
							TestInfo.TestNode.SetCancellationReason("");
							TestInfo.FinalResult = TestResult.Invalid;
							bCanFinalizeTest = false;
							Log.Info("Restarted test {TestName} for requested retry at {StartTime}", TestInfo, CurrentTestStartTime.ToString("h:mm:ss"));
						}
						else
						{
							TestInfo.CancellationReason = !string.IsNullOrEmpty(TestInfo.CancellationReason) ?
								$"Failed to restart during retry. Cancellation reason before retrying: {TestInfo.CancellationReason}" : "Failed to restart during retry.";
							TestInfo.FinalResult = TestResult.Failed;
						}
					}

					if (bCanFinalizeTest)
					{
						Log.Info("{Name} result={Result}", TestInfo, TestInfo.FinalResult);
						if (!string.IsNullOrEmpty(TestInfo.CancellationReason) && TestInfo.FinalResult != TestResult.Cancelled)
						{
							Log.Error(KnownLogEvents.Horde_BuildHealth_Ignore, "{CancellationReason}", TestInfo.CancellationReason);
						}

						if (!Options.DeferReports)
						{
							ReportTestSummary(TestInfo);
						}
						TestInfo.TestNode.SetTestResult(TestInfo.FinalResult);
						// now cleanup
						try
						{
							TestInfo.TestNode.CleanupTest();
						}
						catch (System.Exception ex)
						{
							Log.Error(KnownLogEvents.Horde_BuildHealth_Ignore, "Test {Name} threw an exception while cleaning up. Ex: {Exception}",
								TestInfo.TestNode.Name, ex);
						}
					}

					Log.Info("****************************************************************");
					Log.Info("*");
				}
				catch (System.Exception ex)
				{
					if (TestIsRunning)
					{
						Log.Warning(KnownLogEvents.Horde_BuildHealth_Ignore, "Cancelled Test {Name} threw an exception while stopping. Ex: {Exception}", 
							TestInfo.TestNode.Name, ex);
					}
					else
					{
						Log.Error(KnownLogEvents.Horde_BuildHealth_Ignore, "Test {Name} threw an exception while stopping. Ex: {Exception}",
							TestInfo.TestNode.Name, ex);
					}

					TestInfo.FinalResult = TestResult.Failed;
				}
			}

			return TestInfo.FinalResult;
		}

		/// <summary>
		/// Waits for all pending tests to complete. Returns true/false based on whether
		/// all tests completed successfully.
		/// </summary>
		bool WaitForTests()
		{
			Log.Info("Waiting for {Count} tests to complete", RunningTests.Count);

			DateTime LastUpdateMsg = DateTime.Now;

			bool AllTestsPassed = true;

			while (RunningTests.Count > 0)
			{
				List<TestExecutionInfo > RemainingTests = new List<TestExecutionInfo>();

				foreach (TestExecutionInfo Process in RunningTests)
				{
					
					TestResult Result = TickTest(Process);

					// invalid = no
					if (Result == TestResult.Invalid)
					{ 
						RemainingTests.Add(Process);

						TimeSpan RunningTime = DateTime.Now - Process.TimeSetupEnded;

						if ((DateTime.Now - LastUpdateMsg).TotalSeconds > 60.0f)
						{
							Log.Verbose("Test {Name} is still running. {Elapsed:00} seconds elapsed, will timeout in {Max:00} seconds",
								Process.TestNode.Name,
								RunningTime.TotalSeconds,
								Process.TestNode.MaxDuration - RunningTime.TotalSeconds);

							LastUpdateMsg = DateTime.Now;
						}
					}
					else
					{
						if (Result != TestResult.Passed)
						{
							AllTestsPassed = false;
						}
						Log.Info("Test {Name} Result: {Result}", Process.TestNode.Name, Result);
					}
				}

				RunningTests = RemainingTests;

				Thread.Sleep(1000);
			}

			return AllTestsPassed;
		}
	}
}
