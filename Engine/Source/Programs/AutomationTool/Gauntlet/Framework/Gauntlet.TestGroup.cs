// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Linq;
using System.Text;
using System.Threading.Tasks;

namespace Gauntlet
{

	/// </summary>
	[AttributeUsage(AttributeTargets.Class)]
	public class TestGroup : System.Attribute
	{
		/// <summary>
		/// Names that can refer to this param
		/// </summary>
		public string GroupName;

		public int Priority;

		public TestGroup(string InGroupName, int InPriority=5)
		{
			this.GroupName = InGroupName;
			this.Priority = InPriority;
		}
	}
}
