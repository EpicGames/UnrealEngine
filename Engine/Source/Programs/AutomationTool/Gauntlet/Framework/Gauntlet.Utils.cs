// Copyright Epic Games, Inc. All Rights Reserved.

using AutomationTool;
using EpicGames.Core;
using Gauntlet.TestTracking;
using ImageMagick;
using Microsoft.Extensions.Logging;
using System;
using System.Collections.Generic;
using System.Diagnostics;
using System.IO;
using System.Linq;
using System.Reflection;
using System.Security.Cryptography;
using System.Text;
using System.Text.RegularExpressions;
using System.Threading;
using System.Threading.Tasks;
using UnrealBuildBase;
using UnrealBuildTool;
using static AutomationTool.CommandUtils;
using Logging = Microsoft.Extensions.Logging;

namespace Gauntlet
{
	public static class Globals
	{
		static Params InnerParams = new Params(Environment.GetCommandLineArgs());

		public static Params Params
		{
			get { return InnerParams; }
			set { InnerParams = value; }
		}

		public static string BuildCommand = String.Empty;

		public static readonly DateTime ProgramStartTime = DateTime.UtcNow;

		static string InnerTempDir;
		static string InnerLogDir;
		static string InnerUnrealRootDir;
		static object InnerLockObject = new object();
		static List<Action> InnerAbortHandlers;
		static List<Action> InnerPostAbortHandlers = new List<Action>();
		public static bool CancelSignalled { get; private set; }

		public static bool IsRunningDev => Params.ParseParam("dev");

		/// <summary>
		/// Returns the device pool id 
		/// </summary>
		public static string DevicePoolId
		{
			get
			{
				return Params.ParseValue("devicepool", "");
			}
		}

		/// <summary>
		/// This acts as a broad feature flag to allow for a smoother integration of large flow changes without impacting licensees.
		/// When true, Gauntlet will execute alternative "experimental" code paths in several areas of project, primarily effecting executions of RunUnreal
		/// Most notable areas this is used:
		///		- UnrealSession	| Modifies the code path of LaunchSession to better reserve, setup, and handle errors for devices.
		///		- DevicePool	| Removes the strict expectation that Horde is used as a device service. Allows users to specify new services, such as a cloud provider.
		/// </summary>
		public static bool UseExperimentalFeatures => Params.ParseParam("Experimental") || Params.ParseParam("ExperimentalLaunchFlow");

		public static string TempDir
		{
			get
			{
				if (String.IsNullOrEmpty(InnerTempDir))
				{
					InnerTempDir = Path.Combine(Environment.CurrentDirectory, "GauntletTemp");
				}

				return InnerTempDir;
			}
			set
			{
				InnerTempDir = value;
			}
		}

		public static string LogDir
		{
			get
			{
				if (String.IsNullOrEmpty(InnerLogDir))
				{
					InnerLogDir = Path.Combine(TempDir, "Logs");
				}

				return InnerLogDir;
			}
			set
			{
				InnerLogDir = value;
			}
		}

		public static string UnrealRootDir
		{
			get
			{
				if (String.IsNullOrEmpty(InnerUnrealRootDir))
				{
					InnerUnrealRootDir = Path.GetFullPath(Path.Combine(Path.GetDirectoryName(Assembly.GetEntryAssembly().GetOriginalLocation()), "..", "..", "..", ".."));
				}

				return InnerUnrealRootDir;
			}

		}

		/// <summary>
		/// Return @"\\?\"; adding the prefix to a path string tells the Windows APIs to disable all string parsing and to send the string that follows it straight to the file system.
		/// Allowing for longer than 260 characters path.
		/// </summary>
		public static string LongPathPrefix
		{
			get { return Path.DirectorySeparatorChar == '\\'? @"\\?\" : ""; }
		}


		/// <summary>
		/// Acquired and released during the main Tick of the Gauntlet systems. Use this before touchung anything global scope from a 
		/// test thread.
		/// </summary>
		public static object MainLock
		{
			get { return InnerLockObject; }
		}

		/// <summary>
		/// Allows classes to register for notification of system-wide abort request. On an abort (e.g. ctrl+c) all handlers will 
		/// be called and then shutdown will continue
		/// </summary>
		public static List<Action> AbortHandlers
		{
			get
			{
				if (InnerAbortHandlers == null)
				{
					InnerAbortHandlers = new List<Action>();

					Console.CancelKeyPress += new ConsoleCancelEventHandler((obj, args) =>
					{
						CancelSignalled = true;
						Horde.ReportInvalidatingIssue(Horde.GauntletHordeStepMessage.TestCancelled, "Test was manually cancelled");
						// fire all abort handlers
						foreach (var Handler in AbortHandlers)
						{
							Handler();
						}

						// file all post-abort handlers
						foreach (var Handler in PostAbortHandlers)
						{
							Handler();
						}
					});
				}

				return InnerAbortHandlers;
			}
		}

		/// <summary>
		/// Allows classes to register for post-abort handlers. These are called after all abort handlers have returned
		/// so is the place to perform final cleanup.
		/// </summary>
		public static List<Action> PostAbortHandlers { get { return InnerPostAbortHandlers; } }

		/// <summary>
		/// Used by network functions that need to deal with multiple adapters
		/// </summary>
		public static string PreferredNetworkDomain {  get { return "epicgames.net";  } }
	}

	/// <summary>
	/// Enable/disable verbose logging
	/// </summary>
	public enum LogLevel
	{
		Normal,
		Verbose,
		VeryVerbose
	};

	/// <summary>
	/// Gauntlet Logging helper
	/// </summary>
	public class Log
	{
		public static LogLevel Level = LogLevel.Normal;

		public static bool IsVerbose
		{
			get
			{
				return Level >= LogLevel.Verbose;
			}
		}

		public static bool IsVeryVerbose
		{
			get
			{
				return Level >= LogLevel.VeryVerbose;
			}
		}

		static TextWriter LogFile = null;

		static List<Action<string>> Callbacks;

		static int ECSuspendCount = 0;

		static int SanitizationSuspendCount = 0;

		static ILogger Logger = EpicGames.Core.Log.Logger;

		public static void AddCallback(Action<string> CB)
		{
			if (Callbacks == null)
			{
				Callbacks = new List<Action<string>>();
			}

			Callbacks.Add(CB);
		}

		public static void SaveToFile(string InPath)
		{
			int Attempts = 0;

			if (LogFile != null)
			{
				Console.WriteLine("Logfile already open for writing");
				return;
			}

			do
			{
				string Outpath = InPath;

				if (Attempts > 0)
				{
					string Ext = Path.GetExtension(InPath);
					string BaseName = Path.GetFileNameWithoutExtension(InPath);

					Outpath = Path.Combine(Path.GetDirectoryName(InPath), string.Format("{0}_{1}.{2}", BaseName, Attempts, Ext));
				}

				try
				{
					LogFile = TextWriter.Synchronized(new StreamWriter(Outpath));
				}
				catch (UnauthorizedAccessException Ex)
				{
					Console.WriteLine("Could not open {0} for writing. {1}", Outpath, Ex);
					Attempts++;
				}

			} while (LogFile == null && Attempts < 10);
		}

		static void Flush()
		{
			if (LogFile != null)
			{
				LogFile.Flush();
			}
		}

		public static void SuspendECErrorParsing()
		{
			if (ECSuspendCount++ == 0)
			{
				if (CommandUtils.IsBuildMachine)
				{
					Logger.LogInformation("<-- Suspend Log Parsing -->");
				}
			}
		}

		public static void ResumeECErrorParsing()
		{
			if (--ECSuspendCount == 0)
			{
				if (CommandUtils.IsBuildMachine)
				{
					Logger.LogInformation("<-- Resume Log Parsing -->");
				}
			}
		}

		public static void SuspendSanitization()
		{
			SanitizationSuspendCount++;
		}

		public static void ResumeSanitization()
		{
			--SanitizationSuspendCount;
		}



		/// <summary>
		/// Santi
		/// </summary>
		/// <param name="Input"></param>
		/// <param name="Sanitize"></param>
		/// <returns></returns>
		static private string SanitizeInput(string Input, string[] Sanitize)
		{
			foreach (string San in Sanitize)
			{
				string CleanSan = San;
				if (San.Length > 1)
				{
					CleanSan.Insert(San.Length - 1, "-");
				}
				Input = Regex.Replace(Input, "Warning", CleanSan, RegexOptions.IgnoreCase);
			}

			return Input;
		}

		/// <summary>
		/// Outputs the message to the console with an optional log level and sanitization.
		/// Sanitizing allows errors and exceptions to be passed through to logs without triggering CIS warnings about out log
		/// </summary>
		/// <param name="Message"></param>
		/// <param name="EventId"></param>
		/// <param name="Level"></param>
		/// <param name="Sanitize"></param>
		/// <param name="Args"></param>
		/// <returns></returns>
		static private void OutputMessage(string Message, Logging.EventId EventId, Logging.LogLevel Level = Logging.LogLevel.Information, bool Sanitize=true, params object[] Args)
		{
			// EC detects error statements in the log as a failure. Need to investigate best way of 
			// reporting errors, but not errors we've handled from tools.
			// Probably have Log.Error which does not sanitize?
			if (Sanitize && SanitizationSuspendCount == 0)
			{
				string[] Triggers = { "Warning:", "Error:", "Exception:" };

				foreach (string Trigger in Triggers)
				{
					if (Message.IndexOf(Trigger, StringComparison.OrdinalIgnoreCase) != -1)
					{
						string SafeTrigger = Regex.Replace(Trigger, "i", "1", RegexOptions.IgnoreCase);
						SafeTrigger = Regex.Replace(SafeTrigger, "o", "0", RegexOptions.IgnoreCase);

						Message = Regex.Replace(Message, Trigger, SafeTrigger, RegexOptions.IgnoreCase);
					}
				}		
			}

			if (Globals.Params.ParseParam("timestamp"))
			{
				TimeSpan ElapsedTime = DateTime.UtcNow - Globals.ProgramStartTime;
				Message = "[" + ElapsedTime.ToString() + "] " + Message;
			}

			Logger.Log(Level, EventId, Message, Args);

			if (LogFile != null || Callbacks != null)
			{
				try
				{
					if (Args.Length > 0)
					{
						// Adjust Message and Arguments for string.Format
						int Index = 0;
						string SequenceMessage = Regex.Replace(Message, @"\{[^{}:]+(:[^{}]+)?\}", M => "{" + Index++.ToString() + M.Groups[1].Value + "}", RegexOptions.IgnoreCase);
						if (Index > Args.Length)
						{
							Args = Args.Concat(Enumerable.Repeat("null", Index - Args.Length)).ToArray();
						}
						else if (Index < Args.Length)
						{
							Args = Args.Take(Index).ToArray();
						}
						Message = string.Format(SequenceMessage, Args);
					}

					if (LogFile != null)
					{
						LogFile.WriteLine(Message);
					}

					if (Callbacks != null)
					{
						Callbacks.ForEach(A => A(Message));
					}
				}
				catch (Exception Ex)
				{
					Logger.LogWarning(KnownLogEvents.Gauntlet, "Exception logging '{Message}'. {Exception}", Message, Ex.ToString());
				}
			}
		}	

		static public void Verbose(string Format, params object[] Args)
		{
			if (IsVerbose)
			{
				Verbose(KnownLogEvents.Gauntlet, Format, Args);
			}
		}

		static public void Verbose(Logging.EventId EventId, string Format, params object[] Args)
		{
			if (IsVerbose)
			{
				OutputMessage(Format, EventId, Args: Args);
			}
		}

		static public void VeryVerbose(string Format, params object[] Args)
		{
			if (IsVeryVerbose)
			{
				VeryVerbose(KnownLogEvents.Gauntlet, Format, Args);
			}
		}

		static public void VeryVerbose(Logging.EventId EventId, string Format, params object[] Args)
		{
			if (IsVeryVerbose)
			{
				OutputMessage(Format, EventId, Args: Args);
			}
		}

		static public void Info(string Format, params object[] Args)
		{
			Info(KnownLogEvents.Gauntlet, Format, Args);
		}

		static public void Info(Logging.EventId EventId, string Format, params object[] Args)
		{
			OutputMessage(Format, EventId, Args: Args);
		}

		static public void Warning(string Format, params object[] Args)
		{
			Warning(KnownLogEvents.Gauntlet, Format, Args);
		}

		static public void Warning(Logging.EventId EventId, string Format, params object[] Args)
		{
			OutputMessage(Format, EventId, Logging.LogLevel.Warning, Args: Args);
		}

		static public void Error(string Format, params object[] Args)
		{
			Error(KnownLogEvents.Gauntlet, Format, Args);
		}
		static public void Error(Logging.EventId EventId, string Format, params object[] Args)
		{
			OutputMessage(Format, EventId, Logging.LogLevel.Error, false, Args);
		}

		static public void Critical(string Format, params object[] Args)
		{
			Critical(KnownLogEvents.Gauntlet, Format, Args);
		}
		static public void Critical(Logging.EventId EventId, string Format, params object[] Args)
		{
			OutputMessage(Format, EventId, Logging.LogLevel.Critical, false, Args);
		}
	}

#nullable enable
	/// <summary>
	/// Helper class providing logging methods for test events of various levels.
	/// Used to clearly display test events on the Horde dashboard, eliminates inconsistent reporting.
	/// </summary>
	public class Reporter
	{
		private static volatile Reporter? _Instance;
		private static readonly object _Lock = new object();

		private Reporter()
		{
			;
		}

		public static Reporter Instance
		{
			get
			{
				if (_Instance == null)
				{
					lock (_Lock)
					{
						if (_Instance == null)
						{
							_Instance = new Reporter();
						}
					}
				}
				return _Instance;
			}
		}

		private PhaseQueue? _Phases;
		private Action<TestResult>? _SetUnrealTestResult;
		private BaseTest? _TestNode;

		/// <summary>
		/// This method is used to switch context when running tests in parallel.
		/// The method should be called each time TestExecutor switches between test nodes, such as in node loops, etc.
		/// </summary>
		/// <param name="Phases">Reference to Phases variable in test node</param>
		/// <param name="SetUnrealTestResult">Reference to SetUnrealTestResult method from test node</param>
		/// <param name="TestNode">Reference to test node that is calling this method</param>
		public void UpdateReferences(PhaseQueue Phases, Action<TestResult> SetUnrealTestResult, BaseTest TestNode)
		{
			this._Phases = Phases;
			this._SetUnrealTestResult = SetUnrealTestResult;
			this._TestNode = TestNode;
		}

		/// <summary>
		/// Report a test-halting error that is NOT directly related to the behavior that is being tested.
		/// </summary>
		/// <param name="Message"></param>
		/// <param name="Args"></param>
		public void Interrupt(string Message, params object[] Args)
		{
			Instance?._Phases?.Current?.AddEvent(Logging.LogLevel.Critical, Message, Args);
			Instance?._TestNode?.AddTestEvent(new UnrealTestEvent(EventSeverity.Fatal, Message, Args));
			
			if (_TestNode == null || !_TestNode.LogWarningsAndErrorsAfterSummary)
			{
				Log.Critical(KnownLogEvents.Gauntlet_FatalEvent, Message, Args);
			}

			if (Instance?._TestNode?.GetTestStatus() == TestStatus.Complete && Instance._TestNode.GetTestResult() == TestResult.Passed)
			{
				Instance?._SetUnrealTestResult?.Invoke(TestResult.OperationalException);
			}
		}

		/// <summary>
		/// Report a test failure.  Specifically, the behavior that is targeted by the test case did not function as expected.
		/// </summary>
		/// <param name="Message"></param>
		/// <param name="Args"></param>
		public void Failure(string Message, params object[] Args)
		{
			Instance?._Phases?.Current?.AddEvent(Logging.LogLevel.Error, Message, Args);
			Instance?._TestNode?.AddTestEvent(new UnrealTestEvent(EventSeverity.Error, Message, Args));

			if (_TestNode == null || !_TestNode.LogWarningsAndErrorsAfterSummary)
			{
				Log.Error(KnownLogEvents.Gauntlet_TestEvent, Message, Args);
			}

			if (Instance?._TestNode?.GetTestStatus() == TestStatus.Complete && Instance._TestNode.GetTestResult() == TestResult.Passed)
			{
				Instance?._SetUnrealTestResult?.Invoke(TestResult.Failed);
			}
		}

		/// <summary>
		/// Report a warning with Gauntlet_TestEvent
		/// </summary>
		/// <param name="Message"></param>
		/// <param name="Args"></param>
		public void Warning(string Message, params object[] Args)
		{
			Warning(KnownLogEvents.Gauntlet_TestEvent, Message, Args);
		}

		/// <summary>
		/// Report a warning with required Log Event type
		/// </summary>
		/// <param name="EventId"></param>
		/// <param name="Message"></param>
		/// <param name="Args"></param>
		public void Warning(EventId EventId, string Message, params object[] Args)
		{
			Instance?._Phases?.Current?.AddEvent(Logging.LogLevel.Warning, Message, Args);
			Instance?._TestNode?.AddTestEvent(new UnrealTestEvent(EventSeverity.Warning, Message, Args));

			if (_TestNode == null || !_TestNode.LogWarningsAndErrorsAfterSummary)
			{
				Log.Warning(EventId, Message, Args);
			}
		}

		/// <summary>
		/// Report information
		/// </summary>
		/// <param name="Message"></param>
		/// <param name="Args"></param>
		public void Info(string Message, params object[] Args)
		{
			Instance?._Phases?.Current?.AddEvent(Logging.LogLevel.Information, Message, Args);
			Instance?._TestNode?.AddTestEvent(new UnrealTestEvent(EventSeverity.Info, Message, Args));
			Log.Info(KnownLogEvents.Gauntlet_TestEvent, Message, Args);
		}
	}
#nullable disable

	public class Hasher
	{
		public static string ComputeHash(string Input)
		{
			if (string.IsNullOrEmpty(Input))
			{
				return "0";
			}

			HashAlgorithm Hasher = MD5.Create();  //or use SHA256.Create();

			byte[] Hash = Hasher.ComputeHash(Encoding.UTF8.GetBytes(Input));

			string HashString = "";

			foreach (byte b in Hash)
			{
				HashString += (b.ToString("X2"));
			}

			return HashString;
		}

		public static string ComputeHash(string Input, HashAlgorithm Algo, int MaxLength = 0)
		{
			if (string.IsNullOrEmpty(Input))
			{
				return "0";
			}

			byte[] Hash = Algo.ComputeHash(Encoding.UTF8.GetBytes(Input));
			var SBuilder = new StringBuilder();

			for (int i = 0; i < Hash.Length; i++)
			{
				SBuilder.Append(Hash[i].ToString("x2"));
				if (MaxLength > 0 && SBuilder.Length >= MaxLength)
				{
					if (SBuilder.Length > MaxLength) { SBuilder.Remove(MaxLength, 1); }
					break;
				}
			}

			return SBuilder.ToString();
		}

		static public HashAlgorithm DefaultAlgo = SHA1.Create();
	}

	/*
	 * Helper class that can be used with a using() statement to emit log entries that prevent EC triggering on Error/Warning statements
	*/
	public class ScopedSuspendECErrorParsing : IDisposable
	{
		public ScopedSuspendECErrorParsing()
		{
			Log.SuspendECErrorParsing();
		}

		~ScopedSuspendECErrorParsing()
		{
			Dispose(false);
		}

		#region IDisposable Support
		private bool disposedValue = false; // To detect redundant calls

		protected virtual void Dispose(bool disposing)
		{
			if (!disposedValue)
			{
				Log.ResumeECErrorParsing();

				disposedValue = true;
			}
		}

		// This code added to correctly implement the disposable pattern.
		public void Dispose()
		{
			Dispose(true);
		}
		#endregion
	}

	namespace Utils
	{
		public class TestConstructor
		{
			/// <summary>
			/// Helper function that returns the type of an object based on namespaces and name
			/// </summary>
			/// <param name="Namespaces"></param>
			/// <param name="TestName"></param>
			/// <returns></returns>
			private static Type GetTypeForTest(string TestName, IEnumerable<string> Namespaces)
			{
				var SearchAssemblies = AppDomain.CurrentDomain.GetAssemblies();

				// turn foo into [foo, n1.foo, n2.foo]
				IEnumerable<string> FullNames = new[] { TestName };

				if (Namespaces != null)
				{
					FullNames = FullNames.Concat(Namespaces.Select(N => N + "." + TestName));
				}

				Log.VeryVerbose("Will search {0} for test {1}", string.Join(" ", FullNames), TestName);

				// find all types from loaded assemblies that implement testnode
				List<Type> CandidateTypes = new List<Type>();
				foreach (Assembly Assembly in ScriptManager.AllScriptAssemblies)
				{
					foreach (var Type in Assembly.GetTypes())
					{
						if (typeof(ITestNode).IsAssignableFrom(Type))
						{
							// If there is an Exact match, just take it
							if (Type.FullName == TestName)
							{
								return Type;
							}
							CandidateTypes.Add(Type);
						}
					}
				}

				Log.VeryVerbose("Possible candidates for {0}: {1}", TestName, string.Join(" ", CandidateTypes));

				// check our expanded names.. need to search in namespace order
				IList<Type> MatchingTypes = new List<Type>();
				foreach (string UserTypeName in FullNames)
				{
					// Even tho the user might have specified N1.Foo it still might be Other.N1.Foo so only
					// compare based on the number of namespaces that were specified.
					foreach (Type Candidate in CandidateTypes)
					{
						string[] UserNameComponents = UserTypeName.Split('.');
						string[] TypeNameComponents = Candidate.FullName.Split('.');

						int MissingUserComponents = TypeNameComponents.Length - UserNameComponents.Length;

						if (MissingUserComponents > 0)
						{
							TypeNameComponents = TypeNameComponents.Skip(MissingUserComponents).ToArray();
						}

						IEnumerable<string> Difference = TypeNameComponents.Except(UserNameComponents, StringComparer.OrdinalIgnoreCase);

						if (Difference.Count() == 0)
						{
							Log.VeryVerbose("Found match {0} for user type {1}", Candidate.FullName, UserTypeName);

							// If we have any namespaces, add the candidate
							if(Namespaces.Count() > 0)
							{
								MatchingTypes.Add(Candidate);
							}

							// No namespaces, just return the first type found
							else
							{
								return Candidate;
							}
						}
					}
				}

				// If user has specified at least 1 namespace, we prioritize types that include a provided namespace over types that do not.
				// For example, in the case of the supplied parameters: TestName = BootTest, Namespaces = {"Game"}
				// We prioritize "Game.BootTest" over "UE.BootTest".
				// In the case of multiple name spaces, we default to the first test found within a namespace.
				// For example, in the case of the supplied parameters: TestName = BootTest, Namespaces = {"Game", "UE"}
				// We select "Game.BootTest" over "UE.BootTest".
				// If you maintain many tests with the same base name, you should be as explicit as possible with Namespaces!
				foreach(string Namespace in Namespaces)
				{
					foreach(Type Match in MatchingTypes)
					{
						// Split off the namespace
						string TypeNamespace = Match.FullName.Replace(Match.Name, string.Empty);

						// If it matches one of namespaces, return it
						if (TypeNamespace.Contains(Namespace))
						{
							return Match;
						}
					}
				}

				// If nothing found from the prefered namespaces, then return the first item with less amount of steps.
				if(MatchingTypes.Count() > 0)
				{
					return MatchingTypes.OrderBy(M => M.FullName.Split('.').Length).First();
				}

				throw new AutomationException("Unable to find type {0} in assemblies. Namespaces={1}.", TestName, string.Join(", ", Namespaces));
			}


			/// <summary>
			/// Helper function that returns all types within the specified namespace that are or derive from
			/// the specified type
			/// </summary>
			/// <param name="Namespaces"></param>
			/// <returns></returns>
			public static IEnumerable<Type> GetTypesInNamespaces<BaseType>(IEnumerable<string> Namespaces)
				where BaseType : class
			{
				var AllTypes = AppDomain.CurrentDomain.GetAssemblies().SelectMany(S => S.SafeGetLoadedTypes()).Where(T => typeof(BaseType).IsAssignableFrom(T));

				if (Namespaces.Count() > 0)
				{
					AllTypes = AllTypes.Where(T => T.IsAbstract == false && Namespaces.Contains(T.Namespace.ToString()));
				}

				return AllTypes;
			}

			/// <summary>
			/// Constructs by name a new test of type "TestType" that takes no construction parameters
			/// </summary>
			/// <typeparam name="TestType"></typeparam>
			/// <typeparam name="ParamType"></typeparam>
			/// <param name="TestName"></param>
			/// <param name="Arg"></param>
			/// <param name="Namespaces"></param>
			/// <returns></returns>
			public static TestType ConstructTest<TestType, ParamType>(string TestName, ParamType Arg, IEnumerable<string> Namespaces)
					where TestType : class
			{
				Type NodeType = GetTypeForTest(TestName, Namespaces);

				ConstructorInfo NodeConstructor = null;
				TestType NewNode = null;

				if (Arg != null)
				{
					NodeConstructor = NodeType.GetConstructor(new Type[] { Arg.GetType() });

					if (NodeConstructor != null)
					{
						NewNode = NodeConstructor.Invoke(new object[] { Arg }) as TestType;
					}
				}

				if (NodeConstructor == null)
				{
					NodeConstructor = NodeType.GetConstructor(Type.EmptyTypes);
					if (NodeConstructor != null)
					{
						NewNode = NodeConstructor.Invoke(null) as TestType;
					}
				}

				if (NodeConstructor == null)
				{
					throw new AutomationException("Unable to find constructor for type {0} with our without params", typeof(TestType));
				}

				if (NewNode == null)
				{
					throw new AutomationException("Unable to construct node of type {0}", typeof(TestType));
				}

				return NewNode;
			}


			/// <summary>
			/// Constructs by name a list of tests of type "TestType" that take no construction params
			/// </summary>
			/// <typeparam name="TestType"></typeparam>
			/// <typeparam name="ParamType"></typeparam>
			/// <param name="TestNames"></param>
			/// <param name="Arg"></param>
			/// <param name="Namespaces"></param>
			/// <returns></returns>
			public static IEnumerable<TestType> ConstructTests<TestType, ParamType>(IEnumerable<string> TestNames, ParamType Arg, IEnumerable<string> Namespaces)
					where TestType : class
			{
				List<TestType> Tests = new List<TestType>();

				foreach (var Name in TestNames)
				{
					Tests.Add(ConstructTest<TestType, ParamType>(Name, Arg, Namespaces));
				}

				return Tests;
			}

			/// <summary>
			/// Constructs by name a list of tests of type "TestType" that take no construction params
			/// </summary>
			/// <typeparam name="TestType"></typeparam>
			/// <param name="TestNames"></param>
			/// <param name="Namespaces"></param>
			/// <returns></returns>
			public static IEnumerable<TestType> ConstructTests<TestType>(IEnumerable<string> TestNames, IEnumerable<string> Namespaces)
					where TestType : class
			{
				List<TestType> Tests = new List<TestType>();

				foreach (var Name in TestNames)
				{
					Tests.Add(ConstructTest<TestType, object>(Name, null, Namespaces));
				}

				return Tests;
			}

			/// <summary>
			/// Constructs by name a list of tests of type "TestType" that take no construction params
			/// </summary>
			/// <typeparam name="TestType"></typeparam>
			/// <param name="Group"></param>
			/// <param name="Namespaces"></param>
			/// <returns></returns>
			public static IEnumerable<string> GetTestNamesByGroup<TestType>(string Group, IEnumerable<string> Namespaces)
					where TestType : class
			{
				// Get all types in these namespaces
				IEnumerable<Type> TestTypes = GetTypesInNamespaces<TestType>(Namespaces);

				IEnumerable<string> SortedTests = new string[0];

				// If no group, just return a sorted list
				if (string.IsNullOrEmpty(Group))
				{
					SortedTests = TestTypes.Select(T => T.FullName).OrderBy(S => S);
				}
				else
				{
					Dictionary<string, int> TypesToPriority = new Dictionary<string, int>();

					// Find ones that have a group attribute
					foreach (Type T in TestTypes)
					{
						foreach (object Attrib in T.GetCustomAttributes(true))
						{
							TestGroup TestAttrib = Attrib as TestGroup;

							// Store the type name as a key with the priority as a value
							if (TestAttrib != null && Group.Equals(TestAttrib.GroupName, StringComparison.OrdinalIgnoreCase))
							{
								TypesToPriority[T.FullName] = TestAttrib.Priority;
							}
						}
					}

					// sort by priority then name
					SortedTests = TypesToPriority.Keys.OrderBy(K => TypesToPriority[K]).ThenBy(K => K);
				}
			
				return SortedTests;
			}
		
		}

		public static class InterfaceHelpers
		{
			public static IEnumerable<Type> FindTypes<InterfaceType>(bool bIncludeCompiledScripts = false, bool bConcreteTypesOnly = false)
				where InterfaceType : class
			{
				HashSet<Type> AllTypes = new HashSet<Type>(Assembly.GetExecutingAssembly().GetTypes().Where(Type => typeof(InterfaceType).IsAssignableFrom(Type)));
				if (bConcreteTypesOnly)
				{
					AllTypes = AllTypes.Where(Type => !Type.IsAbstract && !Type.IsInterface).ToHashSet();
				}

				if (bIncludeCompiledScripts)
				{
					foreach (Assembly Assembly in ScriptManager.AllScriptAssemblies)
					{
						HashSet<Type> AssemblyTypes = new HashSet<Type>(Assembly.GetTypes().Where(T => typeof(InterfaceType).IsAssignableFrom(T)));
						if (bConcreteTypesOnly)
						{
							AssemblyTypes = AssemblyTypes.Where(Type => !Type.IsAbstract && !Type.IsInterface).ToHashSet();
						}

						AllTypes.UnionWith(AssemblyTypes);
					}
				}

				return AllTypes;
			}

			public static IEnumerable<InterfaceType> FindImplementations<InterfaceType>(bool bIncludeCompiledScripts = false)
				where InterfaceType : class
			{
				HashSet<Type> AllTypes = FindTypes<InterfaceType>(bIncludeCompiledScripts).ToHashSet();

				List<InterfaceType> ConstructedTypes = new List<InterfaceType>();

				foreach (Type FoundType in AllTypes)
				{
					ConstructorInfo TypeConstructor = FoundType.GetConstructor(Type.EmptyTypes);

					if (TypeConstructor != null && !FoundType.IsAbstract)
					{
						InterfaceType NewInstance = TypeConstructor.Invoke(null) as InterfaceType;
						ConstructedTypes.Add(NewInstance);
					}
				}

				return ConstructedTypes;
			}
			/// <summary>
			/// Check if the method signature is overridden.
			/// </summary>
			/// <param name="ClassType"></param>
			/// <param name="MethodName"></param>
			/// <param name="Signature"></param>
			/// <returns></returns>
			public static bool HasOverriddenMethod(Type ClassType, string MethodName, Type[] Signature)
			{
				MethodInfo Method = ClassType.GetMethod(MethodName, Signature);
				if (Method == null)
				{
					throw new Exception(string.Format("Unknown method {0} from {1} with signature {2}.", MethodName, ClassType.Name, Signature.ToString()));
				}
				bool IsOverridden = Method.GetBaseDefinition().DeclaringType != Method.DeclaringType;
				return Method.GetBaseDefinition().DeclaringType != Method.DeclaringType;
			}

		}


		public static class SystemHelpers
		{
			/// <summary>
			/// Options for copying directories
			/// </summary>
			public enum CopyOptions
			{
				Copy = (1 << 0),        // Normal copy & combine/overwrite
				Mirror = (1 << 1),      // copy + remove files from dest if not in src
				Default = Copy
			}

			/// <summary>
			/// Options that can be specified to the CopyDirectory function.
			/// </summary>
			public class CopyDirectoryOptions
			{
				public CopyOptions Mode { get; set; }

				public int Retries { get; set; }

				public Func<string, string> Transform { get; set; }

				public string Pattern { get; set; }

				public Regex Regex { get; set; }

				public bool Recursive { get; set; }

				public bool Verbose { get; set; }

				public CopyDirectoryOptions()
				{
					Mode = CopyOptions.Copy;
					Retries = 10;
					Transform = delegate (string s)
					{
						return s;
					};
					Pattern = "*";
					Recursive = true;
					Regex = null;
					Verbose = false;
				}

				/// <summary>
				/// Returns true if the pattern indicates entire directory should be copied
				/// </summary>
				public bool IsDirectoryPattern
				{
					get
					{
						return 
							Regex == null &&
							(
								string.IsNullOrEmpty(Pattern)
								|| Pattern.Equals("*")
								|| Pattern.Equals("*.*")
								|| Pattern.Equals("...")
							);
					}
				}
			}

			/// <summary>
			/// Convenience function that removes some of the more esoteric options
			/// </summary>
			/// <param name="SourceDirPath"></param>
			/// <param name="DestDirPath"></param>
			/// <param name="Mode"></param>
			/// <param name="RetryCount"></param>
			public static void CopyDirectory(string SourceDirPath, string DestDirPath, CopyOptions Mode = CopyOptions.Default, int RetryCount = 5)
			{
				CopyDirectory(SourceDirPath, DestDirPath, Mode, delegate (string s) { return s; }, RetryCount);
			}

			/// <summary>
			/// Legacy convenience function that exposes transform & Retry count
			/// </summary>
			/// <param name="SourceDirPath"></param>
			/// <param name="DestDirPath"></param>
			/// <param name="Mode"></param>
			/// <param name="Transform"></param>
			/// <param name="RetryCount"></param>
			public static void CopyDirectory(string SourceDirPath, string DestDirPath, CopyOptions Mode, Func<string, string> Transform, int RetryCount = 5)
			{
				CopyDirectoryOptions Options = new CopyDirectoryOptions();
				Options.Retries = RetryCount;
				Options.Mode = Mode;
				Options.Transform = Transform;

				CopyDirectory(SourceDirPath, DestDirPath, Options);
			}

			/// <summary>
			/// Copies src to dest by comparing files sizes and time stamps and only copying files that are different in src. Basically a more flexible
			/// robocopy
			/// </summary>
			/// <param name="SourceDirPath"></param>
			/// <param name="DestDirPath"></param>
			/// <param name="Options"></param>
			public static void CopyDirectory(string SourceDirPath, string DestDirPath, CopyDirectoryOptions Options)
			{
				DateTime StartTime = DateTime.Now;
				
				DirectoryInfo SourceDir = new DirectoryInfo(SourceDirPath);
				DirectoryInfo DestDir = new DirectoryInfo(DestDirPath);

				if (DestDir.Exists == false)
				{
					DestDir = Directory.CreateDirectory(DestDir.FullName);
				}
				
				bool IsMirroring = (Options.Mode & CopyOptions.Mirror) == CopyOptions.Mirror;

				if (IsMirroring && !Options.IsDirectoryPattern)
				{
					Log.Warning("Can only use mirror with pattern that includes whole directories (e.g. '*')");
					IsMirroring = false;
				}

				IEnumerable<FileInfo> SourceFiles = null;
				FileInfo[] DestFiles = null;

				// find all files. If a directory get them all, else use the pattern/regex
				if (Options.IsDirectoryPattern)
				{
					SourceFiles = GetFiles(SourceDir, "*", Options.Recursive ? SearchOption.AllDirectories : SearchOption.TopDirectoryOnly);
				}
				else
				{
					if (Options.Regex == null)
					{
						SourceFiles = GetFiles(SourceDir, Options.Pattern, Options.Recursive ? SearchOption.AllDirectories : SearchOption.TopDirectoryOnly);
					}
					else
					{
						SourceFiles = GetFiles(SourceDir, "*", Options.Recursive ? SearchOption.AllDirectories : SearchOption.TopDirectoryOnly);

						SourceFiles = SourceFiles.Where(F => Options.Regex.IsMatch(F.Name));
					}
				}

				// Convert dest into a map of relative paths to absolute
				Dictionary<string, System.IO.FileInfo> DestStructure = new Dictionary<string, System.IO.FileInfo>();

				if (IsMirroring)
				{
					DestFiles = GetFiles(DestDir, "*", SearchOption.AllDirectories);

					foreach (FileInfo Info in DestFiles)
					{
						string RelativePath = Info.FullName.Replace(DestDir.FullName, "");

						// remove leading seperator
						if (RelativePath.First() == Path.DirectorySeparatorChar)
						{
							RelativePath = RelativePath.Substring(1);
						}

						DestStructure[RelativePath] = Info;
					}
				}

				// List of source files to copy. The first item is the full (and possibly transformed)
				// dest path, the second is the source
				List<Tuple<FileInfo, FileInfo>> CopyList = new List<Tuple<FileInfo, FileInfo>>();

				// List of relative path files in dest to delete
				List<string> DeletionList = new List<string>();

				foreach (FileInfo SourceInfo in SourceFiles)
				{
					string RelativeSourceFilePath = SourceInfo.FullName.Replace(SourceDir.FullName, "");

					// remove leading seperator
					if (RelativeSourceFilePath.First() == Path.DirectorySeparatorChar)
					{
						RelativeSourceFilePath = RelativeSourceFilePath.Substring(1);
					}

					string RelativeDestFilePath = Options.Transform(RelativeSourceFilePath);

					FileInfo DestInfo = null;

					// We may have destination info if mirroring where we prebuild it all, if not
					// grab it now
					if (DestStructure.ContainsKey(RelativeDestFilePath))
					{
						DestInfo = DestStructure[RelativeDestFilePath];
					}
					else
					{
						string FullDestPath = Path.Combine(DestDir.FullName, RelativeDestFilePath);
						DestInfo = new FileInfo(FullDestPath);						
					}

					if (DestInfo.Exists == false)
					{
						// No copy in dest, add it to the list
						CopyList.Add(new Tuple<FileInfo, FileInfo>(DestInfo, SourceInfo));
					}
					else
					{
						// Check the file is the same version

						// Difference in ticks. Even though we set the dest to the src there still appears to be minute
						// differences in ticks. 1ms is 10k ticks...
						Int64 TimeDelta = Math.Abs(DestInfo.LastWriteTime.Ticks - SourceInfo.LastWriteTime.Ticks);
						Int64 Threshhold = 100000;

						if (DestInfo.Length != SourceInfo.Length ||
							TimeDelta > Threshhold)
						{
							CopyList.Add(new Tuple<FileInfo, FileInfo>(DestInfo, SourceInfo));
						}
						else
						{
							if (Options.Verbose)
							{
								Log.Info("Will skip copy to {0}. File up to date.", DestInfo.FullName);
							}
							else
							{
								Log.Verbose("Will skip copy to {0}. File up to date.", DestInfo.FullName);
							}
						}

						// Remove it from the map
						DestStructure.Remove(RelativeDestFilePath);
					}
				}

				// If set to mirror, delete all the files that were not in source
				if (IsMirroring)
				{
					// Now go through the remaining map items and delete them
					foreach (var Pair in DestStructure)
					{
						DeletionList.Add(Pair.Key);
					}

					foreach (string RelativePath in DeletionList)
					{
						FileInfo DestInfo = new FileInfo(Path.Combine(DestDir.FullName, RelativePath));

						if (!DestInfo.Exists)
						{
							continue;
						}

						if (Options.Verbose)
						{
							Log.Info("Deleting extra file {0}", DestInfo.FullName);
						}
						else
						{
							Log.Verbose("Deleting extra file {0}", DestInfo.FullName);
						}

						try
						{
							// avoid an UnauthorizedAccessException by making sure file isn't read only
							DestInfo.IsReadOnly = false;
							Delete(DestInfo);
						}
						catch (Exception Ex)
						{
							Log.Warning("Failed to delete file {0}. {1}", DestInfo.FullName, Ex);
						}
					}

					// delete empty directories
					DirectoryInfo DestDirInfo = new DirectoryInfo(DestDirPath);

					DirectoryInfo[] AllSubDirs = GetDirectories(DestDirInfo, "*", SearchOption.AllDirectories);

					foreach (DirectoryInfo SubDir in AllSubDirs)
					{
						try
						{
							if (GetFiles(SubDir).Length == 0 && GetDirectories(SubDir).Length == 0)
							{
								if (Options.Verbose)
								{
									Log.Info("Deleting empty dir {0}", SubDir.FullName);
								}
								else
								{
									Log.Verbose("Deleting empty dir {0}", SubDir.FullName);
								}

								Delete(SubDir, true);
							}
						}
						catch (Exception Ex)
						{
							// handle the case where a file is locked
							Log.Info("Failed to delete directory {0}. {1}", SubDir.FullName, Ex);
						}
					}
				}

				CancellationTokenSource CTS = new CancellationTokenSource();

				// todo - make param..
				var POptions = new ParallelOptions { MaxDegreeOfParallelism = 1, CancellationToken = CTS.Token  };

				// install a cancel handler so we can stop parallel-for gracefully
				Action CancelHandler = delegate()
				{
					CTS.Cancel();
				};

				Globals.AbortHandlers.Add(CancelHandler);

				// now do the work
				Parallel.ForEach(CopyList, POptions, FilePair =>
				{
					// ensure path exists
					FileInfo DestInfo = FilePair.Item1;
					FileInfo SrcInfo = FilePair.Item2;

					// ensure directory exists
					DestInfo.Directory.Create();

					int Tries = 0;
					bool Copied = false;

					do
					{
						try
						{
							if (Options.Verbose)
							{
								Log.Info("Copying to {0}", DestInfo.FullName);
							}
							else
							{
								Log.Verbose("Copying to {0}", DestInfo.FullName);
							}

							DestInfo = SrcInfo.CopyTo(DestInfo.FullName, true);

							// Clear attributes and set last write time
							DestInfo.Attributes = FileAttributes.Normal;
							DestInfo.LastWriteTime = SrcInfo.LastWriteTime;
							Copied = true;
						}
						catch (Exception ex)
						{
							if (Tries++ < Options.Retries)
							{
								Log.Info("Copy to {0} failed, retrying {1} of {2} in 30 secs..", DestInfo.FullName, Tries, Options.Retries);
								Log.Verbose("\t{0}", ex);
								Thread.Sleep(30000);
							}
							else
							{
								Log.Info("File Copy failed with {0}.", ex.Message);

								// Warn with message if we're exceeding long path, otherwise throw an exception
								const int MAX_PATH = 260;
								bool LongPath = BuildHostPlatform.Current.Platform == UnrealTargetPlatform.Win64 && (SrcInfo.FullName.Length >= MAX_PATH || DestInfo.FullName.Length >= MAX_PATH);

								if (!LongPath)
								{
									throw new Exception(string.Format("File Copy failed with {0}.", ex.Message));
								}
								else
								{
									string LongPathMessage = (Environment.OSVersion.Version.Major > 6) ?
										"Long path detected, check that long paths are enabled." :
										"Long path detected, OS version doesn't support long paths.";

									// Break out of loop with warning
									Copied = true;

									// Filter out some known unneeded files which can cause this warning, and log the message instead
									string[] Denylist = new string[]{ "UECC-", "PersistentDownloadDir" };
									string Message = string.Format("Long path file copy failed with {0}.  Please verify that this file is not required.", ex.Message);
									if (Denylist.FirstOrDefault(B => { return SrcInfo.FullName.IndexOf(B, StringComparison.OrdinalIgnoreCase) >= 0; }) == null)
									{
										Log.Warning(Message); 
									}
									else
									{
										Log.Info(Message);
									}
								}
							}
						}
					} while (Copied == false);
				});

				TimeSpan Duration = DateTime.Now - StartTime;
				if (Duration.TotalSeconds > 10)
				{
					if (Options.Verbose)
					{
						Log.Info("Copied Directory in {0}", Duration.ToString(@"mm\m\:ss\s"));
					}
					else
					{
						Log.Verbose("Copied Directory in {0}", Duration.ToString(@"mm\m\:ss\s"));
					}
				}

				// remove cancel handler
				Globals.AbortHandlers.Remove(CancelHandler);
			}

			public static string MakePathRelative(string FullPath, string BasePath)
			{
				// base path must be correctly formed!
				if (BasePath.Last() != Path.DirectorySeparatorChar)
				{
					BasePath += Path.DirectorySeparatorChar;
				}					

				var ReferenceUri = new Uri(BasePath);
				var FullUri = new Uri(FullPath);

				return ReferenceUri.MakeRelativeUri(FullUri).ToString();
			}

			public static string CorrectDirectorySeparators(string InPath)
			{
				if (Path.DirectorySeparatorChar == '/')
				{
					return InPath.Replace('\\', Path.DirectorySeparatorChar);
				}
				else
				{
					return InPath.Replace('/', Path.DirectorySeparatorChar);
				}
			}

			public static bool ApplicationExists(string InPath)
			{
				if (File.Exists(InPath))
				{
					return true;
				}

				if (BuildHostPlatform.Current.Platform == UnrealTargetPlatform.Mac)
				{
					if (InPath.EndsWith(".app", StringComparison.OrdinalIgnoreCase))
					{
						return Directory.Exists(InPath);
					}
				}

				return false;
			}

			public static bool IsNetworkPath(string InPath)
			{
				if (InPath.StartsWith("//") || InPath.StartsWith(@"\\"))
				{
					return true;
				}
				
				string RootPath = Path.GetPathRoot(InPath); // get drive's letter

				if (string.IsNullOrEmpty(RootPath))
				{
					return false;
				}

				DriveInfo driveInfo = new System.IO.DriveInfo(RootPath); // get info about the drive
				return driveInfo.DriveType == DriveType.Network; // return true if a network drive
			}

			/// <summary>
			/// Marks a directory for future cleanup
			/// </summary>
			/// <param name="InPath"></param>
			public static void MarkDirectoryForCleanup(string InPath)
			{
				if (Directory.Exists(InPath) == false)
				{
					Directory.CreateDirectory(InPath);
				}

				// write a token, used to detect and old gauntlet-installed builds periodically
				string TokenPath = Path.Combine(InPath, "gauntlet.tempdir");
				File.WriteAllText(TokenPath, "Created by Gauntlet");
			}

			/// <summary>
			/// Removes any directory at the specified path which has a file  matching the provided name
			/// older than the specified number of days. Used by code that writes a .token file to temp
			/// folders
			/// </summary>
			/// <param name="InPath"></param>
			/// <param name="Days"></param>
			public static void CleanupMarkedDirectories(string InPath, int Days)
			{
				DirectoryInfo Di = new DirectoryInfo(InPath);

				if (Di.Exists == false)
				{
					return;
				}

				foreach (DirectoryInfo SubDir in GetDirectories(Di))
				{
					bool HasFile = 
						GetFiles(SubDir).Where(F => {
							int DaysOld = (DateTime.Now - F.LastWriteTime).Days;				
							
							if (DaysOld >= Days)
							{
								// use the old and new tokennames
								return string.Equals(F.Name, "gauntlet.tempdir", StringComparison.OrdinalIgnoreCase) ||
										string.Equals(F.Name, "gauntlet.token", StringComparison.OrdinalIgnoreCase);
							}

							return false;
						}).Count() > 0;

					if (HasFile)
					{
						Log.Info("Removing old directory {0}", SubDir.Name);
						try
						{
							Delete(SubDir, true);
						}
						catch (Exception Ex)
						{
							Log.Warning("Failed to remove old directory {0}. {1}", SubDir.FullName, Ex.Message);
						}
					}
					else
					{
						CleanupMarkedDirectories(SubDir.FullName, Days);
					}
				}				
			}

			/// <summary>
			/// Wrap DirectoryInfo.GetFiles() and retry once if exception is "A retry should be performed."
			/// </summary>
			public static FileInfo[] GetFiles(DirectoryInfo Directory)
			{
				FileInfo[] Files = null;
				try
				{
					Files = Directory.GetFiles();
				}
				catch (Exception ex)
				{
					if (ex.ToString().Contains("A retry should be performed"))
					{
						Log.Info("Retrying Directory.GetFiles() once.");
						Files = Directory.GetFiles();
					}
					else
					{
						throw;
					}
				}
				return Files;
			}

			/// <summary>
			/// Wrap DirectoryInfo.GetFiles(Pattern, SearchOption) and retry once if exception is "A retry should be performed."
			/// </summary>
			public static FileInfo[] GetFiles(DirectoryInfo Directory, string Pattern, SearchOption SearchOption)
			{
				FileInfo[] Files = null;
				try
				{
					Files = Directory.GetFiles(Pattern, SearchOption);
				}
				catch (Exception ex)
				{
					if (ex.ToString().Contains("A retry should be performed"))
					{
						Log.Info("Retrying Directory.GetFiles(Pattern, SearchOption) once.");
						Files = Directory.GetFiles(Pattern, SearchOption);
					}
					else
					{
						throw;
					}
				}
				return Files;
			}

			/// <summary>
			/// Wrap DirectoryInfo.GetDirectories() and retry once if exception is "A retry should be performed."
			/// </summary>
			public static DirectoryInfo[] GetDirectories(DirectoryInfo Directory)
			{
				DirectoryInfo[] Directories = null;
				try
				{
					Directories = Directory.GetDirectories();
				}
				catch (Exception ex)
				{
					if (ex.ToString().Contains("A retry should be performed"))
					{
						Log.Info("Retrying Directory.GetDirectories() once.");
						Directories = Directory.GetDirectories();
					}
					else
					{
						throw;
					}
				}
				return Directories;
			}

			/// <summary>
			/// Wrap DirectoryInfo.GetDirectories(Pattern, SearchOption) and retry once if exception is "A retry should be performed."
			/// </summary>
			public static DirectoryInfo[] GetDirectories(DirectoryInfo Directory, string Pattern, SearchOption SearchOption)
			{
				DirectoryInfo[] Directories = null;
				try
				{
					Directories = Directory.GetDirectories(Pattern, SearchOption);
				}
				catch (Exception ex)
				{
					if (ex.ToString().Contains("A retry should be performed"))
					{
						Log.Info("Retrying Directory.GetDirectories(Pattern, SearchOption) once.");
						Directories = Directory.GetDirectories(Pattern, SearchOption);
					}
					else
					{
						throw;
					}
				}
				return Directories;
			}


			/// <summary>
			/// Wrap DirectoryInfo.GetFileSystemInfos(Pattern, SearchOption) and retry once if exception is "A retry should be performed."
			/// </summary>
			/// <param name="Directory"></param>
			/// <param name="Pattern"></param>
			/// <param name="SearchOption"></param>
			/// <returns></returns>
			public static FileSystemInfo[] GetFileSystemInfos(DirectoryInfo Directory, string Pattern, SearchOption SearchOption)
			{
				FileSystemInfo[] FileInfos = null;
				try
				{
					FileInfos = Directory.GetFileSystemInfos(Pattern, SearchOption);
				}
				catch (Exception ex)
				{
					if (ex.ToString().Contains("A retry should be performed"))
					{
						Log.Info("Retrying Directory.GetFileSystemInfos(Pattern, SearchOption) once.");
						FileInfos = Directory.GetFileSystemInfos(Pattern, SearchOption);
					}
					else
					{
						throw;
					}
				}
				return FileInfos;
			}

			/// <summary>
			/// Deletes the specified directory
			/// </summary>
			/// <param name="Directory">The directory to delete</param>
			/// <param name="bRecursive">Whether or not subdirectories and it's contents should also be deleted</param>
			/// <param name="bForce">If true, a second attempt will me made at file deletion after setting file attributes to normal</param>
			public static void Delete(DirectoryInfo Directory, bool bRecursive, bool bForce = false)
			{
				try
				{
					Log.Info("Deleting directory {Directory}.", Directory);
					Directory.Delete(bRecursive);
				}
				catch (Exception Ex)
				{
					if (Ex.ToString().Contains("A retry should be performed"))
					{
						Log.Info("Retrying deletion once.");
						Directory.Delete(bRecursive);
					}
					else if(bForce)
					{
						Log.Info("Setting files in {Directory} to have normal attributes (no longer read-only) and retrying deletion.", Directory);
						Directory.Attributes = FileAttributes.Normal;

						foreach(FileSystemInfo Info in Directory.EnumerateFiles("*", SearchOption.AllDirectories))
						{
							Log.Info("Deleting file after attributes change: {FileName}", Info.FullName);

							try
							{
								Info.Attributes = FileAttributes.Normal;
							}
							catch (ArgumentException ArgEx)
							{
								Log.Warning("Could not set attributes for {File}. Error: {Error}", Info.FullName, ArgEx.Message);
								// Continue to deletion attempt - we'll try using extended-length path
							}

							// Try to delete the file, with retry logic for locked files
							bool bDeleted = false;
							for (int Attempt = 0; Attempt < 3 && !bDeleted; Attempt++)
							{
								try
								{
									// First try normal deletion
									Info.Delete();
									bDeleted = true;
								}
								catch (System.IO.IOException IoEx) when (IoEx.Message.Contains("because it is being used by another process"))
								{
									// Looking specifically for 
									if (Attempt < 2)
									{
										Log.Info("File {File} is locked. Forcing garbage collection and retrying... (Attempt {Attempt}/3)", Info.FullName, Attempt + 1);
										// Force garbage collection to run finalizers that might be holding file handles
										System.GC.Collect();
										System.GC.WaitForPendingFinalizers();
										System.GC.Collect();
										// Wait a bit for the OS to release the file handle
										System.Threading.Thread.Sleep(500);
									}
									else
									{
										// Final attempt failed, throw the exception
										throw;
									}
								}
								catch (Exception)
								{
									// Handle files with invalid Windows filenames using extended-length path
									Log.Warning("Attempting deletion with extended-length path: {File}", Info.FullName);
									if (Info.FullName.StartsWith(@"\\"))
									{
										// Network or already an extended-length path
										throw;
									}
									string ExtendedPath = @"\\?\" + Info.FullName;
									System.IO.File.Delete(ExtendedPath);
									bDeleted = true;
								}
							}
						}
					}
					else
					{
						Log.Warning("Failed to delete directory {Directory}!", Directory);
					}
				}
			}

			/// <summary>
			/// Deletes the specified file
			/// </summary>
			/// <param name="File">The file to delete</param>
			/// <param name="bForce">If true, a second attempt will me made at file deletion after setting file attributes to normal</param>
			public static void Delete(FileInfo File, bool bForce = false)
			{
				try
				{
					File.Delete();
				}
				catch (Exception Ex)
				{
					if (Ex.ToString().Contains("A retry should be performed"))
					{
						Log.Info("Retrying deletion once.");
						File.Delete();
					}
					else if (bForce)
					{
						Log.Info("Setting files {File} to have normal attributes (no longer read-only) and retrying deletion.", File);
						File.Attributes = FileAttributes.Normal;
						File.Delete(); // throw exceptions here because we have requested a force clean
					}
					else
					{
						Log.Warning("Failed to delete directory {Directory}!", File);
					}
				}
			}

			/// <summary>
			/// Return the fully qualified location of file path, including the Long path prefix if necessary
			/// </summary>
			/// <param name="FilePath"></param>
			/// <returns></returns>
			public static string GetFullyQualifiedPath(string FilePath)
			{
				FilePath = Path.GetFullPath(FilePath);
				if (!FilePath.StartsWith(@"\\") && FilePath.Length > 260)
				{
					return Globals.LongPathPrefix + FilePath;
				}
				return FilePath;
			}
		}

		public class Image
		{
			protected static IEnumerable<FileInfo> GetSupportedFilesAtPath(string InPath)
			{
				string[] Extensions = new[] { ".jpg", ".jpeg", ".png", ".bmp" };

				DirectoryInfo Di = new DirectoryInfo(InPath);

				var Files = SystemHelpers.GetFiles(Di).Where(f => Extensions.Contains(f.Extension.ToLower()));

				return Files;
			}


			public static bool SaveImagesAsGif(IEnumerable<string> FilePaths, string OutPath, uint Delay=100)
			{
				Log.Verbose("Turning {0} files into {1}", FilePaths.Count(), OutPath);
				try
				{
					using (MagickImageCollection Collection = new MagickImageCollection())
					{
						foreach (string File in FilePaths)
						{
							int Index = Collection.Count();
							Collection.Add(File);
							Collection[Index].AnimationDelay = Delay;
						}

						// Optionally reduce colors
						/*QuantizeSettings settings = new QuantizeSettings();
						settings.Colors = 256;
						Collection.Quantize(settings);*/

						foreach (MagickImage image in Collection)
						{
							image.Resize(640, 0);
						}

						// Optionally optimize the images (images should have the same size).
						Collection.Optimize();

						// Save gif
						Collection.Write(OutPath);

						Log.Verbose("Saved {0}", OutPath);
					}
				}
				catch (System.Exception Ex)
				{
					Log.Warning("SaveAsGif failed: {0}", Ex);
					return false;
				}

				return true;
			}

			public static bool SaveImagesAsGif(string InDirectory, string OutPath)
			{
				string[] Extensions = new[] { ".jpg", ".jpeg", ".png", ".bmp" };

				DirectoryInfo Di = new DirectoryInfo(InDirectory);

				var Files = GetSupportedFilesAtPath(InDirectory);

				// sort by creation time
				Files = Files.OrderBy(F => F.CreationTimeUtc);

				if (Files.Count() == 0)
				{
					Log.Info("Could not find files at {0} to Gif-ify", InDirectory);
					return false;
				}

				return SaveImagesAsGif(Files.Select(F => F.FullName), OutPath);
			}

			public static bool ResizeImages(string InDirectory, uint MaxWidth)
			{
				var Files = GetSupportedFilesAtPath(InDirectory);

				if (Files.Count() == 0)
				{
					Log.Verbose("Could not find files at {0} to resize", InDirectory);
					return false;
				}

				Log.Verbose("Reizing {0} files at {1} to have a max width of {2}", Files.Count(), InDirectory, MaxWidth);

				try
				{
					foreach (FileInfo File in Files)
					{
						using (MagickImage Image = new MagickImage(File.FullName))
						{
							if (Image.Width > MaxWidth)
							{
								Image.Resize(MaxWidth, 0);
								Image.Write(File);
							}
						}
					}				
				}
				catch (System.Exception Ex)
				{
					Log.Warning("ResizeImages failed: {0}", Ex);
					return false;
				}

				return true;
			}


			public static bool ConvertImages(string InDirectory, string OutDirectory, string OutExtension, bool DeleteOriginals)
			{
				var Files = GetSupportedFilesAtPath(InDirectory);

				if (Files.Count() == 0)
				{
					Log.Verbose("Could not find files at {0} to convert", InDirectory);
					return false;
				}

				Log.Verbose("Converting {0} files to {1}", Files.Count(), OutExtension);

				try
				{
					List<FileInfo> FilesToCleanUp = new List<FileInfo>();
					foreach (FileInfo File in Files)
					{
						using (MagickImage Image = new MagickImage(File.FullName))
						{
							string OutFile = Path.Combine(OutDirectory, File.Name);
							OutFile = Path.ChangeExtension(OutFile, OutExtension);
							// If we're trying to convert something to itself in place, skip the step.
							if (OutFile != File.FullName)
							{
								Image.Write(OutFile);
								if (DeleteOriginals)
								{
									FilesToCleanUp.Add(File);
								}
							}
						}
					}

					foreach (FileInfo File in FilesToCleanUp)
					{
						SystemHelpers.Delete(File);
					}
				}
				catch (Exception Ex)
				{
					Log.Warning("ConvertImages failed: {0}", Ex);
					try
					{
						if (DeleteOriginals)
						{
							Files.ToList().ForEach(F => SystemHelpers.Delete(F));
						}
					}
					catch (Exception e)
					{
						Log.Warning("Cleaning up original files failed: {0}", e);
					}
					return false;
				}

				return true;
			}
		}
	}

	public static class RegexUtil
	{
		public static bool MatchAndApplyGroups(string InContent, string RegEx, Action<string[]> InFunc)
		{
			return MatchAndApplyGroups(InContent, RegEx, RegexOptions.IgnoreCase, InFunc);
		}

		public static bool MatchAndApplyGroups(string InContent, string RegEx, RegexOptions Options, Action<string[]> InFunc)
		{
			Match M = Regex.Match(InContent, RegEx, Options);

			IEnumerable<string> StringMatches = null;

			if (M.Success)
			{
				StringMatches = M.Groups.Cast<Capture>().Select(G => G.ToString());
				InFunc(StringMatches.ToArray());
			}

			return M.Success;
		}
	}

	public static class DirectoryUtils
	{
		/// <summary>
		/// Enumerate files from a given directory that pass the specified regex
		/// </summary>
		/// <param name="BaseDir">Base directory to search in</param>
		/// <param name="Pattern">Pattern for matching files</param>
		/// <returns>Sequence of file references</returns>
		public static IEnumerable<string> FindFiles(string BaseDir, Regex Pattern)
		{
			IEnumerable<string> Files = System.IO.Directory.EnumerateFiles(BaseDir, "*");

			Files = Files.Where(F => Pattern.IsMatch(F));

			return Files.ToArray();
		}

		/// <summary>
		/// Returns true if any part of the specified path contains 'Component'. E.g. c:\Temp\Foo\Bar would return true for 'Foo'
		/// </summary>
		/// <param name="InPath">Path to search</param>
		/// <param name="InComponent">Component to look for</param>
		/// <param name="InPartialMatch">Whether a partial match is ok. E.g 'Fo' instead of 'Foo'</param>
		/// <returns></returns>
		public static bool PathContainsComponent(string InPath, string InComponent, bool InPartialMatch=false)
		{
			// normalize and split the path
			var Components = new DirectoryInfo(InPath).FullName.Split(Path.DirectorySeparatorChar);

			return InPartialMatch
				? Components.Any(S => S.Contains(InComponent, StringComparison.OrdinalIgnoreCase))
				: Components.Any(S => S.Equals(InComponent, StringComparison.OrdinalIgnoreCase));
		}

		/// <summary>
		/// Returns subdirectories that match the specified regular expression, searching up to the specified maximum depth
		/// </summary>
		/// <param name="BaseDir">Directory to start searching in</param>
		/// <param name="RegexPattern">Regular expression to match on</param>
		/// <param name="RecursionDepth">Depth to search (-1 = unlimited)</param>
		/// <returns></returns>
		public static IEnumerable<DirectoryInfo> FindMatchingDirectories(string BaseDir, string RegexPattern, int RecursionDepth=0)
		{
			List<DirectoryInfo> Found = new List<DirectoryInfo>();

			List<DirectoryInfo> Candidates = new List<DirectoryInfo>(Utils.SystemHelpers.GetDirectories(new DirectoryInfo(BaseDir)));

			Regex Pattern = new Regex(RegexPattern, RegexOptions.IgnoreCase);

			int CurrentDepth = 0;

			do
			{
				IEnumerable<DirectoryInfo> MatchingDirs = Candidates.Where(D => Pattern.IsMatch(D.Name));

				Found.AddRange(MatchingDirs);

				// recurse
				Candidates = Candidates.SelectMany(D => Utils.SystemHelpers.GetDirectories(D)).ToList();

			} while (Candidates.Any() && (CurrentDepth++ < RecursionDepth || RecursionDepth == -1)); ;

			return Found;
		}

		/// <summary>
		/// Returns files, that match the specified regular expression, ignoring case, in the provided folder,
		/// searching up to a maximum depth
		/// </summary>
		/// <param name="BaseDir">Directory to start searching in</param>
		/// <param name="RegexPattern">Regular expression pattern to match on</param>
		/// <param name="RecursionDepth">Optional depth to search (-1 = unlimited)</param>
		/// <returns></returns>
		public static IEnumerable<FileInfo> FindMatchingFiles(string BaseDir, string RegexPattern, int RecursionDepth = 0)
		{
			Regex RegExObj = new Regex(RegexPattern, RegexOptions.IgnoreCase);

			return MatchFiles(BaseDir, RegExObj, RecursionDepth);
		}

		/// <summary>
		/// Returns files that match a specified regular expression in the provided folder, searching up to a maximum depth
		/// </summary>
		/// <param name="BaseDir">Directory to start searching in</param>
		/// <param name="RegExObj">RegEx to match on</param>
		/// <param name="RecursionDepth">Optional depth to search (-1 = unlimited)</param>
		/// <returns></returns>
		public static IEnumerable<FileInfo> FindMatchingFiles(string BaseDir, Regex RegExObj, int RecursionDepth = 0)
		{
			return MatchFiles(BaseDir, RegExObj, RecursionDepth);
		}

		/// <summary>
		/// Returns files that match the given RegEx from the provided folder, searching up to a maximum depth
		/// </summary>
		/// <param name="BaseDir">Directory to start searching in</param>
		/// <param name="RegExObj">RegEx to match on</param>
		/// <param name="RecursionDepth">Depth to search (-1 = unlimited)</param>
		/// <returns></returns>
		private static IEnumerable<FileInfo> MatchFiles(string BaseDir, Regex RegExObj, int RecursionDepth)
		{
			List<FileInfo> Found = new List<FileInfo>();

			IEnumerable<DirectoryInfo> CandidateDirs = new DirectoryInfo[] { new DirectoryInfo(BaseDir) };

			int CurrentDepth = 0;

			do
			{
				// check for matching files in this set of directories
				IEnumerable<FileInfo> MatchingFiles = CandidateDirs.SelectMany(D => Utils.SystemHelpers.GetFiles(D)).Where(F => RegExObj.IsMatch(F.Name));

				Found.AddRange(MatchingFiles);

				// recurse into this set of directories
				CandidateDirs = CandidateDirs.SelectMany(D => Utils.SystemHelpers.GetDirectories(D)).ToList();

			} while (CandidateDirs.Any() && (CurrentDepth++ < RecursionDepth || RecursionDepth == -1)); ;

			return Found;
		}
	}

	public static class FileUtils
	{
		/// <summary>
		/// Sanitize filename
		/// </summary>
		/// <param name="Name"></param>
		/// <returns></returns>
		static public string SanitizeFilename(string Name)
		{
			return Regex.Replace(Name, @"[^a-z0-9_\-.]+", "_", RegexOptions.IgnoreCase);
		}

		/// <summary>
		/// Convert file path to Unix-style path
		/// </summary>
		/// <param name="Path"></param>
		/// <returns></returns>
		static public string ConvertPathToUnix(string Path)
		{
			return Path.Replace("\\", "/");
		}
	}

	public static class ProcessUtils
	{
		/// <summary>
		/// Create a text file with a header listing Gauntlet Test and command line used.
		/// Return the StreamWriter instance used to create the file.
		/// Intended to be used for log file.
		/// </summary>
		/// <param name="FilePath"></param>
		/// <param name="Commandline"></param>
		/// <returns></returns>
		static public StreamWriter CreateWriterForProcessLog(string FilePath, string Commandline)
		{
			StreamWriter Writer = File.CreateText(FilePath);
			Writer.WriteLine("------ Gauntlet Test ------");
			Writer.WriteLine($"Command Line: {Commandline}");
			Writer.WriteLine("---------------------------");

			return Writer;
		}

		/// <summary>
		/// Write output log from IAppInstance if related instance log size is under size limit.
		/// </summary>
		/// <param name="AppInstance"></param>
		/// <param name="FilePath"></param>
		/// <param name="Commandline"></param>
		/// <returns></returns>
		static public bool WriteOutputLog(IAppInstance AppInstance, string FilePath, string Commandline)
		{
			CheckProcessLogReachedSizeLimit(AppInstance);
			StreamWriter Writer = CreateWriterForProcessLog(FilePath, Commandline);
			Writer.Write(AppInstance.StdOut);
			Writer.Close();
			return true;
		}

		/// <summary>
		/// Get log size limit for log file in bytes. Return 0 if -NoMaxLogSize is used. 
		/// </summary>
		/// <returns></returns>
		static public long GetProcessLogSizeLimit()
		{
			return Globals.Params.ParseParam("NoMaxLogSize") ? 0 : 1024 * 1024 * 1024;
		}

		static public void CheckProcessLogReachedSizeLimit(IAppInstance AppInstance)
		{
			long MaxLogSize = GetProcessLogSizeLimit();
			if (MaxLogSize > 0)
			{
				long LogSize = AppInstance.StdOut.Length * sizeof(char);
				if (LogSize > MaxLogSize)
				{
					throw new AutomationException("Log reached 1Gb size limit.");
				}
			}
		}

		/// <summary>
		/// Check if LogFile size reached limit. Throw AutomationException if reached.
		/// </summary>
		/// <param name="LogFile"></param>
		/// <exception cref="AutomationException"></exception>
		static public void CheckProcessLogReachedSizeLimit(FileReference LogFile)
		{
			long MaxLogSize = GetProcessLogSizeLimit();
			if (MaxLogSize > 0)
			{
				long LogSize = LogFile.ToFileInfo().Length;
				if (LogSize > MaxLogSize)
				{
					throw new AutomationException("Log reached 1Gb size limit.");
				}
			}
		}

		/// <summary>
		/// Get CircularLogBuffer line capacity limit.
		/// </summary>
		/// <returns></returns>
		static public int GetCircularLogBufferCapacityLimit()
		{
			return Globals.Params.ParseValue("Gauntlet.LogBufferLineCapacity", 1024);
		}

		/// <summary>
		/// Create CircularLogBuffer using Gauntlet default buffer line limit
		/// </summary>
		/// <returns></returns>
		static public CircularLogBuffer CreateLogBuffer()
		{
			return new CircularLogBuffer(GetCircularLogBufferCapacityLimit());
		}


		/// <summary>
		/// Try to get a log file. Return the available file path.
		/// </summary>
		/// <param name="AppName"></param>
		/// <returns></returns>
		static public string GetLogFilePath(string AppName, string LocalCache)
		{
			string LogFilenameBase = String.Format("{0}_{1}", AppName, DateTime.Now.ToString("yyyy.MM.dd-HH.mm.ss"));
			for (int Attempt = 1; Attempt < 100; ++Attempt)
			{
				if (!Directory.Exists(LocalCache) && !Directory.CreateDirectory(LocalCache).Exists)
				{
					break;
				}
				string LogFilenameBaseToCreate = LogFilenameBase;
				if (Attempt > 1)
				{
					LogFilenameBaseToCreate += "_" + Attempt;
				}
				LogFilenameBaseToCreate += ".log";
				string LogFilenameToCreate = Path.Combine(LocalCache, LogFilenameBaseToCreate);
				if (File.Exists(LogFilenameToCreate))
				{
					continue;
				}

				return LogFilenameToCreate;
			}

			return string.Empty;
		}

		/// <summary>
		/// Local logs cache
		/// </summary>
		static public string LocalLogsPath => Path.Combine(Globals.TempDir, "DeviceCache", "Logs");
	}

	/// <summary>
	/// Interface for stream reader used in context of log
	/// </summary>
	public interface ILogStreamReader
	{
		/// <summary>
		/// Set desired line index to start reading line
		/// </summary>
		/// <param name="LineIndex"></param>
		void SetLineIndex(int LineIndex);

		/// <summary>
		/// Get line index used to read next line
		/// </summary>
		/// <returns></returns>
		int GetLineIndex();

		/// <summary>
		/// Get next line at line index and increment line index by one.
		/// Return null if no line is available.
		/// </summary>
		/// <returns></returns>
		string GetNextLine();

		/// <summary>
		/// Return full content of available underlying stream.
		/// This is for backward compatibility. It is not efficient and defy the purpose of using a stream reader.
		/// </summary>
		/// <returns></returns>
		string GetContent();

		/// <summary>
		/// Generate an enumerator that iterates all available lines from last line index (and update it).
		/// </summary>
		/// <returns></returns>
		IEnumerable<string> EnumerateNextLines();

		/// <summary>
		/// Get available line count from the underlying stream
		/// </summary>
		/// <returns></returns>
		int GetAvailableLineCount();

		/// <summary>
		/// Clone the stream reader including current line index
		/// </summary>
		/// <returns></returns>
		ILogStreamReader Clone();
	}

	/// <summary>
	/// Provide a ILogStreamReader from a string using a callback to pull that string dynamically.
	/// Used in conjunction with IProcessResult to read process output.
	/// </summary>
	public class DynamicStringReader : ILogStreamReader
	{
		private Func<string> _getLog;
		private int _lineIndex = 0;
		private int _targetLineIndex = 0;
		private int _contentCharIndex = 0;

		public DynamicStringReader(Func<string> Callback)
		{
			_lineIndex = 0;
			_targetLineIndex = 0;
			_contentCharIndex = 0;
			_getLog = Callback;
		}

		public void SetLineIndex(int LineIndex)
		{
			if (LineIndex < 0)
			{
				throw new AutomationException("DynamicStringReader does not support negative indexing.");
			}

			_targetLineIndex = LineIndex;
		}

		public int GetLineIndex() => _targetLineIndex;

		public string GetNextLine()
		{
			string StdOut = _getLog();
			if (_targetLineIndex != _lineIndex)
			{
				SkipToTargetIndex(StdOut);
			}

			if (_contentCharIndex >= StdOut.Length)
			{
				return null;
			}

			int EndIdx = StdOut.IndexOf('\n', _contentCharIndex);
			if (EndIdx == -1)
			{
				EndIdx = StdOut.Length;
			}
			string Line = StdOut.Substring(_contentCharIndex, EndIdx - _contentCharIndex).TrimEnd('\r');
			_contentCharIndex = EndIdx + 1;
			_targetLineIndex = ++_lineIndex;
			return Line;
		}

		private void SkipToTargetIndex(string StdOut)
		{
			// Check if _lineIndex needs to skip to _targetIndex
			if (_targetLineIndex < _lineIndex)
			{
				_lineIndex = 0;
				_contentCharIndex = 0;
			}

			while (_contentCharIndex < StdOut.Length && _lineIndex < _targetLineIndex)
			{
				int NextIdx = StdOut.IndexOf('\n', _contentCharIndex);
				if (NextIdx == -1)
				{
					NextIdx = StdOut.Length;
				}
				_contentCharIndex = NextIdx + 1;
				_lineIndex++;
			}
			_targetLineIndex = _lineIndex;
		}

		public IEnumerable<string> EnumerateNextLines()
		{
			string Line = GetNextLine();
			while (Line != null)
			{
				yield return Line;
				Line = GetNextLine();
			}
		}

		public string GetContent() => _getLog();

		public int GetAvailableLineCount() => _getLog().Count(C => C == '\n');

		public ILogStreamReader Clone()
		{
			DynamicStringReader Clone = new DynamicStringReader(_getLog);
			Clone.SetLineIndex(GetLineIndex());
			return Clone;
		}
	}

	/// <summary>
	/// Provide a ILogStreamReader instance from a text file
	/// </summary>
	public class LogFileReader : ILogStreamReader, IDisposable
	{
		private string _filePath;
		private StreamReader _stream;
		private FileStream _file;
		private int _lineIndex;
		private int _targetLineIndex;

		public LogFileReader(string FilePath)
		{
			_filePath = FilePath;
			_file = File.Open(FilePath, FileMode.Open, FileAccess.Read, FileShare.ReadWrite);
			_stream = new StreamReader(_file);
			_lineIndex = 0;
			_targetLineIndex = 0;
		}

		~LogFileReader()
		{
			Dispose();
		}

		public void Dispose()
		{
			_stream.Dispose();
			_file.Dispose();
		}

		public void SetLineIndex(int Index)
		{
			_targetLineIndex = Index;
		}

		public int GetLineIndex() => _targetLineIndex;

		public string GetNextLine()
		{
			if (_targetLineIndex < _lineIndex)
			{
				_stream.DiscardBufferedData();
				_stream.BaseStream.Seek(0, SeekOrigin.Begin);
				_lineIndex = 0;
			}
			while(_lineIndex < _targetLineIndex)
			{
				if (_stream.ReadLine() != null)
				{
					_lineIndex++;
				}
				else
				{
					_targetLineIndex = _lineIndex;
				}
			}
			string Line = _stream.ReadLine();
			if (Line != null)
			{
				_targetLineIndex = ++_lineIndex;
			}
			return Line;
		}

		public string GetContent()
		{
			long CurrentCharIndex = _stream.BaseStream.Position;
			if (CurrentCharIndex != 0)
			{
				_stream.DiscardBufferedData();
				_stream.BaseStream.Seek(0, SeekOrigin.Begin);
			}
			string Content = _stream.ReadToEnd();
			_stream.DiscardBufferedData();
			_stream.BaseStream.Seek(CurrentCharIndex, SeekOrigin.Begin);
			return Content;
		}

		public IEnumerable<string> EnumerateNextLines()
		{
			string Line = GetNextLine();
			while (Line != null)
			{
				yield return Line;
				Line = GetNextLine();
			}
		}

		public int GetAvailableLineCount()
		{
			long CurrentCharIndex = _stream.BaseStream.Position;
			_stream.DiscardBufferedData();
			_stream.BaseStream.Seek(0, SeekOrigin.Begin);
			int LineCount = 0;
			int Char = '\0';
			do
			{
				Char = _stream.Read();
				if (Char == '\n')
				{
					LineCount++;
				}
			} while (Char != -1);
			_stream.DiscardBufferedData();
			_stream.BaseStream.Seek(CurrentCharIndex, SeekOrigin.Begin);
			return LineCount;
		}

		public ILogStreamReader Clone()
		{
			LogFileReader Clone = new LogFileReader(_filePath);
			Clone.SetLineIndex(GetLineIndex());
			return Clone;
		}
	}

	/// <summary>
	/// Provide a ILogStreamReader instance from a CircularLogBuffer
	/// </summary>
	public class LogBufferReader : ILogStreamReader
	{
		private int _lineIndex = 0;
		private CircularLogBuffer _buffer;

		public LogBufferReader(CircularLogBuffer Buffer)
		{
			_lineIndex = Buffer.GetFirstAvailableLineIndex();
			_buffer = Buffer;
		}

		public void SetLineIndex(int LineIndex)
		{
			if (LineIndex < 0)
			{
				_lineIndex = _buffer.GetLastLineIndex() + 1 + LineIndex;
			}
			else
			{
				_lineIndex = Math.Min(LineIndex, _buffer.GetLastLineIndex() + 1);
			}
		}

		public int GetLineIndex() => _lineIndex;

		public string GetNextLine()
		{
			_lineIndex = Math.Max(_buffer.GetFirstAvailableLineIndex(), _lineIndex);
			string Line = _buffer.GetLine(_lineIndex);
			if (Line != null)
			{
				_lineIndex++;
			}
			return Line;
		}

		public IEnumerable<string> EnumerateNextLines()
		{
			string Line = GetNextLine();
			while (Line != null)
			{
				yield return Line;
				Line = GetNextLine();
			}
		}

		public string GetContent() => _buffer.GetContent();

		public int GetAvailableLineCount() => _buffer.GetAvailableLineCount();

		public ILogStreamReader Clone()
		{
			LogBufferReader Clone = new LogBufferReader(_buffer);
			Clone.SetLineIndex(GetLineIndex());
			return Clone;
		}
	}

	/// <summary>
	/// Circular log buffer separating input as line. Capacity being based on lines.
	/// </summary>
	public class CircularLogBuffer : IEnumerable<string>
	{
		private string[] _list;
		private int _capacity = 0;
		private int _lineIndex = -1;
		private ReaderWriterLockSlim _lock = new ReaderWriterLockSlim();

		public CircularLogBuffer(int Capacity)
		{
			if (Capacity <= 0)
			{
				throw new AutomationException("CircularLogBuffer capacity must be larger than 0.");
			}

			_list = new string[Capacity];
			_capacity = Capacity;
			_lineIndex = -1;
		}

		/// <summary>
		/// Allow to feed the log buffer with text content. The input is parsed and separated into lines.
		/// </summary>
		/// <param name="Content"></param>
		/// <returns></returns>
		public CircularLogBuffer Feed(string Content)
		{
			for (int BaseIdx = 0; BaseIdx < Content.Length;)
			{
				int EndIdx = Content.IndexOf('\n', BaseIdx);
				if (EndIdx == -1)
				{
					EndIdx = Content.Length;
				}
				AppendLine(Content.Substring(BaseIdx, EndIdx - BaseIdx).TrimEnd('\r'));
				BaseIdx = EndIdx + 1;
			}

			return this;
		}

		/// <summary>
		/// Append input line to the log buffer
		/// </summary>
		/// <param name="Item"></param>
		/// <returns></returns>
		public CircularLogBuffer AppendLine(string Item)
		{
			_lock.EnterWriteLock();
			try
			{
				_lineIndex++;
				_list[_lineIndex % _capacity] = Item;
			}
			finally
			{
				_lock.ExitWriteLock();
			}

			return this;
		}

		/// <summary>
		/// Get last line index input
		/// </summary>
		/// <returns></returns>
		public int GetLastLineIndex()
		{
			return _lineIndex;
		}

		/// <summary>
		/// Get the first/oldest available line index.
		/// If the capacity was reached, the first lines are lost,
		/// the returned line index will be that first available line
		/// </summary>
		/// <returns></returns>
		public int GetFirstAvailableLineIndex()
		{
			return _lineIndex < _capacity ? 0 : _lineIndex - _capacity + 1;
		}

		/// <summary>
		/// Available line count.
		/// If the capacity was reached, the capacity will be returned.
		/// </summary>
		/// <returns></returns>
		public int GetAvailableLineCount()
		{
			return _lineIndex < _capacity ? _lineIndex + 1 : _capacity;
		}

		/// <summary>
		/// Get line at index
		/// </summary>
		/// <param name="LineIndex"></param>
		/// <returns></returns>
		public string GetLine(int LineIndex)
		{
			if (LineIndex < 0)
			{
				LineIndex = _lineIndex + 1 + LineIndex;
			}

			if (_lineIndex < 0 || LineIndex > _lineIndex || LineIndex < GetFirstAvailableLineIndex())
			{
				return null;
			}

			_lock.EnterReadLock();
			try
			{
				return _list[LineIndex % _capacity];
			}
			finally
			{
				_lock.ExitReadLock();
			}
		}

		/// <summary>
		/// Return an instance of LogBufferReader
		/// </summary>
		/// <returns></returns>
		public LogBufferReader GetReader()
		{
			return new LogBufferReader(this);
		}

		/// <summary>
		/// Return an enumerator that iterate all available lines
		/// </summary>
		/// <returns></returns>
		public IEnumerator<string> GetEnumerator()
		{
			if (_lineIndex == -1)
			{
				yield break;
			}

			int Cursor = _lineIndex < _capacity ? -1 : _lineIndex;
			// If the circular buffer is at-capacity, we start at cursor index otherwise -1
			// Then increment by 1, until circled back to the actual cursor index.
			do
			{
				_lock.EnterReadLock();
				try
				{
					Cursor = (Cursor + 1) % _capacity;
					yield return _list[Cursor];
				}
				finally
				{
					_lock.ExitReadLock();
				}
			} while (Cursor != _lineIndex % _capacity);
		}

		/// <summary>
		/// Return an enumerator that iterate all available lines
		/// </summary>
		/// <returns></returns>
		System.Collections.IEnumerator System.Collections.IEnumerable.GetEnumerator()
		{
			return this.GetEnumerator();
		}

		/// <summary>
		/// Return the full available content as a string
		/// </summary>
		/// <returns></returns>
		public string GetContent()
		{
			return string.Join('\n', this);
		}
	}

	/// <summary>
	/// Logger to file with interface to get a log reader
	/// </summary>
	public class FileLogger
	{
		private CircularLogBuffer _buffer;
		private string _filepath;
		private string _commandline;
		private StreamWriter _writer;

		/// <summary>
		/// Instantiate Logger to file with interface to get a log reader
		/// </summary>
		/// <param name="FilePath">Target log file path</param>
		/// <param name="CommandLine">Command line that producing the log output</param>
		/// <param name="bKeepStreamOpen">If true, the stream writer is kept open until closed through a call to StreamClose(). The logger can still append lines when the stream is closed.</param>
		public FileLogger(string FilePath, string CommandLine, bool bKeepStreamOpen = true)
		{
			_buffer = ProcessUtils.CreateLogBuffer();
			_filepath = FilePath;
			_commandline = CommandLine;
			_writer = null;
			try
			{
				_writer = ProcessUtils.CreateWriterForProcessLog(FilePath, CommandLine);
				if (!bKeepStreamOpen)
				{
					_writer.Close();
					_writer = null;
				}
			}
			catch (IOException Ex)
			{
				Log.Warning("Could not write log file '{Filename}'.\n {Exception}", _filepath, Ex);
				_filepath = null;
			}
		}

		~FileLogger()
		{
			CloseStream();
		}

		public void CloseStream()
		{
			if (_writer != null)
			{
				_writer.Close();
				_writer = null;
			}
		}

		public void AppendLine(string Data)
		{
			if (!string.IsNullOrEmpty(_filepath))
			{
				try
				{
					if (_writer == null)
					{
						File.AppendAllText(_filepath, Data + '\n');
					}
					else
					{
						_writer.Write(Data + '\n');
						_writer.Flush();
					}
				}
				catch (Exception Ex)
				{
					Log.Warning("Could not write in log file '{Filename}'.\n {Exception}", _filepath, Ex);
					_filepath = null;
				}
			}

			foreach (string Line in Data.Split('\n'))
			{
				_buffer.AppendLine(Line.TrimEnd('\r'));
			}
		}

		public ILogStreamReader GetReader()
		{
			if (string.IsNullOrEmpty(_filepath) || !File.Exists(_filepath))
			{
				return GetBufferReader();
			}

			return new LogFileReader(_filepath);
		}

		public ILogStreamReader GetBufferReader()
		{
			return _buffer.GetReader();
		}

		public void CopyFile(string FilePath)
		{
			FileReference NewFilePathRef = new FileReference(FilePath);
			if (FilePath != _filepath)
			{
				if (!Directory.Exists(NewFilePathRef.Directory.FullName))
				{
					Directory.CreateDirectory(NewFilePathRef.Directory.FullName);
				}
				if (string.IsNullOrEmpty(_filepath))
				{
					StreamWriter Writer = ProcessUtils.CreateWriterForProcessLog(NewFilePathRef.FullName, _commandline);
					foreach (string Line in _buffer)
					{
						Writer.WriteLine(Line);
					}
					Writer.Close();
					_filepath = NewFilePathRef.FullName;
				}
				else
				{
					ProcessUtils.CheckProcessLogReachedSizeLimit(new FileReference(_filepath));
					File.Copy(_filepath, NewFilePathRef.FullName, true);
				}
			}
		}
	}

	/// <summary>
	/// Extend IProcessResult to add log and buffer readers
	/// </summary>
	public interface ILongProcessResult : IProcessResult
	{
		/// <summary>
		/// Return an ILogStreamReader using the log file as a source
		/// </summary>
		/// <returns></returns>
		ILogStreamReader GetLogReader();

		/// <summary>
		/// Return an ILogStreamReader using the log buffer as a source
		/// </summary>
		/// <returns></returns>
		ILogStreamReader GetLogBufferReader();

		/// <summary>
		/// Append string to the log output
		/// </summary>
		/// <param name="InData"></param>
		/// <param name="IsStdErr"></param>
		void AppendToOutput(string InData, bool IsStdErr);
	}

	/// <summary>
	/// Represent a process running for a long time.
	/// Output is redirected to a circular log buffer unless ERunOptions.NoStdOutRedirect is set in the options.
	/// An output filter callback can be set to modify process output.
	/// </summary>
	public class LongProcessResult : ILongProcessResult
	{
		/// <summary>
		/// Delegate to filter output. If the output is coming from StdErr, bIsStdErr is true.
		/// If the return value is null, the line is skipped in the final output.
		/// </summary>
		/// <param name="Message"></param>
		/// <param name="bIsStdErr"></param>
		/// <returns></returns>
		public delegate string OutputFilterCallbackType(string Message, bool bIsStdErr);

		public int ExitCode { get; set; }

		public bool bExitCodeSuccess => ExitCode == 0;

		private Process LocalProcess { get; set; }

		private string AppName { get; set; }

		private string CommandLine { get; set; }

		private FileLogger Logger { get; set; }

		private string LocalCachePath { get; set; }

		private bool bOutputToConsole { get; set; }

		private object ProcSyncObject;
		private object OutputSyncObject;

		private AutoResetEvent StdOutCloseHandle = new AutoResetEvent(false);
		private AutoResetEvent StdErrCloseHandle = new AutoResetEvent(false);

		private OutputFilterCallbackType OutputFilterCallback;

		public LongProcessResult(string Command, string Args, ERunOptions Options, OutputFilterCallbackType OutputCallback = null, string WorkingDir = null, string LocalCache = null)
		{
			ExitCode = -1;
			Logger = null;
			LocalCachePath = string.IsNullOrEmpty(LocalCache)? ProcessUtils.LocalLogsPath : Path.Combine(LocalCache, "Logs");
			ProcSyncObject = new object();
			OutputSyncObject = new object();
			OutputFilterCallback = OutputCallback;
			ProcessStartInfo StartInfo;
			if (OperatingSystem.IsWindows())
			{
				StartInfo = new ProcessStartInfo(Command, Args);
			}
			else
			{
				StartInfo = new ProcessStartInfo(Command, GauntletCommandLine.GetArgumentList(Args));
			}

			if (WorkingDir != null)
			{
				StartInfo.WorkingDirectory = WorkingDir;
			}
			AppName = Path.GetFileNameWithoutExtension(StartInfo.FileName);
			CommandLine = $"{Path.GetFileName(StartInfo.FileName)} {Args}";
			if (!Options.HasFlag(ERunOptions.NoLoggingOfRunCommand))
			{
				Log.Info("Running: {CommandLine}", CommandLine);
			}

			LocalProcess = new Process();
			LocalProcess.StartInfo = StartInfo;
			bOutputToConsole = Options.HasFlag(ERunOptions.AllowSpew);
			if (!Options.HasFlag(ERunOptions.NoStdOutRedirect))
			{
				TryCreateLogFile();
				LocalProcess.StartInfo.RedirectStandardOutput = true;
				LocalProcess.StartInfo.RedirectStandardError = true;
				LocalProcess.StartInfo.StandardOutputEncoding = new UTF8Encoding(false, false);
				LocalProcess.OutputDataReceived += StdOut;
				LocalProcess.ErrorDataReceived += StdErr;
			}

			try
			{
				LocalProcess.Start();
				if (!Options.HasFlag(ERunOptions.NoStdOutRedirect))
				{
					LocalProcess.BeginOutputReadLine();
					LocalProcess.BeginErrorReadLine();
				}
			}
			catch (Exception ex)
			{
				throw new AutomationException(ex, "Failed to start local process for action (\"{0}\"): {1} {2}", ex.Message, LocalProcess.StartInfo.FileName, LocalProcess.StartInfo.Arguments);
			}

			RegisterProcessExit();
		}

		private void TryCreateLogFile()
		{
			string LogFilename = ProcessUtils.GetLogFilePath(AppName, LocalCachePath);
			if (!string.IsNullOrEmpty(LogFilename))
			{
				Logger = new FileLogger(LogFilename, CommandLine);
			}
		}

		public void StdOut(object sender, DataReceivedEventArgs Args)
		{
			if (Args.Data != null)
			{
				AppendToOutput(Args.Data, false);
			}
			else
			{
				StdOutCloseHandle.Set();
			}
		}

		public void StdErr(object sender, DataReceivedEventArgs Args)
		{
			if (Args.Data != null)
			{
				AppendToOutput(Args.Data, true);
			}
			else
			{
				StdErrCloseHandle.Set();
			}
		}

		public void AppendToOutput(string InData, bool IsStdErr)
		{
			lock (OutputSyncObject)
			{
				string Data = OutputFilterCallback != null ? OutputFilterCallback(InData, IsStdErr) : InData;
				if (Data != null)
				{
					Logger?.AppendLine(Data);
					if(bOutputToConsole)
					{
						EpicGames.Core.Log.WriteLine(LogEventType.Console, Data);
					}
				}
			}
		}

		public string Output => Logger?.GetReader().GetContent();

		public ILogStreamReader GetLogReader() => Logger?.GetReader();

		public ILogStreamReader GetLogBufferReader() => Logger?.GetBufferReader();

		public FileReference WriteOutputToFile(string FilePath)
		{
			if (Logger == null)
			{
				return null;
			}

			Logger.CopyFile(FilePath);

			return new FileReference(FilePath);
		}

		public bool HasExited
		{
			get
			{
				bool bHasExited = true;
				lock (ProcSyncObject)
				{
					if (LocalProcess != null)
					{
						bHasExited = LocalProcess.HasExited;
						if (bHasExited)
						{
							lock (OutputSyncObject)
							{
								Logger?.CloseStream();
							}
							ExitCode = LocalProcess.ExitCode;
						}
					}
				}
				return bHasExited;
			}
		}

		public Process ProcessObject
		{
			get { return LocalProcess; }
		}

		public void DisposeProcess()
		{
			if (LocalProcess != null)
			{
				LocalProcess.Dispose();
				LocalProcess = null;
			}
		}

		~LongProcessResult()
		{
			DisposeProcess();
			Logger?.CloseStream();
		}

		public string GetProcessName()
		{
			if (LocalProcess != null)
			{
				return LocalProcess.ProcessName;
			}

			return AppName;
		}

		public void StopProcess(bool KillDescendants = true)
		{
			if (LocalProcess != null && !HasExited)
			{
				Process ProcToKill = null;
				lock (ProcSyncObject)
				{
					ProcToKill = LocalProcess;
					LocalProcess = null;
				}
				string ProcessName = ProcToKill.ProcessName;
				try
				{
					ProcToKill.Kill(KillDescendants);
					ProcToKill.WaitForExit(60000);
					if (!ProcToKill.HasExited)
					{
						Log.Verbose("Process {ProcessName} failed to exit.", ProcessName);
					}
					else
					{
						ExitCode = ProcToKill.ExitCode;
						Log.Verbose("Process {ProcessName} successfully exited.", ProcessName);
						OnProcessExited();
					}
					ProcToKill.Close();
				}
				catch (Exception Ex)
				{
					Log.Warning("Exception while trying to kill process {ProcessName}:\n{Exception}", ProcessName, LogUtils.FormatException(Ex));
				}
			}
		}

		private void Kill(bool KillDescendants = true)
		{
			if (LocalProcess != null && !HasExited)
			{
				Process ProcToKill = null;
				lock (ProcSyncObject)
				{
					ProcToKill = LocalProcess;
					LocalProcess = null;
				}
				ProcToKill.Kill(KillDescendants);
			}
		}

		private void ProcessExit(object sender, EventArgs e)
		{
			Kill();
		}

		public void OnProcessExited()
		{
			AppDomain Domain = AppDomain.CurrentDomain;
			Domain.ProcessExit -= ProcessExit;
			Domain.DomainUnload -= ProcessExit;
		}

		private void RegisterProcessExit()
		{
			AppDomain Domain = AppDomain.CurrentDomain;
			Domain.ProcessExit += ProcessExit;
			Domain.DomainUnload += ProcessExit;
		}

		public void WaitForExit()
		{
			if ((LocalProcess != null) && !LocalProcess.HasExited)
			{
				Process Proc = null;
				lock (ProcSyncObject)
				{
					Proc = LocalProcess;
				}
				Proc.WaitForExit();
				ExitCode = Proc.ExitCode;
				lock (ProcSyncObject)
				{
					LocalProcess = null;
				}
			}

			// Wait for outputs closure
			int WaitTimeout = 10000;
			if (!(StdOutCloseHandle.WaitOne(WaitTimeout) && StdErrCloseHandle.WaitOne(WaitTimeout)))
			{
				Log.Info("Outputs did not close in time after process {ProcessName} exited.", LocalProcess.ProcessName);
			}
			lock (OutputSyncObject)
			{
				Logger?.CloseStream();
			}
		}
	}
}
