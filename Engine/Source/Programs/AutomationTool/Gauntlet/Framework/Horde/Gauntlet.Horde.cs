// Copyright Epic Games, Inc. All Rights Reserved.

using AutomationTool;
using EpicGames.Horde.Jobs;
using System;
using System.Collections.Generic;
using System.IO;
using System.Linq;
using System.Text;
using System.Text.Json;
using System.Text.Json.Serialization;

#nullable enable

namespace Gauntlet
{
	public static class Horde
	{

		static public bool IsHordeJob
		{
			get
			{
				return !string.IsNullOrEmpty(JobId);
			}
		}

		static public string? JobId
		{
			get
			{
				return Environment.GetEnvironmentVariable("UE_HORDE_JOBID");
			}
		}

		static public string? StepId
		{
			get
			{
				return Environment.GetEnvironmentVariable("UE_HORDE_STEPID");
			}
		}

		static public string? StepName
		{
			get
			{
				return Environment.GetEnvironmentVariable("UE_HORDE_STEPNAME");
			}
		}

		public enum JobStepMessageLevels
		{
			None,
			Info,
			Warning,
			Error
		}

		public class HordeStepMessage
		{
			public JobStepMessageLevels Severity { get; private set; }
			public string Message { get; private set; }
			public HordeStepMessage(string InMessage, JobStepMessageLevels InSeverity)
			{
				Message = InMessage;
				Severity = InSeverity;
			}
		}
		

		public partial class GauntletHordeStepMessage
		{
			public static HordeStepMessage TestStepMessage = new HordeStepMessage("TestCategory", JobStepMessageLevels.Error);

			// Failures unrelated to game functionality
			public static HordeStepMessage CannotAccessBuild = new HordeStepMessage("CannotAccessBuild", JobStepMessageLevels.Error);
			public static HordeStepMessage NotEnoughDriveSpace = new HordeStepMessage("NotEnoughDriveSpace", JobStepMessageLevels.Error);
			public static HordeStepMessage InvalidBuildSource = new HordeStepMessage("InvalidBuildSource", JobStepMessageLevels.Error);
			public static HordeStepMessage InvalidPerfSpec = new HordeStepMessage("InvalidPerfSpec", JobStepMessageLevels.Error);
			public static HordeStepMessage InstallationFailed = new HordeStepMessage("InstallationFailed", JobStepMessageLevels.Error);
			public static HordeStepMessage InvalidConfig = new HordeStepMessage("InvalidConfig", JobStepMessageLevels.Error);
			public static HordeStepMessage NotEnoughDevices = new HordeStepMessage("NotEnoughDevices", JobStepMessageLevels.Error);
			public static HordeStepMessage BuildNotFound = new HordeStepMessage("BuildNotFound", JobStepMessageLevels.Error);
			public static HordeStepMessage BuildNotSpecified = new HordeStepMessage("BuildNotSpecified", JobStepMessageLevels.Error);
			public static HordeStepMessage FailedToStart = new HordeStepMessage("FailedToStart", JobStepMessageLevels.Error);
			public static HordeStepMessage InvalidBackend = new HordeStepMessage("InvalidBackend", JobStepMessageLevels.Error);
			public static HordeStepMessage WrongAccountUsed = new HordeStepMessage("WrongAccountUsed", JobStepMessageLevels.Error);
			public static HordeStepMessage TestCancelled = new HordeStepMessage("TestCancelled", JobStepMessageLevels.Error);
			public static HordeStepMessage InvalidProject = new HordeStepMessage("InvalidProject", JobStepMessageLevels.Error);
			public static HordeStepMessage NoVerseFiles = new HordeStepMessage("NoVerseFiles", JobStepMessageLevels.Error);
			public static HordeStepMessage InvalidValues = new HordeStepMessage("InvalidValues", JobStepMessageLevels.Error);
			public static HordeStepMessage PermissionsFailure = new HordeStepMessage("PermissionsFailure", JobStepMessageLevels.Error);
			public static HordeStepMessage InvalidTestData = new HordeStepMessage("InvalidTestData", JobStepMessageLevels.Error);
			public static HordeStepMessage PushChangesFailure = new HordeStepMessage("PushChangesFailure", JobStepMessageLevels.Error);
			public static HordeStepMessage CompileError = new HordeStepMessage("CompileError", JobStepMessageLevels.Error);

			// Failures related to game/service functionality
			public static HordeStepMessage LoginFailed = new HordeStepMessage("LoginFailed", JobStepMessageLevels.Error);
			public static HordeStepMessage MatchmakingFailed = new HordeStepMessage("MatchmakingFailed", JobStepMessageLevels.Error);
			public static HordeStepMessage TestTimeoutExceeded = new HordeStepMessage("TestTimeoutExceeded", JobStepMessageLevels.Error);
			public static HordeStepMessage HeartbeatTimeout = new HordeStepMessage("HeartbeatTimeout", JobStepMessageLevels.Error);
			public static HordeStepMessage OverlayException = new HordeStepMessage("OverlayException", JobStepMessageLevels.Error);
			public static HordeStepMessage TestCasesFailed = new HordeStepMessage("TestCasesFailed", JobStepMessageLevels.Error);
			public static HordeStepMessage MissingArtifacts = new HordeStepMessage("MissingArtifacts", JobStepMessageLevels.Error);
			public static HordeStepMessage UnacceptablePerf = new HordeStepMessage("UnacceptablePerf", JobStepMessageLevels.Error);
			public static HordeStepMessage UnacceptableMem = new HordeStepMessage("UnacceptableMem", JobStepMessageLevels.Error);
			public static HordeStepMessage HardwareFailure = new HordeStepMessage("HardwareFailure", JobStepMessageLevels.Error);
			public static HordeStepMessage ServerCrashDetected = new HordeStepMessage("ServerCrashDetected", JobStepMessageLevels.Error);
			public static HordeStepMessage ClientCrashDetected = new HordeStepMessage("ClientCrashDetected", JobStepMessageLevels.Error);
			public static HordeStepMessage EditorCrashDetected = new HordeStepMessage("EditorCrashDetected", JobStepMessageLevels.Error);
			public static HordeStepMessage ServerError = new HordeStepMessage("ServerError", JobStepMessageLevels.Error);
			public static HordeStepMessage SyncError = new HordeStepMessage("SyncError", JobStepMessageLevels.Error);
			public static HordeStepMessage ContentError = new HordeStepMessage("ContentError", JobStepMessageLevels.Error);
			public static HordeStepMessage UploadError = new HordeStepMessage("UploadError", JobStepMessageLevels.Error);
			public static HordeStepMessage LobbyFailure = new HordeStepMessage("LobbyFailure", JobStepMessageLevels.Error);

			// Warning states for if things are getting close to bad
			public static HordeStepMessage TestWarningsDetected = new HordeStepMessage("TestWarningsDetected", JobStepMessageLevels.Warning);
			public static HordeStepMessage PerfWarningsDetected = new HordeStepMessage("PerfWarningsDetected", JobStepMessageLevels.Warning);
			public static HordeStepMessage MemWarningsDetected = new HordeStepMessage("MemWarningsDetected", JobStepMessageLevels.Warning);
			public static HordeStepMessage ReplayGettingStale = new HordeStepMessage("ReplayGettingStale", JobStepMessageLevels.Warning);
			public static HordeStepMessage ImproperRpcCallDetected = new HordeStepMessage("ImproperRpcCallDetected", JobStepMessageLevels.Warning);

			// Success states
			public static HordeStepMessage TestSucceeded = new HordeStepMessage("TestSucceeded", JobStepMessageLevels.Info);
			public static HordeStepMessage TestDataGenerated = new HordeStepMessage("TestDataGenerated", JobStepMessageLevels.Info);

			// Failed states
			public static HordeStepMessage ZeroTestsRun = new HordeStepMessage("ZeroTestsRun", JobStepMessageLevels.Error);
			public static HordeStepMessage CleanupFailure = new HordeStepMessage("CleanupFailure", JobStepMessageLevels.Error);

		}

		private static string? curStepMessage { get; set; } = null;
		private static string? curStepSeverity { get; set; } = null;
		private static string? curStepColor { get; set; } = null;

		/// <summary>
		/// Update the step message (short message written in-line in step view)
		/// These messages should be generic so we can aggregate multiple instances of it.
		/// Good message example "TestTimedOut", or "IslandNotFound", or "LoginFailed"
		/// Bad message example "This test timed out after 3 hours", "7 of 8 islands found"
		/// </summary>
		/// <param name="InStepMessage">A predefined object that describes what happened.</param>
		public static void UpdateStepMessage(HordeStepMessage InStepMessage)
		{
			curStepMessage = InStepMessage.Message;
			curStepSeverity = InStepMessage.Severity.ToString();
		}

		/// <summary>
		/// Labels a test as being invalid, and provides reason why to the summary report, with a concise message to also be shown in step view. This should ONLY
		/// be used right before an exception is thrown. If it is being used outside of a currently running test (as in, the test is actively ticking), you want to set
		/// bGenerateSummary to true
		/// </summary>
		/// <param name="InMessage">A predefined object that describes what happened.</param>
		/// <param name="IssueDescription">In-depth description of the issue. If the issue type was NotEnoughDevices, for example, this is where you would say what sort of device
		/// you were looking for.</param>
		/// <param name="bGenerateSummary"> Whether or not we should immediately generate the summary log. If this is being called from outside of a test node, set to yes.</param>
		public static void ReportInvalidatingIssue(HordeStepMessage InMessage, string IssueDescription, bool bGenerateSummary = true)
		{
			string InterruptColorOverride = "User1";
			curStepColor = InterruptColorOverride;
			string InvalidatingIssueString = $"**Invalidating issue found:** {InMessage.Message}\n";
			OnStepSummary += () => InvalidatingIssueString;
			if (!string.IsNullOrEmpty(IssueDescription))
			{
				OnStepSummary += () => $"More details: {IssueDescription}";
			}

			UpdateStepMessage(InMessage);
			if (bGenerateSummary)
			{
				GenerateStepSummary();
			}
		}

		/// <summary>
		/// Whether to generate test data v1 model when writing test results for Horde
		/// </summary>
		static public bool UseTestDataV1
		{
			get
			{
				return Globals.Params.ParseParam("UseTestDataV1");
			}
		}

		public class JobStepReport
		{
			public string scope { get; } = "Step";
			public string? name { get; set; }
			public string placement { get; } = "Summary";
			public string? fileName { get; set; }
			public string? message { get; set; }
			public string? severity { get; set; }
			public string? color { get; set; }
		}

		public class JobSummaryReport
		{
			public string scope { get; } = "Job";
			public string? name { get; set; }
			public string placement { get; } = "Summary";
			public string? fileName { get; set; }
		}

		static bool bHasGeneratedStepReport = false;
		static bool bHasGeneratedJobReport = false;

		public delegate string SummaryLineDelegate();
		/// <summary>
		/// Delegate called when the Horde step summary is generated.
		/// Returned string is appended as a line to the summary report.
		/// </summary>
		/// <returns></returns>
		public static SummaryLineDelegate? OnStepSummary;

		/// <summary>
		/// Delegate called when the Horde step summary is finalized.
		/// Returned string is appended as a line to the summary report.
		/// </summary>
		/// <returns></returns>
		public static SummaryLineDelegate? OnStepSummaryFinalization;

		/// <summary>
		/// Delegate called when the Horde job summary is generated.
		/// Return stringis append as a line to the job summary report.
		/// </summary>
		public static SummaryLineDelegate? OnJobSummary;

		public static void AddJobBuildMetaData(IBuildSource buildSource)
		{
			try
			{
				if (!IsHordeJob)
				{
					return;
				}

				if (buildSource != null)
				{
					CommandUtils.AddHordeJobMetadata(EpicGames.Horde.Jobs.JobId.Parse(JobId!), null, new Dictionary<string, List<string>> { { StepId!, new List<string> { $"BuildName=${buildSource.BuildName}?display=true" } } }).Wait();
				}

			}
			catch (Exception Ex)
			{
				Log.Info("Exception while tagging Horde job with build metadata\n{0}\n", Ex.Message);
			}
		}

		public static string GenerateArtifactLink(string ArtifactPath)
		{
			// file:// is attractive here, and file://\\computer\share\folder\file.txt does work ok in Chrome, but it does not Firefox
			// so, lacking a uniform way to link to a network filesystem, just put the path here as static text and escaped for markdown
			return $"* **Netshare Artifacts**: {System.Text.RegularExpressions.Regex.Replace(ArtifactPath, @"([|\\*\+])", @"\$1")}";
		}

		public static void GenerateStepSummary()
		{
			try
			{
				if (!IsHordeJob)
				{
					return;
				}

				if (bHasGeneratedStepReport)
				{
					Log.Info("Trying to generate a step report when we have already generated one. ");
					return;
				}

				StringBuilder Markdown = new StringBuilder();

				if (OnStepSummary != null)
				{
					foreach (SummaryLineDelegate callback in OnStepSummary.GetInvocationList())
					{
						Markdown.AppendLine(callback());
					}
				}
				if (OnStepSummaryFinalization != null)
				{
					foreach (SummaryLineDelegate callback in OnStepSummaryFinalization.GetInvocationList())
					{
						Markdown.AppendLine(callback());
					}
				}

				// detect if any testdata was generated
				if (!UseTestDataV1)
				{
					DirectoryInfo TestDataOutputFolder = new DirectoryInfo(Path.Combine(CommandUtils.CmdEnv.EngineSavedFolder, "TestData"));
					if (TestDataOutputFolder.Exists && TestDataOutputFolder.GetFiles("*.json", SearchOption.TopDirectoryOnly).Any())
					{
						Markdown.AppendLine($"* **Test Results**: [Open report](/test-automation?job={JobId}&step={StepId})");
					}
					else
					{
						Log.Verbose("Did not add Horde report link as any test data files in '{0}' was found.\n", TestDataOutputFolder.FullName);
					}
				}

				if (Markdown.Length > 0)
				{
					string LogFolder = CommandUtils.CmdEnv.LogFolder;
					string MarkdownFilename = "GauntletStepDetails.md";

					File.WriteAllText(Path.Combine(LogFolder, MarkdownFilename), Markdown.ToString());
					JsonSerializerOptions options = new()
					{
						DefaultIgnoreCondition = JsonIgnoreCondition.WhenWritingNull
					};
					JobStepReport report = new JobStepReport() { name = "Gauntlet Step Details", fileName = MarkdownFilename, message = curStepMessage, severity = curStepSeverity, color = curStepColor };
					File.WriteAllText(Path.Combine(LogFolder, "GauntletStepDetails.report.json"), JsonSerializer.Serialize(report, options));
				}
				bHasGeneratedStepReport = true;
			}
			catch (Exception Ex)
			{
				Log.Info("Exception while generating Horde step summary\n{0}\n", Ex);
			}
		}

		public static void GenerateJobSummary()
		{
			try
			{
				if (!IsHordeJob)
				{
					return;
				}

				if (bHasGeneratedJobReport)
				{
					Log.Info("Trying to generate a job summary report when we have already generated one. ");
					return;
				}

				StringBuilder Markdown = new StringBuilder();

				if (OnJobSummary != null)
				{
					foreach (SummaryLineDelegate callback in OnJobSummary.GetInvocationList())
					{
						Markdown.AppendLine(callback());
					}
				}

				if (Markdown.Length > 0)
				{
					// a blank line to exit markdown paragraph
					Markdown.AppendLine();

					string LogFolder = CommandUtils.CmdEnv.LogFolder;
					string MarkdownFilename = "GauntletJobSummary.md";

					File.WriteAllText(Path.Combine(LogFolder, MarkdownFilename), Markdown.ToString());
					JsonSerializerOptions options = new()
					{
						DefaultIgnoreCondition = JsonIgnoreCondition.WhenWritingNull
					};
					JobSummaryReport report = new JobSummaryReport() { name = $"{StepName} Summary", fileName = MarkdownFilename };
					File.WriteAllText(Path.Combine(LogFolder, "GauntletJobSummary.report.json"), JsonSerializer.Serialize(report, options));
				}
				bHasGeneratedJobReport = true;
			}
			catch (Exception Ex)
			{
				Log.Info("Exception while generating Horde job summary\n{0}\n", Ex);
			}
		}

	}
}
