// Copyright Epic Games, Inc. All Rights Reserved.

using EpicGame;
using System.Threading;
using System.Threading.Tasks;

#nullable enable

namespace Gauntlet
{
	/// <summary>
	/// A Test Node which simplifies writing asynchronous tests
	/// </summary>
	public abstract class AsyncEpicGameTestNode<TConfigClass>(UnrealTestContext InContext) : EpicGameTestNode<TConfigClass>(InContext)
		where TConfigClass : EpicGameTestConfig, new()
	{
		public RpcTargetRegistry TargetRegistry { get; private set; } = new();
		public CancellationTokenSource SessionCancelTokenSource { get; private set; } = new();

		private Task<TestResult>? TestTask;
		private AsyncSessionState CurrentState = AsyncSessionState.Init;
		private enum AsyncSessionState
		{
			Init,
			BeforeSession,
			SessionStart,
			SessionExecuting,
			AfterSession,
		}

		/// <summary>
		/// Subclasses should define their logic by overriding ExecuteTestAsync
		/// </summary>
		public override void TickTest()
		{
			base.TickTest();

			switch (CurrentState)
			{
				case AsyncSessionState.Init:
					InitializeSession();
					CurrentState = AsyncSessionState.BeforeSession;
					break;
				case AsyncSessionState.BeforeSession:
					BeforeTest();
					CurrentState = AsyncSessionState.SessionStart;
					break;
				case AsyncSessionState.SessionStart:
					ConfigureCancellation();
					TestTask = ExecuteTestAsync(SessionCancelTokenSource.Token);
					CurrentState = AsyncSessionState.SessionExecuting;
					break;
				case AsyncSessionState.SessionExecuting:
					if (TestTask!.IsCompleted)
					{
						CurrentState = AsyncSessionState.AfterSession;
					}
					break;
				case AsyncSessionState.AfterSession:
					TestResult SessionResult = TestResult.Invalid;
					SessionResult = TestTask!.GetAwaiter().GetResult();
					SetTestResult(SessionResult); // called before AfterTest to allow derived class to override by also calling SetTestResult
					AfterTest(ref SessionResult);
					MarkTestComplete();
					break;
				default:
					throw new InterruptTestException($"Invalid {nameof(AsyncSessionState)} '{CurrentState}' detected while executing RPC test {GetType().Name}");
			}
		}

		/// <inheritdoc/>
		public override void StopTest(StopReason InReason)
		{
			SessionCancelTokenSource.Cancel(); // likely already cancelled, but make certain
			base.StopTest(InReason);
		}

		private void InitializeSession()
		{
			RpcExecutor.Instance.StartListenThread();
			RpcExecutor.Instance.SetRpcTargetRegistry(TargetRegistry);
			TargetRegistry.UpdateAvailableRpcForAllTargets();
			TargetRegistry.UpdateIPsForRemoteAndNonDesktopTargets(UnrealApp, ReportInterrupt);
		}

		/// <summary>
		/// Sets conditions for canceling the test session
		/// </summary>
		protected void ConfigureCancellation()
		{
			Globals.AbortHandlers.Add( () =>
			{
				Log.Info($"Test session for {Name} cancelled by user input");
				SessionCancelTokenSource.Cancel();
				SetTestResult(TestResult.Cancelled);
			});
		}

		/// <inheritdoc/>
		public override void CleanupTest()
		{
			SessionCancelTokenSource.Cancel();

			TargetRegistry.Dispose();
			TestTask?.Dispose();
			SessionCancelTokenSource.Dispose();
			base.CleanupTest();
		}

		/// <summary>
		/// Async logic for this test session. Tasks will be run on the default thread pool.
		/// </summary>
		/// <param name="CancelToken">signals early termination requests</param>
		/// <returns>TestResult describing the test status</returns>
		protected abstract Task<TestResult> ExecuteTestAsync(CancellationToken CancelToken);

		/// <summary>
		/// Additional setup steps run immediately before ExecuteTestAsync
		/// </summary>
		public virtual void BeforeTest()
		{
			// intentionally empty
		}

		/// <summary>
		/// Additional teardown steps to run immediately after ExecuteTestAsync
		/// </summary>
		/// <param name="FinishedTestResult">The recorded result from ExecuteTestAsync. Can be overridden with SetTestResult().</param>
		public virtual void AfterTest(ref readonly TestResult FinishedTestResult)
		{
			// intentionally empty
		}
	}
}
