// Copyright Epic Games, Inc. All Rights Reserved.

using System.Net.Http;
using System.Threading.Tasks;

namespace Gauntlet
{
	public class GauntletHttpClient : HttpClient
	{
		public GauntletHttpClient() : this(new HttpClientHandler())
		{
		}

		public GauntletHttpClient(HttpMessageHandler Handler) : base(Handler, true)
		{
		}
		
		public async Task<HttpResponseMessage> GetRequestAsync(string Route)
		{
			LogOutgoingRequestURI(Route);
			return await GetAsync(Route);
		}

		public async Task<HttpResponseMessage> SendRequestAsync(HttpRequestMessage Msg)
		{
			LogOutgoingRequestURI(Msg.RequestUri.ToString());
			await LogOutgoingRequestPayload(Msg);
			return await SendAsync(Msg);
		}

		private async Task LogOutgoingRequestPayload(HttpRequestMessage Msg)
		{
			string Payload = await Msg.Content.ReadAsStringAsync();

			if (string.IsNullOrEmpty(Payload))
			{
				return;
			}

			Log.Verbose("Request Payload: {0}", Payload);
		}

		public void LogOutgoingRequestURI(string Route)
		{
			Log.Verbose("Making Http Request to URI:{0}{1}", BaseAddress, Route);
		}
	}

	/// <summary>
	/// Generic HTTP response class that 
	/// </summary>
	public class GauntletHttpResponse
	{
		public bool Success { get; set; }
		public string Reason { get; set; }
	}
}
