// Copyright Epic Games, Inc. All Rights Reserved.

using EpicGames.Core;
using Newtonsoft.Json;
using System;
using System.Collections.Concurrent;
using System.Collections.Generic;
using System.Diagnostics;
using System.IO;
using System.Linq;
using System.Net;
using System.Net.Http;
using System.Net.NetworkInformation;
using System.Net.Sockets;
using System.Runtime.CompilerServices;
using System.Text;
using System.Text.Json.Nodes;
using System.Text.RegularExpressions;
using System.Threading;
using ThreadState = System.Threading.ThreadState;

namespace Gauntlet
{
	/// <summary>
	/// Contains all relevant information for a registered RPC route on a target
	/// </summary>
	[DebuggerDisplay("{Name}")]
	public class RpcDefinition
	{
		public string Name;
		public string Route;
		public string Verb;
		public string InputContentType;
		public List<RpcArgumentDesc> Args;
	}
	/// <summary>
	/// Definition of a ledger entry when looking at our request history
	/// </summary>
	public class RpcLedgerEntry
	{
		public string RpcName;
		public string RequestTimestamp;
		public string RequestBody;
	}

	/// <summary>
	/// Definition of an individual argument expected by an RPC. 
	/// </summary>
	[DebuggerDisplay("{Name}")]
	public class RpcArgumentDesc
	{
		public string Name;
		public string Type;
		public string Desc;
		public bool Optional;
	}
	/// <summary>
	/// Full response body and code.
	/// </summary>
	public class ResponseData
	{
		public string ResponseBody;
		public int ResponseCode;
		public ResponseData()
		{
			ResponseBody = "";
			ResponseCode = (int)HttpStatusCode.OK;
		}
	}

	/// <summary>
	/// Description for a route mapping that we are listening on from Gauntlet. 
	/// </summary>
	public class RouteMapping
	{
		public string Path;
		public string Verb;
		public RouteMapping(string InPath, string InVerb = "post")
		{
			Path = InPath.ToLower();
			Verb = InVerb.ToLower();
		}

		public override bool Equals(object TargetObject)
		{
			return this.Equals(TargetObject as RouteMapping);
		}

		public bool Equals(RouteMapping InMap)
		{
			return (InMap.Path.ToLower() == Path.ToLower() && InMap.Verb.ToLower() == Verb.ToLower());
		}

		public override int GetHashCode()
		{
			string OverallString = string.Format("{0}|{1}", Path.ToLower(), Verb.ToLower());
			return OverallString.GetHashCode();
		}
		public override string ToString()
		{
			return string.Format("{0}|{1}", Path.ToLower(), Verb.ToLower());
		}

	}

	/// <summary>
	/// Singleton responsible for all of the communication from Gauntlet to the targets.
	/// </summary>
	public partial class RpcExecutor
	{
		private static RpcExecutor ExecutorInstance = null;
		private static readonly object ThreadLock = new object();
		public static GauntletHttpClient ExecutorClient = new GauntletHttpClient();

		protected HttpListener RequestListener;
		protected Thread HttpListenThread;
		protected static int GauntletRPCListenPort = 23232;
		protected RpcTargetRegistry ActiveTargetRegistry = null;

		public static RpcExecutor Instance
		{
			get
			{
				lock (ThreadLock)
				{
					if (ExecutorInstance == null)
					{
						ExecutorInstance = new RpcExecutor();
						ExecutorClient.Timeout = Globals.Params.ParseParam("NoRPCHttpTimeout") ? Timeout.InfiniteTimeSpan 
							: TimeSpan.FromMilliseconds(Globals.Params.ParseValue("RPCHttpTimeoutInMilliseconds", 100 * 1000));
					}
					return ExecutorInstance;
				}
			}
		}

		private RpcExecutor()
		{
		}
		~RpcExecutor()
		{
			ActiveTargetRegistry = null;
			if (HttpListenThread != null && HttpListenThread.ThreadState == ThreadState.Running)
			{
				bHasReceivedTerminationCall = true;
				RequestListener.Abort();
			}
		}

		/// <summary>
		/// Enables a message queue for each target application in the current test, reducing async lock contention
		/// </summary>
		/// <param name="CurrentRegistry">RpcTargetRegistry for the currently running test</param>
		public void SetRpcTargetRegistry(RpcTargetRegistry CurrentRegistry, [CallerFilePath] string Caller = "", [CallerLineNumber] int Line = 0)
		{
			if (ActiveTargetRegistry != null && ActiveTargetRegistry != CurrentRegistry)
			{
				Log.Warning($"{nameof(SetRpcTargetRegistry)} unexpectedly invoked from {Caller} line {Line} while another registry was already set. This may disable RPC communication in another running test.");
			}
			ActiveTargetRegistry = CurrentRegistry;
		}

		public static string GetListenAddressAndPort()
		{
			string LocalIP = "127.0.0.1";
			try
			{
				IEnumerable<NetworkInterface> NetworkInterfaces = NetworkInterface.GetAllNetworkInterfaces()
					.Where(ni => ni.OperationalStatus == OperationalStatus.Up &&
								ni.NetworkInterfaceType != NetworkInterfaceType.Loopback);

				foreach (var ni in NetworkInterfaces)
				{
					IPInterfaceProperties IpProperties = ni.GetIPProperties();
					UnicastIPAddressInformation Ipv4Address = IpProperties.UnicastAddresses
						.FirstOrDefault(ip => ip.Address.AddressFamily == AddressFamily.InterNetwork);

					if (Ipv4Address != null)
					{
						LocalIP = Ipv4Address.Address.ToString();
						break;
					}
				}
			}
			catch (Exception ex)
			{
				Log.Verbose($"Failed to get network IP, using loopback: {ex.Message}");
			}
			return String.Format("{0}:{1}", LocalIP, GauntletRPCListenPort);
		}

		public ConcurrentQueue<GauntletIncomingRpcMessage> IncomingMessageQueue = [];
		protected ConcurrentDictionary<GauntletRpcDelegateRegistryFilter, GauntletIncomingRpcMessageDelegate> MessageDelegates = [];
		protected ConcurrentDictionary<RouteMapping, GauntletHttpRouteDelegate> HttpRoutes = [];

		protected object ReceivedTerminationCallLock = new ();
		private bool _bHasReceivedTerminationCall = false;
		public bool bHasReceivedTerminationCall
		{
			get
			{
				lock(ReceivedTerminationCallLock)
				{
					return _bHasReceivedTerminationCall;
				}
			}
			private set
			{
				lock (ReceivedTerminationCallLock)
				{
					_bHasReceivedTerminationCall = value;
				}
			}
		}
		public void ListenLoopThread()
		{

			RequestListener = new HttpListener();

			try
			{
				RequestListener.Prefixes.Add(String.Format("http://{0}/", GetListenAddressAndPort()));
				Log.Info("Base URL is: " + String.Format("http://{0}/", GetListenAddressAndPort()));
				RegisterHttpRoute(new RouteMapping("/sendmessage"), HandleIncomingMessage, true);
				RequestListener.Start();
			}
			catch (Exception ex)
			{
				Log.Warning("Not able to open listener on external port unless running as admin. Error Message:" + ex.Message);
				return;
			}
			Console.WriteLine(String.Format("Listening on port {0}...", GauntletRPCListenPort));

			// We always want to be listening for requests on this thread until we kill the loop.
			while (!bHasReceivedTerminationCall && RequestListener.IsListening)
			{
				IAsyncResult ListenerContext = RequestListener.BeginGetContext(new AsyncCallback(ReceiveListenRequest), RequestListener);
				ListenerContext.AsyncWaitHandle.WaitOne();
			}

		}


		public virtual void HandleIncomingMessage(string MessageBody, ResponseData CallResponse)
		{
			CallResponse.ResponseCode = (int)HttpStatusCode.OK;
			GauntletIncomingRpcMessage IncomingMessage = JsonConvert.DeserializeObject<GauntletIncomingRpcMessage>(MessageBody);
			if (IncomingMessage != null)
			{
				FireRelevantMessageDelegates(IncomingMessage);

				if (null == ActiveTargetRegistry)
				{
					IncomingMessageQueue.Enqueue(IncomingMessage);
				}
				else
				{
					RpcTarget Target = ActiveTargetRegistry.GetTarget(IncomingMessage.SenderId);
					if( null == Target )
					{
						Log.Warning($"No RPC target registered for '{IncomingMessage.SenderId}' in active registry; routing to central queue to prevent message loss (Category: '{IncomingMessage.Category}').");
						IncomingMessageQueue.Enqueue(IncomingMessage);
					}
					else
					{
						Target.IncomingMessageQueue.Enqueue(IncomingMessage);
					}
				}
				CallResponse.ResponseBody = String.Format("New message enqueued! Source: {0} \n Category: {1} \n Payload: {2} \n", IncomingMessage.SenderId, IncomingMessage.Category, IncomingMessage.Payload);
			}
			else
			{
				CallResponse.ResponseBody = "Body could not be processed. Contents: " + MessageBody;
				CallResponse.ResponseCode = (int)HttpStatusCode.UnprocessableEntity;
			}
			Log.Verbose(CallResponse.ResponseBody);
		}

		public void ReceiveListenRequest(IAsyncResult ListenResult)
		{
			HttpListener RequestListener = (HttpListener)ListenResult.AsyncState;
			if (bHasReceivedTerminationCall)
			{
				return;
			}
			HttpListenerContext ListenerContext = RequestListener.EndGetContext(ListenResult);
			HttpListenerRequest ListenerRequest = ListenerContext.Request;


			RouteMapping CallPath = new RouteMapping(ListenerContext.Request.Url.LocalPath, ListenerContext.Request.HttpMethod);
			Log.Verbose(string.Format("In ReceiveListenRequest callback for {0} | {1}", CallPath.Path, CallPath.Verb));
			ResponseData CallResponse = new ResponseData();
			CallResponse.ResponseBody = "This endpoint is not handled";

			foreach (KeyValuePair<RouteMapping, GauntletHttpRouteDelegate> RouteEntry in HttpRoutes)
			{
				if (CallPath.Equals(RouteEntry.Key))
				{
					try
					{
						using (HttpListenerResponse ListenerResponse = ListenerContext.Response)
						{
							if (ListenerResponse == null)
							{
								return;
							}
							ListenerResponse.Headers.Set("Content-Type", "text/plain");

							using (Stream ResponseStream = ListenerResponse.OutputStream)
							{
								if (ResponseStream == null)
								{
									return;
								}
								StreamReader RequestReader = new StreamReader(ListenerRequest.InputStream, ListenerRequest.ContentEncoding);
								string RequestBody = RequestReader.ReadToEnd();
								RouteEntry.Value.Invoke(RequestBody, CallResponse);
								ListenerContext.Response.StatusCode = CallResponse.ResponseCode;
								byte[] ResponseBuffer = Encoding.UTF8.GetBytes(CallResponse.ResponseBody);
								if (ResponseBuffer == null)
								{
									return;
								}
								ListenerResponse.ContentLength64 = ResponseBuffer.Length;
								ResponseStream.Write(ResponseBuffer, 0, ResponseBuffer.Length);
							}
						}
					}
					catch (Exception Error)
					{
						if (bHasReceivedTerminationCall)
						{
							Log.SuspendECErrorParsing();
							Log.Info("Prevented Shutdown failure. Discarded response for call to {CallPath} because we are in the process of shutting down. Error: {ErrorMessage}", CallPath.Path, Error.Message);
							Log.ResumeECErrorParsing();
						}
						else
						{
							Log.SuspendECErrorParsing();
							Log.Warning("Call to {CallPath} failed. Error: {ErrorMessage}", CallPath.Path, Error.Message);
							Log.ResumeECErrorParsing();
						}
					}
					return;
				}
			}

			try 
			{ 
				using (HttpListenerResponse ListenerResponse = ListenerContext.Response)
				{
					ListenerResponse.Headers.Set("Content-Type", "text/plain");

					using (Stream ResponseStream = ListenerResponse.OutputStream)
					{
						ListenerContext.Response.StatusCode = (int)HttpStatusCode.NotFound;
						byte[] ResponseBuffer = Encoding.UTF8.GetBytes(CallResponse.ResponseBody);
						ListenerResponse.ContentLength64 = ResponseBuffer.Length;
						ResponseStream.Write(ResponseBuffer, 0, ResponseBuffer.Length);
					}
				}
			}
			catch (Exception Error)
			{
				if (bHasReceivedTerminationCall)
				{
					Log.SuspendECErrorParsing();
					Log.Info("Prevented Shutdown failure. Discarded response for call to {CallPath} because we are in the process of shutting down. Error: {ErrorMessage}", CallPath.Path, Error.Message);
					Log.ResumeECErrorParsing();
				}
				else
				{
					Log.SuspendECErrorParsing();
					Log.Warning("Call to {CallPath} failed. Error: {ErrorMessage}", CallPath.Path, Error.Message);
					Log.ResumeECErrorParsing();
				}
			}
		}
		public List<GauntletIncomingRpcMessage> GetNewMessages(bool bClearMessageQueue = true)
		{
			List<GauntletIncomingRpcMessage> NewMessages = [];
			int TargetsMessageCount = 0;
			if(null != ActiveTargetRegistry)
			{
				foreach (RpcTarget Target in ActiveTargetRegistry.RpcTargets.Values)
				{
					TargetsMessageCount += Target.IncomingMessageQueue.Count;
				}
			}
			if (IncomingMessageQueue.Count == 0 && TargetsMessageCount == 0)
			{
				return NewMessages;
			}

			PopulateNewMessagesFromCentralQueue(NewMessages, bClearMessageQueue);
			PopulateNewMessagesFromActiveTargetQueues(NewMessages, bClearMessageQueue);
			return NewMessages;
		}

		/// <summary>
		/// Returns the string representations of all messages across all queues, without modifying the queues.
		/// </summary>
		/// <returns>List of message strings formatted as "[SenderId][Category] Payload"</returns>
		public List<string> GetAllMessageStrings()
		{
			return GetNewMessages(bClearMessageQueue: false)
				.Select(Msg => $"[{Msg.SenderId}][{Msg.Category}] {Msg.Payload}")
				.ToList();
		}

		private void PopulateNewMessagesFromQueue(ConcurrentQueue<GauntletIncomingRpcMessage> Queue, List<GauntletIncomingRpcMessage> Messages, bool bClearMessageQueue)
		{
			int InitialCount = Queue.Count;
			if (InitialCount > 0)
			{
				if (bClearMessageQueue)
				{
					for(int MessageIndex = 0; MessageIndex < InitialCount; ++MessageIndex)
					{
						if(Queue.TryDequeue(out GauntletIncomingRpcMessage NewMessage))
						{
							Messages.Add(new GauntletIncomingRpcMessage(NewMessage));
						}
						else
						{
							Log.Warning("Message queue unexpectedly dequeued from another thread. RPC messages may be missing.");
						}
					}
				}
				else
				{
					foreach (GauntletIncomingRpcMessage NewMessage in Queue)
					{
						Messages.Add(new GauntletIncomingRpcMessage(NewMessage));
					}
				}
			}
		}

		private void PopulateNewMessagesFromCentralQueue(List<GauntletIncomingRpcMessage> Messages, bool bClearMessageQueue = true)
		{
			PopulateNewMessagesFromQueue(IncomingMessageQueue, Messages, bClearMessageQueue);
		}

		private void PopulateNewMessagesFromActiveTargetQueues(List<GauntletIncomingRpcMessage> Messages, bool bClearMessageQueue = true)
		{
			if (null != ActiveTargetRegistry)
			{
				foreach (RpcTarget Target in ActiveTargetRegistry.RpcTargets.Values)
				{
					PopulateNewMessagesFromQueue(Target.IncomingMessageQueue, Messages, bClearMessageQueue);
				}
			}
		}

		/// <summary>
		/// Dequeues a number messages, allowing new messages to be concurrently enqueued
		/// </summary>
		private void SafeClearMessages(ConcurrentQueue<GauntletIncomingRpcMessage> Queue, int MessagesToClear)
		{
			for (int MessageIndex = 0; MessageIndex < MessagesToClear; ++MessageIndex)
			{
				Queue.TryDequeue(out GauntletIncomingRpcMessage Unused);
			}
		}

		public void ClearIncomingMessageQueue()
		{
			SafeClearMessages(IncomingMessageQueue, IncomingMessageQueue.Count);
			if (null != ActiveTargetRegistry)
			{
				foreach (RpcTarget Target in ActiveTargetRegistry.RpcTargets.Values)
				{
					SafeClearMessages(Target.IncomingMessageQueue, Target.IncomingMessageQueue.Count);
				}
			}
		}

		public void ClearMessageDelegates()
		{
			MessageDelegates.Clear();
		}

		/// <summary>
		/// Checks a single message for matches against a list of message filters to see if it matches any of them.
		/// </summary>
		/// <param name="NewMessage">Gauntlet inbound message to be checked if it matches the filters</param>
		/// <param name="MessageFilters">List of filters that represent the message(s) that we are looking
		/// for </param>
		/// <returns></returns>
		public int GetMessageFilterMatches(GauntletIncomingRpcMessage NewMessage, List<GauntletRpcDelegateRegistryFilter> MessageFilters)
		{
			int FiltersMatched = 0;

			if (NewMessage == null)
			{
				Log.Warning("Cannot check message because it is null.");
				return 0;
			}

			if (MessageFilters == null || MessageFilters.Count == 0)
			{
				Log.Warning("No Filter provided, cannot match without a filter.");
				return 0;
			}

			foreach (GauntletRpcDelegateRegistryFilter MessageFilter in MessageFilters.Where(filter => NewMessage.MatchesMessageFilter(filter)).ToList())
			{
				string SenderId = string.IsNullOrEmpty(NewMessage.SenderId) ? "Any Sender ID" : NewMessage.SenderId;
				string Category = string.IsNullOrEmpty(NewMessage.Category) ? "Any Category" : NewMessage.Category;
				string Payload = string.IsNullOrEmpty(NewMessage.Payload) ? "Any Payload" : NewMessage.Payload;

				Log.Info("Incoming message from RPC Target: {0} Found! Category: {1} Payload: {2}", SenderId, Category, Payload);

				FiltersMatched++;
			}

			return FiltersMatched;
		}

		/// <summary>
		/// Process a message queue and return how many messages match a filter, optionally retaining any unmatched messages. <see cref="GauntletRpcDelegateRegistryFilter"/>
		/// </summary>
		/// <param name="MessageQueue">Queue to search for matches</param>
		/// <param name="QueueLock">Lock object guarding the queue</param>
		/// <param name="MessageFilters">List of Filters that match message properties</param>
		/// <param name="bSaveUnmatchedMessages">Preserve entries in MessageQueue which are not matched. When false all entries are purged from the queue.</param>
		/// <returns>Count of matched messages</returns>
		private int ConsumeIncomingMessagesFromQueue(ConcurrentQueue<GauntletIncomingRpcMessage> MessageQueue, List<GauntletRpcDelegateRegistryFilter> MessageFilters, bool bSaveUnmatchedMessages)
		{
			ArgumentNullException.ThrowIfNull(MessageQueue);
			ArgumentNullException.ThrowIfNull(MessageFilters);
			if (MessageFilters.Count == 0)
			{
				Log.Warning("Requested Incoming Messages to be matched, but no filters were provided to match against.");
				return 0;
			}

			int MatchesFound = 0;
			List<GauntletIncomingRpcMessage> UnmatchedMessages = [];
			List<GauntletIncomingRpcMessage> NewMessages = [];
			PopulateNewMessagesFromQueue(MessageQueue, NewMessages, bClearMessageQueue: true);
			foreach (var Message in NewMessages)
			{
				if (GetMessageFilterMatches(Message, MessageFilters) > 0)
				{
					++MatchesFound;
				}
				else if(bSaveUnmatchedMessages)
				{
					UnmatchedMessages.Add(Message);
				}
			}
			
			foreach (var Message in UnmatchedMessages)
			{
				MessageQueue.Enqueue(Message);
			}
			Log.Verbose("Matched {0} messages from {1} filters.", MatchesFound, MessageFilters.Count);
			return MatchesFound;
		}

		/// <summary>
		/// Process an RpcTarget's queue and return how many messages match a filter, optionally retaining any unmatched messages. <see cref="GauntletRpcDelegateRegistryFilter"/>
		/// </summary>
		/// <param name="Target">Queue to search for matches</param>
		/// <param name="MessageFilters">List of Filters that match message properties</param>
		/// <param name="bSaveUnmatchedMessages">Preserve entries in MessageQueue which are not matched. When false all entries are purged from the queue.</param>
		/// <returns>Count of matched messages</returns>
		public int ConsumeIncomingMessagesFromTarget(RpcTarget Target, List<GauntletRpcDelegateRegistryFilter> MessageFilters, bool bSaveUnmatchedMessages)
		{
			return ConsumeIncomingMessagesFromQueue(Target.IncomingMessageQueue, MessageFilters, bSaveUnmatchedMessages);
		}

		/// <summary>
		/// Process all active queues and return how many messages match a filter, optionally retaining any unmatched messages. <see cref="GauntletRpcDelegateRegistryFilter"/>
		/// </summary>
		/// <param name="MessageFilters">List of Filters that match message properties</param>
		/// <param name="bSaveUnmatchedMessages">Preserve entries in MessageQueue which are not matched. When false all entries are purged from the queue.</param>
		/// <returns>Count of matched messages</returns>
		public int ConsumeIncomingMessagesFromAllQueues(List<GauntletRpcDelegateRegistryFilter> MessageFilters, bool bSaveUnmatchedMessages)
		{
			int MatchesFound = ConsumeIncomingMessagesFromQueue(IncomingMessageQueue, MessageFilters, bSaveUnmatchedMessages);

			if (null != ActiveTargetRegistry)
			{
				foreach (RpcTarget Target in ActiveTargetRegistry.RpcTargets.Values)
				{
					MatchesFound += ConsumeIncomingMessagesFromTarget(Target, MessageFilters, bSaveUnmatchedMessages);
				}
			}
			return MatchesFound;
		}

		/// <summary>
		/// Checks the IncomingMessages queue for messages matching a list of GauntletRpcDelegateRegistryFilters, which represent 
		/// all of the messages that we are looking for. 
		/// </summary>
		/// <param name="MessageFilters">List of GauntletRpcDelegateRegistryFilters which hold the message properties we are checking
		/// for. DesiredSenderId, DesiredCategory, and DesiredPayload can be null and will act as wild cards allowing you to match on 
		/// specific messages or groups of messages.</param>
		/// <param name="bSaveUnmatchedMessages">If true, messages that are in the IncomingMessages queue but do not match our filter 
		/// will be preserved and can be later matched. If false, then all new messages retrieved are purged regardless of if they were 
		/// matched or not. Matched messages are always removed.</param>
		/// <returns>The number of matched messages found.</returns>
		public int CheckIncomingMessages(List<GauntletRpcDelegateRegistryFilter> MessageFilters, bool bSaveUnmatchedMessages)
		{
			Log.Info("Checking for new Messages from {0} Targets. Category: {1}, Contents: {2}", MessageFilters.Count, MessageFilters[0].DesiredCategory, MessageFilters[0].DesiredPayload);
			int MatchesFound = ConsumeIncomingMessagesFromAllQueues(MessageFilters, bSaveUnmatchedMessages);
			Log.Info("Found {0} messages from {1} targets.", MatchesFound, MessageFilters.Count);
			return MatchesFound;
		}

		/// <summary>
		/// Checks the IncomingMessages queue for messages matching a GauntletRpcDelegateRegistryFilter, which represents
		/// the message(s) that we are looking for. 
		/// </summary>
		/// <param name="MessageFilter">Single GauntletRpcDelegateRegistryFilters which holds the message properties we are checking
		/// for. DesiredSenderId, DesiredCategory, and DesiredPayload can be null and will act as wild cards allowing you to match on 
		/// specific messages or groups of messages.</param>
		/// <param name="bSaveUnmatchedMessages">If true, messages that are in the IncomingMessages queue but do not match our filter 
		/// will be preserved and can be later matched. If false, then all new messages retrieved are purged regardless of if they were 
		/// matched or not. Matched messages are always removed.</param>
		/// <returns>The number of matched messages found.</returns>
		[Obsolete($"Instead use {nameof(ConsumeIncomingMessagesFromTarget)} on the target message queue(s)", false)]
		public int CheckIncomingMessages(GauntletRpcDelegateRegistryFilter MessageFilter, bool bSaveUnmatchedMessages)
		{
			return CheckIncomingMessages(new List<GauntletRpcDelegateRegistryFilter>(new GauntletRpcDelegateRegistryFilter[] { MessageFilter }), bSaveUnmatchedMessages);
		}

		/// <summary>
		/// Checks the IncomingMessages queue messages matching the sender ID, category and payload contents. 
		/// </summary>
		/// <param name="TargetSenderId">Targets to check if there are incoming messages for. Can be null/empty to match for any Sender</param>
		/// <param name="TargetMessageCategory">Category of message to be checked. Can be null/empty to match for any Category</param>
		/// <param name="TargetMessagePayload">All or part of the message payload to be checked. Is case sensitive. Can be null/empty 
		/// to match for any payload</param>
		/// <param name="bSaveUnmatchedMessages">If true, messages that are in the IncomingMessages queue but do not match our filter 
		/// will be preserved and can be later matched. If false, then all new messages retrieved are purged regardless of if they were 
		/// matched or not. Matched messages are always removed.</param>
		/// <returns>The number of matched messages found.</returns>
		[Obsolete($"Instead use {nameof(ConsumeIncomingMessagesFromTarget)} on the target message queue(s)", false)]
		public int CheckIncomingMessages(string TargetSenderId, string TargetMessageCategory, string TargetMessagePayload, bool bSaveUnmatchedMessages)
		{
			GauntletRpcDelegateRegistryFilter NewFilter = new GauntletRpcDelegateRegistryFilter(TargetSenderId, TargetMessageCategory, TargetMessagePayload, false);
			return CheckIncomingMessages(NewFilter, bSaveUnmatchedMessages);
		}

		/// <summary>
		/// Checks the IncomingMessages queue for a message matching a category and payload contents for a list of RPC Targets
		/// </summary>
		/// <param name="InTargets">Targets to check if there are incoming messages for</param>
		/// <param name="TargetMessageCategory">Category of message to be checked.</param>
		/// <param name="TargetMessagePayload">All or part of the contens of the message payload to be checked. Is case sensitive. </param>
		/// <param name="bSaveUnmatchedMessages">If true, messages that are in the IncomingMessages queue but do not match our filter 
		/// will be preserved and can be later matched. If false, then all new messages retrieved are purged regardless of if they were 
		/// matched or not. Matched messages are always removed.</param>
		/// <returns>The number of matched messages found.</returns>
		public int CheckIncomingMessages(List<RpcTarget> InTargets, string TargetMessageCategory, string TargetMessagePayload, bool bSaveUnmatchedMessages)
		{
			List<GauntletRpcDelegateRegistryFilter> NewFilters = new List<GauntletRpcDelegateRegistryFilter>();

			foreach(RpcTarget Target in InTargets)
			{
				NewFilters.Add(new GauntletRpcDelegateRegistryFilter(Target.TargetName, TargetMessageCategory, TargetMessagePayload, false));
			}

			return CheckIncomingMessages(NewFilters, bSaveUnmatchedMessages);

		}

		/// <summary>
		/// Checks the IncomingMessages queue for a message matching a category and payload contents for a single RPC Target
		/// </summary>
		/// <param name="InTarget">Targets to check if there are incoming messages for</param>
		/// <param name="TargetMessageCategory">Category of message to be checked.</param>
		/// <param name="TargetMessagePayload">All or part of the contens of the message payload to be checked. Is case sensitive. </param>
		/// <param name="bSaveUnmatchedMessages">If true, messages that are in the IncomingMessages queue but do not match our filter 
		/// will be preserved and can be later matched. If false, then all new messages retrieved are purged regardless of if they were 
		/// matched or not. Matched messages are always removed.</param>
		/// <returns>The number of matched messages found.</returns>
		[Obsolete($"Instead use {nameof(ConsumeIncomingMessagesFromTarget)} on the target message queue(s)", false)]
		public int CheckIncomingMessages(RpcTarget InTarget, string TargetMessageCategory, string TargetMessagePayload, bool bSaveUnmatchedMessages)
		{
			GauntletRpcDelegateRegistryFilter NewFilter = new GauntletRpcDelegateRegistryFilter(InTarget.TargetName, TargetMessageCategory, TargetMessagePayload, false);
			return CheckIncomingMessages(NewFilter, bSaveUnmatchedMessages);
		}

		/// <summary>
		/// Basic filter class that allows users to filter what incoming messages they would like to fire their delegate functions on.
		/// Null values act as wild cards.
		/// </summary>
		public class GauntletRpcDelegateRegistryFilter
		{
			public string DesiredSenderId;
			public string DesiredCategory;
			public string DesiredPayload;
			public bool bDeregisterAfterFirstTrigger;
			public GauntletRpcDelegateRegistryFilter(string InSender, string InCategory, string InPayload, bool bInDeregisterAfterFirstTrigger)
			{
				DesiredSenderId = InSender;
				DesiredCategory = InCategory;
				DesiredPayload = InPayload;
				bDeregisterAfterFirstTrigger = bInDeregisterAfterFirstTrigger;
			}

		}

		/// <summary>
		/// Basic class for handling and deserializing incoming messages from Unreal processes
		/// </summary>
		public class GauntletIncomingRpcMessage
		{
			public string SenderId;
			public string Category;
			public string Payload;

			public GauntletIncomingRpcMessage()
			{
				SenderId = string.Empty;
				Category = string.Empty;
				Payload = string.Empty;

			}

			public GauntletIncomingRpcMessage(GauntletIncomingRpcMessage CopyTarget)
			{
				SenderId = CopyTarget.SenderId;
				Category = CopyTarget.Category;
				Payload = CopyTarget.Payload;
			}

			public GauntletIncomingRpcMessage(string InSender, string InCategory, string InPayload)
			{
				SenderId = InSender;
				Category = InCategory;
				Payload = InPayload;
			}

			public bool MatchesMessageFilter(GauntletRpcDelegateRegistryFilter MessageFilter)
			{
				if (!string.IsNullOrEmpty(MessageFilter.DesiredSenderId) && SenderId != MessageFilter.DesiredSenderId)
				{
					return false;
				}
				if (!string.IsNullOrEmpty(MessageFilter.DesiredCategory) && Category != MessageFilter.DesiredCategory)
				{
					return false;
				}
				if (!string.IsNullOrEmpty(MessageFilter.DesiredPayload) && Payload != MessageFilter.DesiredPayload)
				{
					return false;
				}
				return true;
			}
		} 

		/// <summary>
		/// Delegate type for incoming messages from Unreal processes.
		/// </summary>
		public delegate void GauntletIncomingRpcMessageDelegate(GauntletIncomingRpcMessage Message);

		/// <summary>
		///  Delegate registry function. Set any fields that don't matter to the filter to "" or String.Empty, and your delegate
		///  will fire off any time a message comes in that matches all criteria of your filter. It does not fuzzy match.
		/// </summary>
		/// <param name="FunctionToRun"> Delegate of function to run. Signature is void YourFunction(GauntletIncomingRpcMessage IncMessage)</param>
		/// <param name="SenderFilter">If you care about what process the message is sent from, put it here.</param>
		/// <param name="CategoryFilter">Message category you care about filtering on.</param>
		/// <param name="PayloadFilter">Exact message payload you'd likte to trigger on.</param>
		public virtual void RegisterMessageDelegate(GauntletIncomingRpcMessageDelegate FunctionToRun, string SenderFilter = "", string CategoryFilter = "", string PayloadFilter = "", bool bDeregisterAfterTrigger = false)
		{
			if (string.IsNullOrEmpty(SenderFilter) && string.IsNullOrEmpty(CategoryFilter) && string.IsNullOrEmpty(PayloadFilter))
			{
				Log.Warning("Cannot register Delegate with no filter at all.");
				return;
			}
			
			if(MessageDelegates.TryAdd(new GauntletRpcDelegateRegistryFilter(SenderFilter, CategoryFilter, PayloadFilter, bDeregisterAfterTrigger), FunctionToRun))
			{
				Log.Info("New delegate registered: " + SenderFilter + " | " + CategoryFilter + " | " + PayloadFilter);
			}
			else
			{
				Log.Error("Unexpectedly failed to register new delegate with duplicate filter");
			}
		}

		/// <summary>
		/// Delegate deregistry function. We will remove the delegate only if it matches everything passed in exactly.
		/// </summary>
		/// <param name="FunctionToRun">Function delegate name.</param>
		/// <param name="SenderFilter">Sender filter for the delegate trigger.</param>
		/// <param name="CategoryFilter">Category filter for the delegate trigger.</param>
		/// <param name="PayloadFilter">Payload filter for the delegate trigger.</param>
		/// <returns></returns>
		public virtual bool DeregisterMessageDelegate(GauntletIncomingRpcMessageDelegate FunctionToRun, string SenderFilter = "", string CategoryFilter = "", string PayloadFilter = "")
		{
			bool bDeregisteredDelegate = false;
			foreach (GauntletRpcDelegateRegistryFilter MessageFilter in MessageDelegates.Keys)
			{
				if (MessageFilter.DesiredSenderId == SenderFilter && MessageFilter.DesiredCategory == CategoryFilter && MessageFilter.DesiredPayload == PayloadFilter)
				{
					if (FunctionToRun == MessageDelegates[MessageFilter])
					{
						if(MessageDelegates.TryRemove(MessageFilter, out var Unused))
						{
							bDeregisteredDelegate = true;
							Log.Verbose("Delegate removed for filter: Sender: " + SenderFilter + " | Category: " + CategoryFilter + " | Payload: " + PayloadFilter);
						}
						else
						{
							Log.Verbose("Delegate unexpectedly already removed");
						}
					}
				}
			}

			return bDeregisteredDelegate;
		}


		protected virtual void FireRelevantMessageDelegates(GauntletIncomingRpcMessage NewMessage)
		{
			var MatchedFilters = MessageDelegates.Keys
				.Where(key => NewMessage.MatchesMessageFilter(key))
				.ToList();

			foreach (GauntletRpcDelegateRegistryFilter MessageFilter in MatchedFilters)
			{
				MessageDelegates[MessageFilter].Invoke(NewMessage);
				if (MessageFilter.bDeregisterAfterFirstTrigger)
				{
					MessageDelegates.TryRemove(MessageFilter, out var Unused);
				}
			}
		}

		public void StartListenThread()
		{
			if (HttpListenThread == null)
			{
				HttpListenThread = new Thread(ListenLoopThread);
				HttpListenThread.Start();
			}
			else
			{
				Gauntlet.Log.Warning(KnownLogEvents.Gauntlet, "RPC Executor tried to start a new listening thread but there's already an active one in the \"{RPCListeningThreadState}\" state", HttpListenThread.ThreadState);
				if (HttpListenThread.ThreadState != ThreadState.Running && !RequestListener.IsListening)
				{
					Gauntlet.Log.Info("Trying to force start RPC listen thread");
					bHasReceivedTerminationCall = false;
					HttpListenThread = new Thread(ListenLoopThread);
					HttpListenThread.Start();
				}
			}
		}

		public void ShutdownListenThread()
		{
			Gauntlet.Log.Verbose("RPC Executor shutting down Listen Thread");
			if (HttpListenThread != null)
			{
				bHasReceivedTerminationCall = true;
				RequestListener.Abort();
			}
		}

		/// <summary>
		/// Delegate type for handling messages on a http route.
		/// </summary>
		public delegate void GauntletHttpRouteDelegate(string MessageBody, ResponseData CallResponse);

		public void RegisterHttpRoute(RouteMapping Path, GauntletHttpRouteDelegate RouteDelegate, bool bOverrideIfExists = false)
		{
			if (HttpRoutes.ContainsKey(Path))
			{
				if (bOverrideIfExists)
				{
					Log.Info(string.Format("Overwriting endpoint {0} | {1} with new delegate.", Path.Path, Path.Verb));
					HttpRoutes[Path] = RouteDelegate;
				}
				else
				{
					Log.Error(string.Format("Unable to register endpoint {0} | {1}  as that route is already in use and bOverrideIfExists is set to false.", Path.Path, Path.Verb));
				}
			}
			else
			{
				Log.Info(string.Format("Registering endpoint {0} | {1} with new delegate.", Path.Path, Path.Verb));
				if(!HttpRoutes.TryAdd(Path, RouteDelegate))
				{
					Log.Error($"Unexpectedly encountered existing RouteMapping for {Path}");
				}
			}
		}

		public void DeregisterHttpRoute(RouteMapping Path)
		{
			if (HttpRoutes.ContainsKey(Path) && HttpRoutes.TryRemove(Path, out var Unused))
			{
				Log.Info(string.Format("Deregistered endpoint {0}", Path));
			}
			else
			{
				Log.Info(string.Format("Failed to deregister nonexistant endpoint {0}", Path));
			}
		}

		public static HttpResponseMessage CallRpc(RpcTarget InTarget, string RpcName, int Timeout = 5000, Dictionary<string, object> InArgs = null, string InContentType = "application/json")
		{
			string RpcUrl = InTarget.CreateRpcUrl(RpcName);
			RpcDefinition RpcEntry = InTarget.GetRpc(RpcName);

			Log.VeryVerbose($"Calling RPC {RpcName} on target {InTarget.TargetName} at {RpcUrl}");
			if (string.IsNullOrEmpty(RpcUrl) || RpcEntry == null)
			{
				Log.Verbose($"Failed Calling RPC {RpcName}");
				return null;
			}
			if (InArgs == null)
			{
				InArgs = new Dictionary<string, object>();
			}

			HttpMethod MethodToUse = new HttpMethod(RpcEntry.Verb);
			HttpRequestMessage RpcRequest = new HttpRequestMessage(MethodToUse, RpcUrl);
			RpcRequest.Headers.Add("rpcname", RpcName);
			RpcRequest.Headers.Connection.Add("keep-alive");
			RpcRequest.Headers.Add("Keep-Alive", $"{Timeout}");
			if (RpcEntry.Args != null && RpcEntry.Args.Count > 0)
			{
				foreach (RpcArgumentDesc ArgDesc in RpcEntry.Args)
				{
					object ArgValue;

					if (!InArgs.TryGetValue(ArgDesc.Name, out ArgValue))
					{
						if ( !ArgDesc.Optional )
						{
							Log.Warning(string.Format("RPC {0} lists mandatory argument {1} of type {2} which was not provided. Please either repair argument list or update RPC Definition.",
								RpcName, ArgDesc.Name, ArgDesc.Type));
						}
						continue;
					}
					
					if (ArgValue == null)
					{
						Log.Warning(string.Format("RPC {0} argument {1} of expected type {2} is null. If null is a possible value please make value optional in RPC instead.",
								RpcName, ArgDesc.Name, ArgDesc.Type));
						continue;
					}
					bool CastSucceeded = false;
					switch (ArgDesc.Type.ToLower())
					{
						case "bool":
							{
								CastSucceeded = ArgValue is bool;
								break;
							}
						case "int":
							{
								CastSucceeded = ArgValue is int;
								break;
							}
						case "float":
							{
								CastSucceeded = ArgValue is float;
								break;
							}
						case "double":
							{
								CastSucceeded = ArgValue is double;
								break;
							}
						case "string":
							{
								CastSucceeded = ArgValue is string;
								break;
							}
						case "object":
							{
								CastSucceeded = ArgValue is not null;
								break;
							}
						default:
							{
								// If listed object type is something not handled in this switch, either add a new handler or we basically just
								// accept that it is a var type that is not going to be automatically linted.
								break;
							}
					}
					if (!CastSucceeded)
					{
						Log.Warning(string.Format("RPC {0} unable to cast arg {1} value \"{2}\" to defined type {3}. Please either repair argument list or update RPC Definition.",
							RpcName, ArgDesc.Name, ArgValue, ArgDesc.Type));
					}
				}
			}
			if (InArgs.Count > 0)
			{
				ASCIIEncoding Encoding = new ASCIIEncoding();
				string JsonBody = JsonConvert.SerializeObject(InArgs);
				byte[] RequestBytes = Encoding.GetBytes(JsonBody);
				RpcRequest.Content = new StringContent(JsonBody);
			}
			// We might timeout calling CallRpc on a client that's closing - we do not want this to be a fail;
			try
			{
				HttpResponseMessage RpcResponse = ExecutorClient.Send(RpcRequest);
				return RpcResponse;
			}
			catch (Exception Error)
			{
				string Message = $"CallRpc for: {RpcName} failed with following exception: {Error}";

				Log.SuspendECErrorParsing();
				if (InTarget.IsInvalidated())
				{
					Log.Verbose(Message);
				}
				else
				{
					Log.Info(Message);
				}
				Log.ResumeECErrorParsing();
				return null;
			}
		}
		
		public static HttpResponseMessage CallRPCWithUrlParams(RpcTarget InTarget, string RpcName, Dictionary<string, object> InArgs, int Timeout = 5000,  string InContentType = "application/json")
        {
        	Log.VeryVerbose($"Calling RPC {RpcName}");
        	string RpcUrl = InTarget.CreateRpcUrl(RpcName);
        	RpcDefinition RpcEntry = InTarget.GetRpc(RpcName);
        	if (string.IsNullOrEmpty(RpcUrl) || RpcEntry == null)
        	{
        		Log.Verbose($"Failed Calling RPC {RpcName}");
        		return null;
        	}
            
            //format the arguments for the url, sanitizing to remove invalid characters e.g. & = and /
            //only special characters allowed are _ - and . 
            string UrlArgs = string.Empty;
            string SanitizingRegex = "[^a-zA-Z0-9._-]";
            foreach (KeyValuePair<string, object> Argument in InArgs)
            {
	            string Key = Regex.Replace(Argument.Key, SanitizingRegex, string.Empty);
	            if (Key != Argument.Key)
	            {
					Log.Verbose($"Argument Key {Argument.Key} sanitized to remove invalid characters. New Value: {Key}");    
	            }

	            string ArgumentValue = $"{Argument.Value}";
	            string Value = Regex.Replace(ArgumentValue, SanitizingRegex, string.Empty);
	            if (Value != ArgumentValue)
	            {
		            Log.Verbose($"Argument Value {ArgumentValue} sanitized to remove invalid characters. New Value: {Value}");   
	            }
	            
	            UrlArgs += $"{Key}={Value}";
	            if (!Argument.Equals(InArgs.Last()))
	            {
		            UrlArgs += "&";
	            }
            }
            RpcUrl = String.Format($"{RpcUrl}?{UrlArgs}");
            HttpMethod MethodToUse = new HttpMethod(RpcEntry.Verb);
        	HttpRequestMessage RpcRequest = new HttpRequestMessage(MethodToUse, RpcUrl);
			RpcRequest.Headers.Connection.Add("keep-alive");
			RpcRequest.Headers.Add("Keep-Alive", "600");

			// We might timeout calling CallRpc on a client that's closing - we do not want this to be a fail;
			try
        	{
				HttpResponseMessage RpcResponse = ExecutorClient.Send(RpcRequest);
        		return RpcResponse;
        	}
        	catch (Exception Error)
        	{
				string Message = $"CallRpc for: {RpcName} failed with following exception: {Error}";

				Log.SuspendECErrorParsing();
				if (InTarget.IsInvalidated())
				{
					Log.Verbose(Message);
				}
				else
				{
					Log.Info(Message);
				}
				Log.ResumeECErrorParsing();
        		return null;
        	}
        }
	}
}