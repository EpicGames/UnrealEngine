// Copyright Epic Games, Inc. All Rights Reserved.

#nullable enable

using System;
using System.Collections.Generic;
using System.Diagnostics.CodeAnalysis;
using System.Net.Http;
using System.Threading;
using System.Threading.Tasks;
using Newtonsoft.Json;

namespace Gauntlet;

public partial class RpcExecutor
{
	/// <summary>
	/// Asynchronously send an RPC to the target.
	/// </summary>
	/// <param name="InTarget">
	/// The <see cref="RpcTarget"/> to send the RPC to.
	/// </param>
	/// <param name="InRpcName">
	/// The name of the RPC to send.
	/// </param>
	/// <param name="InArgs">
	/// A map of parameters
	/// </param>
	/// <param name="InCancellationToken">
	/// The <see cref="CancellationToken"/> to monitor for a cancellation request.
	/// </param>
	/// <returns>
	/// A <see cref="Task{TResult}"/> representing the target's response.
	/// </returns>
	/// <exception cref="ArgumentException">
	/// The <paramref name="InTarget"/>, <paramref name="InRpcName"/>, or <paramref name="InArgs"/> parameters are invalid.
	/// </exception>
	public static Task<HttpResponseMessage> CallRpcAsync(
		RpcTarget InTarget,
		string InRpcName,
		IReadOnlyDictionary<string, object> InArgs,
		CancellationToken InCancellationToken = default)
	{
		ArgumentNullException.ThrowIfNull(InTarget);
		ArgumentException.ThrowIfNullOrEmpty(InRpcName);
		ArgumentNullException.ThrowIfNull(InArgs);

		if (!TryGetRpcDefinition(InTarget, InRpcName, out var RpcEntry))
		{
			throw new ArgumentException(@"Failed to find RPC definition.", nameof(InRpcName));
		}

		if (!TryGetRpcUrl(InTarget, InRpcName, out var RpcUrl))
		{
			// There is no reason this would ever hit as the prior exception would have hit
			throw new ArgumentException(@"Failed to create RPC url.", nameof(InRpcName));
		}

		ValidateRpcArguments(RpcEntry, InArgs);

		var MethodToUse = new HttpMethod(RpcEntry.Verb);

		var RpcRequest = new HttpRequestMessage(MethodToUse, RpcUrl);
		RpcRequest.Headers.Connection.Add("keep-alive");
		RpcRequest.Headers.Add("Keep-Alive", "600");
		RpcRequest.Headers.Add("rpcname", InRpcName);

		if (InArgs.Count > 0)
		{
			var JsonBody = JsonConvert.SerializeObject(InArgs);

			RpcRequest.Content = new StringContent(JsonBody);
		}

		Log.VeryVerbose($"calling RPC {InRpcName} on target {InTarget.TargetName} at {RpcUrl}");

		return ExecutorClient.SendAsync(RpcRequest, InCancellationToken);
	}

	private static void ValidateRpcArguments(RpcDefinition InRpcDefinition, IReadOnlyDictionary<string, object> InArgs)
	{
		// Should this be a custom exception type? RpcArgumentException or something?

		if (InRpcDefinition.Args is null)
		{
			return;
		}

		foreach (var RpcArgument in InRpcDefinition.Args)
		{
			if (!InArgs.TryGetValue(RpcArgument.Name, out var Value))
			{
				if (!RpcArgument.Optional)
				{
					throw new ArgumentException($@"Required RPC argument '{RpcArgument.Name}' is missing.");
				}

				continue;
			}

			// Most of the existing codebase doesn't have nullable enabled.
			// Guard against potential null reference exceptions.
			if (Value is null)
			{
				throw new ArgumentException($@"RPC argument '{RpcArgument.Name}' is null. It should be marked as optional if not required.");
			}

			if (RpcArgumentTypes.TryGetValue(RpcArgument.Type, out var Type))
			{
				var ValueType = Value.GetType();

				if (Value.GetType() != Type)
				{
					throw new ArgumentException($"Invalid type for RPC argument '{RpcArgument.Name}'. Expected {Type.Name}. Was {ValueType.Name}");
				}
			}
			else
			{
				// CallRpc just allows this so we will as well
			}
		}
	}

	private static readonly IReadOnlyDictionary<string, Type> RpcArgumentTypes = new Dictionary<string, Type>
	{
		{ "bool", typeof(bool) },
		{ "int", typeof(int) },
		{ "float", typeof(float) },
		{ "double", typeof(double) },
	};

	private static bool TryGetRpcDefinition(RpcTarget InTarget, string InName, [NotNullWhen(true)] out RpcDefinition? OutDefinition)
	{
		OutDefinition = InTarget.GetRpc(InName);

		return OutDefinition is not null;
	}

	private static bool TryGetRpcUrl(RpcTarget InTarget, string InName, [NotNullWhen(true)] out string? OutUrl)
	{
		OutUrl = InTarget.CreateRpcUrl(InName);

		return !string.IsNullOrEmpty(OutUrl);
	}
}