// Copyright Epic Games, Inc. All Rights Reserved.

using AutomationTool;
using Newtonsoft.Json;
using System;
using System.Collections.Concurrent;
using System.Collections.Generic;
using System.IO;
using System.Linq;
using System.Net;
using System.Net.Http;
using System.Net.NetworkInformation;
using UnrealBuildTool;
using static Gauntlet.RpcExecutor;

namespace Gauntlet
{
	/// <summary>
	///  Contains all information relevant to RPC Targets (Unreal Processes) 
	/// </summary>
	public class RpcTarget
	{
		public string IpAddress;
		public int Port;
		public UnrealTargetRole TargetType;
		public string TargetName;
		public List<RpcDefinition> AvailableRPCs;
		/// <summary>
		/// If this is true, we know the target is capable of sending messages to us and has available RPCs and we no longer need to manually poll those processes for messages.
		/// </summary>
		public bool HasSentMessage = false;

		/// <summary>
		/// Returns true if a valid numeric IP address has been set, a valid IP has been received from the game via /updateip,
		/// or the address is localhost (which is a valid local target on Win64 even without a /updateip update).
		/// Returns false if the address is still an unresolved hostname or placeholder.
		/// </summary>
		public bool HasValidIpAddress => IPAddress.TryParse(IpAddress, out _) ||
			string.Equals(IpAddress, "localhost", StringComparison.OrdinalIgnoreCase);
		public bool IsDeferStart = false;
		protected bool bIsInvalidated = false;
		public ConcurrentQueue<GauntletIncomingRpcMessage> IncomingMessageQueue = new ();

		public RpcTarget(string InIpAddress, int InPort, string InName, UnrealTargetRole InType = UnrealTargetRole.Client)
		{
			IpAddress = InIpAddress;
			Port = InPort;
			AvailableRPCs = new List<RpcDefinition>();
			TargetType = InType;
			TargetName = InName;
		}

		public RpcDefinition GetRpc(string RpcName)
		{
			foreach (RpcDefinition CurrentEntry in AvailableRPCs)
			{
				if (CurrentEntry.Name == RpcName)
				{
					return CurrentEntry;
				}
			}
			return null;
		}

		public string CreateRpcUrl(string RpcName)
		{
			RpcDefinition TargetRPC = GetRpc(RpcName);
			if (TargetRPC == null)
			{
				return String.Empty;
			}
			return string.Format("http://{0}:{1}{2}", IpAddress, Port, TargetRPC.Route);
		}

		public bool UpdateAvailableRpcs()
		{
			string StatusUrl = string.Format("http://{0}:{1}/listrpcs", IpAddress, Port);
			HttpRequestMessage ListRpcRequest = new HttpRequestMessage(HttpMethod.Get, StatusUrl);
			try
			{
				using (HttpResponseMessage RpcResponse = RpcExecutor.ExecutorClient.Send(ListRpcRequest))
				{
					string ResponseString = "";
					using (StreamReader ResponseReader = new StreamReader(RpcResponse.Content.ReadAsStream()))
					{
						ResponseString = ResponseReader.ReadToEnd();
						ResponseReader.Close();
					}
					if(ResponseString.Contains("errorCode") || ResponseString.Contains("errorMessage"))
					{
						if(ResponseString.Contains("route_handler_not_found"))
						{
							Log.Error($"Failed to call {StatusUrl} - The RPC was not identified by the target \"{TargetName}\".");
						}
						else if(ResponseString.Contains("mismatched_content_length_body_too_large"))
						{
							Log.Warning($"Failed to call {StatusUrl} (\"{TargetName}\") - Response body exceeded the declared Content-Length. The target's RPC list may have changed during serialization. This is typically transient and will resolve on the next update.");
						}
						else
						{
							Log.Error($"Failed to call {StatusUrl} (\"{TargetName}\") - Reason unknown. Please check the following Response String and update the error handler: {ResponseString}");
						}
						return false;
					}
					AvailableRPCs = JsonConvert.DeserializeObject<List<RpcDefinition>>(ResponseString);
				}
				
				Log.Verbose($"{TargetName} received new list of {AvailableRPCs.Count} available RPCs.");
			}
			catch (Exception e)
			{
				Log.Info(e.Message);
				// Not sure what to put here as RpcRegistry could quite possibly not be initialized by design.
				return false;
			}
			return AvailableRPCs.Count > 0;
		}

		// Logging Helper extensions that prepends the TargetName to the log message to help identify which process it came from
		// returns the formatted message
		public string LogInfo(string Format, params object[] Args)
		{
			return LogMessage(Microsoft.Extensions.Logging.LogLevel.Information, Format, Args);
		}

		public string LogWarning(string Format, params object[] Args)
		{
			return LogMessage(Microsoft.Extensions.Logging.LogLevel.Warning, Format, Args);
		}

		public string LogError(string Format, params object[] Args)
		{
			return LogMessage(Microsoft.Extensions.Logging.LogLevel.Error, Format, Args);
		}
		public string LogVerbose(string Format, params object[] Args)
		{
			return LogMessage(Microsoft.Extensions.Logging.LogLevel.Debug, Format, Args);
		}

		private string LogMessage(Microsoft.Extensions.Logging.LogLevel LogLevel, string Format, params object[] Args)
		{
			// Prepend TargetName as long as it exists.
			if (!string.IsNullOrEmpty(TargetName))
			{
				Format = string.Format("{0} - {1}", TargetName, Format);
			}

			switch (LogLevel)
			{
				case Microsoft.Extensions.Logging.LogLevel.Warning:
					Log.Warning(Format, Args);
					break;
				case Microsoft.Extensions.Logging.LogLevel.Error:
					Log.Error(Format, Args);
					break;
				case Microsoft.Extensions.Logging.LogLevel.Debug:
					Log.Verbose(Format, Args);
					break;
				default:
					Log.Info(Format, Args);
					break;
			}

			return string.Format(Format, Args);
		}

		public void SetIsInvalidated(bool InIsInvalidated = true)
		{
			bIsInvalidated = InIsInvalidated;
		}

		public bool IsInvalidated()
		{
			return bIsInvalidated;
		}
	}

	public class RpcTargetRegistry : IDisposable
	{
		#region RPC Target Accessors
		/// <summary>
		/// Quick accessor for the Editor Process RPC Target(if it is used)
		/// </summary>
		public RpcTarget EditorTarget { get; private set; }
		/// <summary>
		/// Quick accessor for the Server Process RPC Target(if it is used)
		/// </summary>
		public RpcTarget ServerTarget { get; private set; }
		/// <summary>
		/// Quick accessor for the Client Process RPC Target(if it is used). *Note* If multiple clients are in use, this will point to the first one registered.
		/// </summary>
		public RpcTarget ClientTarget { get; private set; }
		#endregion

		#region Multi-process Indexes
		/// <summary>
		/// When creating multiple RPC targets for each process role, these indexes are used to keep track
		/// of how many have been configured. This value is appended to the SenderId and increments the 
		/// port so each process has a unique name and port.
		/// </summary>
		private int CurrentClientRpcTargetIndex = 0;
		private int CurrentServerRpcTargetIndex = 0;
		private int CurrentEditorRpcTargetIndex = 0;
		#endregion

		#region RPC Target Configuration Base Values
		/// <summary>
		/// Base value used to build the SenderId of a RPC Target.
		/// </summary>
		private string BaseEditorRpcTargetName = "EditorRpcTarget_";
		/// <summary>
		/// Unique ID used by the Server to send or identify receipt of RPC's
		/// </summary>
		private string BaseServerRpcTargetName = "ServerRpcTarget_";
		/// <summary>
		/// Unique ID used by the Client to send or identify receipt of RPC's
		/// </summary>
		private string BaseClientRpcTargetName = "ClientRpcTarget_";

		/// <summary>
		/// Desired default URI. For tests running on the same machine, can be set to the same value.
		/// </summary>
		private string BaseRpcTargetUri = "localhost";

		/// <summary>
		/// Message sent by an RPC enabled process 
		/// </summary>
		private string RpcUpdateMessage = "RpcListUpdated";

		// UE Process RPC Base Listening Ports
		/// <summary>
		/// Base listening port for RPC calls to clients. Recommend to use ports be in the 12000 range.
		/// </summary>
		private int BaseClientRpcTargetListeningPort = 12321;

		/// <summary>
		/// Base listening port for RPC calls to the editor. Recommend to use ports be in the 11000 range.
		/// </summary>
		private int BaseEditorRpcTargetListeningPort = 11321;

		/// <summary>
		/// Base listening port for RPC calls to servers. Recommend to use ports be in the 13000 range.
		/// </summary>
		private int BaseServerRpcTargetListeningPort = 13321;

		#endregion // RPC Target Configuration Base Values

		/// <summary>
		/// Roles which have a RpcTarget registered, only one target should register for a single role
		/// </summary>
		private readonly ConcurrentDictionary<UnrealTestRole, RpcTarget> RoleTargets = new(); 

		/// <summary>
		/// All registered RPC Targets stored here, in a thread-safe container.
		/// </summary>
		public readonly ConcurrentDictionary<string, RpcTarget> RpcTargets = new();


		public RpcTargetRegistry()
		{
			RpcExecutor.Instance.RegisterHttpRoute(new RouteMapping("/updateip"), UpdateTargetIp, true);
			RpcExecutor.Instance.RegisterMessageDelegate(ReceivedRpcListUpdateMessage, "", "", RpcUpdateMessage);
		}

		~RpcTargetRegistry()
		{
			ReleaseUnmanagedResources();
		}

		public void AddNewTarget(string InName, string IpAddress = "localhost", int Port = 12321, UnrealTargetRole InType = UnrealTargetRole.Client)
		{
			AddNewTarget(new RpcTarget(IpAddress, Port, InName, InType));
		}

		public void AddNewTarget(RpcTarget InTarget)
		{
			if (GetTarget(InTarget.IpAddress, InTarget.Port) == null)
			{
				RpcTargets.TryAdd(InTarget.TargetName, InTarget);
				CacheFirstRpcTargetByType(InTarget.TargetName, InTarget.TargetType);
			}
		}

		public RpcTarget GetTarget(string InIpAddress, int InPort)
		{
			foreach (RpcTarget Registry in RpcTargets.Values)
			{
				if (Registry.IpAddress == InIpAddress && Registry.Port == InPort)
				{
					return Registry;
				}
			}
			return null;
		}

		public RpcTarget GetTarget(string InTargetName)
		{
			if (!RpcTargets.ContainsKey(InTargetName))
			{
				return null;
			}

			return RpcTargets[InTargetName];
		}

		public void ReceivedRpcListUpdateMessage(RpcExecutor.GauntletIncomingRpcMessage InMessage)
		{
			RpcTarget InTarget = GetTarget(InMessage.SenderId);
			if (InTarget != null)
			{
				InTarget.HasSentMessage = InTarget.UpdateAvailableRpcs();
			}
		}

		public void UpdateAvailableRpcForAllTargets()
		{
			foreach (RpcTarget NextTarget in RpcTargets.Values)
			{
				if (!NextTarget.HasSentMessage)
				{
					NextTarget.UpdateAvailableRpcs();
				}
			}
		}

		public List<RpcTarget> GetAllTargetsOfType(UnrealTargetRole InTargetType)
		{
			return RpcTargets.Values.Where(t => t.TargetType == InTargetType).ToList();
		}

		public List<RpcTarget> GetAllClients()
		{
			return GetAllTargetsOfType(UnrealTargetRole.Client);
		}

		public List<RpcTarget> GetAllServers()
		{
			return GetAllTargetsOfType(UnrealTargetRole.Server);
		}

		public virtual void UpdateTargetIp(string MessageBody, ResponseData CallResponse)
		{
			ConsoleColor PrevForegroundColor = Console.ForegroundColor;
			Console.ForegroundColor = ConsoleColor.Magenta;
			CallResponse.ResponseCode = (int)HttpStatusCode.OK;
			Dictionary<string, string> IncomingMessage = JsonConvert.DeserializeObject<Dictionary<string, string>>(MessageBody);
			if (!IncomingMessage.ContainsKey("newip") || !IncomingMessage.ContainsKey("target"))
			{
				CallResponse.ResponseCode = (int)HttpStatusCode.BadRequest;
				CallResponse.ResponseBody = "UpdateTargetIp: Request requires both newip and target values in body";
				return;
			}
			
			if (!IPAddress.TryParse(IncomingMessage["newip"], out IPAddress NewIp))
			{
				CallResponse.ResponseCode = (int)HttpStatusCode.BadRequest;
				CallResponse.ResponseBody = "UpdateTargetIp: newip is invalid";
				return;
			}

			RpcTarget DesiredTarget = GetTarget(IncomingMessage["target"]);
			if (DesiredTarget == null)
			{
				CallResponse.ResponseCode = (int)HttpStatusCode.BadRequest;
				CallResponse.ResponseBody = "UpdateTargetIp: Target not found";
				return;
			}

			// Some platform subsocket systems return an IPv6 address that is mapped to an IPv4.
			// The http client requires special formatting of the uri's to handle these, so instead just convert them to a plain IPv4 address
			DesiredTarget.IpAddress = NewIp.MapToIPv4().ToString();
			CallResponse.ResponseBody = String.Format("UpdateTargetIp: Target {0} IP updated to {1}", DesiredTarget.TargetName, DesiredTarget.IpAddress);
			Console.WriteLine(CallResponse.ResponseBody);
			Console.ForegroundColor = PrevForegroundColor;
			Log.Info(CallResponse.ResponseBody);
			DesiredTarget.UpdateAvailableRpcs();
		}

		#region RpcTarget Creation Methods
		/// <summary>
		/// Creates a Client RpcTarget with a guaranteed unique port and name.
		/// Post-Increments the target index.
		/// </summary>
		private RpcTarget GetNextAvailableRpcTarget(UnrealTargetRole Role)
		{
			int TargetPort = 0;
			string TargetName = string.Empty;
			try
			{
				switch (Role)
				{
					case UnrealTargetRole.Client:
					case UnrealTargetRole.EditorGame:
						TargetPort = BaseClientRpcTargetListeningPort + CurrentClientRpcTargetIndex;
						TargetName = BaseClientRpcTargetName + CurrentClientRpcTargetIndex.ToString();
						CurrentClientRpcTargetIndex++;
						break;
					case UnrealTargetRole.Editor:
					case UnrealTargetRole.CookedEditor:
						TargetPort = BaseEditorRpcTargetListeningPort + CurrentEditorRpcTargetIndex;
						TargetName = BaseEditorRpcTargetName + CurrentEditorRpcTargetIndex.ToString();
						CurrentEditorRpcTargetIndex++;
						break;
					case UnrealTargetRole.Server:
					case UnrealTargetRole.EditorServer:
						TargetPort = BaseServerRpcTargetListeningPort + CurrentServerRpcTargetIndex;
						TargetName = BaseServerRpcTargetName + CurrentServerRpcTargetIndex.ToString();
						CurrentServerRpcTargetIndex++;
						break;
					default:
						string ErrorMsg = "Trying to initialize RPC Target without an unknown UnrealTargetRole!";
						Log.Error(ErrorMsg);
						throw new TestException(ErrorMsg);
				}

				RpcTarget NextAvailableTarget = new RpcTarget(BaseRpcTargetUri, TargetPort, TargetName, Role);
				return NextAvailableTarget;
			}
			catch (TestException)
			{
				throw;
			}
			catch (Exception Ex)
			{
				string ErrorMsg = string.Format("GetNextAvailableRpcTarget failed due to general exception: Message: {0}, Call Stack: {1}", Ex.Message, Ex.StackTrace);
				Log.Error(ErrorMsg);
				throw new TestException(ErrorMsg);
			}
		}
		#endregion

		#region Target Config Helpers

		private bool InitialTargetRoleIsSet(UnrealTargetRole TargetRoleType) => TargetRoleType switch
		{
			UnrealTargetRole.Client => ClientTarget is not null,
			UnrealTargetRole.EditorGame => ClientTarget is not null,
			UnrealTargetRole.Server => ServerTarget is not null,
			UnrealTargetRole.EditorServer => ServerTarget is not null,
			UnrealTargetRole.Editor => EditorTarget is not null,
			UnrealTargetRole.CookedEditor => EditorTarget is not null,
			_ => false,
		};
		
		/// <summary>
		/// Caches the first retrieved Client, Editor or Server to a class variables for simplified access.
		/// </summary>
		/// <param name="RpcTargetName">TargetName of the RpcTarget we are trying to cache</param>
		/// <param name="TargetType">The <see cref="UnrealTargetRole"/> being cached.</param>
		private void CacheFirstRpcTargetByType(string RpcTargetName, UnrealTargetRole TargetType)
		{
			if (!InitialTargetRoleIsSet(TargetType))
			{
				RpcTarget RpcTargetToBeCached = GetTarget(RpcTargetName);
				switch (TargetType)
				{
					case UnrealTargetRole.Client:
					case UnrealTargetRole.EditorGame:
						ClientTarget = RpcTargetToBeCached;
						break;
					case UnrealTargetRole.Editor:
					case UnrealTargetRole.CookedEditor:
						EditorTarget = RpcTargetToBeCached;
						break;
					case UnrealTargetRole.Server:
					case UnrealTargetRole.EditorServer:
						ServerTarget = RpcTargetToBeCached;
						break;
					case UnrealTargetRole.Unknown:
					case UnrealTargetRole.Host:
					default:
						Log.Error($"No handling for setting {TargetType} client/editor/server target property on {nameof(RpcTargetRegistry)}.");
						break;
				}
			}
		}

		/// <summary>
		/// This is a command config helper to quickly add the correct command line parameters to Enable RPC's for the desired role
		/// and register the RpcTarget with the RpcTargetRegistry, if not already registered.
		/// </summary>
		/// <param name="TestRole">Role to enable </param>
		/// <param name="PortOverride"></param>
		/// <param name="UriOverride"></param>
		/// <exception cref="Exception"></exception>
		public void EnableRpcForRole(UnrealTestRole TestRole, int PortOverride = 0, string UriOverride = "")
		{
			try
			{
				RpcTarget CurrentTarget;
				if (RoleTargets.TryGetValue(TestRole, out CurrentTarget))
				{
					Log.Verbose($"RPC already enabled for {TestRole.Type}:{TestRole.GetHashCode()}");
				}
				else
				{
					CurrentTarget = GetNextAvailableRpcTarget(TestRole.Type);
					if(RoleTargets.TryAdd(TestRole, CurrentTarget))
					{
						AddNewTarget(CurrentTarget);
						CacheFirstRpcTargetByType(CurrentTarget.TargetName, TestRole.Type);
					}
					else
					{
						// somewhere else raced to add the same key
						CurrentTarget = RoleTargets[TestRole];
						Log.VeryVerbose($"RPC already enabled while trying to register for {TestRole.Type}:{TestRole.GetHashCode()}");
					}
					Log.Verbose($"RPC enabled for {TestRole.Type}:{TestRole.GetHashCode()}");
				}

				if (PortOverride != 0)
				{
					CurrentTarget.Port = PortOverride;
				}

				if (!string.IsNullOrEmpty(UriOverride))
				{
					CurrentTarget.IpAddress = UriOverride;
				}
				else
				{
					CurrentTarget.IpAddress = ResolveProcessUri(TestRole);
				}

				if (TestRole.DeferredLaunch)
				{
					CurrentTarget.IsDeferStart = true;
				}

				// defines the port the role's process should listen on and adds to command line
				TestRole.CommandLineParams.AddRawCommandline(string.Format("-rpcport={0}", CurrentTarget.Port));
				// Override HTTPServer.Listeners .ini settings
				TestRole.CommandLineParams.AddRawCommandline(string.Format("-ini:Engine:[HTTPServer.Listeners]:DefaultBindAddress={0},[HTTPServer.Listeners]:DefaultReuseAddressAndPort=true -logcmds=\"LogSockets VeryVerbose\"", CurrentTarget.IpAddress));
				// tells process what its sender ID is
				TestRole.CommandLineParams.Add("rpcsenderid", CurrentTarget.TargetName);
				// setup gauntlet ip and port so messages can be sent from process
				TestRole.CommandLineParams.Add("externalrpclistenaddress", RpcExecutor.GetListenAddressAndPort());
			}
			catch (TestException)
			{
				throw;
			}
			catch (Exception Ex)
			{
				string ErrorMsg = string.Format("EnableRpcForRole failed due to general exception: Message: {0}, Call Stack: {1}", Ex.Message, Ex.StackTrace);
				Log.Error(ErrorMsg);
				throw new TestException(ErrorMsg);
			}
		}

		/// <summary>
		/// Replace the current IP address values with the values obtained during devices reservation.
		/// IP addresses are replaced for all remote devices, as well as for local non-desktop devices.
		/// </summary>
		/// <param name="UnrealApp"></param>
		/// <param name="ActionReportInterrupt">Required for correct completion of test in case of failure and facilitating debugging.</param>
		/// <exception cref="TestException"></exception>
		public void UpdateIPsForRemoteAndNonDesktopTargets(UnrealSession UnrealApp, Action<string, object[]> ActionReportInterrupt)
		{
			foreach (ITargetDevice ReservedDevice in UnrealApp.UnrealDeviceReservation.ReservedDevices)
			{
				bool IsLocalDevice = ReservedDevice.Name.StartsWith("LocalDevice");

				if ((ReservedDevice is not TargetDeviceDesktopCommon) || (!IsLocalDevice))
				{
					UnrealSessionRole TargetRole = UnrealApp.RolesToDevices.FirstOrDefault(x => x.Value == ReservedDevice).Key;
					string RpcTargetName = TargetRole.CommandLineParams.GetParamValue("rpcsenderid").ToString();

					try
					{
						this.RpcTargets[RpcTargetName].IpAddress = ReservedDevice.Name;
					}
					catch (KeyNotFoundException Ex)
					{
						string ErrorMsg = string.Format("Test failed due to absence of required RpcTargetName in RpcTargets dictionary." +
														" Message: {0}, Call Stack: {1}", Ex.Message, Ex.StackTrace);
						ActionReportInterrupt(ErrorMsg, [Ex]);
						
						throw new TestException(ErrorMsg);
					}
				}
			}
		}

		/// <summary>
		/// This logic is similar to determining if -multihome should be used so being on VPN doesn't
		/// cause IP's to conflict.
		/// </summary>
		/// <param name="TestRole"></param>
		/// <returns>URI to be used for the TestRole passed in.</returns>
		/// <exception cref="AutomationException"></exception>
		private string ResolveProcessUri(UnrealTestRole TestRole)
		{
			string ProcessUri = BaseRpcTargetUri;

			// Default to the first address with a valid prefix
			var LocalAddress = Dns.GetHostEntry(Dns.GetHostName()).AddressList
				.Where(o => o.AddressFamily == System.Net.Sockets.AddressFamily.InterNetwork
					&& o.GetAddressBytes()[0] != 169)
				.FirstOrDefault();

			var ActiveInterfaces = NetworkInterface.GetAllNetworkInterfaces()
				.Where(I => I.OperationalStatus == OperationalStatus.Up);

			bool MultipleInterfaces = ActiveInterfaces.Count() > 1;

			if (MultipleInterfaces)
			{
				// Now, lots of Epic PCs have virtual adapters etc, so see if there's one that's on our network and if so use that IP
				var PreferredInterface = ActiveInterfaces
					.Where(I => I.GetIPProperties().DnsSuffix.Equals("epicgames.net", StringComparison.OrdinalIgnoreCase))
					.SelectMany(I => I.GetIPProperties().UnicastAddresses)
					.Where(A => A.Address.AddressFamily == System.Net.Sockets.AddressFamily.InterNetwork)
					.FirstOrDefault();

				if (PreferredInterface != null)
				{
					LocalAddress = PreferredInterface.Address;
				}
			}

			if (LocalAddress == null)
			{
				throw new AutomationException("Could not find local IP address");
			}

			// see if ip was overridden on command line
			string RequestedServerIp = Globals.Params.ParseValue("serverip", "");
			string RequestedClientIp = Globals.Params.ParseValue("clientip", "");
			string ServerIp = string.IsNullOrEmpty(RequestedServerIp) ? LocalAddress.ToString() : RequestedServerIp;
			string ClientIp = string.IsNullOrEmpty(RequestedClientIp) ? LocalAddress.ToString() : RequestedClientIp;

			// Do we need to bind to a specific IP?
			if (TestRole.Type.IsServer() && (MultipleInterfaces || !string.IsNullOrEmpty(RequestedServerIp)))
			{
				ProcessUri = ServerIp;
			}

			// client too, but only desktop platforms
			if (TestRole.Type.IsClient() && (MultipleInterfaces || !string.IsNullOrEmpty(RequestedClientIp)))
			{
				string Platform = BuildHostPlatform.Current.Platform.ToString();
				if (Platform == "Win64" || Platform == "Mac")
				{
					ProcessUri = ClientIp;
				}
			}

			if (TestRole.Type.IsEditor() && MultipleInterfaces)
			{
				ProcessUri = LocalAddress.ToString();
			}

			return ProcessUri;
		}
		#endregion //Target Config Helpers

		private void ReleaseUnmanagedResources()
		{
			if (RpcExecutor.Instance != null)
			{
				Log.Verbose("Cleaning RPC executor instance");
				RpcExecutor.Instance.ShutdownListenThread();
				RpcExecutor.Instance.ClearIncomingMessageQueue();
				RpcExecutor.Instance.ClearMessageDelegates();
			}
			RpcTargets.Clear();
			
			EditorTarget = null;
			CurrentEditorRpcTargetIndex = 0;
			ServerTarget = null;
			CurrentServerRpcTargetIndex = 0;
			ClientTarget = null;
			CurrentClientRpcTargetIndex = 0;
		}

		public void Dispose()
		{
			ReleaseUnmanagedResources();
		}
	}
}
