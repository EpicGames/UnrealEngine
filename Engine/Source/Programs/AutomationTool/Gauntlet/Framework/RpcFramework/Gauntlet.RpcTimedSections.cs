// Copyright Epic Games, Inc. All Rights Reserved.

#nullable enable
using Gauntlet.Report;
using Gauntlet.TestTracking;
using System;
using System.Collections.Concurrent;
using System.Collections.Generic;
using System.Diagnostics;
using System.Linq;
using System.Text;
using static Gauntlet.HordeReport.AutomatedTestSessionData;
using Logging = Microsoft.Extensions.Logging;

namespace Gauntlet
{
	/// <summary>
	/// This component collects information about the execution of a given code block
	/// between calls to the StartSection and FinishSection methods. It collects information
	/// about the code block's run duration, depth, and start and finish times relative to 
	/// the test case's stopwatch.
	/// </summary>
	public class RpcTimedSections
	{
		public ConcurrentStack<TimedSection> ActiveSections { get; } = new();
		private ConcurrentDictionary<int, ExecutedSectionData> FinishedSections { get; } = new();

		/// <summary>
		/// Timer that is running during the whole test.
		/// May be started explicitly at the required place by call Stopwatch.StartNew().
		/// Automatically starts when the StartSection method is first called if it has not been started explicitly before.
		/// Based on this timer Start and End Offsets are calculated.
		/// </summary>
		public Stopwatch? ExecutionStopwatch { get; set; }

		private PhaseQueue? Phases;
		private RpcTargetRegistry TargetRegistry;

		/// <summary>
		/// The component can operate in two modes: with automatic Phase creation for each
		/// Section and without Phase creation.By default, Phases are not created.
		/// </summary>
		/// <param name="targetRegistry"></param>
		/// <param name="phases">Initialized object if Phase generation is required, and null if not.</param>
		public RpcTimedSections(RpcTargetRegistry TargetRegistry, PhaseQueue? Phases = null)
		{
			this.TargetRegistry = TargetRegistry;
			this.Phases = Phases;
		}

		/// <summary>
		/// Represents a Section that is currently running. When the Section completes its work,
		/// the data is transferred to the ExecutedSectionData object in accordance with the SectionIndex.
		/// </summary>
		public class TimedSection
		{
			public Stopwatch? Watch { get; set; }
			public string? SectionName { get; set; }

			/// <summary>
			/// Section ordinal number
			/// </summary>
			public int SectionIndex { get; set; }
		}

		/// <summary>
		/// Data container that contain a full set of data about Section. It is mainly filled in at the end of the Section's work.
		/// Also used to automatically add test date to the report.
		/// </summary>
		public class ExecutedSectionData : IPhaseDataHandler
		{
			public string? SectionName { get; set; }
			public double DurationInSeconds { get; set; }

			/// <summary>
			/// Section start time relative to test stopwatch
			/// </summary>
			public double StartOffsetSec { get; set; }

			/// <summary>
			/// Section end time relative to test stopwatch
			/// </summary>
			public double EndOffsetSec { get; set; }

			/// <summary>
			/// Section Nesting. Root level is 0.
			/// </summary>
			public int Depth { get; set; }

			/// <summary>
			/// Called when test phase is added to report
			/// </summary>
			/// <param name="ReportedPhase"></param>
			public void OnAddToReport(TestPhase ReportedPhase)
			{
				if (ReportedPhase.outcome == TestPhaseOutcome.Interrupted
					&& !ReportedPhase.GetStream().GetEvents()
						.Where(E => E.Level == Logging.LogLevel.Error || E.Level == Logging.LogLevel.Critical).Any())
				{
					ReportedPhase.GetStream().AddError("Failed to complete phase. See failing nested phases and Closure for details.");
				}
			}
		}

		/// <summary>
		/// The beginning of the code block for which information needs to be collected. Blocks can be nested
		/// </summary>
		/// <param name="Name">Code block name</param>
		public void StartSection(string Name)
		{
			TimedSection Section = new();
			Section.Watch = Stopwatch.StartNew();

			if (ExecutionStopwatch == null)
			{
				Log.Warning($"ExecutionStopwatch wasn't explicitly started. Automatically starting it during start of section '{Name}'");
				ExecutionStopwatch = Stopwatch.StartNew();
			}

			double SectionStartExecutionOffset = ExecutionStopwatch?.Elapsed.TotalSeconds ?? 0;

			Section.SectionName = Name;
			Section.SectionIndex = FinishedSections.Count;
			int Depth = ActiveSections.Count;
			string Line = $"{new string('=', Depth)}> START: {Name}";
			LogAllInfo(Line);
			ActiveSections.Push(Section);
			ExecutedSectionData SectionData = new ExecutedSectionData
			{
				SectionName = Section.SectionName,
				Depth = Depth,
				StartOffsetSec = SectionStartExecutionOffset,
			};

			if (!FinishedSections.TryAdd(FinishedSections.Count, SectionData))
			{
				Log.Warning($"Unable to add new item '{SectionData.SectionName}' to FinishedSections");
				return;
			}

			// If not need to create Phases for Sections, then skip this code
			if (Phases != null)
			{
				Phases.Start(Name, SectionData);
			}
		}

		/// <summary>
		/// Returns the currently active section, or an empty section if none are active
		/// </summary>
		/// <returns>Item from top of stack, or empty object if stack is empty</returns>
		public TimedSection GetCurrentActiveSection()
		{
			if (ActiveSections.TryPeek(out TimedSection? Section))
			{
				return Section;
			}

			Log.Warning("Unable to peek value from ActiveSections, the stack may be empty.");

			return new();
		}

		/// <summary>
		/// The finish of the code block for which information needs to be collected
		/// </summary>
		public void FinishSection()
		{
			bool PopResult = ActiveSections.TryPop(out TimedSection? Section);

			if (!PopResult)
			{
				Log.Warning("Unable to pop TimedSection from ActiveSections stack");

				if (ActiveSections.Count == 0)
				{
					Log.Warning("Count of Active Sections is 0. Looks like FinishSection called without calling StartSection first");
				}

				return;
			}

			Section?.Watch?.Stop();
			double SectionStopExecutionOffset = ExecutionStopwatch?.Elapsed.TotalSeconds ?? 0;

			string Line = $"{new string('=', ActiveSections.Count)}> END: {Section?.SectionName} at " +
							$"{SectionStopExecutionOffset:0.00}s (took {Section?.Watch?.Elapsed.TotalSeconds:0.00}s)";
			LogAllInfo(Line);

			int VerifiedSectionIndex = Section != null ? Section.SectionIndex : 0;
			ExecutedSectionData FinishedSectionData = FinishedSections[VerifiedSectionIndex];
			FinishedSectionData.DurationInSeconds = Section?.Watch?.Elapsed.TotalSeconds ?? 0;
			FinishedSectionData.EndOffsetSec = SectionStopExecutionOffset;

			// If not need to create Phases for Sections, then skip this code
			if (Phases != null)
			{
				if (!string.IsNullOrEmpty(Section?.SectionName)) Phases.Get(Section.SectionName)?.End();
				if (ActiveSections.Count > 0)
				{
					TimedSection SectionParent = GetCurrentActiveSection();

					if (SectionParent != null && !string.IsNullOrEmpty(SectionParent.SectionName))
					{
						Phases.SetCurrent(SectionParent.SectionName);
					}
				}
			}
		}

		/// <summary>
		/// Log the required message to the logs of all targets and the console
		/// </summary>
		/// <param name="Message"></param>
		public void LogAllInfo(string Message)
		{
			Log.Info(Message);

			foreach (var RpcTarget in TargetRegistry.RpcTargets)
			{
				RpcLibrary.LogMessageInTarget(RpcTarget.Value, Message);
			}
		}

		/// <summary>
		/// Returns information about Finished Sections with their Depth and Duration, or about 
		/// Active Sections if any section was not completed in formatted and readable view.
		/// Can be used as part of test run report
		/// </summary>
		/// <returns></returns>
		public string GetFormattedSummary()
		{
			StringBuilder SB = new();

			if (ActiveSections.Count > 0)
			{
				// If we still have active sections, then the test didn't finish.
				// Leave some breadcrumbs to make it easy to see where we stopped.
				SB.AppendLine("## Active sections:");
				int Depth = 0;
				foreach (var Section in ActiveSections.Reverse())
				{
					SB.AppendLine(new string('=', Depth) + $"> {Section.SectionName}");
					Depth += 1;
				}
				SB.AppendLine();
			}
			else
			{
				// Otherwise, we have finished the run and should print out some nice timings.
				SB.AppendLine("## Finished sections:");
				int NumSectionsWritten = 0;
				foreach (var Section in FinishedSections.OrderBy(data => data.Value.StartOffsetSec))
				{
					if (Section.Value.DurationInSeconds > 0)
					{
						NumSectionsWritten++;
						SB.AppendLine(new string('=', Section.Value.Depth) + $"> {Section.Value.SectionName} ({Section.Value.DurationInSeconds:0.00}s)");
					}
				}
				if (NumSectionsWritten == 0)
				{
					SB.AppendLine("(no sections finished, this run likely crashed very early)");
				}
				SB.AppendLine();
			}

			return SB.ToString();
		}

		/// <summary>
		/// Returns all gathered information about finished sections.
		/// The code between the stopwatch start point and the first section start is considered a "Boot" section.
		/// </summary>
		/// <returns></returns>
		public IList<object> GetSectionsData()
		{
			IList<object> SectionsData = [];

			// Additional section measuring time between boot and the first section
			double BootDuration = FinishedSections.Count > 0 ? FinishedSections[0].StartOffsetSec : 0.0;
			Dictionary<string, object> BootSectionData = new()
			{
				{ "SectionName", "Boot" },
				{ "Depth", 0 },
				{ "StartOffsetSec", 0.0 },
				{ "EndOffsetSec", BootDuration },
			};
			SectionsData.Add(BootSectionData);

			foreach (var section in FinishedSections)
			{
				Dictionary<string, object> CurrentSectionData = [];
				CurrentSectionData.Add("SectionName", section.Value.SectionName ?? "null");
				CurrentSectionData.Add("Depth", section.Value.Depth);
				CurrentSectionData.Add("StartOffsetSec", section.Value.StartOffsetSec);
				CurrentSectionData.Add("EndOffsetSec", section.Value.EndOffsetSec);
				SectionsData.Add(CurrentSectionData);
			}

			return SectionsData;
		}
	}
}
