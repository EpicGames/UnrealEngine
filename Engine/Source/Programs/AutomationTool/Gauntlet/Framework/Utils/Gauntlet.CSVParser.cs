// Copyright Epic Games, Inc. All Rights Reserved.

using System.IO;
using System.Collections.Generic;
using AutomationTool;
using System.Linq;


namespace Gauntlet
{
	public static class CSVParser
	{
		public static List<Dictionary<string, string>> Load(string File, string DefaultForMissingColumns = null)
		{
			List<Dictionary<string, string>> Rows = new List<Dictionary<string, string>>();
			// Parse csv file
			using (StreamReader Reader = new StreamReader(File))
			{
				// Parse Header
				string FirstLine = Reader.ReadLine();
				int LineNumber = 1;
				string[] HeaderValues = FirstLine.Split(',');
				// Parse the rest of the data
				while (!Reader.EndOfStream)
				{
					Dictionary<string, string> OneRow = new Dictionary<string, string>();
					string Line = Reader.ReadLine();
					LineNumber++;
					string[] Values = Line.Split(',');

					if (DefaultForMissingColumns != null)
					{
						if (Values.Length < HeaderValues.Length)
						{
							
							Values = Enumerable.Concat(Values,Enumerable.Repeat(DefaultForMissingColumns, HeaderValues.Length - Values.Length)).ToArray();
						}
					}

					if (Values.Length >= HeaderValues.Length)
					{
						for (int i = 0; i < HeaderValues.Length; i++)
						{
							OneRow[HeaderValues[i]] = Values[i];
						}
						Rows.Add(OneRow);
					}
					else
					{
						throw new AutomationException(string.Format("Not enough number of columns at line {0}.", LineNumber));
					}
				}
			}

			return Rows;
		}
	}
}