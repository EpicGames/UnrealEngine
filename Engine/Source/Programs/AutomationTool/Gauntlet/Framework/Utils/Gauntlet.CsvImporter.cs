// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Diagnostics.CodeAnalysis;
using System.Linq;
using System.IO;

namespace Gauntlet
{
	public enum CsvImportType
	{
		/// <summary>
		/// Standard import pointing to a csv file.
		/// </summary>
		CsvFile,

		/// <summary>
		/// An import that has no csv and any metadata is provided via the manifest.
		/// </summary>
		Inline
	}

	public interface ICsvMetadata
	{
		/// <summary>
		/// Get as a completed dict;
		/// </summary>
		public Dictionary<string, dynamic> ToDict();
	}

	/// <summary>
	/// Interface for a generic Csv Import.
	/// CsvImportEntry should suit most needs, but if you need something custom you can implement this interface.
	/// </summary>
	public interface ICsvImport
	{
		/// <summary>
		/// The type of import.
		/// </summary>
		public CsvImportType ImportType { get; }

		/// <summary>
		/// Friendly name of the import for logging purposes.
		/// </summary>
		public string ImportName { get; }

		/// <summary>
		/// The path to the csv file.
		/// </summary>
		[AllowNull]
		public string CsvFilename { get; }

		/// <summary>
		/// The path to the log file (Optional).
		/// </summary>
		[AllowNull]
		public string LogFilename { get; set; }

		/// <summary>
		/// Any additional metadata associated with this csv.
		/// Corresponds to 'CustomFields' in the datasource config.
		/// </summary>
		public Dictionary<string, dynamic> CustomFields { get; }

		/// <summary>
		/// Metadata for this import which varies on the import type.
		/// For CsvFile imports, this is additional metadata that's included with the csv's metadata (not currently implemented).
		/// For inline imports, this is the only source of metadata as there's no csv file.
		/// </summary>
		[AllowNull]
		public ICsvMetadata Metadata { get; set; }

		/// <summary>
		/// Additional csv stat averages to include.
		/// Currently only implemented for inline imports.
		/// </summary>
		[AllowNull]
		public Dictionary<string, double> AdditionalStatAverages { get; }

		/// <summary>
		/// Any additional files that should be bundled with this csv.
		/// Key is the file type, value is the path to the file.
		/// </summary>
		public IReadOnlyDictionary<string, string> AdditionalFiles { get; }

		/// <summary>
		/// Adds an additional file to this import.
		/// </summary>
		/// <param name="FileType">The unique FileType.</param>
		/// <param name="Filename">The path to the file.</param>
		public void AddAdditionalFile(string FileType, string Filename);
	}

	/// <summary>
	/// Generic Csv import entry.
	/// </summary>
	public class CsvImportEntry : ICsvImport
	{
		public CsvImportType ImportType { get; private set; } = CsvImportType.CsvFile;
		public string ImportName { get; private set; }

		public string CsvFilename { get; private set; } = null;
		public string LogFilename { get; set; } = null;
		public Dictionary<string, dynamic> CustomFields { get; private set; }
		public ICsvMetadata Metadata { get; set; } = null;
		public IReadOnlyDictionary<string, string> AdditionalFiles { get; private set; }
		public Dictionary<string, double> AdditionalStatAverages { get; private set; } = null;

		/// <summary>
		/// CsvFile import constructor.
		/// Deprecated: Please use CreateCsvFileImportEntry.
		/// </summary>
		/// <param name="CsvFilename">The path to the csv to import.</param>
		/// <param name="LogFilename">Optional path to a corresponding log to bundle with this csv.</param>
		/// <param name="CustomFields">Optional metadata corresponding to 'CustomFields' in the datasource config.</param>
		public CsvImportEntry(string CsvFilename, string LogFilename = null, Dictionary<string, dynamic> CustomFields = null)
		{
			this.ImportType = CsvImportType.CsvFile;
			this.ImportName = Path.GetFileName(CsvFilename);
			this.CsvFilename = CsvFilename;
			this.LogFilename = LogFilename;
			this.CustomFields = CustomFields ?? new Dictionary<string, dynamic>();
		}

		/// <summary>
		/// Creates an import entry for a csv file.
		/// </summary>
		/// <param name="CsvFilename">The path to the csv to import.</param>
		/// <param name="LogFilename">Optional path to a corresponding log to bundle with this csv.</param>
		/// <param name="CustomFields">Optional metadata corresponding to 'CustomFields' in the datasource config.</param>
		public static CsvImportEntry CreateCsvFileImportEntry(string CsvFilename, string LogFilename = null, Dictionary<string, dynamic> CustomFields = null)
		{
			return new CsvImportEntry(CsvFilename, LogFilename, CustomFields);
		}

		/// <summary>
		/// Creates an inline Import entry.
		/// </summary>
		/// <param name="ImportName">Identifier for this import, primarily for logging.</param>
		/// <param name="InlineMetadata">Metadata that would normally be provided by a csv. Must contain necessary fields.</param>
		/// <param name="CustomFields">Optional datasource specific metadata. Corresponds to "CustomFields" in datasource config.</param>
		/// <param name="AdditionalStatAverages">
		/// Optional dictionary of stat data to include in the import. These stats are treated like regular csv stat averages.
		/// It's expected that these stats have been sanitized by the user. Any invalid stats will cause an ArgumentException to be thrown.
		/// You can use the CsvStatSanitizer helper class to sanitize your stats.
		/// </param>
		public static CsvImportEntry CreateInlineImportEntry(string ImportName, ICsvMetadata CsvMetadata, Dictionary<string, double> AdditionalStatAverages = null, Dictionary<string, dynamic> CustomFields = null)
		{
			if (CsvMetadata == null)
			{
				throw new ArgumentNullException("CsvMetadata", "Inline imports must have valid metadata.");
			}

			if (AdditionalStatAverages != null)
			{
				foreach ((string StatName, double Value) in AdditionalStatAverages)
				{
					if (!CsvStatSanitizer.IsValidStatName(StatName))
					{
						throw new ArgumentException($"AdditionalStat {StatName} is invalid. Please sanitize stats with CsvStatSanitizer first.", "AdditionalStatAverages");
					}
				}
			}

			return new CsvImportEntry()
			{
				ImportType = CsvImportType.Inline,
				ImportName = ImportName,
				Metadata = CsvMetadata,
				CustomFields = CustomFields ?? new Dictionary<string, dynamic>(),
				CsvFilename = null,
				AdditionalStatAverages = AdditionalStatAverages
			};
		}

		public void AddAdditionalFile(string FileType, string Filename)
		{
			Dictionary<string, string> MutableAdditionalFiles = AdditionalFiles != null 
				? AdditionalFiles.ToDictionary(Kvp => Kvp.Key, Kvp => Kvp.Value)
				: new Dictionary<string, string>();
			
			if (MutableAdditionalFiles.ContainsKey(FileType))
			{
				throw new Exception("Filetype " + FileType + " already registered with CSV " + CsvFilename);
			}

			MutableAdditionalFiles[FileType] = Filename;
			AdditionalFiles = MutableAdditionalFiles;
		}

		private CsvImportEntry()
		{
		}
	}

	/// <summary>
	/// Csv Importer interface.
	/// </summary>
	public interface ICsvImporter
	{
		public void Import(IEnumerable<ICsvImport> Imports);
		public void Import(IEnumerable<ICsvImport> Imports, out string URLLink);
	}

	/// <summary>
	/// Helper class to sanitize stat names.
	/// </summary>
	public class CsvStatSanitizer
	{
		// Based on FCsvStatNameValidator::ValidCharacters in CsvProfiler.cpp
		private static string ValidCharacters = "0123456789abcdefghijklmnopqrstuvwxyzABCDEFGHIJKLMNOPQRSTUVWXYZ /_-[]()#.:";

		/// <summary>
		/// Returns true if the stat name only contains valid characters.
		/// </summary>
		public static bool IsValidStatName(string StatName)
		{
			return StatName.All(c => ValidCharacters.Contains(c));
		}

		/// <summary>
		/// Returns a new stat string with all invalid characters removed.
		/// </summary>
		public static string GetSanitizedStatName(string StatName)
		{
			return SanitizeNameWithReplacement(StatName, (c) => null);
		}

		/// <summary>
		/// Returns a new stat string with all invalid character replaced with the result of ReplacementFunc(char).
		/// If ReplacementFunc(c) returns null that character is skipped.
		/// </summary>
		public static string SanitizeNameWithReplacement(string StatName, Func<char, char?> ReplacementFunc)
		{
			string SanitizedName = string.Empty;
			foreach (char c in StatName)
			{
				if (!ValidCharacters.Contains(c))
				{
					char? ReplacementChar = ReplacementFunc(c);
					if (ReplacementChar != null)
					{
						SanitizedName += ReplacementChar;
					}
				}
				else
				{
					SanitizedName += c;
				}
			}
			return SanitizedName;
		}
	}
}
