// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.IO;
using System.Linq;
using System.Text.Json;
using System.Text.RegularExpressions;
using System.Text.Json.Serialization;
using AutomationTool;
using Microsoft.CodeAnalysis;
using EpicGames.Core;
using Logging = Microsoft.Extensions.Logging;
using System.Globalization;
using static Gauntlet.HordeReport.TestDataCollection;
using Gauntlet.Report;
using System.Diagnostics.CodeAnalysis;

namespace Gauntlet
{
	public static class HordeReport
	{
		/// <summary>
		/// Default location to store Test Data 
		/// </summary>
		public static string DefaultTestDataDir
		{
			get
			{
				return Path.GetFullPath(Environment.GetEnvironmentVariable("UE_TESTDATA_DIR") ?? Path.Combine(CommandUtils.CmdEnv.EngineSavedFolder, "TestData"));
			}
		}
		/// <summary>
		/// Default location to store test Artifacts 
		/// </summary>
		public static string DefaultArtifactsDir
		{
			get
			{
				return Path.GetFullPath(Environment.GetEnvironmentVariable("UE_ARTIFACTS_DIR") ?? CommandUtils.CmdEnv.LogFolder);
			}
		}

		public abstract class BaseHordeReport : BaseTestReport
		{
			/// <summary>
			/// Horde report version
			/// </summary>
			public override int Version => 1;

			protected string OutputArtifactPath;
			protected HashSet<string> ArtifactProcessedHashes;
			protected Dictionary<string, object> ExtraReports;

			/// <summary>
			/// Attach Artifact to the Test Report
			/// </summary>
			/// <param name="ArtifactPath"></param>
			/// <param name="Name"></param>
			/// <returns>return true if the file was successfully attached</returns>
			public override bool AttachArtifact(string ArtifactPath, string Name = null)
			{
				return AttachArtifact(ArtifactPath, Name, false);
			}
			/// <summary>
			/// Attach Artifact to the Test Report
			/// </summary>
			/// <param name="ArtifactPath"></param>
			/// <param name="Name"></param>
			/// <param name="Overwrite"></param>
			/// <returns>return true if the file was successfully attached</returns>
			public bool AttachArtifact(string ArtifactPath, string Name = null, bool Overwrite = false)
			{
				if (string.IsNullOrEmpty(OutputArtifactPath))
				{
					throw new InvalidOperationException("OutputArtifactPath must be set before attaching any artifact");
				}

				// Generate a hash from the artifact path
				string ArtifactHash = Hasher.ComputeHash(ArtifactPath, Hasher.DefaultAlgo, 8);

				if (ArtifactProcessedHashes == null)
				{
					ArtifactProcessedHashes = new HashSet<string>();
				}
				else
				{
					if (ArtifactProcessedHashes.Contains(ArtifactHash))
					{
						// already processed
						return true;
					}
				}

				// Mark as processed. Even in case of failure, we don't want to try over again.
				ArtifactProcessedHashes.Add(ArtifactHash);

				string TargetPath = Utils.SystemHelpers.GetFullyQualifiedPath(Path.Combine(OutputArtifactPath, Name ?? Path.GetFileName(ArtifactPath)));
				ArtifactPath = Utils.SystemHelpers.GetFullyQualifiedPath(ArtifactPath);
				bool isFileExist = File.Exists(ArtifactPath);
				if (isFileExist && (!File.Exists(TargetPath) || Overwrite))
				{
					try
					{
						string TargetDirectry = Path.GetDirectoryName(TargetPath);
						if (!Directory.Exists(TargetDirectry)) { Directory.CreateDirectory(TargetDirectry); }
						File.Copy(ArtifactPath, TargetPath, true);
						return true;
					}
					catch (Exception Ex)
					{
						Log.Error("Failed to copy artifact '{0}'. {1}", Path.GetFileName(ArtifactPath), Ex);
					}
					return false;
				}
				return isFileExist;
			}

			/// <summary>
			/// Set Output Artifact Path and create the directory if missing
			/// </summary>
			/// <param name="InPath"></param>
			/// <returns></returns>
			public void SetOutputArtifactPath(string InPath)
			{
				OutputArtifactPath = Path.GetFullPath(InPath);

				if (!Directory.Exists(OutputArtifactPath))
				{
					Directory.CreateDirectory(OutputArtifactPath);
				}
				Log.Verbose(string.Format("Test Report output artifact path is set to: {0}", OutputArtifactPath));
			}

			/// <summary>
			/// Attach a log path to a device instance name
			/// </summary>
			/// <param name="InstanceName"></param>
			/// <param name="LogPath"></param>
			/// <param name="LogName"></param>
			/// <returns></returns>
			public abstract bool AttachDeviceLog(string InstanceName, string LogPath, string LogName);

			public void AttachDependencyReport(object InReport, string Key = null)
			{
				if (ExtraReports == null)
				{
					ExtraReports = new Dictionary<string, object>();
				}
				if (InReport is BaseHordeReport InHordeReport)
				{
					Key = InHordeReport.GetTestDataKey(Key);
				}
				ExtraReports[Key] = InReport;
			}

			public override Dictionary<string, object> GetReportDependencies()
			{
				var Reports = base.GetReportDependencies();
				if (ExtraReports != null)
				{
					ExtraReports.ToList().ForEach(Item => Reports.Add(Item.Key, Item.Value));
				}
				return Reports;
			}

			/// <summary>
			/// Return the Key to be used for the TestData
			/// </summary>
			/// <param name="BaseKey"></param>
			/// <returns></returns>
			public virtual string GetTestDataKey(string BaseKey = null)
			{
				if (BaseKey == null)
				{
					return Type;
				}
				return string.Format("{0}::{1}", Type, BaseKey);
			}

			/// <summary>
			/// Return the data to be stored in the TestData
			/// </summary>
			/// <returns></returns>
			public virtual object GetTestData()
			{
				return this;
			}
		}

		#region Legacy Implementations
		/// <summary>
		/// Contains detailed information about device that run tests
		/// </summary>
		public class Device
		{
			public string DeviceName { get; set; }
			public string Instance { get; set; }
			public string Platform { get; set; }
			public string OSVersion { get; set; }
			public string Model { get; set; }
			public string GPU { get; set; }
			public string CPUModel { get; set; }
			public int RAMInGB { get; set; }
			public string RenderMode { get; set; }
			public string RHI { get; set; }
			public string AppInstanceLog { get; set; }
		}

		/// <summary>
		/// Contains reference to files used or generated for file comparison
		/// </summary>
		public class ComparisonFiles
		{
			public string Difference { get; set; }
			public string Approved { get; set; }
			public string Unapproved { get; set; }
		}
		/// <summary>
		/// Contains information about test artifact
		/// </summary>
		public class Artifact
		{
			public Artifact()
			{
				Files = new ComparisonFiles();
			}

			public string Id { get; set; }
			public string Name { get; set; }
			public string Type { get; set; }
			public ComparisonFiles Files { get; set; }
		}
		/// <summary>
		/// Contains information about test entry event
		/// </summary>
		public class Event
		{
			public EventType Type { get; set; }
			public string Message { get; set; }
			public string Context { get; set; }
			public string Artifact { get; set; }
		}
		/// <summary>
		/// Contains information about test entry
		/// </summary>
		public class Entry
		{
			public Entry()
			{
				Event = new Event();
			}

			public Event Event { get; set; }
			public string Filename { get; set; }
			public int LineNumber { get; set; }
			public string Timestamp { get; set; }
		}
		/// <summary>
		/// Contains detailed information about test result. This is to what TestPassResult refere to for each test result. 
		/// </summary>
		public class TestResultDetailed
		{
			public TestResultDetailed()
			{
				Artifacts = new List<Artifact>();
				Entries = new List<Entry>();
			}

			public string TestDisplayName { get; set; }
			public string FullTestPath { get; set; }
			public TestStateType State { get; set; }
			public string DeviceInstance { get; set; }
			public int Warnings { get; set; }
			public int Errors { get; set; }
			public List<Artifact> Artifacts { get; set; }
			public List<Entry> Entries { get; set; }

			/// <summary>
			/// Add a new Artifact to the test result and return it 
			/// </summary>
			public Artifact AddNewArtifact()
			{
				Artifact NewArtifact = new Artifact();
				Artifacts.Add(NewArtifact);

				return NewArtifact;
			}

			/// <summary>
			/// Add a new Entry to the test result and return it 
			/// </summary>
			public Entry AddNewEntry()
			{
				Entry NewEntry = new Entry();
				Entries.Add(NewEntry);

				return NewEntry;
			}
		}
		/// <summary>
		/// Contains a brief information about test result.
		/// </summary>
		public class TestResult
		{
			public TestResult()
			{
				TestDetailed = new TestResultDetailed();
			}

			public string TestDisplayName
			{
				get { return TestDetailed.TestDisplayName; }
				set { TestDetailed.TestDisplayName = value; }
			}
			public string FullTestPath
			{
				get { return TestDetailed.FullTestPath; }
				set { TestDetailed.FullTestPath = value; }
			}
			public TestStateType State
			{
				get { return TestDetailed.State; }
				set { TestDetailed.State = value; }
			}
			public string DeviceInstance
			{
				get { return TestDetailed.DeviceInstance; }
				set { TestDetailed.DeviceInstance = value; }
			}
			public int Errors
			{
				get { return TestDetailed.Errors; }
				set { TestDetailed.Errors = value; }
			}
			public int Warnings
			{
				get { return TestDetailed.Warnings; }
				set { TestDetailed.Warnings = value; }
			}

			public string ArtifactName { get; set; }


			private TestResultDetailed TestDetailed { get; set; }

			/// <summary>
			/// Return the underlying TestResultDetailed 
			/// </summary>
			public TestResultDetailed GetTestResultDetailed()
			{
				return TestDetailed;
			}
			/// <summary>
			/// Set the underlying TestResultDetailed
			/// </summary>
			public void SetTestResultDetailed(TestResultDetailed InTestResultDetailed)
			{
				TestDetailed = InTestResultDetailed;
			}
		}

		/// <summary>
		/// Contains information about an entire test pass 
		/// </summary>
		public class UnrealEngineTestPassResults : BaseHordeReport
		{
			public override string Type
			{
				get { return "Unreal Automated Tests"; }
			}

			public UnrealEngineTestPassResults() : base()
			{
				Devices = new List<Device>();
				Tests = new List<TestResult>();
			}

			public List<Device> Devices { get; set; }
			public string ReportCreatedOn { get; set; }
			public string ReportURL { get; set; }
			public int SucceededCount { get; set; }
			public int SucceededWithWarningsCount { get; set; }
			public int FailedCount { get; set; }
			public int NotRunCount { get; set; }
			public int InProcessCount { get; set; }
			public float TotalDurationSeconds { get; set; }
			public List<TestResult> Tests { get; set; }

			/// <summary>
			/// Add a new Device to the pass results and return it 
			/// </summary>
			private Device AddNewDevice()
			{
				Device NewDevice = new Device();
				Devices.Add(NewDevice);

				return NewDevice;
			}

			public override bool AttachDeviceLog(string InstanceName, string LogPath, string LogName)
			{
				if (AttachArtifact(LogPath, LogName))
				{
					Device item = Devices.LastOrDefault(D => D.Instance == InstanceName);
					if (item != null)
					{
						item.AppInstanceLog = LogName;
						return true;
					}
				}
				return false;
			}

			/// <summary>
			/// Add a new TestResult to the pass results and return it 
			/// </summary>
			private TestResult AddNewTestResult()
			{
				TestResult NewTestResult = new TestResult();
				Tests.Add(NewTestResult);

				return NewTestResult;
			}

			public override void AddEvent(EventType Type, string Message, params object[] Args)
			{
				throw new System.NotImplementedException("AddEvent not implemented");
			}

			/// <summary>
			/// Convert UnrealAutomatedTestPassResults to Horde data model
			/// </summary>
			/// <param name="InTestPassResults"></param>
			/// <param name="ReportURL"></param>
			public static UnrealEngineTestPassResults FromUnrealAutomatedTests(UnrealAutomatedTestPassResults InTestPassResults, string ReportURL)
			{
				UnrealEngineTestPassResults OutTestPassResults = new UnrealEngineTestPassResults();
				if (InTestPassResults.Devices != null)
				{
					foreach (UnrealAutomationDevice InDevice in InTestPassResults.Devices)
					{
						Device ConvertedDevice = OutTestPassResults.AddNewDevice();
						ConvertedDevice.DeviceName = InDevice.DeviceName;
						ConvertedDevice.Instance = InDevice.Instance;
						ConvertedDevice.Platform = InDevice.Platform;
						ConvertedDevice.OSVersion = InDevice.OSVersion;
						ConvertedDevice.Model = InDevice.Model;
						ConvertedDevice.GPU = InDevice.GPU;
						ConvertedDevice.CPUModel = InDevice.CPUModel;
						ConvertedDevice.RAMInGB = InDevice.RAMInGB;
						ConvertedDevice.RenderMode = InDevice.RenderMode;
						ConvertedDevice.RHI = InDevice.RHI;
					}
				}
				OutTestPassResults.ReportCreatedOn = InTestPassResults.ReportCreatedOn;
				OutTestPassResults.ReportURL = ReportURL;
				OutTestPassResults.SucceededCount = InTestPassResults.Succeeded;
				OutTestPassResults.SucceededWithWarningsCount = InTestPassResults.SucceededWithWarnings;
				OutTestPassResults.FailedCount = InTestPassResults.Failed;
				OutTestPassResults.NotRunCount = InTestPassResults.NotRun;
				OutTestPassResults.InProcessCount = InTestPassResults.InProcess;
				OutTestPassResults.TotalDurationSeconds = InTestPassResults.TotalDuration;
				if (InTestPassResults.Tests != null)
				{
					foreach (UnrealAutomatedTestResult InTestResult in InTestPassResults.Tests)
					{
						TestResult ConvertedTestResult = OutTestPassResults.AddNewTestResult();
						ConvertedTestResult.TestDisplayName = InTestResult.TestDisplayName;
						ConvertedTestResult.FullTestPath = InTestResult.FullTestPath;
						ConvertedTestResult.State = InTestResult.State;
						ConvertedTestResult.DeviceInstance = InTestResult.DeviceInstance.FirstOrDefault();
						Guid TestGuid = Guid.NewGuid();
						ConvertedTestResult.ArtifactName = TestGuid + ".json";
						InTestResult.ArtifactName = ConvertedTestResult.ArtifactName;
						// Copy Test Result Detail
						TestResultDetailed ConvertedTestResultDetailed = ConvertedTestResult.GetTestResultDetailed();
						ConvertedTestResultDetailed.Errors = InTestResult.Errors;
						ConvertedTestResultDetailed.Warnings = InTestResult.Warnings;
						foreach (UnrealAutomationArtifact InTestArtifact in InTestResult.Artifacts)
						{
							Artifact NewArtifact = ConvertedTestResultDetailed.AddNewArtifact();
							NewArtifact.Id = InTestArtifact.Id;
							NewArtifact.Name = InTestArtifact.Name;
							NewArtifact.Type = InTestArtifact.Type;
							ComparisonFiles ArtifactFiles = NewArtifact.Files;
							ArtifactFiles.Difference = InTestArtifact.Files.GetValueOrDefault("difference");
							ArtifactFiles.Approved = InTestArtifact.Files.GetValueOrDefault("approved");
							ArtifactFiles.Unapproved = InTestArtifact.Files.GetValueOrDefault("unapproved");
						}
						foreach (UnrealAutomationEntry InTestEntry in InTestResult.Entries)
						{
							Entry NewEntry = ConvertedTestResultDetailed.AddNewEntry();
							NewEntry.Filename = InTestEntry.Filename;
							NewEntry.LineNumber = InTestEntry.LineNumber;
							NewEntry.Timestamp = InTestEntry.Timestamp;
							Event EntryEvent = NewEntry.Event;
							EntryEvent.Artifact = InTestEntry.Event.Artifact;
							EntryEvent.Context = InTestEntry.Event.Context;
							EntryEvent.Message = InTestEntry.Event.Message;
							EntryEvent.Type = InTestEntry.Event.Type;
						}
					}
				}
				return OutTestPassResults;
			}

			/// <summary>
			/// Copy Test Results Artifacts
			/// </summary>
			/// <param name="ReportPath"></param>
			/// <param name="InOutputArtifactPath"></param>
			public void CopyTestResultsArtifacts(string ReportPath, string InOutputArtifactPath)
			{
				SetOutputArtifactPath(InOutputArtifactPath);
				int ArtifactsCount = 0;
				foreach (TestResult OutputTestResult in Tests)
				{
					TestResultDetailed OutputTestResultDetailed = OutputTestResult.GetTestResultDetailed();
					// copy artifacts
					foreach (Artifact TestArtifact in OutputTestResultDetailed.Artifacts)
					{

						string[] ArtifactPaths = { TestArtifact.Files.Difference, TestArtifact.Files.Approved, TestArtifact.Files.Unapproved };
						foreach (string ArtifactPath in ArtifactPaths)
						{
							if (!string.IsNullOrEmpty(ArtifactPath) && AttachArtifact(Path.Combine(ReportPath, ArtifactPath), ArtifactPath)) { ArtifactsCount++; }
						}
					}
					// write test json blob
					string TestResultFilePath = Path.Combine(OutputArtifactPath, OutputTestResult.ArtifactName);
					try
					{
						File.WriteAllText(TestResultFilePath, JsonSerializer.Serialize(OutputTestResultDetailed, GetDefaultJsonOptions()));
						ArtifactsCount++;
					}
					catch (Exception Ex)
					{
						Log.Error("Failed to save Test Result for '{0}'. {1}", OutputTestResult.TestDisplayName, Ex);
					}
				}
				Log.Verbose("Copied {0} artifacts for Horde to {1}", ArtifactsCount, OutputArtifactPath);
			}
		}

		/// <summary>
		/// Contains information about a test session 
		/// </summary>
		public class AutomatedTestSessionData_legacy : BaseHordeReport
		{
			public override string Type
			{
				get { return "Automated Test Session"; }
			}


			public class TestResult
			{
				public string Name { get; set; }
				public List<string> Tags { get; set; }
				public string TestUID { get; set; }
				public string Suite { get; set; }
				public TestStateType State { get; set; }
				public List<string> DeviceAppInstanceName { get; set; }
				public uint ErrorCount { get; set; }
				public uint WarningCount { get; set; }
				public string ErrorHashAggregate { get; set; }
				public string DateTime { get; set; }
				public float TimeElapseSec { get; set; }
				// not part of json output
				public List<TestEvent> Events;
				public SortedSet<string> ErrorHashes;
				public TestResult(string InName, List<string> InTags, string InSuite, TestStateType InState, List<string> InDevices, string InDateTime)
				{
					Name = InName;
					Tags = InTags;
					TestUID = GenerateHash(Name);
					Suite = InSuite;
					State = InState;
					DeviceAppInstanceName = InDevices;
					ErrorHashes = new SortedSet<string>();
					ErrorHashAggregate = "";
					DateTime = InDateTime;
					Events = new List<TestEvent>();
				}
				private string GenerateHash(string InName)
				{
					return Hasher.ComputeHash(InName, Hasher.DefaultAlgo, 8);
				}
				public TestEvent NewEvent(string InDateTime, EventType InType, string InMessage, string InTag, string InContext = null)
				{
					TestEvent NewItem = new TestEvent(InType, InMessage, InTag, InContext ?? "", InDateTime);
					Events.Add(NewItem);
					switch (InType)
					{
						case EventType.Error:
							ErrorCount++;
							if (ErrorHashes.Count == 0 || !ErrorHashes.Contains(NewItem.Hash))
							{
								// Avoid error duplication and order sensitivity
								ErrorHashes.Add(NewItem.Hash);
								if (ErrorHashes.Count > 1)
								{
									ErrorHashAggregate = GenerateHash(string.Join("", ErrorHashes.ToArray()));
								}
								else
								{
									ErrorHashAggregate = NewItem.Hash;
								}
							}
							break;
						case EventType.Warning:
							WarningCount++;
							break;
					}

					return NewItem;
				}
				public TestEvent GetLastEvent()
				{
					if (Events.Count == 0)
					{
						throw new AutomationException("A test event must be added first to the report!");
					}
					return Events.Last();
				}
			}
			public class TestSession
			{
				public string DateTime { get; set; }
				public float TimeElapseSec { get; set; }
				public Dictionary<string, TestResult> Tests { get; set; }
				public string TestResultsTestDataUID { get; set; }
				public TestSession()
				{
					Tests = new Dictionary<string, TestResult>();
					TestResultsTestDataUID = Guid.NewGuid().ToString();
				}
				public TestResult NewTestResult(string InName, List<string> InTags, string InSuite, TestStateType InState, List<string> InDevices, string InDateTime)
				{
					TestResult NewItem = new TestResult(InName, InTags, InSuite, InState, InDevices, InDateTime);
					Tests[NewItem.TestUID] = NewItem;
					return NewItem;
				}
			}
			public class Device
			{
				public string Name { get; set; }
				public string AppInstanceName { get; set; }
				public string AppInstanceLog { get; set; }
				public Dictionary<string, string> Metadata { get; set; }
				public Device(string InName, string InInstance, string InInstanceLog)
				{
					Name = InName;
					AppInstanceName = InInstance;
					AppInstanceLog = InInstanceLog;
					Metadata = new Dictionary<string, string>();
				}
				public void SetMetaData(string Key, string Value)
				{
					Metadata.Add(Key, Value);
				}
			}
			public class TestArtifact
			{
				public string Tag { get; set; }
				public string ReferencePath { get; set; }
			}
			public class TestEvent
			{
				public string Message { get; set; }
				public string Context { get; set; }
				public EventType Type { get; set; }
				public string Tag { get; set; }
				public string Hash { get; set; }
				public string DateTime { get; set; }
				public List<TestArtifact> Artifacts { get; set; }
				public TestEvent(EventType InType, string InMessage, string InTag, string InContext, string InDateTime)
				{
					Type = InType;
					Message = InMessage;
					Context = InContext;
					Tag = InTag;
					Hash = InType != EventType.Info ? GenerateEventHash() : "";
					DateTime = InDateTime;
					Artifacts = new List<TestArtifact>();
				}
				public TestArtifact NewArtifacts(string InTag, string InReferencePath)
				{
					TestArtifact NewItem = new TestArtifact();
					NewItem.Tag = InTag;
					NewItem.ReferencePath = InReferencePath;
					Artifacts.Add(NewItem);
					return NewItem;
				}
				private string FilterEvent(string Text)
				{
					// Filter out time stamp in message event
					// [2021.05.20-09.55.28:474][  3]
					Regex Regex_timestamp = new Regex(@"\[[0-9]+\.[0-9]+\.[0-9]+-[0-9]+\.[0-9]+\.[0-9]+(:[0-9]+)?\](\[ *[0-9]+\])?");
					Text = Regex_timestamp.Replace(Text, "");

					// Filter out worker number in message event
					// Worker #13
					Regex Regex_workernumber = new Regex(@"Worker #[0-9]+");
					Text = Regex_workernumber.Replace(Text, "");

					// Filter out path
					// D:\build\U5+R5.0+Inc\Sync\Engine\Source\Runtime\...
					Regex Regex_pathstring = new Regex(@"([A-Z]:)?([\\/][^\\/]+)+[\\/]");
					Text = Regex_pathstring.Replace(Text, "");

					// Filter out hexadecimal string in message event
					// 0x00007ffa7b04e533
					// Mesh 305f21682cf3a231a0947ffb35c51567
					// <lambda_2a907a23b64c2d53ad869d04fdc6d423>
					Regex Regex_hexstring = new Regex(@"(0x)?[0-9a-fA-F]{6,32}");
					Text = Regex_hexstring.Replace(Text, "");

					return Text;
				}
				private string GenerateEventHash()
				{
					string FilteredEvent = FilterEvent(string.Format("{0}{1}{2}{3}", Type.ToString(), Message, Context, Tag));
					return Hasher.ComputeHash(FilteredEvent, Hasher.DefaultAlgo, 8);
				}
			}
			public class TestResultData
			{
				public List<TestEvent> Events
				{
					get { return Result.Events; }
					set { Result.Events = value; }
				}
				private TestResult Result;
				public TestResultData(TestResult InTestResult)
				{
					Result = InTestResult;
				}
			}
			public class IndexedError
			{
				public string Message { get; set; }
				public string Tag { get; set; }
				public List<string> TestUIDs { get; set; }
				public IndexedError(string InMessage, string InTag)
				{
					Message = InMessage;
					Tag = InTag;
					TestUIDs = new List<string>();
				}
			}

			public AutomatedTestSessionData_legacy(string InName) : base()
			{
				Name = InName;
				PreFlightChange = "";
				TestSessionInfo = new TestSession();
				Devices = new List<Device>();
				IndexedErrors = new Dictionary<string, IndexedError>();
				TestResults = new Dictionary<string, TestResultData>();
			}

			public string Name { get; set; }
			public string PreFlightChange { get; set; }
			public TestSession TestSessionInfo { get; set; }
			public List<Device> Devices { get; set; }
			/// <summary>
			/// Database side optimization: Index the TestUID by TestError hash.
			/// Key is meant to be the TestError hash and Value the list of related TestUID.
			/// </summary>
			public Dictionary<string, IndexedError> IndexedErrors { get; set; }
			/// <summary>
			/// Allow the Database to pull only one test result from the TestData.
			/// Key is meant to be the TestUID and Value the test result detailed events.
			/// </summary>
			private Dictionary<string, TestResultData> TestResults { get; set; }

			private string CurrentTestUID;

			private Device NewDevice(string InName, string InInstance, string InInstanceLog = null)
			{
				Device NewItem = new Device(InName, InInstance, InInstanceLog);
				Devices.Add(NewItem);
				return NewItem;
			}

			private void IndexTestError(string ErrorHash, string TestUID, string Message, string Tag)
			{
				if (!IndexedErrors.ContainsKey(ErrorHash))
				{
					IndexedErrors[ErrorHash] = new IndexedError(Message, Tag);
				}
				else if (IndexedErrors[ErrorHash].TestUIDs.Contains(TestUID))
				{
					// already stored
					return;
				}
				IndexedErrors[ErrorHash].TestUIDs.Add(TestUID);
			}

			private TestResult GetCurrentTest()
			{
				if (string.IsNullOrEmpty(CurrentTestUID))
				{
					throw new AutomationException("A test must be set to the report!");
				}
				return TestSessionInfo.Tests[CurrentTestUID];
			}

			/// <summary>
			/// Add a Test entry and set it as current test
			/// </summary>
			/// <param name="InName"></param>
			/// <param name="InSuite"></param>
			/// <param name="InDevices"></param>
			/// <param name="InDateTime"></param>
			public void SetTest(string InName, List<string> InTags, string InSuite, List<string> InDevices, string InDateTime = null)
			{
				TestResult Test = TestSessionInfo.NewTestResult(InName, InTags, InSuite, TestStateType.Unknown, InDevices, InDateTime);
				CurrentTestUID = Test.TestUID;
				// Index the test result data.
				TestResults[Test.TestUID] = new TestResultData(Test);
			}
			/// <summary>
			/// Set current test state
			/// </summary>
			/// <param name="InState"></param>
			public void SetTestState(TestStateType InState)
			{
				TestResult CurrentTest = GetCurrentTest();
				CurrentTest.State = InState;
			}
			/// <summary>
			/// Set current test time elapse in second
			/// </summary>
			/// <param name="InTimeElapseSec"></param>
			public void SetTestTimeElapseSec(float InTimeElapseSec)
			{
				TestResult CurrentTest = GetCurrentTest();
				CurrentTest.TimeElapseSec = InTimeElapseSec;
			}
			/// <summary>
			/// Add event to current test
			/// </summary>
			/// <param name="Type"></param>
			/// <param name="Message"></param>
			/// <param name="Args"></param>
			public override void AddEvent(EventType Type, string Message, params object[] Args)
			{
				Dictionary<string, object> Properties = new Dictionary<string, object>();
				MessageTemplate.ParsePropertyValues(Message, Args, Properties);
				Message = MessageTemplate.Render(Message, Properties);
				AddEvent(DateTime.UtcNow.ToString(UnrealAutomationEntry.DateTimeFormat), Type, Message, "gauntlet", null);
			}
			/// <summary>
			/// Overload of AddEvent, add even to current test with date time
			/// </summary>
			/// <param name="InDateTime"></param>
			/// <param name="InType"></param>
			/// <param name="InMessage"></param>
			/// <param name="InTag"></param>
			/// <param name="InContext"></param>
			public void AddEvent(string InDateTime, EventType InType, string InMessage, string InTag, string InContext)
			{
				TestResult CurrentTest = GetCurrentTest();
				TestEvent Event = CurrentTest.NewEvent(InDateTime, InType, InMessage, InTag, InContext);
				if (InType == EventType.Error)
				{
					IndexTestError(Event.Hash, CurrentTest.TestUID, InMessage, InTag);
				}
			}
			/// <summary>
			/// Add artifact to last test event
			/// </summary>
			/// <param name="InTag"></param>
			/// <param name="InFilePath"></param>
			/// <param name="InReferencePath"></param>
			/// <returns></returns>
			public bool AddArtifactToLastEvent(string InTag, string InFilePath, string InReferencePath = null)
			{
				TestEvent LastEvent = GetCurrentTest().GetLastEvent();
				if (AttachArtifact(InFilePath, InReferencePath))
				{
					if (InReferencePath == null)
					{
						InReferencePath = Path.GetFileName(InFilePath);
					}
					LastEvent.NewArtifacts(InTag, InReferencePath);

					return true;
				}

				return false;
			}

			public override bool AttachDeviceLog(string InstanceName, string LogPath, string LogName)
			{
				if (AttachArtifact(LogPath, LogName))
				{
					Device item = Devices.LastOrDefault(D => D.AppInstanceName == InstanceName);
					if (item != null)
					{
						item.AppInstanceLog = LogName;
						return true;
					}
				}
				return false;
			}

			public override string GetTestDataKey(string BaseKey = null)
			{
				return base.GetTestDataKey(); // Ignore BaseKey
			}
			public override Dictionary<string, object> GetReportDependencies()
			{

				var Reports = base.GetReportDependencies();
				// Test Result Details
				var Key = string.Format("{0} Result Details::{1}", Type, TestSessionInfo.TestResultsTestDataUID);
				Reports[Key] = TestResults;

				return Reports;
			}

			/// <summary>
			/// Convert UnrealAutomatedTestPassResults to Horde data model
			/// </summary>
			/// <param name="InTestPassResults"></param>
			/// <param name="InName"></param>
			/// <param name="InSuite"></param>
			/// <param name="InReportPath"></param>
			/// <param name="InHordeArtifactPath"></param>
			/// <returns></returns>
			public static AutomatedTestSessionData_legacy FromUnrealAutomatedTests(UnrealAutomatedTestPassResults InTestPassResults, string InName, string InSuite, string InReportPath, string InHordeArtifactPath)
			{
				AutomatedTestSessionData_legacy OutTestPassResults = new AutomatedTestSessionData_legacy(InName);
				OutTestPassResults.SetOutputArtifactPath(InHordeArtifactPath);
				if (InTestPassResults.Devices != null)
				{
					foreach (UnrealAutomationDevice InDevice in InTestPassResults.Devices)
					{
						Device ConvertedDevice = OutTestPassResults.NewDevice(InDevice.DeviceName, InDevice.Instance, InDevice.AppInstanceLog);
						ConvertedDevice.SetMetaData("platform", InDevice.Platform);
						ConvertedDevice.SetMetaData("os_version", InDevice.OSVersion);
						ConvertedDevice.SetMetaData("model", InDevice.Model);
						ConvertedDevice.SetMetaData("gpu", InDevice.GPU);
						ConvertedDevice.SetMetaData("cpumodel", InDevice.CPUModel);
						ConvertedDevice.SetMetaData("ram_in_gb", InDevice.RAMInGB.ToString());
						ConvertedDevice.SetMetaData("render_mode", InDevice.RenderMode);
						ConvertedDevice.SetMetaData("rhi", InDevice.RHI);
					}
				}
				OutTestPassResults.TestSessionInfo.DateTime = InTestPassResults.ReportCreatedOn;
				OutTestPassResults.TestSessionInfo.TimeElapseSec = InTestPassResults.TotalDuration;
				if (InTestPassResults.Tests != null)
				{
					foreach (UnrealAutomatedTestResult InTestResult in InTestPassResults.Tests)
					{
						string TestDateTime = InTestResult.DateTime;
						if (TestDateTime == "0001.01.01-00.00.00")
						{
							// Special case: when UE Test DateTime is set this way, it means the value was 0 or null before getting converted to json.
							TestDateTime = null;
						}
						OutTestPassResults.SetTest(InTestResult.FullTestPath, InTestResult.Tags, InSuite, InTestResult.DeviceInstance, InTestResult.DateTime);
						foreach (UnrealAutomationEntry InTestEntry in InTestResult.Entries)
						{
							string Tag = "entry";
							string Context = InTestEntry.Event.Context;
							string Artifact = InTestEntry.Event.Artifact;
							UnrealAutomationArtifact IntArtifactEntry = null;
							// If Artifact values is not null nor a bunch on 0, then we have a file attachment.
							if (!string.IsNullOrEmpty(Artifact) && Artifact.Substring(0, 4) != "0000")
							{
								IntArtifactEntry = InTestResult.Artifacts.Find(A => A.Id == Artifact);
								if (IntArtifactEntry != null && IntArtifactEntry.Type == "Comparison")
								{
									// UE for now only produces one type of artifact that can be attached to an event and that's image comparison
									Tag = "image comparison";
									Context = IntArtifactEntry.Name;
								}
							}
							if (string.IsNullOrEmpty(Context) && !string.IsNullOrEmpty(InTestEntry.Filename))
							{
								Context = string.Format("{0} @line:{1}", InTestEntry.Filename, InTestEntry.LineNumber.ToString());
							}
							// Tag critical failure as crash
							if (InTestEntry.Event.Type == EventType.Error && Tag == "entry" && InTestEntry.Event.IsCriticalFailure)
							{
								Tag = "crash";
							}

							OutTestPassResults.AddEvent(InTestEntry.Timestamp, InTestEntry.Event.Type, InTestEntry.Event.Message, Tag, Context);
							bool IsInfo = InTestEntry.Event.Type == EventType.Info;
							// Add Artifacts
							if (IntArtifactEntry != null)
							{
								List<string> FailedToAttached = new List<string>();
								string ArtifactFilePath = IntArtifactEntry.Files.GetValueOrDefault("difference");
								if (!IsInfo && !string.IsNullOrEmpty(ArtifactFilePath))
								{
									if (!OutTestPassResults.AddArtifactToLastEvent(
										"difference",
										Path.Combine(InReportPath, ArtifactFilePath),
										ArtifactFilePath
									))
									{
										FailedToAttached.Add(ArtifactFilePath);
									}
								}
								ArtifactFilePath = IntArtifactEntry.Files.GetValueOrDefault("approved");
								if (!IsInfo && !string.IsNullOrEmpty(ArtifactFilePath))
								{
									if (!OutTestPassResults.AddArtifactToLastEvent(
										"approved",
										Path.Combine(InReportPath, ArtifactFilePath),
										ArtifactFilePath
									))
									{
										FailedToAttached.Add(ArtifactFilePath);
									}
								}
								ArtifactFilePath = IntArtifactEntry.Files.GetValueOrDefault("unapproved");
								if (!string.IsNullOrEmpty(ArtifactFilePath))
								{
									string AbsoluteLocation = Path.Combine(InReportPath, ArtifactFilePath);
									if (!OutTestPassResults.AddArtifactToLastEvent(
										"unapproved",
										AbsoluteLocation,
										ArtifactFilePath
									))
									{
										FailedToAttached.Add(ArtifactFilePath);
									}
									// Add Json meta data if any
									string MetadataLocation = Utils.SystemHelpers.GetFullyQualifiedPath(Path.GetDirectoryName(AbsoluteLocation));
									if (!IsInfo && Directory.Exists(MetadataLocation))
									{
										string[] JsonMetadataFiles = System.IO.Directory.GetFiles(MetadataLocation, "*.json");
										if (JsonMetadataFiles.Length > 0)
										{
											int LastSlash = ArtifactFilePath.LastIndexOf("/");
											string RelativeLocation = ArtifactFilePath.Substring(0, LastSlash);
											OutTestPassResults.AddEvent(
												InTestEntry.Timestamp,
												EventType.Info,
												"The image reference can be updated by pointing the Screen Comparison tab from the Test Automation window to the artifacts from this test.",
												"image comparison metadata", null
											);
											foreach (string JsonFile in JsonMetadataFiles)
											{
												string JsonArtifactName = RelativeLocation + "/" + Path.GetFileName(JsonFile);
												if (!OutTestPassResults.AddArtifactToLastEvent("json metadata", JsonFile, JsonArtifactName))
												{
													FailedToAttached.Add(JsonArtifactName);
												}
											}
										}
									}
								}
								if (FailedToAttached.Count() > 0)
								{
									foreach (var Item in FailedToAttached)
									{
										OutTestPassResults.AddEvent(EventType.Warning, string.Format("Failed to attached Artifact {0}.", Item));
									}
								}
							}
						}
						OutTestPassResults.SetTestState(InTestResult.State);
						OutTestPassResults.SetTestTimeElapseSec(InTestResult.Duration);
					}
				}
				return OutTestPassResults;
			}
		}

		/// <summary>
		/// Contains Test success/failed status and a list of errors and warnings
		/// </summary>
		public class SimpleTestReport : BaseHordeReport
		{
			public class TestRole
			{
				public string Type { get; set; }
				public string Platform { get; set; }
				public string Configuration { get; set; }

				public TestRole(UnrealTestRole Role, UnrealTestRoleContext Context)
				{
					Type = Role.Type.ToString();
					Platform = Context.Platform.ToString();
					Configuration = Context.Configuration.ToString();
				}
			}

			public override string Type
			{
				get { return "Simple Report"; }
			}

			public SimpleTestReport() : base()
			{

			}

			public SimpleTestReport(Gauntlet.TestResult TestResult, UnrealTestContext Context, UnrealTestConfiguration Configuration) : base()
			{

				BuildChangeList = Context.BuildInfo.Changelist;
				this.TestResult = TestResult.ToString();

				// populate roles
				UnrealTestRole MainRole = Configuration.GetMainRequiredRole();
				this.MainRole = new TestRole(MainRole, Context.GetRoleContext(MainRole.Type));

				IEnumerable<UnrealTestRole> Roles = Configuration.RequiredRoles.Values.SelectMany(V => V);
				foreach (UnrealTestRole Role in Roles)
				{
					this.Roles.Add(new TestRole(Role, Context.GetRoleContext(Role.Type)));
				}
			}

			public string TestName { get; set; }
			public string Description { get; set; }
			public string ReportCreatedOn { get; set; }
			public float TotalDurationSeconds { get; set; }
			public bool HasSucceeded { get; set; }
			public string Status { get; set; }
			public string URLLink { get; set; }
			public int BuildChangeList { get; set; }
			public TestRole MainRole { get; set; }
			public List<TestRole> Roles { get; set; } = new List<TestRole>();
			public string TestResult { get; set; }
			public List<String> Logs { get; set; } = new List<String>();
			public List<String> Errors { get; set; } = new List<String>();
			public List<String> Warnings { get; set; } = new List<String>();

			public override void AddEvent(EventType Type, string Message, params object[] Args)
			{
				Dictionary<string, object> Properties = new Dictionary<string, object>();
				MessageTemplate.ParsePropertyValues(Message, Args, Properties);
				Message = MessageTemplate.Render(Message, Properties);

				switch (Type)
				{
					case EventType.Error:
						Errors.Add(Message);
						break;
					case EventType.Warning:
						Warnings.Add(Message);
						break;
				}
				// The rest is ignored with this report type.
			}

			public override bool AttachArtifact(string ArtifactPath, string Name = null)
			{
				if (base.AttachArtifact(ArtifactPath, Name))
				{
					Logs.Add(string.Format("Attached artifact: {0}", Name ?? Path.GetFileName(ArtifactPath)));
					return true;
				}
				return false;
			}

			public override bool AttachDeviceLog(string InstanceName, string LogPath, string LogName)
			{
				if (AttachArtifact(LogPath, LogName))
				{
					Logs.Add(string.Format("Attached Log for device {0}: {1}", InstanceName, LogName));
				}
				return true;
			}
		}
		#endregion

#nullable enable

		/// <summary>
		/// Contains information about a test session (v2)
		/// </summary>
		public class AutomatedTestSessionData : BaseHordeReport
		{
			public override string Type => "Automated Test Session";

			public const int DEFAULT_VERSION = 2;
			public override int Version => DEFAULT_VERSION;

			/// <summary>
			/// Unique key that identify the test and group the different sessions
			/// </summary>
			public string Key { get; set; }

			public override string GetTestDataKey(string? BaseKey = null)
			{
				return Key; // Ignore BaseKey
			}

			/// <summary>
			/// Stored data of the session
			/// </summary>
			protected TestSession Data { get; set; }

			public override IDataBlob GetTestData()
			{
				return Data;
			}

			/// <summary>
			/// Utility function to generate a valid key from any name
			/// </summary>
			/// <param name="InName"></param>
			/// <returns></returns>
			private static string GenerateKey(string InName)
			{
				return Hasher.ComputeHash(InName, Hasher.DefaultAlgo, 8);
			}

			/// <summary>
			/// Cursor to the current phase
			/// </summary>
			private TestPhase? CurrentPhase { get; set; }

			/// <summary>
			/// Static constructor
			/// </summary>
			static AutomatedTestSessionData()
			{
				// Make sure to register the JSON formatter for TestLogValue type
				LogValueFormatter.RegisterFormatter(typeof(TestLogValue), new TestLogValueFormatter());
			}

			public AutomatedTestSessionData(string InKey, string? InName = null) : base()
			{
				Key = InKey;
				Data = new TestSession(string.IsNullOrEmpty(InName) ? InKey : InName, this);
			}

			public AutomatedTestSessionData(ITestAuditMetadata InAudit) : this(InAudit.TestKey, InAudit.TestName)
			{
				Data.summary.commandline = InAudit.TestCommandLine;

				AddTestRecipe(InAudit.TestNodeClass, InAudit.TestVersion.ToString(), InAudit.TestTimeoutMinutes, InAudit.TestParams);

				if (InAudit.TestIntent != null
					|| InAudit.TestHarness != null
					|| InAudit.TestProfile != null
					|| InAudit.TestTeam != null)
				{
					AddTestDescription(InAudit.TestTeam?.ToString(), InAudit.TestIntent, InAudit.TestHarness?.ToString(), InAudit.TestProfile?.ToString());
				}

				string[]? InTags = InAudit.TestTags?.Select(Tag => Tag.ToString()).ToArray();
				if (InTags != null)
				{
					AddTestTags(InTags);
				}
			}

			/// <summary>
			/// Represent a Device used during a test
			/// </summary>
			public class TestDevice
			{
				public string name { get; set; }
				[JsonIgnore]
				public string key { get; private set; }
				public string? appInstanceLogPath { get; set; }
				public Dictionary<string, string> metadata { get; set; }

				public TestDevice(string InName, string InKey)
				{
					name = InName;
					key = InKey;
					metadata = new Dictionary<string, string>();
				}
			}

			/// <summary>
			/// Summary for the test session
			/// </summary>
			public class TestSessionSummary
			{
				public string testName { get; set; }
				public string? dateTime { get; set; }
				public float timeElapseSec { get; set; }
				public string? commandline {  get; set; }
				public int phasesTotalCount { get; set; }
				public int phasesSucceededCount { get; set; }
				public int phasesUndefinedCount { get; set; }
				public int phasesFailedCount { get; set; }

				public TestSessionSummary(string InName)
				{
					testName = InName;
				}
			}

			/// <summary>
			/// Description of the test
			/// </summary>
			public class TestDescription
			{
				public string? team { get; set; }
				public string? profile { get; set; }
				public string? harness { get; set; }
				public string? intent { get; set; }

				public TestDescription(string? InTeam = null, string? InProfile = null, string? InHarness = null, string? InIntent = null)
				{
					team = InTeam;
					profile = InProfile;
					harness = InHarness;
					intent = InIntent;
				}
			}

			/// <summary>
			/// Recipe to run the test
			/// </summary>
			public class TestRecipe
			{
				public string nodeClass { get; set; }
				public string version { get; set; }
				public float timeoutMinutes { get; set; }
				[JsonPropertyName("params")]
				public Dictionary<string, object>? @params { get; set; }
				public TestRecipe(string InNodeClass, string InVersion, float InTimeoutMinutes = 0)
				{
					nodeClass = InNodeClass;
					version = InVersion;
					timeoutMinutes = InTimeoutMinutes;
				}
			}

			[JsonConverter(typeof(JsonTryParseEnumConverter))]
			public enum TestPhaseOutcome
			{
				Unknown,
				NotRun,
				Interrupted,
				Failed,
				Success,
				Skipped
			}

			/// <summary>
			/// Represent a phase of a test
			/// </summary>
			public class TestPhase
			{
				public string key { get; set; }
				public string name { get; set; }
				public string? dateTime { get; set; }
				public float timeElapseSec { get; set; }
				public TestPhaseOutcome outcome { get; set; }
				public string? errorFingerprint { get; set; }
				public bool? hasWarning { get; set; }
				public HashSet<string>? tags { get; set; }
				public HashSet<string> deviceKeys { get; set; }
				public string? eventStreamPath { get; set; }

				private TestSession? session { get; set; }

				private TestEventStream eventStream { get; set; }

				public TestPhase(string InName, string? InKey = null, TestSession? InSession = null)
				{
					key = string.IsNullOrEmpty(InKey) ? GenerateKey(InName) : ValidateKey(InKey);
					name = InName;
					deviceKeys = new HashSet<string>();
					outcome = TestPhaseOutcome.Unknown;
					eventStream = new TestEventStream(session?.report, this);
					if (InSession != null)
					{
						AttachToSession(InSession);
					}
				}

				/// <summary>
				/// Set timing information about the phase
				/// </summary>
				/// <param name="StartTime"></param>
				/// <param name="TimeElapse"></param>
				public void SetTiming(DateTime StartTime, float TimeElapse)
				{
					dateTime = StartTime.ToString("s", CultureInfo.InvariantCulture);
					timeElapseSec = TimeElapse;
				}

				/// <summary>
				/// Set the outcome of the phase, it will update the test session phase counts
				/// </summary>
				/// <param name="OutCome"></param>
				public void SetOutcome(TestPhaseOutcome OutCome)
				{
					UpdatePhasesCount(PhaseCount.Out);
					outcome = OutCome;
					UpdatePhasesCount();
				}

				/// <summary>
				/// Set the outcome of the phase, it will update the test session phase counts
				/// </summary>
				/// <param name="State"></param>
				public void SetOutcome(TestStateType State)
				{
					UpdatePhasesCount(PhaseCount.Out);
					outcome = TestStateToPhaseOutcome(State);
					UpdatePhasesCount();
				}

				/// <summary>
				/// Set the outcome of the phase using Gauntlet TestResult type, it will update the test session phase counts
				/// </summary>
				/// <param name="State"></param>
				public void SetOutcome(Gauntlet.TestResult State)
				{
					UpdatePhasesCount(PhaseCount.Out);
					outcome = TestResultToPhaseOutcome(State);
					UpdatePhasesCount();
				}

				/// <summary>
				/// Attach the phase to a session, making changes to the phase outcome tracked in that session
				/// </summary>
				/// <param name="InSession"></param>
				public void AttachToSession(TestSession InSession)
				{
					session = InSession;
					if (session.report != null)
					{
						eventStream.AttachToReport(session.report);
					}
					session.phases.Add(this);
					session.summary.phasesTotalCount += 1;
					UpdatePhasesCount();
				}

				private enum PhaseCount
				{
					In,
					Out
				}
				/// <summary>
				/// Update the phases count in the session based on the phase outcome
				/// </summary>
				/// <param name="Direction"></param>
				private void UpdatePhasesCount(PhaseCount Direction = PhaseCount.In)
				{
					if (session == null)
					{
						return;
					}
					int Incr = Direction == PhaseCount.In ? 1 : -1;
					switch (outcome)
					{
						case TestPhaseOutcome.Failed:
							session.summary.phasesFailedCount += Incr;
							break;

						case TestPhaseOutcome.Success:
							session.summary.phasesSucceededCount += Incr;
							break;

						case TestPhaseOutcome.Skipped:
							// Remove from total count
							session.summary.phasesTotalCount -= Incr;
							break;

						default:
							session.summary.phasesUndefinedCount += Incr;
							break;
					}
				}

				/// <summary>
				/// Compile the aggregate errors
				/// </summary>
				public void CompileErrorFingerprint()
				{
					if (!string.IsNullOrEmpty(errorFingerprint) && errorFingerprint.Length > 8)
					{
						errorFingerprint = Hasher.ComputeHash(errorFingerprint, Hasher.DefaultAlgo, 8);
					}
				}

				/// <summary>
				/// Add tags to the phase
				/// </summary>
				/// <param name="NewTags"></param>
				public void AddTags(IEnumerable<string> NewTags)
				{
					if (tags == null)
					{
						tags = new(new TagComparer());
					}
					tags.UnionWith(NewTags);
				}

				/// <summary>
				/// Add tags to the phase
				/// </summary>
				/// <param name="NewTags"></param>
				public void AddTags(IEnumerable<TestTag> NewTags)
				{
					AddTags(NewTags.Select(Tag => Tag.ToString()));
				}

				/// <summary>
				/// Get the event stream associated with this phase
				/// </summary>
				/// <returns></returns>
				/// <exception cref="AutomationException"></exception>
				public TestEventStream GetStream()
				{
					return eventStream;
				}
			}

			internal class TagComparer : IEqualityComparer<string>
			{
				public bool Equals(string? TagA, string? TagB)
				{
					if (TagA == null && TagB == null)
					{
						return true;
					}
					else if (TagA == null || TagB == null)
					{
						return false;
					}
					return TagA.Equals(TagB, StringComparison.CurrentCultureIgnoreCase);
				}

				public int GetHashCode([DisallowNull] string Tag)
				{
					return Tag.ToLower().GetHashCode();
				}
			}

			/// <summary>
			/// fork of the test session data (summary, phases, devices, eventStreams)
			/// </summary>
			public class TestSession : IDataBlob
			{
				public TestSessionSummary summary { get; set; }
				public List<TestPhase> phases { get; set; }
				public Dictionary<string, TestDevice> devices { get; set; }
				public int version { get { return report?.Version ?? DEFAULT_VERSION /* default to test data v2 report version */; } }
				public Dictionary<string, string> metadata { get { return report?.Metadata ?? new(); } }
				public HashSet<string>? tags { get; set; }
				public TestDescription? description { get; set; }
				public TestRecipe? recipe { get; set; }

				[JsonIgnore]
				// back pointer
				public AutomatedTestSessionData? report { get; protected set; }

				public TestSession(string InName, AutomatedTestSessionData? InReport = null)
				{
					summary = new TestSessionSummary(InName);
					phases = new List<TestPhase>();
					devices = new Dictionary<string, TestDevice>();
					report = InReport;
				}
			}

			/// <summary>
			/// Set the session timing information
			/// </summary>
			/// <param name="StartTime"></param>
			/// <param name="TimeElapse"></param>
			public virtual void SetSessionTiming(DateTime StartTime, float TimeElapse)
			{
				Data.summary.dateTime = StartTime.ToString("s", CultureInfo.InvariantCulture);
				Data.summary.timeElapseSec = TimeElapse;
			}

			/// <summary>
			/// Add a Device to the session (or replace it) and return the new instance.
			/// </summary>
			/// <param name="InName"></param>
			/// <param name="InKey"></param>
			/// <returns></returns>
			public virtual TestDevice AddDevice(string InName, string? InKey = null)
			{
				string Key = ValidateKey(InKey ?? InName);
				if (Data.devices.ContainsKey(Key))
				{
					// remove to replace
					Data.devices.Remove(Key);
					Log.Verbose("Device (Key) '{Name}' already exists in the report. It is being replaced.", InName, Key);
				}
				TestDevice Device = new TestDevice(InName, Key);
				Data.devices.Add(Key, Device);
				return Device;
			}

			/// <summary>
			/// Return the device associated with the Key
			/// </summary>
			/// <param name="Key"></param>
			/// <returns></returns>
			public virtual TestDevice? GetDevice(string Key)
			{
				return Data.devices.GetValueOrDefault(Key);
			}

			/// <summary>
			/// Return all the devices associated with the report
			/// </summary>
			/// <returns></returns>
			public virtual List<TestDevice> GetDevices()
			{
				return Data.devices.Values.ToList();
			}

			public override bool AttachDeviceLog(string InstanceName, string LogPath, string LogName)
			{
				if (AttachArtifact(LogPath, LogName))
				{
					var Device = Data.devices.GetValueOrDefault(InstanceName);
					if (Device != null)
					{
						string LogPathForHorde = Path.GetRelativePath(Globals.UnrealRootDir, Path.Combine(OutputArtifactPath, LogName));
						Device.appInstanceLogPath = FileUtils.ConvertPathToUnix(LogPathForHorde);
						return true;
					}
				}

				return false;
			}

			/// <summary>
			/// Add a phase to the test session
			/// </summary>
			/// <param name="InName"></param>
			/// <param name="InKey"></param>
			/// <returns></returns>
			public virtual TestPhase AddPhase(string InName, string? InKey = null)
			{
				TestPhase Phase = new TestPhase(InName, InKey, Data);
				CurrentPhase = Phase;
				return Phase;
			}

			/// <summary>
			/// Get the phase associated with Key
			/// </summary>
			/// <param name="Key"></param>
			/// <returns></returns>
			public virtual TestPhase? GetPhase(string Key)
			{
				return Data.phases.FirstOrDefault(P => P.key == Key);
			}

			/// <summary>
			/// Set the current session phase with the phase associated with Key 
			/// </summary>
			/// <param name="Key"></param>
			/// <exception cref="AutomationException"></exception>
			public virtual void SetCurrentPhase(string Key)
			{
				TestPhase? Phase = GetPhase(Key);
				if (Phase == null)
				{
					throw new AutomationException("Could not find Phase '{0}'", Key);
				}
				CurrentPhase = Phase;
			}

			/// <summary>
			/// Set the current session phase with the input phase (and attach that phase the session with not already)
			/// </summary>
			/// <param name="InPhase"></param>
			public virtual void SetCurrentPhase(TestPhase InPhase)
			{
				InPhase.AttachToSession(Data);
				CurrentPhase = InPhase;
			}

			/// <summary>
			/// Attach the input phase to the test session
			/// </summary>
			/// <param name="InPhase"></param>
			public virtual void AttachPhase(TestPhase InPhase)
			{
				InPhase.AttachToSession(Data);
			}

			/// <summary>
			/// Get the current session phase 
			/// </summary>
			/// <returns></returns>
			public virtual TestPhase? GetCurrentPhase()
			{
				return CurrentPhase;
			}

			/// <summary>
			/// Add tags to the test
			/// </summary>
			/// <param name="NewTags"></param>
			public virtual void AddTestTags(IEnumerable<string> NewTags)
			{
				if (Data.tags == null)
				{
					Data.tags = new(new TagComparer());
				}
				Data.tags.UnionWith(NewTags);
			}

			/// <summary>
			/// Add test description
			/// </summary>
			/// <param name="Team"></param>
			/// <param name="Intent"></param>
			/// <param name="Harness"></param>
			/// <param name="Profile"></param>
			public virtual void AddTestDescription(string? Team, string? Intent, string? Harness, string? Profile)
			{
				Data.description = new TestDescription(Team, Profile, Harness, Intent);
			}

			/// <summary>
			/// Add recipe to run the same test
			/// </summary>
			/// <param name="NodeClass"></param>
			/// <param name="Version"></param>
			/// <param name="TimeoutMinutes"></param>
			/// <param name="Params"></param>
			public virtual void AddTestRecipe(string NodeClass, string Version, float TimeoutMinutes = 0, Dictionary<string, object>? Params = null)
			{
				Data.recipe = new TestRecipe(NodeClass, Version, TimeoutMinutes);
				Data.recipe.@params = Params;
			}

			/// <summary>
			/// Represent the event stream of a phase
			/// </summary>
			public class TestEventStream
			{
				// Redact numbers and timestamp like things
				static Regex Regex_numbertimestampstring = new Regex(@"\d[\d.,:]*", RegexOptions.Compiled);
				// Redact path
				static Regex Regex_pathstring = new Regex(@"(?:(?<![A-Z])|HTTPS*:)(?:[A-Z]:|[/\\])[\w/\\.:+?=%-]+", RegexOptions.Compiled);
				// Redact hex strings
				static Regex Regex_hexstring = new Regex(@"(?:0X|#)[0-9A-F]+", RegexOptions.Compiled);

				private AutomatedTestSessionData? report { get; set; }
				private TestPhase phase { get; set; }
				private List<LogEvent> stream { get; set; }

				public TestEventStream(AutomatedTestSessionData? InReport, TestPhase InPhase)
				{
					report = InReport;
					phase = InPhase;
					stream = new List<LogEvent>();
				}

				/// <summary>
				/// Return true if there is any LogEvent in this stream
				/// </summary>
				/// <returns></returns>
				public bool HasAnyEvent()
				{
					return stream.Any();
				}

				/// <summary>
				/// Get the list of LogEvents
				/// </summary>
				/// <returns></returns>
				public IEnumerable<LogEvent> GetEvents()
				{
					return stream;
				}

				/// <summary>
				/// Attach the event stream to a report
				/// </summary>
				/// <param name="InReport"></param>
				public virtual void AttachToReport(AutomatedTestSessionData InReport)
				{
					report = InReport;
				}

				/// <summary>
				/// Attached the input LogEvent to the stream
				/// </summary>
				/// <param name="Event"></param>
				public virtual void AddEvent(LogEvent Event)
				{
					stream.Add(Event);
					switch (Event.Level)
					{
						case Logging.LogLevel.Critical:
						case Logging.LogLevel.Error:
							phase.errorFingerprint = (phase.errorFingerprint ?? "") + GenerateHash(Event.Message);
							break;

						case Logging.LogLevel.Warning:
							phase.hasWarning = true;
							break;
					}
				}

				private string FilterEvent(string Text)
				{
					Text = Text.Trim().ToUpperInvariant();
					Text = Regex_pathstring.Replace(Text, "");
					Text = Regex_hexstring.Replace(Text, "H");
					Text = Regex_numbertimestampstring.Replace(Text, "n");

					return Text;
				}
				private string GenerateHash(string InText)
				{
					return Hasher.ComputeHash(FilterEvent(InText), Hasher.DefaultAlgo, 8);
				}

				/// <summary>
				/// Add event with optional formatting inputs
				/// </summary>
				/// <param name="Time"></param>
				/// <param name="Level"></param>
				/// <param name="Message"></param>
				/// <param name="Args"></param>
				public virtual void AddEvent(DateTime Time, Logging.LogLevel Level, string Message, params object[] Args)
				{
					string? Format = null;
					Dictionary<string, object?>? Properties = null;
					if (Args.Any())
					{
						Format = Message;
						Properties = new Dictionary<string, object?>();
						MessageTemplate.ParsePropertyValues(Format, Args, Properties!);
						Message = MessageTemplate.Render(Format, Properties!);
					}
					Logging.EventId EventId = Level == Logging.LogLevel.Critical ? KnownLogEvents.Gauntlet_FatalEvent : KnownLogEvents.Gauntlet_TestEvent;
					LogEvent Event = new LogEvent(Time, Level, EventId, Message, Format, Properties, null);
					AddEvent(Event);
				}

				/// <summary>
				/// Add event with optional formatting inputs
				/// </summary>
				/// <param name="Level"></param>
				/// <param name="Message"></param>
				/// <param name="Args"></param>
				public virtual void AddEvent(Logging.LogLevel Level, string Message, params object[] Args)
				{
					AddEvent(DateTime.UtcNow, Level, Message, Args);
				}

				/// <summary>
				/// Add Info event with optional formatting inputs
				/// </summary>
				/// <param name="Format"></param>
				/// <param name="Args"></param>
				public virtual void AddInfo(string Format, params object[] Args)
				{
					AddEvent(Logging.LogLevel.Information, Format, Args);
				}

				/// <summary>
				/// Add Warning event with optional formatting inputs
				/// </summary>
				/// <param name="Format"></param>
				/// <param name="Args"></param>
				public virtual void AddWarning(string Format, params object[] Args)
				{
					AddEvent(Logging.LogLevel.Warning, Format, Args);
				}

				/// <summary>
				/// Add Error event with optional formatting inputs
				/// </summary>
				/// <param name="Format"></param>
				/// <param name="Args"></param>
				public virtual void AddError(string Format, params object[] Args)
				{
					AddEvent(Logging.LogLevel.Error, Format, Args);
				}

				/// <summary>
				/// Add event using ITestEvent as input
				/// </summary>
				/// <param name="Event"></param>
				public virtual void AddEvent(ITestEvent Event)
				{
					string Message = Event.Summary;
					Logging.LogLevel Level = EventSeverityToLogLevel(Event.Severity);
					List<object> Params = new List<object>();
					if (Event.Details.Any())
					{
						Message += "\n{Details}";
						Params.Add(string.Join("\n", Event.Details));
					}
					if (Event.Callstack.Any())
					{
						Message += "\n{Callstack}";
						Params.Add(string.Join("\n", Event.Callstack));
					}
					AddEvent(Event.Time, Level, Message, Args: Params.ToArray());
				}

				/// <summary>
				/// Add URL link
				/// </summary>
				/// <param name="Time"></param>
				/// <param name="Level"></param>
				/// <param name="Name"></param>
				/// <param name="URL"></param>
				public virtual void AddURLLink(DateTime Time, Logging.LogLevel Level, string Name, string URL)
				{
					string Token = TestLogValueTypes.url.ToLower();
					var Properties = new Dictionary<string, object?>() { { Token, TestLogValue.CreateURLLink(Name, URL) } };
					string Format = $"{{{Token}}}";
					string LogMessage = MessageTemplate.Render(Format, Properties);
					LogEvent Event = new LogEvent(Time, Level, KnownLogEvents.Gauntlet_TestEvent, LogMessage, Format, Properties, null);

					AddEvent(Event);
				}

				/// <summary>
				/// Add Artifact event
				/// </summary>
				/// <param name="Time"></param>
				/// <param name="Level"></param>
				/// <param name="Format"></param>
				/// <param name="Artifact"></param>
				public virtual void AddArtifact(DateTime Time, Logging.LogLevel Level, string Format, ILogArtifact Artifact)
				{
					if (report == null)
					{
						AddWarning("Phase not attached to a report. Failed to attached Artifact {Name}.", Artifact.ToString()!);
					}
					else
					{
						Artifact.AttachTo(report);
					}

					var Properties = Artifact.ToProperties();
					string Message = MessageTemplate.Render(Format, Properties);

					LogEvent Event = new LogEvent(Time, Level, KnownLogEvents.Gauntlet_TestEvent, Message, Format, Properties, null);
					AddEvent(Event);
				}

				/// <summary>
				/// Add Image Comparison event
				/// </summary>
				/// <param name="Time"></param>
				/// <param name="Level"></param>
				/// <param name="Format"></param>
				/// <param name="Artifact"></param>
				public virtual void AddImageComparison(DateTime Time, Logging.LogLevel Level, string Format, ImageComparisonFiles Artifact)
				{
					bool IsError = Level == Logging.LogLevel.Error;
					if (!IsError)
					{
						// Don't push references if it is not a failure
						Artifact.Files[TestLogValueTypes.approved] = null;
						Artifact.Files[TestLogValueTypes.approvedMetadata] = null;
					}

					AddArtifact(Time, Level, Format, Artifact);
					if (IsError)
					{
						AddInfo("The image reference can be updated by pointing the Screen Comparison tab from the Test Automation window to the artifacts from this test.");
					}
				}
			}

			/// <summary>
			/// Attached the input LogEvent to the stream to current event stream
			/// </summary>
			/// <param name="Event"></param>
			public virtual void AddEvent(LogEvent Event)
			{
				GetEventStream().AddEvent(Event);
			}

			/// <summary>
			/// Add event with optional formatting inputs to current event stream
			/// </summary>
			/// <param name="Time"></param>
			/// <param name="Level"></param>
			/// <param name="Message"></param>
			/// <param name="Args"></param>
			public virtual void AddEvent(DateTime Time, Logging.LogLevel Level, string Message, params object[] Args)
			{
				GetEventStream().AddEvent(Time, Level, Message, Args);
			}

			/// <summary>
			/// Add event with optional formatting inputs to current event stream
			/// </summary>
			/// <param name="Type"></param>
			/// <param name="Message"></param>
			/// <param name="Args"></param>
			public override void AddEvent(EventType Type, string Message, params object[] Args)
			{
				AddEvent(DateTime.UtcNow, EventTypeToLogLevel(Type), Message, Args);
			}

			/// <summary>
			/// Attach Artifact to the report
			/// </summary>
			/// <param name="FilePath"></param>
			/// <param name="Name"></param>
			/// <returns></returns>
			protected virtual bool AttachArtifact(ref string FilePath, string? Name = null)
			{
				if (AttachArtifact(FilePath, Name))
				{
					FilePath = Path.Combine(OutputArtifactPath, Name ?? Path.GetFileName(FilePath));
					FilePath = FileUtils.ConvertPathToUnix(Path.GetRelativePath(Globals.UnrealRootDir, FilePath));
					return true;
				}
				return false;
			}

			/// <summary>
			/// Attach artifact to the report, warn if it fails to do it
			/// </summary>
			/// <param name="FilePath"></param>
			/// <param name="Name"></param>
			/// <returns></returns>
			protected virtual bool AttachArtifactOrWarn(ref string? FilePath, string? Name = null)
			{
				if (FilePath != null && !AttachArtifact(ref FilePath, Name))
				{
					AddWarning("Failed to attached Artifact {FilePath}.", Path.GetRelativePath(Globals.UnrealRootDir, FilePath));
					FilePath = null;
					return false;
				}
				return true;
			}

			private static class TestLogValueTypes
			{
				// Log entry
				public static readonly Utf8String Type = new Utf8String("$type");
				public static readonly Utf8String Text = new Utf8String("$text");
				// Generic artifacts
				public static readonly string artifacts = "Artifacts";
				// Embedded HTML
				public static readonly string embeddedHtml = "Embedded HTML";
				// Image compare
				public static readonly string imageCompare = "Image Compare";
				public static readonly string unapproved = "unapproved";
				public static readonly string unapprovedMetadata = "unapproved_metadata";
				public static readonly string approved = "approved";
				public static readonly string approvedMetadata = "approved_metadata";
				public static readonly string difference = "difference";
				public static readonly string differenceReport = "difference_report";
				// URL link
				public static readonly string url = "URL";

			}

			/// <summary>
			/// Represent a Log value. Necessary to support LogValue annotation in TestData.
			/// </summary>
			public class TestLogValue
			{
				public Utf8String Type { get; protected set; }
				public string Text { get; protected set; }
				public Dictionary<Utf8String, object?> Properties { get; protected set; }

				public TestLogValue(string InType, string InText, Dictionary<Utf8String, object?> InProperties)
				{
					Type = new Utf8String(InType);
					Text = InText;
					Properties = InProperties;
				}

				public override string ToString()
				{
					return Text;
				}

				/// <summary>
				/// Utility function to create log value for URL link
				/// </summary>
				public static TestLogValue CreateURLLink(string Text, string URL)
				{
					return new TestLogValue(TestLogValueTypes.url, Text, new Dictionary<Utf8String, object?>() { { new Utf8String("href"), URL } });
				}
			}

			/// <summary>
			/// TestLogValue Formatter
			/// </summary>
			public class TestLogValueFormatter : ILogValueFormatter
			{
				public void Format(object value, Utf8JsonWriter writer)
				{
					TestLogValue entry = (TestLogValue)value;
					writer.WriteStartObject();
					writer.WriteString(TestLogValueTypes.Type, entry.Type);
					writer.WriteString(TestLogValueTypes.Text, entry.ToString());
					if (entry.Properties != null)
					{
						foreach (KeyValuePair<Utf8String, object?> pair in entry.Properties)
						{
							writer.WritePropertyName(pair.Key);
							writer.WriteStringValue(pair.Value?.ToString());
						}
					}
					writer.WriteEndObject();
				}
			}

			/// <summary>
			/// Interface for artifact in Log event
			/// </summary>
			public interface ILogArtifact
			{
				public bool AttachTo(AutomatedTestSessionData Report);
				Dictionary<string, object?> ToProperties();
			}

			/// <summary>
			/// Generic Artifact Files utility objects to attached files to report and generate event properties
			/// </summary>
			public class ArtifactFiles : ILogArtifact
			{
				public string Type { get; protected set; }
				public string Token { get; protected set; }
				public string Name { get; protected set; }
				public Dictionary<string, string?> Files { get; protected set; }
				public string RootPath { get; protected set; }

				public ArtifactFiles(string InType, string InName, Dictionary<string, string?> InFiles, string InRootPath)
				{
					Type = InType;
					Token = Type.ToLower().Replace(" ", "_");
					Name = InName;
					Files = new(InFiles);
					RootPath = InRootPath;
				}

				public ArtifactFiles(string InName, Dictionary<string, string?> InFiles, string InRootPath)
					: this(TestLogValueTypes.artifacts, InName, InFiles, InRootPath) { }

				private bool AttachFileTo(ref string? FilePath, AutomatedTestSessionData Report)
				{
					if (string.IsNullOrEmpty(FilePath))
					{
						// Ignore this file
						return true;
					}
					string? Name = null;
					if (!Path.IsPathFullyQualified(FilePath))
					{
						Name = FilePath;
						if (!string.IsNullOrEmpty(RootPath))
							FilePath = Path.Combine(RootPath, FilePath);
					}
					return Report.AttachArtifactOrWarn(ref FilePath, Name);
				}

				public bool AttachTo(AutomatedTestSessionData Report)
				{
					bool AllAttached = true;
					foreach (string Key in Files.Keys.ToList())
					{
						string? FilePath = Files[Key];
						if (!AttachFileTo(ref FilePath, Report))
						{
							AllAttached = AllAttached && false;
						}
						Files[Key] = FilePath;
					}

					return AllAttached;
				}

				public Dictionary<string, object?> ToProperties()
				{
					Dictionary<Utf8String, object?> Prop = new Dictionary<Utf8String, object?>();
					foreach(var Item in Files)
					{
						if (!string.IsNullOrEmpty(Item.Value))
						{
							Prop.Add(new Utf8String(Item.Key.ToLower()), Item.Value);
						}
					}
					TestLogValue Value = new TestLogValue(Type, Name, Prop);
					Dictionary<string, object?> EventProperties = new Dictionary<string, object?> { { Token, Value } };

					return EventProperties;
				}
			}

			/// <summary>
			/// Utility object to attached ImageComparison artifacts and generate related event properties
			/// </summary>
			public class ImageComparisonFiles : ArtifactFiles, ILogArtifact
			{
				public ImageComparisonFiles(string InName, Dictionary<string, string?> InFiles, string InRootPath, bool AddMetadataFiles = false) : base(TestLogValueTypes.imageCompare, InName, InFiles, InRootPath)
				{
					string? FilePath = Files.GetValueOrDefault(TestLogValueTypes.unapproved);
					if (AddMetadataFiles && !string.IsNullOrEmpty(FilePath))
					{
						Files.Add(TestLogValueTypes.unapprovedMetadata, Path.ChangeExtension(FilePath, ".json"));
					}
					FilePath = Files.GetValueOrDefault(TestLogValueTypes.approved);
					if (AddMetadataFiles && !string.IsNullOrEmpty(FilePath))
					{
						Files.Add(TestLogValueTypes.approvedMetadata, Path.ChangeExtension(FilePath, ".json"));
					}
					FilePath = Files.GetValueOrDefault(TestLogValueTypes.difference);
					if (AddMetadataFiles && !string.IsNullOrEmpty(FilePath))
					{
						Files.Add(TestLogValueTypes.differenceReport, Path.Combine(Path.GetDirectoryName(FilePath)!, "Report.json"));
					}
				}
			}

			/// <summary>
			/// Utility object to attached Embedded HTML file and generate related event properties
			/// </summary>
			public class EmbeddedHTMLFile : ArtifactFiles, ILogArtifact
			{
				public EmbeddedHTMLFile(string InName, string InFile, string InRootPath)
					: base(TestLogValueTypes.embeddedHtml, InName, new Dictionary<string, string?>() { { "path", InFile } }, InRootPath) { }
			}

			private static Logging.LogLevel EventTypeToLogLevel(EventType Type)
			{
				Logging.LogLevel Level = Logging.LogLevel.None;
				switch (Type)
				{
					case EventType.Info:
						Level = Logging.LogLevel.Information;
						break;

					case EventType.Warning:
						Level = Logging.LogLevel.Warning;
						break;

					case EventType.Error:
						Level = Logging.LogLevel.Error;
						break;
				}

				return Level;
			}

			private static Logging.LogLevel EventSeverityToLogLevel(EventSeverity Type)
			{
				Logging.LogLevel Level = Logging.LogLevel.None;
				switch (Type)
				{
					case EventSeverity.Info:
						Level = Logging.LogLevel.Information;
						break;

					case EventSeverity.Warning:
						Level = Logging.LogLevel.Warning;
						break;

					case EventSeverity.Error:
						Level = Logging.LogLevel.Error;
						break;

					case EventSeverity.Fatal:
						Level = Logging.LogLevel.Critical;
						break;
				}

				return Level;
			}

			private static TestPhaseOutcome TestStateToPhaseOutcome(TestStateType State)
			{
				TestPhaseOutcome Outcome = TestPhaseOutcome.Unknown;
				switch (State)
				{
					case TestStateType.Fail:
						Outcome = TestPhaseOutcome.Failed;
						break;

					case TestStateType.InProcess:
					case TestStateType.Interrupted:
						Outcome = TestPhaseOutcome.Interrupted;
						break;

					case TestStateType.NotRun:
					case TestStateType.Unknown:
						Outcome = TestPhaseOutcome.NotRun;
						break;

					case TestStateType.Success:
						Outcome = TestPhaseOutcome.Success;
						break;

					case TestStateType.Skipped:
						Outcome = TestPhaseOutcome.Skipped;
						break;
				}
				return Outcome;
			}

			private static TestPhaseOutcome TestResultToPhaseOutcome(Gauntlet.TestResult State)
			{
				TestPhaseOutcome Outcome = TestPhaseOutcome.Unknown;
				switch (State)
				{
					case Gauntlet.TestResult.Failed:
						Outcome = TestPhaseOutcome.Failed;
						break;

					case Gauntlet.TestResult.TimedOut:
					case Gauntlet.TestResult.Cancelled:
					case Gauntlet.TestResult.WantRetry:
					case Gauntlet.TestResult.OperationalException:
						Outcome = TestPhaseOutcome.Interrupted;
						break;

					case Gauntlet.TestResult.Invalid:
						Outcome = TestPhaseOutcome.NotRun;
						break;

					case Gauntlet.TestResult.Passed:
						Outcome = TestPhaseOutcome.Success;
						break;

					case Gauntlet.TestResult.InsufficientDevices:
						Outcome = TestPhaseOutcome.Skipped;
						break;
				}
				return Outcome;
			}

			/// <summary>
			/// Return the event stream of the target phase or current phase if not specified
			/// </summary>
			/// <param name="InPhaseKey"></param>
			/// <returns></returns>
			/// <exception cref="AutomationException"></exception>
			public virtual TestEventStream GetEventStream(string? InPhaseKey = null)
			{
				if (InPhaseKey == null && CurrentPhase == null)
				{
					AddPhase("Main");
				}
				string PhaseKey = string.IsNullOrEmpty(InPhaseKey) ? CurrentPhase!.key : InPhaseKey;
				TestPhase? Phase = GetPhase(PhaseKey);
				if (Phase == null)
				{
					throw new AutomationException("No Phase '{0}' set in Test '{1}'", PhaseKey, Key);
				}
				return Phase.GetStream();
			}

			/// <inheritdoc/>
			public override void FinalizeReport()
			{
				// store the event streams
				string OutputDir = Path.Combine(OutputArtifactPath, "EventStreams");
				if (!Directory.Exists(OutputDir))
				{
					Directory.CreateDirectory(OutputDir);
				}
				Log.Verbose("Writing Phase Event Streams at {0}", OutputDir);
				foreach (TestPhase Phase in Data.phases)
				{
					string? OutputFilePath = null;
					TestEventStream Stream = Phase.GetStream();
					// skip if the event stream is empty
					if (Stream.HasAnyEvent())
					{
						string FileName = $"EventStreams/{Phase.key}.json";
						OutputFilePath = Path.Combine(OutputArtifactPath, FileName);
						// write phase event stream
						try
						{
							File.WriteAllText(OutputFilePath, JsonSerializer.Serialize(Stream.GetEvents(), GetDefaultJsonOptions()));
							if(!AttachArtifact(ref OutputFilePath, FileName))
							{
								OutputFilePath = null;
							}
						}
						catch (Exception Ex)
						{
							Log.Error("Failed to save Event Stream for '{0}'. {1}", Phase.key, Ex);
						}
						// finalize error fingerprint
						Phase.CompileErrorFingerprint();
					}
					// update phase
					Phase.eventStreamPath = OutputFilePath;
					if (Phase.outcome == TestPhaseOutcome.Unknown)
					{
						Phase.SetOutcome(Phase.errorFingerprint == null ? TestStateType.Success : TestStateType.Fail);
					}
					// propagate phase tags to test tags
					if (Phase.tags != null)
					{
						AddTestTags(Phase.tags);
					}
				}
			}

			private static DateTime ConvertUETimeStringToDateTime(string StringTime)
			{
				return UnrealAutomationEntry.GetTimestampAsDateTime(StringTime);
			}

			/// <summary>
			/// Convert UnrealAutomatedTestPassResults to Horde data model
			/// </summary>
			/// <param name="InTestPassResults"></param>
			/// <param name="InReportPath"></param>
			/// <returns></returns>
			public void PopulateFromUnrealAutomatedTests(UnrealAutomatedTestPassResults InTestPassResults, string InReportPath)
			{
				if (string.IsNullOrEmpty(Data.summary.dateTime))
				{
					DateTime SessionTime = ConvertUETimeStringToDateTime(InTestPassResults.ReportCreatedOn);
					SetSessionTiming(SessionTime, InTestPassResults.TotalDuration);
				}

				if (InTestPassResults.Devices != null)
				{
					foreach (UnrealAutomationDevice InDevice in InTestPassResults.Devices)
					{
						TestDevice ConvertedDevice = AddDevice(InDevice.DeviceName, InDevice.Instance);
						ConvertedDevice.appInstanceLogPath = InDevice.AppInstanceLog;
						ConvertedDevice.metadata.Add("platform", InDevice.Platform);
						ConvertedDevice.metadata.Add("os_version", InDevice.OSVersion);
						ConvertedDevice.metadata.Add("model", InDevice.Model);
						ConvertedDevice.metadata.Add("gpu", InDevice.GPU);
						ConvertedDevice.metadata.Add("cpumodel", InDevice.CPUModel);
						ConvertedDevice.metadata.Add("ram_in_gb", InDevice.RAMInGB.ToString());
						ConvertedDevice.metadata.Add("render_mode", InDevice.RenderMode);
						ConvertedDevice.metadata.Add("rhi", InDevice.RHI);
					}
				}
				if (InTestPassResults.Tests != null)
				{
					foreach (UnrealAutomatedTestResult InTestResult in InTestPassResults.Tests)
					{
						TestPhase Phase = AddPhase(InTestResult.FullTestPath);
						if (InTestResult.Tags.Count > 0)
						{
							Phase.AddTags(InTestResult.Tags);
						}
						Phase.deviceKeys = InTestResult.DeviceInstance.ToHashSet();
						string TestDateTime = InTestResult.DateTime;
						DateTime PhaseTime = ConvertUETimeStringToDateTime(TestDateTime);
						Phase.SetTiming(PhaseTime, InTestResult.Duration);
						Phase.SetOutcome(InTestResult.State);
						TestEventStream Stream = Phase.GetStream();
						bool hasCriticalFailure = false;
						foreach (UnrealAutomationEntry InTestEntry in InTestResult.Entries)
						{
							string Artifact = InTestEntry.Event.Artifact;
							// If Artifact values is not null nor a bunch on 0, then we have a file attachment.
							if (!string.IsNullOrEmpty(Artifact) && Artifact.Substring(0, 4) != "0000")
							{
								UnrealAutomationArtifact? ArtifactEntry = InTestResult.Artifacts.Find(A => A.Id == Artifact);
								if (ArtifactEntry != null)
								{
									DateTime EventTime = ConvertUETimeStringToDateTime(InTestEntry.Timestamp);
									Logging.LogLevel LogLevel = EventTypeToLogLevel(InTestEntry.Event.Type);
									string EntryName = ArtifactEntry.Name;
									if (ArtifactEntry.Type == "Comparison")
									{
										ImageComparisonFiles ComparisonFiles = new ImageComparisonFiles(EntryName, ArtifactEntry.Files, InReportPath, true);
										string Format = InTestEntry.Event.Message.Replace(EntryName, $"{{{ComparisonFiles.Token}}}");
										Stream.AddImageComparison(EventTime, LogLevel, Format, ComparisonFiles);

									}
									else
									{
										ArtifactFiles Files = new ArtifactFiles(ArtifactEntry.Type, EntryName, ArtifactEntry.Files, InReportPath);
										string Format = InTestEntry.Event.Message.Replace(EntryName, $"{{{Files.Token}}}");
										Stream.AddArtifact(EventTime, LogLevel, Format, Files);
									}
									continue;
								}
							}
							Stream.AddEvent(InTestEntry.AsLogEvent());
							hasCriticalFailure = hasCriticalFailure || InTestEntry.Event.IsCriticalFailure;
						}
						if (hasCriticalFailure)
						{
							Phase.SetOutcome(TestPhaseOutcome.Interrupted);
						}
					}
				}
			}
		}

		/// <summary>
		/// Container for Test Data items 
		/// </summary>
		public class TestDataCollection
		{
			public interface IDataBlob
			{
				int version { get; }
				Dictionary<string, string> metadata { get; }
			}
			public class DataItem
			{
				public string? key { get; set; }
				public object? data { get; set; }
			}
			public TestDataCollection()
			{
				items = new List<DataItem>();
			}

			public DataItem AddNewTestReport(object InData, string? InKey = null)
			{
				Dictionary<string, object>? ExtraItems = null;
				if (InData is BaseHordeReport InHordeReport)
				{
					InHordeReport.FinalizeReport();
					ExtraItems = InHordeReport.GetReportDependencies();
					InKey = InHordeReport.GetTestDataKey(InKey);
					InData = InHordeReport.GetTestData();
				}
				DataItem NewDataItem = new DataItem()
				{
					key = InKey,
					data = InData,
				};

				var FoundItemIndex = items.FindIndex(I => I.key == InKey);
				if (FoundItemIndex == -1)
				{
					items.Add(NewDataItem);
				}
				else
				{
					items[FoundItemIndex] = NewDataItem;
				}

				if (ExtraItems != null && ExtraItems.Count() > 0)
				{
					foreach (string Key in ExtraItems.Keys)
					{
						DataItem ExtraDataItem = new DataItem()
						{
							key = Key,
							data = ExtraItems[Key]
						};
						items.Add(ExtraDataItem);
					}
				}

				return NewDataItem;
			}

			public List<DataItem> items { get; set; }

			/// <summary>
			/// Write Test Data Collection to json
			/// </summary>
			/// <param name="OutputTestDataFilePath"></param>
			/// <param name="bIncrementNameIfFileExists"></param>
			public void WriteToJson(string OutputTestDataFilePath, bool bIncrementNameIfFileExists = false)
			{
				string OutputTestDataDir = Path.GetDirectoryName(OutputTestDataFilePath)!;
				if (!Directory.Exists(OutputTestDataDir))
				{
					Directory.CreateDirectory(OutputTestDataDir);
				}
				if (File.Exists(OutputTestDataFilePath) && bIncrementNameIfFileExists)
				{
					// increment filename if file exists
					string Ext = Path.GetExtension(OutputTestDataFilePath);
					string Filename = OutputTestDataFilePath.Replace(Ext, "");
					int Incr = 0;
					do
					{
						Incr++;
						OutputTestDataFilePath = string.Format("{0}{1}{2}", Filename, Incr, Ext);
					} while (File.Exists(OutputTestDataFilePath));
				}
				// write test pass summary
				Log.Verbose("Writing Test Data Collection for Horde at {0}", OutputTestDataFilePath);
				try
				{
					File.WriteAllText(OutputTestDataFilePath, JsonSerializer.Serialize(this, GetDefaultJsonOptions()));
				}
				catch (Exception Ex)
				{
					Log.Error("Failed to save Test Data Collection for Horde. {0}", Ex);
				}
			}
		}
		private static JsonSerializerOptions GetDefaultJsonOptions()
		{
			return new JsonSerializerOptions
			{
				// Pretty print when running locally
				WriteIndented = !CommandUtils.IsBuildMachine,
				// Keep file size small by ignoring null value
				DefaultIgnoreCondition = JsonIgnoreCondition.WhenWritingNull
			};
		}

		/// <summary>
		/// Validate Key name, intended to validate a key that is going to be used as property name (ie: Phase key).
		/// </summary>
		/// <param name="InKey"></param>
		/// <returns></returns>
		/// <exception cref="AutomationException"></exception>
		private static string ValidateKey(string InKey)
		{
			if (string.IsNullOrEmpty(InKey))
			{
				throw new AutomationException("Key must not be empty.");
			}
			if (InKey.StartsWith("$"))
			{
				throw new AutomationException("Key must not start with the '$' character.");
			}
			if (InKey.Contains("."))
			{
				throw new AutomationException("Key must not contain '.' characters.");
			}
			if (InKey.Length > 1024)
			{
				throw new AutomationException("Key must not have more than 1024 characters.");
			}
			return InKey;
		}
	}
}
