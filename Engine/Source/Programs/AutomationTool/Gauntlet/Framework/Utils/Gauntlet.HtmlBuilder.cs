// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Linq;
using System.Text;
using System.Threading.Tasks;

namespace Gauntlet
{
	public class HtmlBuilder
	{
		
		protected StringBuilder SB;
		protected bool CurrentlyBuildingTable = false;
		/// <summary>
		/// All of the possible ways to modify a stock table cell.
		/// </summary>
		public class TableCellInfo
		{
			public bool WithBorder = true;
			public bool CenterAlign = true;
			/// <summary>
			/// Number of columns the cell spans.
			/// </summary>
			public int CellColumnWidth = 1;
			public string BackgroundColor = "#ffffff";
			public string BorderType = "border:1px solid black;"; 
			public string TableContents;
			public void ResetToDefaults()
			{
				CellColumnWidth = 1;
				BackgroundColor = "#ffffff";
				BorderType = "border:1px solid black;";
				CenterAlign = true;
				WithBorder = true;
			}
		}

		/// <summary>
		/// All of the possible ways to modify a basic string via formatting.
		/// </summary>
		public class TextOptions
		{
			public bool Italics = false;
			public bool Bold = false;
			public string TextColorOverride = string.Empty;
			public string Hyperlink = string.Empty;
			public string ApplyToString(string TextToAlter)
			{
				string OutString = TextToAlter;
				if (Italics)
				{
					OutString = string.Format("<i>{0}</i>", OutString);
				}
				if (Bold)
				{
					OutString = string.Format("<b>{0}</b>", OutString);
				}
				if (!string.IsNullOrEmpty(TextColorOverride))
				{
					OutString = string.Format("<font color=\"{0}\">{1}</font>", TextColorOverride, OutString);
				}
				if (!string.IsNullOrEmpty(Hyperlink))
				{
					OutString = string.Format("<a href=\"{0}\">{1}</a>", Hyperlink, OutString);
				}
				return OutString;
			}

		}

		public HtmlBuilder()
		{
			SB = new StringBuilder();
		}

		
		/// <summary>
		/// Returns our formatted text
		/// </summary>
		/// <returns></returns>
		public override string ToString()
		{
			return SB.ToString();
		}

		/// <summary>
		/// Returns true if the correct body ends with a new line
		/// </summary>
		public bool EndsWithNewLine
		{
			get { return SB.Length == 0 || SB.ToString().EndsWith("<br>") ; }
		}

		/// <summary>
		/// Ensures any text after this starts on a new line
		/// </summary>
		/// <returns></returns>
		public HtmlBuilder EnsureEndsWithNewLine()
		{
			if (SB.Length > 0 && !EndsWithNewLine)
			{
				NewLine();
			}
			return this;
		}

		/// <summary>
		/// Returns true if the correct body ends with a new line
		/// </summary>
		public bool EndsWithRowClose
		{
			get { return SB.Length == 0 || SB.ToString().EndsWith("</tr>"); }
		}

		/// <summary>
		/// Ensures any text after this starts on a new line
		/// </summary>
		/// <returns></returns>
		public HtmlBuilder EnsureEndsWithRowClose()
		{
			if (SB.Length > 0 && !EndsWithRowClose)
			{
				SB.Append("</tr>");
			}
			return this;
		}

		/// <summary>
		/// Open up a new table, if one doesn't already exist, to start adding cells to.
		/// </summary>
		/// <returns></returns>
		public HtmlBuilder StartBuildingTable()
		{
			if (!CurrentlyBuildingTable)
			{
				EnsureEndsWithNewLine();
				SB.Append("<table style=\"border: 0px; padding: 0px;\">");
				CurrentlyBuildingTable = true;
			}
			return this;
		}

		/// <summary>
		/// Open a new row. Close any existing open rows if they already exist.
		/// </summary>
		/// <returns></returns>
		public HtmlBuilder StartRow()
		{
			EnsureEndsWithRowClose();
			SB.Append("<tr>");
			return this;
		}
		/// <summary>
		/// Finalize a row with the proper tags.
		/// </summary>
		/// <returns></returns>
		public HtmlBuilder EndRow()
		{
			SB.Append("</tr>");
			return this;
		}

		protected string CreateCell(string TableContents, TableCellInfo CellInfo)
		{

			return string.Format("<td bgcolor=\'{0}\' colspan=\'{1}\' {2} {3}> {4}</td>", 
				CellInfo.BackgroundColor,
				CellInfo.CellColumnWidth,
				CellInfo.CenterAlign ? "align=center" : "",
				CellInfo.WithBorder ? string.Format(" style=\"{0}\"", CellInfo.BorderType) : "",
				TableContents);
		}
		/// <summary>
		/// Add a new cell with any formatting applied.
		/// </summary>
		/// <param name="TableContents"></param>
		/// <param name="CellInfo"></param>
		/// <returns></returns>
		public HtmlBuilder AddNewCell(string TableContents, TableCellInfo CellInfo = null)
		{
			SB.Append(CreateCell(TableContents, CellInfo != null ? CellInfo : new TableCellInfo()));
			return this;
		}

		/// <summary>
		/// Finish up an existing table. Close off any existing rows before closing the table and newlining.
		/// </summary>
		/// <returns></returns>
		public HtmlBuilder FinalizeTable()
		{
			if (CurrentlyBuildingTable)
			{
				EnsureEndsWithRowClose();
				SB.Append("</table><br>");
				CurrentlyBuildingTable = false;
			}
			return this;
		}

		public HtmlBuilder NewLine()
		{
			SB.Append("<br>");
			return this;
		}

		/// <summary>
		/// Insert a hyperlink
		/// </summary>
		/// <param name="URL"></param>
		/// <param name="Text"></param>
		/// <returns></returns>
		public HtmlBuilder Hyperlink(string URL, string Text)
		{
			SB.AppendFormat("<a href=\"{0}\">{1}</a>", URL, Text);
			return this;
		}

		/// <summary>
		///Insert an H1 Header
		/// </summary>
		/// <param name="Text"></param>
		/// <returns></returns>
		public HtmlBuilder H1(string Text)
		{
			EnsureEndsWithNewLine();
			SB.AppendFormat("<h1>{0}</h1>", Text);
			return this;
		}

		/// <summary>
		/// Insert an H2 header
		/// </summary>
		/// <param name="Text"></param>
		/// <returns></returns>
		public HtmlBuilder H2(string Text)
		{
			EnsureEndsWithNewLine();
			SB.AppendFormat("<h2>{0}</h2>", Text);
			return this;
		}

		/// <summary>
		/// Insert an H3 header
		/// </summary>
		/// <param name="Text"></param>
		/// <returns></returns>
		public HtmlBuilder H3(string Text)
		{
			EnsureEndsWithNewLine();
			SB.AppendFormat("<h3>{0}</h3>", Text);
			return this;
		}

		/// <summary>
		/// Insert an H4 Header
		/// </summary>
		/// <param name="Text"></param>
		/// <returns></returns>
		public HtmlBuilder H4(string Text)
		{
			EnsureEndsWithNewLine();
			SB.AppendFormat("<h4>{0}</h4>", Text);
			return this;
		}
		/// <summary>
		/// Insert an H5 header
		/// </summary>
		/// <param name="Text"></param>
		/// <returns></returns>
		public HtmlBuilder H5(string Text)
		{
			EnsureEndsWithNewLine();
			SB.AppendFormat("<h5>{0}</h5>", Text);
			return this;
		}

		/// <summary>
		/// Insert an ordered (numbered) list
		/// </summary>
		/// <param name="Items"></param>
		/// <returns></returns>
		public HtmlBuilder OrderedList(IEnumerable<string> Items)
		{
			EnsureEndsWithNewLine();
			SB.Append("<ol type=\"1\">");
			foreach (string Item in Items)
			{
				SB.AppendFormat("<li>{0}</li>", Item);
				SB.AppendLine();
			}
			SB.Append("</ol>");
			NewLine();

			return this;
		}

		/// <summary>
		/// Insert an unordered (bulleted) list
		/// </summary>
		/// <param name="Items"></param>
		/// <returns></returns>
		public HtmlBuilder UnorderedList(IEnumerable<string> Items)
		{
			EnsureEndsWithNewLine();
			SB.Append("<ul style=\"list-style-type:disc\">");
			foreach (string Item in Items)
			{
				SB.AppendFormat("<li>{0}</li>", Item);
				SB.AppendLine();
			}
			SB.Append("</ul>");
			NewLine();

			return this;
		}

		/// <summary>
		/// Append the provided Markdown to our body
		/// </summary>
		/// <param name="RHS"></param>
		/// <returns></returns>
		public HtmlBuilder Append(HtmlBuilder RHS)
		{
			SB.Append(RHS.ToString());
			return this;
		}

		/// <summary>
		/// Append the provided text to our body. No formatting will be applied
		/// </summary>
		/// <param name="RHS"></param>
		/// <returns></returns>
		public HtmlBuilder Append(string RHS)
		{
			SB.Append(RHS);
			return this;
		}



		/// <summary>
		/// Append the provided text to our body. No formatting will be applied
		/// </summary>
		/// <param name="RHS"></param>
		/// <param name="InOptions"></param>
		/// <returns></returns>
		public HtmlBuilder AppendFormatted(string RHS, TextOptions InOptions)
		{
			if (InOptions == null)
			{
				SB.Append(RHS);
			}
			else
			{
				SB.Append(InOptions.ApplyToString(RHS));
			}
			return this;
		}

	}
}
