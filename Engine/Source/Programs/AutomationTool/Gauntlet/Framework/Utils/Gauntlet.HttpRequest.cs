// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Net;
using System.Net.Http;
using System.Net.Http.Headers;
using System.Text;
using System.Threading.Tasks;
using AutomationTool;

namespace Gauntlet
{
	public static class HttpRequest
	{
		public static AuthenticationHeaderValue Authentication(string Target, string Token)
		{
			return new AuthenticationHeaderValue(Target, Token);
		}
		public class Connection
		{
			private HttpClient Client;
			public Connection(string Server, AuthenticationHeaderValue Auth)
			{
				Client = new HttpClient();
				Client.BaseAddress = new Uri(Server);
				if (Auth == null)
				{
					throw new AutomationException(string.Format("Missing Authorization Header for {0}", Server));
				}
				Client.DefaultRequestHeaders.Authorization = Auth;
			}
			public HttpResponse PostJson(string Path, string JsonString)
			{

				HttpRequestMessage Request = new HttpRequestMessage(HttpMethod.Post, Path);
				Request.Content = new StringContent(JsonString, Encoding.UTF8, "application/json");
				using (HttpResponseMessage Response = Client.SendAsync(Request).Result)
				{
					return new HttpResponse(Response.StatusCode, Response.Content.ReadAsStringAsync().Result);
				}
			}
		}
		public class HttpResponse
		{
			public string Content;
			public HttpStatusCode StatusCode;
			public HttpResponse(HttpStatusCode InCode, string InContent)
			{
				StatusCode = InCode;
				Content = InContent;
			}
			public bool IsSuccessStatusCode { get { return StatusCode == HttpStatusCode.OK; } }
		}
	}
}