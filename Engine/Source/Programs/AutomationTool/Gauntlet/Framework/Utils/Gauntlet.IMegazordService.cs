// Copyright Epic Games, Inc. All Rights Reserved.

using System.Threading.Tasks;

#nullable enable

namespace Gauntlet
{
	/// <summary>
	/// Cross-boundary contract for the device farm Megazord API.
	/// Public engine code (e.g. <see cref="TargetDeviceIOS"/>) calls into the
	/// Megazord service via this interface so that restricted implementation
	/// details remain inside <c>Engine/Restricted/NotForLicensees</c>.
	///
	/// This is a partial interface: platform-specific methods are declared in
	/// sibling files under the public platform paths (e.g. iOS contributes
	/// <c>GetDeviceLockdownKeyAsync</c> from
	/// <c>Engine/Source/Programs/AutomationTool/Gauntlet/Platform/IOS/Gauntlet.IMegazordService.cs</c>).
	/// </summary>
	public partial interface IMegazordService
	{
		/// <summary>
		/// Query the IP address of a device from the Megazord API using its UUID or serial number.
		/// </summary>
		/// <param name="DeviceUUID">Device UUID or serial number</param>
		/// <returns>IP address string if found, null otherwise</returns>
		Task<string?> GetDeviceIPAddressAsync(string DeviceUUID);
	}
}
