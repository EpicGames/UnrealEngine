// Copyright Epic Games, Inc. All Rights Reserved.

using Newtonsoft.Json;
using System;
using System.Collections.Generic;
using System.IO;
using System.Linq;
using System.Text;
using System.Threading.Tasks;

namespace Gauntlet
{
	/// <summary>
	/// Information that defines a device
	/// </summary>
	public class IgnoredTestIssues
	{
		public string TestName = "";

		public string[] IgnoredEnsures = new string[0];

		public string[] IgnoredWarnings = new string[0];

		public string[] IgnoredErrors = new string[0];
	}
		
	/// <summary>
	/// A class that can be serialized in from JSON file and used in a TestNode to ignore specific warnings/errors if so desired. 
	/// Note - this is not used in the base Gauntlet classes but rather is intended for games where accepting *some* amount of issues
	/// may be necessary for a time. E.g. Issues that have been logged and are awaiting a fix, or a development branch that inherited
	/// some known issues from the mainline
	/// </summary>
	public class IgnoredIssueConfig
	{
		public IgnoredIssueConfig()
		{

		}

		private IEnumerable<IgnoredTestIssues> IgnoredIssues { get; set; } = Enumerable.Empty<IgnoredTestIssues>();

		/// <summary>
		/// Load from the provided JOSN file.
		/// [
		///		{
		///			"TestName" : "Game.AbilityTests",
		///			"IgnoredWarnings" : [
		///				"PNG Error: Duplicate ICCP"
		///			],
		///			"IgnoredEnsures" : [
		///				"Ensure condition failed: GlobalAbilityTaskCount < 1000",
		///			]
		///		}
		/// ]
		/// </summary>
		/// <param name="InPath"></param>
		/// <returns></returns>
		public bool LoadFromFile(string InPath)
		{
			try
			{
				IgnoredIssues = JsonConvert.DeserializeObject<IgnoredTestIssues[]>(File.ReadAllText(InPath));
			}
			catch (Exception Ex)
			{
				Log.Warning("Failed to load issue definition from {0}. {1}", InPath, Ex.Message);
				return false;
			}

			return true;
		}

		/// <summary>
		/// Returns true if the provided ensure should be ignored
		/// </summary>
		/// <param name="InTestName"></param>
		/// <param name="InEnsureMsg"></param>
		/// <returns></returns>
		public bool IsEnsureIgnored(string InTestName, string InEnsureMsg)
		{
			IEnumerable<IgnoredTestIssues> TestDefinitions = IgnoredIssues.Where(I => I.TestName == "*" || I.TestName.Equals(InTestName, StringComparison.OrdinalIgnoreCase));
			return TestDefinitions.Any(D => D.IgnoredEnsures.Any(Ex => InEnsureMsg.IndexOf(Ex, StringComparison.OrdinalIgnoreCase) >= 0));

		}

		/// <summary>
		/// Returns true if the provided warning should be ignored for the specified test name
		/// </summary>
		/// <param name="InTestName"></param>
		/// <param name="InWarning"></param>
		/// <returns></returns>
		public bool IsWarningIgnored(string InTestName, string InWarning)
		{
			IEnumerable<IgnoredTestIssues> TestDefinitions = IgnoredIssues.Where(I => I.TestName == "*" || I.TestName.Equals(InTestName, StringComparison.OrdinalIgnoreCase));
			return TestDefinitions.Any(D => D.IgnoredWarnings.Any( Ex => InWarning.IndexOf(Ex, StringComparison.OrdinalIgnoreCase) >= 0));

		}

		/// <summary>
		/// Returns true if the provided error should be ignored for the specified test name
		/// </summary>
		/// <param name="InTestName"></param>
		/// <param name="InError"></param>
		/// <returns></returns>
		public bool IsErrorIgnored(string InTestName, string InError)
		{
			IEnumerable<IgnoredTestIssues> TestDefinitions = IgnoredIssues.Where(I => I.TestName.Equals(InTestName, StringComparison.OrdinalIgnoreCase));
			return TestDefinitions.Any(D => D.IgnoredErrors.Any(Ex => InError.IndexOf(Ex, StringComparison.OrdinalIgnoreCase) >= 0));
		}

		/// <summary>
		/// Returns true if the provided error should be ignored for the specified test name
		/// </summary>
		/// <param name="InTestName"></param>
		/// <param name="InEntry"></param>
		/// <returns></returns>
		public bool IsLogEntryIgnored(string InTestName, UnrealLog.LogEntry InEntry)
		{
			if (InEntry.Level == UnrealLog.LogLevel.Warning)
			{
				return IsWarningIgnored(InTestName, InEntry.Message);
			}
			else if (InEntry.Level == UnrealLog.LogLevel.Error)
			{
				return IsErrorIgnored(InTestName, InEntry.Message);
			}
			return false;
		}
	}
}
