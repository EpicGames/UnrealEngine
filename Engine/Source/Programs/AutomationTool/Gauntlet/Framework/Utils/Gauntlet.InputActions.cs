// Copyright Epic Games, Inc. All Rights Reserved.

using System.Collections.Generic;
using System.Reflection;
using System.Text.Json;
using System.Text.Json.Serialization;
using UnrealBuildTool;

namespace Gauntlet.Utils
{
	public class FKey
	{
		public string KeyName { get; set; }

		public FKey(string InName)
		{
			this.KeyName = InName;
		}

		// We might need this in the future, but there's some issues serializing FKeyDetails as it has a cyclic reference to the FKey object itself (In pointer form in C++)
		// public FKeyDetails KeyDetails


	}
	
	// Basically a replica of FKeyDetails from InputCoreTypes, intended to be serialized and then deserialized into its UObject type
	// We may end up genuinely needing this for Axis-based input (For example MouseWheelAxis or Mouse2D), but it's not needed right now for regular key-presses
	public struct FKeyDetails
	{
		public enum EKeyFlags
		{
			GamepadKey					= 1 << 0,
			Touch						= 1 << 1,
			MouseButton					= 1 << 2,
			ModifierKey					= 1 << 3,
			NotBlueprintBindableKey		= 1 << 4,
			Axis1D						= 1 << 5,
			Axis3D						= 1 << 6,
			UpdateAxisWithoutSamples	= 1 << 7,
			NotActionBindableKey		= 1 << 8,
			Deprecated					= 1 << 9,
			ButtonAxis					= 1 << 10,
			Axis2D						= 1 << 11,
			Gesture						= 1 << 12,
			NoFlags						= 0
		}

		public enum EInputAxisType
		{
			None,
			Button,
			Axis1D,
			Axis2D,
			Axis3D
		}

		private FKey Key;
		public int PairedAxis = 0;
		public FKey PairedAxisKey;
		public string MenuCategory;

		public int bIsModifierKey;
		public int bIsGamepadKey;
		public int bIsTouch;
		public int bIsMouseButton;
		public int bIsBindableInBlueprints;
		public int bShouldUpdateAxisWithoutSamples;
		public int bIsbindableToActions;
		public int bIsDeprecated;
		public int bIsGesture;
		public EInputAxisType AxisType;

		public string LongDisplayName;
		public string ShortDisplayName;

		public FKeyDetails(FKey InKey, string InLongDisplayName, string InShortDisplayName = "", int InKeyFlags = 0)
		{
			this.Key = InKey;
			this.LongDisplayName = InLongDisplayName;
			this.ShortDisplayName = InShortDisplayName;

			this.bIsGamepadKey = (InKeyFlags & (int)EKeyFlags.GamepadKey) != 0 ? 1 : 0;
			this.bIsModifierKey = (InKeyFlags & (int)EKeyFlags.ModifierKey) != 0 ? 1 : 0;
			this.bIsTouch = (InKeyFlags & (int)EKeyFlags.Touch) != 0 ? 1 : 0;
			this.bIsMouseButton = (InKeyFlags & (int)EKeyFlags.MouseButton) != 0 ? 1 : 0;
			this.bIsBindableInBlueprints = (~InKeyFlags & (int)EKeyFlags.NotBlueprintBindableKey) != 0 && (~InKeyFlags & (int)EKeyFlags.Deprecated) != 0 ? 1 : 0;
			this.bShouldUpdateAxisWithoutSamples = (InKeyFlags & (int)EKeyFlags.UpdateAxisWithoutSamples) != 0 ? 1 : 0;
			this.bIsbindableToActions = (~InKeyFlags & (int)EKeyFlags.NotActionBindableKey) != 0 && (~InKeyFlags & (int)EKeyFlags.Deprecated) != 0 ? 1 : 0;
			this.bIsDeprecated = (InKeyFlags & (int)EKeyFlags.Deprecated) != 0 ? 1 : 0;
			this.bIsGesture = (InKeyFlags & (int)EKeyFlags.Gesture) != 0 ? 1 : 0;

			// Set MenuCategory based on above flags
			if(bIsGamepadKey == 1)
			{
				this.MenuCategory = "Gamepad";
			}
			else if(bIsMouseButton == 1)
			{
				this.MenuCategory = "Mouse";
			}
			else
			{
				this.MenuCategory = "Keyboard";
			}

			// Defaulting the Axis stuff to null for now
			PairedAxisKey = null;
			AxisType = 0;
		}

		public FKey GetKey()
		{
			return this.Key;
		}

		public string GetLongDisplayName()
		{
			return this.LongDisplayName;
		}

		public string GetShortDisplayName()
		{
			return this.ShortDisplayName;
		}

		public bool IsGamepadKey()
		{
			return this.bIsGamepadKey == 1;
		}

		public bool IsModifierKey()
		{
			return this.bIsModifierKey == 1;
		}

		public bool IsTouch()
		{
			return this.bIsTouch == 1;
		}

		public bool IsMouseButton()
		{
			return this.bIsMouseButton == 1;
		}

		public bool IsGesture()
		{
			return this.bIsGesture == 1;
		}

		public string GetMenuCategory()
		{
			return this.MenuCategory;
		}

		public int GetPairedAxis()
		{
			return this.PairedAxis;
		}

		public FKey GetPairedAxisKey()
		{
			return this.PairedAxisKey;
		}

		public EInputAxisType GetAxisType()
		{
			return this.AxisType;
		}

	}

	public struct EKeys
	{
		public static FKey A = new FKey("A");
		public static FKey B = new FKey("B");
		public static FKey C = new FKey("C");
		public static FKey D = new FKey("D");
		public static FKey E = new FKey("E");
		public static FKey F = new FKey("F");
		public static FKey G = new FKey("G");
		public static FKey H = new FKey("H");
		public static FKey I = new FKey("I");
		public static FKey J = new FKey("J");
		public static FKey K = new FKey("K");
		public static FKey L = new FKey("L");
		public static FKey M = new FKey("M");
		public static FKey N = new FKey("N");
		public static FKey O = new FKey("O");
		public static FKey P = new FKey("P");
		public static FKey Q = new FKey("Q");
		public static FKey R = new FKey("R");
		public static FKey S = new FKey("S");
		public static FKey T = new FKey("T");
		public static FKey U = new FKey("U");
		public static FKey V = new FKey("V");
		public static FKey W = new FKey("W");
		public static FKey X = new FKey("X");
		public static FKey Y = new FKey("Y");
		public static FKey Z = new FKey("Z");
		public static FKey MouseX = new FKey("MouseX");
		public static FKey MouseY = new FKey("MouseY");
		public static FKey Mouse2D = new FKey("Mouse2D");
		public static FKey MouseScrollUp = new FKey("MouseScrollUp");
		public static FKey MouseScrollDown = new FKey("MouseScrollDown");
		public static FKey MouseWheelAxis = new FKey("MouseWheelAxis");
		public static FKey LeftMouseButton = new FKey("LeftMouseButton");
		public static FKey RightMouseButton = new FKey("RightMouseButton");
		public static FKey MiddleMouseButton = new FKey("MiddleMouseButton");
		public static FKey ThumbMouseButton = new FKey("ThumbMouseButton");
		public static FKey ThumbMouseButton2 = new FKey("ThumbMouseButton2");

		// Separate commands here for Mac
		public static FKey BackSpace = new FKey("Backspace");
		public static FKey Delete = new FKey("Delete");
		public static FKey Tab = new FKey("Tab");
		public static FKey Enter = new FKey("Enter");
		public static FKey Pause = new FKey("Pause");
		public static FKey CapsLock = new FKey("CapsLock");
		public static FKey Escape = new FKey("Escape");
		public static FKey SpaceBar = new FKey("SpaceBar");
		public static FKey PageUp = new FKey("PageUp");
		public static FKey PageDown = new FKey("PageDown");
		public static FKey End = new FKey("End");
		public static FKey Home = new FKey("Home");
		public static FKey Left = new FKey("Left");
		public static FKey Up = new FKey("Up");
		public static FKey Right = new FKey("Right");
		public static FKey Down = new FKey("Down");
		public static FKey Insert = new FKey("Insert");
		public static FKey Zero = new FKey("0");
		public static FKey One = new FKey("1");
		public static FKey Two = new FKey("2");
		public static FKey Three = new FKey("3");
		public static FKey Four = new FKey("4");
		public static FKey Five = new FKey("5");
		public static FKey Six = new FKey("6");
		public static FKey Seven = new FKey("7");
		public static FKey Eight = new FKey("8");
		public static FKey Nine = new FKey("9");
		public static FKey NumPadZero = new FKey("NumPadZero");
		public static FKey NumPadOne = new FKey("NumPadOne");
		public static FKey NumPadTwo = new FKey("NumPadTwo");
		public static FKey NumPadThree = new FKey("NumPadThree");
		public static FKey NumPadFour = new FKey("NumPadFour");
		public static FKey NumPadFive = new FKey("NumPadFive");
		public static FKey NumPadSix = new FKey("NumPadSix");
		public static FKey NumPadSeven = new FKey("NumPadSeven");
		public static FKey NumPadEight = new FKey("NumPadEight");
		public static FKey NumPadNine = new FKey("NumPadNine");
		public static FKey F1 = new FKey("F1");
		public static FKey F2 = new FKey("F2");
		public static FKey F3 = new FKey("F3");
		public static FKey F4 = new FKey("F4");
		public static FKey F5 = new FKey("F5");
		public static FKey F6 = new FKey("F6");
		public static FKey F7 = new FKey("F7");
		public static FKey F8 = new FKey("F8");
		public static FKey F9 = new FKey("F9");
		public static FKey F10 = new FKey("F10");
		public static FKey F11 = new FKey("F11");
		public static FKey F12 = new FKey("F12");
		public static FKey NumLock = new FKey("NumLock");
		public static FKey LeftShift = new FKey("LeftShift");
		public static FKey RightShift = new FKey("RightShift");
		public static FKey LeftControl = new FKey("LeftControl");
		public static FKey RightControl = new FKey("RightControl");
		public static FKey LeftAlt = new FKey("LeftAlt");
		public static FKey RightAlt = new FKey("RightAlt");
		public static FKey LeftCommand = new FKey("LeftCommand");
		public static FKey RightCommand = new FKey("RightCommand");
		public static FKey Semicolon = new FKey("Semicolon");
		public static FKey EqualsKey = new FKey("Equals");
		public static FKey Comma = new FKey("Comma");
		public static FKey Underscore = new FKey("Underscore");
		public static FKey Hyphen = new FKey("Hyphen");
		public static FKey Period = new FKey("Period");
		public static FKey Slash = new FKey("Slash");
		public static FKey Tilde = new FKey("`");
		public static FKey LeftBracket = new FKey("LeftBracket");
		public static FKey Backslash = new FKey("Backslash");
		public static FKey RightBracket = new FKey("RightBracket");
		public static FKey Apostrophe = new FKey("Apostrophe");
		public static FKey Ampersand = new FKey("Ampersand");
		public static FKey Asterix = new FKey("Asterix");
		public static FKey Caret = new FKey("Caret");
		public static FKey Colon = new FKey("Colon");
		public static FKey Dollar = new FKey("Dollar");
		public static FKey Exclamation = new FKey("Exclamation");
		public static FKey LeftParantheses = new FKey("LeftParantheses");
		public static FKey RightParantheses = new FKey("RightParantheses");
		public static FKey Quote = new FKey("Quote");

		// In C++ this uses FString::Chr
		public static FKey A_AccentGrave = new FKey($"{(char)224}");
		public static FKey E_AccentGrave = new FKey($"{(char)232}");
		public static FKey E_AccentAigu = new FKey($"{(char)233}");
		public static FKey C_Cedille = new FKey($"{(char)231}");
		public static FKey Section = new FKey($"{(char)167}");

		// Gamepad Axis Stuff
		// These are still mostly untested and are here for parity until we can confirm how to properly do analog inputs
		public static FKey Gamepad_Left2D = new FKey("Gamepad_Left2D");
		public static FKey Gamepad_LeftX = new FKey("Gamepad_LeftX");
		public static FKey Gamepad_LeftY = new FKey("Gamepad_LeftY");
		public static FKey Gamepad_Right2D = new FKey("Gamepad_Right2D");
		public static FKey Gamepad_RightX = new FKey("Gamepad_RightX");
		public static FKey Gamepad_RightY = new FKey("Gamepad_RightY");
		public static FKey Gamepad_Special_Left_X = new FKey("Gamepad_Special_Left_X");
		public static FKey Gamepad_Special_Left_Y = new FKey("Gamepad_Special_Left_Y");
		public static FKey Gamepad_LeftTrigger = new FKey("Gamepad_LeftTrigger");
		public static FKey Gamepad_RightTrigger = new FKey("Gamepad_RightTrigger");
		public static FKey Gamepad_LeftThumbstick = new FKey("Gamepad_LeftThumbstick");
		public static FKey Gamepad_RightThumbstick = new FKey("Gamepad_RightThumbstick");
		public static FKey Gamepad_LeftTriggerAxis = new FKey("Gamepad_LeftTriggerAxis");
		public static FKey Gamepad_RightTriggerAxis = new FKey("Gamepad_RightTriggerAxis");

		// Gamepad Generic
		public static FKey Gamepad_DPad_Up = new FKey("Gamepad_DPad_Up");
		public static FKey Gamepad_DPad_Down = new FKey("Gamepad_DPad_Down");
		public static FKey Gamepad_DPad_Right = new FKey("Gamepad_DPad_Right");
		public static FKey Gamepad_DPad_Left = new FKey("Gamepad_DPad_Left");
		public static FKey Gamepad_LeftStick_Up = new FKey("Gamepad_LeftStick_Up");
		public static FKey Gamepad_LeftStick_Down = new FKey("Gamepad_LeftStick_Down");
		public static FKey Gamepad_LeftStick_Right = new FKey("Gamepad_LeftStick_Right");
		public static FKey Gamepad_LeftStick_Left = new FKey("Gamepad_LeftStick_Left");
		public static FKey Gamepad_RightStick_Up = new FKey("Gamepad_RightStick_Up");
		public static FKey Gamepad_RightStick_Down = new FKey("Gamepad_RightStick_Down");
		public static FKey Gamepad_RightStick_Right = new FKey("Gamepad_RightStick_Right");
		public static FKey Gamepad_RightStick_Left = new FKey("Gamepad_RightStick_Left");
		public static FKey Gamepad_Special_Left = new FKey("Gamepad_Special_Left");
		public static FKey Gamepad_Special_Right = new FKey("Gamepad_Special_Right");
		public static FKey Gamepad_FaceButton_Bottom = new FKey("Gamepad_FaceButton_Bottom");
		public static FKey Gamepad_FaceButton_Right = new FKey("Gamepad_FaceButton_Right");
		public static FKey Gamepad_FaceButton_Left = new FKey("Gamepad_FaceButton_Left");
		public static FKey Gamepad_FaceButton_Top = new FKey("Gamepad_FaceButton_Top");
		public static FKey Gamepad_LeftShoulder = new FKey("Gamepad_LeftShoulder");
		public static FKey Gamepad_RightShoulder = new FKey("Gamepad_RightShoulder");

		// Motion Controls
		public static FKey Tilt = new FKey("Tilt");
		public static FKey RotationRate = new FKey("RotationRate");
		public static FKey Gravity = new FKey("Gravity");
		public static FKey Acceleration = new FKey("Acceleration");

		// Gesture Controls
		public static FKey Gesture_Pinch = new FKey("Gesture_Pinch");
		public static FKey Gesture_Flick = new FKey("Gesture_Flick");
		public static FKey Gesture_Rotate = new FKey("Gesture_Rotate");

		// Steam Controls
		public static FKey Steam_Touch_0 = new FKey("Steam_Touch_0");
		public static FKey Steam_Touch_1 = new FKey("Steam_Touch_1");
		public static FKey Steam_Touch_2 = new FKey("Steam_Touch_2");
		public static FKey Steam_Touch_3 = new FKey("Steam_Touch_3");
		public static FKey Steam_Back_Left = new FKey("Steam_Back_Left");
		public static FKey Steam_Back_Right = new FKey("Steam_Back_Right");

		// Xbox One Global Speech commands
		public static FKey Global_Menu = new FKey("Global_Menu");
		public static FKey Global_View = new FKey("Global_View");
		public static FKey Global_Pause = new FKey("Global_Pause");
		public static FKey Global_Play = new FKey("Global_Play");
		public static FKey Global_Back = new FKey("Global_Back");

		// Android-specific Controls
		public static FKey Android_Back = new FKey("Android_Back");
		public static FKey Android_Volume_Up = new FKey("Android_Volume_Up");
		public static FKey Android_Volume_Down = new FKey("Android_Volume_Down");
		public static FKey Android_Menu = new FKey("Android_Menu");

		// Vive Controls
		public static FKey Vive_Left_Grip_Click = new FKey("Vive_Left_Grip_Click");
		public static FKey Vive_Left_Menu_Click = new FKey("Vive_Left_Menu_Click");
		public static FKey Vive_Left_Trigger_Click = new FKey("Vive_Left_Trigger_Click");
		public static FKey Vive_Left_Trigger_Axis = new FKey("Vive_Left_Trigger_Axis");
		public static FKey Vive_Left_Trackpad_2D = new FKey("Vive_Left_Trackpad_2D");
		public static FKey Vive_Left_Trackpad_X = new FKey("Vive_Left_Trackpad_X");
		public static FKey Vive_Left_Trackpad_Y = new FKey("Vive_Left_Trackpad_Y");
		public static FKey Vive_Left_Trackpad_Click = new FKey("Vive_Left_Trackpad_Click");
		public static FKey Vive_Left_Trackpad_Touch = new FKey("Vive_Left_Trackpad_Touch");
		public static FKey Vive_Left_Trackpad_Up = new FKey("Vive_Left_Trackpad_Up");
		public static FKey Vive_Left_Trackpad_Down = new FKey("Vive_Left_Trackpad_Down");
		public static FKey Vive_Left_Trackpad_Left = new FKey("Vive_Left_Trackpad_Left");
		public static FKey Vive_Left_Trackpad_Right = new FKey("Vive_Left_Trackpad_Right");
		public static FKey Vive_Right_Grip_Click = new FKey("Vive_Right_Grip_Click");
		public static FKey Vive_Right_Menu_Click = new FKey("Vive_Right_Menu_Click");
		public static FKey Vive_Right_Trigger_Click = new FKey("Vive_Right_Trigger_Click");
		public static FKey Vive_Right_Trigger_Axis = new FKey("Vive_Right_Trigger_Axis");
		public static FKey Vive_Right_Trackpad_2D = new FKey("Vive_Right_Trackpad_2D");
		public static FKey Vive_Right_Trackpad_X = new FKey("Vive_Right_Trackpad_X");
		public static FKey Vive_Right_Trackpad_Y = new FKey("Vive_Right_Trackpad_Y");
		public static FKey Vive_Right_Trackpad_Click = new FKey("Vive_Right_Trackpad_Click");
		public static FKey Vive_Right_Trackpad_Touch = new FKey("Vive_Right_Trackpad_Touch");
		public static FKey Vive_Right_Trackpad_Up = new FKey("Vive_Right_Trackpad_Up");
		public static FKey Vive_Right_Trackpad_Down = new FKey("Vive_Right_Trackpad_Down");
		public static FKey Vive_Right_Trackpad_Left = new FKey("Vive_Right_Trackpad_Left");
		public static FKey Vive_Right_Trackpad_Right = new FKey("Vive_Right_Trackpad_Right");

		// Mixed Reality Controls
		public static FKey MixedReality_Left_Menu_Click = new FKey("MixedReality_Left_Menu_Click");
		public static FKey MixedReality_Left_Grip_Click = new FKey("MixedReality_Left_Grip_Click");
		public static FKey MixedReality_Left_Trigger_Click = new FKey("MixedReality_Left_Trigger_Click");
		public static FKey MixedReality_Left_Trigger_Axis = new FKey("MixedReality_Left_Trigger_Axis");
		public static FKey MixedReality_Left_Thumbstick_2D = new FKey("MixedReality_Left_Thumbstick_2D");
		public static FKey MixedReality_Left_Thumbstick_X = new FKey("MixedReality_Left_Thumbstick_X");
		public static FKey MixedReality_Left_Thumbstick_Y = new FKey("MixedReality_Left_Thumbstick_Y");
		public static FKey MixedReality_Left_Thumbstick_Click = new FKey("MixedReality_Left_Thumbstick_Click");
		public static FKey MixedReality_Left_Thumbstick_Up = new FKey("MixedReality_Left_Thumbstick_Up");
		public static FKey MixedReality_Left_Thumbstick_Down = new FKey("MixedReality_Left_Thumbstick_Down");
		public static FKey MixedReality_Left_Thumbstick_Left = new FKey("MixedReality_Left_Thumbstick_Left");
		public static FKey MixedReality_Left_Thumbstick_Right = new FKey("MixedReality_Left_Thumbstick_Right");
		public static FKey MixedReality_Left_Trackpad_2D = new FKey("MixedReality_Left_Trackpad_2D");
		public static FKey MixedReality_Left_Trackpad_X = new FKey("MixedReality_Left_Trackpad_X");
		public static FKey MixedReality_Left_Trackpad_Y = new FKey("MixedReality_Left_Trackpad_Y");
		public static FKey MixedReality_Left_Trackpad_Click = new FKey("MixedReality_Left_Trackpad_Click");
		public static FKey MixedReality_Left_Trackpad_Touch = new FKey("MixedReality_Left_Trackpad_Touch");
		public static FKey MixedReality_Left_Trackpad_Up = new FKey("MixedReality_Left_Trackpad_Up");
		public static FKey MixedReality_Left_Trackpad_Down = new FKey("MixedReality_Left_Trackpad_Down");
		public static FKey MixedReality_Left_Trackpad_Left = new FKey("MixedReality_Left_Trackpad_Left");
		public static FKey MixedReality_Left_Trackpad_Right = new FKey("MixedReality_Left_Trackpad_Right");
		public static FKey MixedReality_Right_Menu_Click = new FKey("MixedReality_Right_Menu_Click");
		public static FKey MixedReality_Right_Grip_Click = new FKey("MixedReality_Right_Grip_Click");
		public static FKey MixedReality_Right_Trigger_Click = new FKey("MixedReality_Right_Trigger_Click");
		public static FKey MixedReality_Right_Trigger_Axis = new FKey("MixedReality_Right_Trigger_Axis");
		public static FKey MixedReality_Right_Thumbstick_2D = new FKey("MixedReality_Right_Thumbstick_2D");
		public static FKey MixedReality_Right_Thumbstick_X = new FKey("MixedReality_Right_Thumbstick_X");
		public static FKey MixedReality_Right_Thumbstick_Y = new FKey("MixedReality_Right_Thumbstick_Y");
		public static FKey MixedReality_Right_Thumbstick_Click = new FKey("MixedReality_Right_Thumbstick_Click");
		public static FKey MixedReality_Right_Thumbstick_Up = new FKey("MixedReality_Right_Thumbstick_Up");
		public static FKey MixedReality_Right_Thumbstick_Down = new FKey("MixedReality_Right_Thumbstick_Down");
		public static FKey MixedReality_Right_Thumbstick_Left = new FKey("MixedReality_Right_Thumbstick_Left");
		public static FKey MixedReality_Right_Thumbstick_Right = new FKey("MixedReality_Right_Thumbstick_Right");
		public static FKey MixedReality_Right_Trackpad_2D = new FKey("MixedReality_Right_Trackpad_2D");
		public static FKey MixedReality_Right_Trackpad_X = new FKey("MixedReality_Right_Trackpad_X");
		public static FKey MixedReality_Right_Trackpad_Y = new FKey("MixedReality_Right_Trackpad_Y");
		public static FKey MixedReality_Right_Trackpad_Click = new FKey("MixedReality_Right_Trackpad_Click");
		public static FKey MixedReality_Right_Trackpad_Touch = new FKey("MixedReality_Right_Trackpad_Touch");
		public static FKey MixedReality_Right_Trackpad_Up = new FKey("MixedReality_Right_Trackpad_Up");
		public static FKey MixedReality_Right_Trackpad_Down = new FKey("MixedReality_Right_Trackpad_Down");
		public static FKey MixedReality_Right_Trackpad_Left = new FKey("MixedReality_Right_Trackpad_Left");
		public static FKey MixedReality_Right_Trackpad_Right = new FKey("MixedReality_Right_Trackpad_Right");

		// Oculus Touch controls
		public static FKey OculusTouch_Left_X_Click = new FKey("OculusTouch_Left_X_Click");
		public static FKey OculusTouch_Left_Y_Click = new FKey("OculusTouch_Left_Y_Click");
		public static FKey OculusTouch_Left_X_Touch = new FKey("OculusTouch_Left_X_Touch");
		public static FKey OculusTouch_Left_Y_Touch = new FKey("OculusTouch_Left_Y_Touch");
		public static FKey OculusTouch_Left_Menu_Click = new FKey("OculusTouch_Left_Menu_Click");
		public static FKey OculusTouch_Left_Grip_Click = new FKey("OculusTouch_Left_Grip_Click");
		public static FKey OculusTouch_Left_Grip_Axis = new FKey("OculusTouch_Left_Grip_Axis");
		public static FKey OculusTouch_Left_Trigger_Click = new FKey("OculusTouch_Left_Trigger_Click");
		public static FKey OculusTouch_Left_Trigger_Axis = new FKey("OculusTouch_Left_Trigger_Axis");
		public static FKey OculusTouch_Left_Trigger_Touch = new FKey("OculusTouch_Left_Trigger_Touch");
		public static FKey OculusTouch_Left_Thumbstick_2D = new FKey("OculusTouch_Left_Thumbstick_2D");
		public static FKey OculusTouch_Left_Thumbstick_X = new FKey("OculusTouch_Left_Thumbstick_X");
		public static FKey OculusTouch_Left_Thumbstick_Y = new FKey("OculusTouch_Left_Thumbstick_Y");
		public static FKey OculusTouch_Left_Thumbstick_Click = new FKey("OculusTouch_Left_Thumbstick_Click");
		public static FKey OculusTouch_Left_Thumbstick_Touch = new FKey("OculusTouch_Left_Thumbstick_Touch");
		public static FKey OculusTouch_Left_Thumbstick_Up = new FKey("OculusTouch_Left_Thumbstick_Up");
		public static FKey OculusTouch_Left_Thumbstick_Down = new FKey("OculusTouch_Left_Thumbstick_Down");
		public static FKey OculusTouch_Left_Thumbstick_Left = new FKey("OculusTouch_Left_Thumbstick_Left");
		public static FKey OculusTouch_Left_Thumbstick_Right = new FKey("OculusTouch_Left_Thumbstick_Right");
		public static FKey OculusTouch_Right_A_Click = new FKey("OculusTouch_Right_A_Click");
		public static FKey OculusTouch_Right_B_Click = new FKey("OculusTouch_Right_B_Click");
		public static FKey OculusTouch_Right_A_Touch = new FKey("OculusTouch_Right_A_Touch");
		public static FKey OculusTouch_Right_B_Touch = new FKey("OculusTouch_Right_B_Touch");
		public static FKey OculusTouch_Right_Grip_Click = new FKey("OculusTouch_Right_Grip_Click");
		public static FKey OculusTouch_Right_Grip_Axis = new FKey("OculusTouch_Right_Grip_Axis");
		public static FKey OculusTouch_Right_Trigger_Click = new FKey("OculusTouch_Right_Trigger_Click");
		public static FKey OculusTouch_Right_Trigger_Axis = new FKey("OculusTouch_Right_Trigger_Axis");
		public static FKey OculusTouch_Right_Trigger_Touch = new FKey("OculusTouch_Right_Trigger_Touch");
		public static FKey OculusTouch_Right_Thumbstick_2D = new FKey("OculusTouch_Right_Thumbstick_2D");
		public static FKey OculusTouch_Right_Thumbstick_X = new FKey("OculusTouch_Right_Thumbstick_X");
		public static FKey OculusTouch_Right_Thumbstick_Y = new FKey("OculusTouch_Right_Thumbstick_Y");
		public static FKey OculusTouch_Right_Thumbstick_Click = new FKey("OculusTouch_Right_Thumbstick_Click");
		public static FKey OculusTouch_Right_Thumbstick_Touch = new FKey("OculusTouch_Right_Thumbstick_Touch");
		public static FKey OculusTouch_Right_Thumbstick_Up = new FKey("OculusTouch_Right_Thumbstick_Up");
		public static FKey OculusTouch_Right_Thumbstick_Down = new FKey("OculusTouch_Right_Thumbstick_Down");
		public static FKey OculusTouch_Right_Thumbstick_Left = new FKey("OculusTouch_Right_Thumbstick_Left");
		public static FKey OculusTouch_Right_Thumbstick_Right = new FKey("OculusTouch_Right_Thumbstick_Right");

		// Valve Index Controls
		public static FKey ValveIndex_Left_A_Click = new FKey("ValveIndex_Left_A_Click");
		public static FKey ValveIndex_Left_B_Click = new FKey("ValveIndex_Left_B_Click");
		public static FKey ValveIndex_Left_A_Touch = new FKey("ValveIndex_Left_A_Touch");
		public static FKey ValveIndex_Left_B_Touch = new FKey("ValveIndex_Left_B_Touch");
		public static FKey ValveIndex_Left_Grip_Axis = new FKey("ValveIndex_Left_Grip_Axis");
		public static FKey ValveIndex_Left_Grip_Force = new FKey("ValveIndex_Left_Grip_Force");
		public static FKey ValveIndex_Left_Trigger_Click = new FKey("ValveIndex_Left_Trigger_Click");
		public static FKey ValveIndex_Left_Trigger_Axis = new FKey("ValveIndex_Left_Trigger_Axis");
		public static FKey ValveIndex_Left_Trigger_Touch = new FKey("ValveIndex_Left_Trigger_Touch");
		public static FKey ValveIndex_Left_Thumbstick_2D = new FKey("ValveIndex_Left_Thumbstick_2D");
		public static FKey ValveIndex_Left_Thumbstick_X = new FKey("ValveIndex_Left_Thumbstick_X");
		public static FKey ValveIndex_Left_Thumbstick_Y = new FKey("ValveIndex_Left_Thumbstick_Y");
		public static FKey ValveIndex_Left_Thumbstick_Click = new FKey("ValveIndex_Left_Thumbstick_Click");
		public static FKey ValveIndex_Left_Thumbstick_Touch = new FKey("ValveIndex_Left_Thumbstick_Touch");
		public static FKey ValveIndex_Left_Thumbstick_Up = new FKey("ValveIndex_Left_Thumbstick_Up");
		public static FKey ValveIndex_Left_Thumbstick_Down = new FKey("ValveIndex_Left_Thumbstick_Down");
		public static FKey ValveIndex_Left_Thumbstick_Left = new FKey("ValveIndex_Left_Thumbstick_Left");
		public static FKey ValveIndex_Left_Thumbstick_Right = new FKey("ValveIndex_Left_Thumbstick_Right");
		public static FKey ValveIndex_Left_Trackpad_2D = new FKey("ValveIndex_Left_Trackpad_2D");
		public static FKey ValveIndex_Left_Trackpad_X = new FKey("ValveIndex_Left_Trackpad_X");
		public static FKey ValveIndex_Left_Trackpad_Y = new FKey("ValveIndex_Left_Trackpad_Y");
		public static FKey ValveIndex_Left_Trackpad_Force = new FKey("ValveIndex_Left_Trackpad_Force");
		public static FKey ValveIndex_Left_Trackpad_Touch = new FKey("ValveIndex_Left_Trackpad_Touch");
		public static FKey ValveIndex_Left_Trackpad_Up = new FKey("ValveIndex_Left_Trackpad_Up");
		public static FKey ValveIndex_Left_Trackpad_Down = new FKey("ValveIndex_Left_Trackpad_Down");
		public static FKey ValveIndex_Left_Trackpad_Left = new FKey("ValveIndex_Left_Trackpad_Left");
		public static FKey ValveIndex_Left_Trackpad_Right = new FKey("ValveIndex_Left_Trackpad_Right");
		public static FKey ValveIndex_Right_A_Click = new FKey("ValveIndex_Right_A_Click");
		public static FKey ValveIndex_Right_B_Click = new FKey("ValveIndex_Right_B_Click");
		public static FKey ValveIndex_Right_A_Touch = new FKey("ValveIndex_Right_A_Touch");
		public static FKey ValveIndex_Right_B_Touch = new FKey("ValveIndex_Right_B_Touch");
		public static FKey ValveIndex_Right_Grip_Axis = new FKey("ValveIndex_Right_Grip_Axis");
		public static FKey ValveIndex_Right_Grip_Force = new FKey("ValveIndex_Right_Grip_Force");
		public static FKey ValveIndex_Right_Trigger_Click = new FKey("ValveIndex_Right_Trigger_Click");
		public static FKey ValveIndex_Right_Trigger_Axis = new FKey("ValveIndex_Right_Trigger_Axis");
		public static FKey ValveIndex_Right_Trigger_Touch = new FKey("ValveIndex_Right_Trigger_Touch");
		public static FKey ValveIndex_Right_Thumbstick_2D = new FKey("ValveIndex_Right_Thumbstick_2D");
		public static FKey ValveIndex_Right_Thumbstick_X = new FKey("ValveIndex_Right_Thumbstick_X");
		public static FKey ValveIndex_Right_Thumbstick_Y = new FKey("ValveIndex_Right_Thumbstick_Y");
		public static FKey ValveIndex_Right_Thumbstick_Click = new FKey("ValveIndex_Right_Thumbstick_Click");
		public static FKey ValveIndex_Right_Thumbstick_Touch = new FKey("ValveIndex_Right_Thumbstick_Touch");
		public static FKey ValveIndex_Right_Thumbstick_Up = new FKey("ValveIndex_Right_Thumbstick_Up");
		public static FKey ValveIndex_Right_Thumbstick_Down = new FKey("ValveIndex_Right_Thumbstick_Down");
		public static FKey ValveIndex_Right_Thumbstick_Left = new FKey("ValveIndex_Right_Thumbstick_Left");
		public static FKey ValveIndex_Right_Thumbstick_Right = new FKey("ValveIndex_Right_Thumbstick_Right");
		public static FKey ValveIndex_Right_Trackpad_2D = new FKey("ValveIndex_Right_Trackpad_2D");
		public static FKey ValveIndex_Right_Trackpad_X = new FKey("ValveIndex_Right_Trackpad_X");
		public static FKey ValveIndex_Right_Trackpad_Y = new FKey("ValveIndex_Right_Trackpad_Y");
		public static FKey ValveIndex_Right_Trackpad_Force = new FKey("ValveIndex_Right_Trackpad_Force");
		public static FKey ValveIndex_Right_Trackpad_Touch = new FKey("ValveIndex_Right_Trackpad_Touch");
		public static FKey ValveIndex_Right_Trackpad_Up = new FKey("ValveIndex_Right_Trackpad_Up");
		public static FKey ValveIndex_Right_Trackpad_Down = new FKey("ValveIndex_Right_Trackpad_Down");
		public static FKey ValveIndex_Right_Trackpad_Left = new FKey("ValveIndex_Right_Trackpad_Left");
		public static FKey ValveIndex_Right_Trackpad_Right = new FKey("ValveIndex_Right_Trackpad_Right");
		
		public EKeys()
		{
			UnrealTargetPlatform.TryParse(Globals.Params.ParseValue("Platform", "Win64"), out UnrealTargetPlatform Platform);
			if (Platform == UnrealTargetPlatform.Mac)
			{
				BackSpace = new FKey("Delete");
				Delete = new FKey("ForwardDelete");
			}

		}

	}

	// Same as EInputEvent from EngineBaseTypes.h
	public enum EInputEvent
	{
		IE_Pressed,
		IE_Released,
		IE_Repeat,
		IE_DoubleClick,
		IE_Axis,
		IE_Max

	}

	public class InputAction(FKey InKey, EInputEvent InInputEvent, double InXDelta = 0, double InYDelta = 0, bool InTapped = false)
	{
		[JsonPropertyName("inputkey")]
		public FKey Key { get; } = InKey;

		[JsonPropertyName("inputevent")]
		public EInputEvent KeyAction { get; } = InInputEvent;

		[JsonPropertyName("xdelta")]
		public double XDelta { get; } = InXDelta;

		[JsonPropertyName("ydelta")]
		public double YDelta { get;  } = InYDelta;

		[JsonPropertyName("tapped")]
		public bool Tapped { get;  } = InTapped;

		/// <summary>
		/// Converts the instance's properties into a dictionary suitable for RPC method arguments.
		/// Properties decorated with <see cref="JsonPropertyNameAttribute"/> are serialized and added to the dictionary.
		/// </summary>
		/// <returns>A dictionary with JSON property names as keys and serialized property values as JSON strings.</returns>
		/// <remarks>
		/// If this pattern is needed across multiple classes, consider creating an extension method
		/// on <see cref="object"/> to avoid code duplication.
		/// </remarks>
		public Dictionary<string, object> AsRpcArgs()
		{
			Dictionary<string, object> Args = [];
			PropertyInfo[] Properties = GetType().GetProperties();

			foreach (var Property in Properties)
			{
				JsonPropertyNameAttribute JsonAttribute = Property.GetCustomAttribute<JsonPropertyNameAttribute>();
				if (JsonAttribute != null)
				{
					object Value = Property.GetValue(this);
					Args[JsonAttribute.Name] = JsonSerializer.Serialize(Value);
				}
			}

			return Args;
		}
	}
}