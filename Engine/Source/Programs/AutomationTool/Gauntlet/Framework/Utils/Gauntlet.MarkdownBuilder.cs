// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Linq;
using System.Text;
using System.Threading.Tasks;

namespace Gauntlet
{
	/// <summary>
	/// Helper class to generate text formatted as markdown
	/// </summary>
	public class MarkdownBuilder
	{
		protected StringBuilder SB;

		public MarkdownBuilder()
		{
			SB = new StringBuilder();
		}

		public MarkdownBuilder(string Contents)
		{
			SB = new StringBuilder(Contents);
		}

		/// <summary>
		/// Returns our formatted text
		/// </summary>
		/// <returns></returns>
		public override string ToString()
		{
			return SB.ToString();
		}

		/// <summary>
		/// Returns true if the correct body ends with a new line
		/// </summary>
		public bool EndsWithNewLine
		{
			get { return SB.Length == 0 || SB[SB.Length-1] == '\n'; }
		}
		
		/// <summary>
		/// Ensures any text after this starts on a new line
		/// </summary>
		/// <returns></returns>
		public MarkdownBuilder EnsureEndsWithNewLine()
		{
			if (SB.Length > 0 && !EndsWithNewLine)
			{
				NewLine();
			}

			return this;
		}


		/// <summary>
		///Insert an H1 Header
		/// </summary>
		/// <param name="Text"></param>
		/// <returns></returns>
		public MarkdownBuilder H1(string Text)
		{
			EnsureEndsWithNewLine();
			SB.AppendFormat("# {0}", Text);
			return this;
		}

		/// <summary>
		/// Insert an H2 header
		/// </summary>
		/// <param name="Text"></param>
		/// <returns></returns>
		public MarkdownBuilder H2(string Text)
		{
			EnsureEndsWithNewLine();
			SB.AppendFormat("## {0}", Text);
			return this;
		}

		/// <summary>
		/// Insert an H3 header
		/// </summary>
		/// <param name="Text"></param>
		/// <returns></returns>
		public MarkdownBuilder H3(string Text)
		{
			EnsureEndsWithNewLine();
			SB.AppendFormat("### {0}", Text);
			return this;
		}

		/// <summary>
		/// Insert an H4 Header
		/// </summary>
		/// <param name="Text"></param>
		/// <returns></returns>
		public MarkdownBuilder H4(string Text)
		{
			EnsureEndsWithNewLine();
			SB.AppendFormat("##### {0}", Text);
			return this;
		}

		/// <summary>
		/// Insert an H5 header
		/// </summary>
		/// <param name="Text"></param>
		/// <returns></returns>
		public MarkdownBuilder H5(string Text)
		{
			EnsureEndsWithNewLine();
			SB.AppendFormat("###### {0}\n", Text);
			return this;
		}

		/// <summary>
		/// Insert a newline
		/// </summary>
		/// <returns></returns>
		public MarkdownBuilder NewLine()
		{
			SB.AppendLine();
			return this;
		}

		/// <summary>
		/// Insert an H5 header
		/// </summary>
		/// <param name="Count"></param>
		/// <returns></returns>
		public MarkdownBuilder HorizontalLine(int Count=80)
		{
			EnsureEndsWithNewLine();
			for (int i = 0; i < Count; i++)
			{
				SB.Append("-");
			}
			SB.AppendLine();
			return this;
		}

		/// <summary>
		/// Insert a paragraph block
		/// </summary>
		/// <param name="Text"></param>
		/// <returns></returns>
		public MarkdownBuilder Paragraph(string Text)
		{
			EnsureEndsWithNewLine();
			SB.Append(Text);
			NewLine();
			NewLine();
			return this;
		}

		/// <summary>
		/// Insert an ordered (numbered) list
		/// </summary>
		/// <param name="Items"></param>
		/// <returns></returns>
		public MarkdownBuilder OrderedList(IEnumerable<string> Items)
		{
			EnsureEndsWithNewLine();
			int Index = 1;

			foreach (string Item in Items)
			{
				SB.AppendFormat("{0}. {1}", Index++, Item);
				SB.AppendLine();
			}
			NewLine();

			return this;
		}

		/// <summary>
		/// Insert an unordered (bulleted) list
		/// </summary>
		/// <param name="Items"></param>
		/// <returns></returns>
		public MarkdownBuilder UnorderedList(IEnumerable<string> Items)
		{
			EnsureEndsWithNewLine();
			foreach (string Item in Items)
			{
				if (Item != null)
				{
					SB.AppendFormat("* {0}", Item);
					SB.AppendLine();
				}
			}
			NewLine();

			return this;
		}

		/// <summary>
		/// Append the provided Markdown to our body
		/// </summary>
		/// <param name="RHS"></param>
		/// <returns></returns>
		public MarkdownBuilder Append(MarkdownBuilder RHS)
		{
			EnsureEndsWithNewLine();
			SB.Append(RHS.ToString());
			return this;
		}

		/// <summary>
		/// Append the provided text to our body. No formatting will be applied
		/// </summary>
		/// <param name="RHS"></param>
		/// <returns></returns>
		public MarkdownBuilder Append(string RHS)
		{
			EnsureEndsWithNewLine();
			SB.Append(RHS);
			return this;
		}
	}
}
