// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.IO;
using System.Linq;
using UnrealBuildTool;
using static Gauntlet.HordeReport;
using static Gauntlet.HordeReport.AutomatedTestSessionData;

namespace Gauntlet
{

	public delegate void PopulateTestPhasesReport(AutomatedTestSessionData TestReport);

	public class RoleReportData
	{
		public DateTime SessionStartTime { get; set; }
		public ITargetDevice TargetDevice { get;set; }
		public string RoleType { get; set; }
		public UnrealTargetPlatform Platform { get; set; }
		public UnrealTargetConfiguration Configuration { get; set; }
		public string ProjectName { get; set; }
		public string HordeArtifactsDir { get; set; }
		public List<UnrealTestEvent> InitializationEvents { get; set; }
		public List<UnrealTestEvent> ClosureEvents { get; set; }
	}
	public class MinimalHordeReportGenerator
	{
		private RoleReportData RoleReportData;

		public MinimalHordeReportGenerator(RoleReportData InRoleReportData)
		{
			RoleReportData = InRoleReportData;

			if (RoleReportData.SessionStartTime == DateTime.MinValue)
			{
				RoleReportData.SessionStartTime = DateTime.Now;
			}
		}

		private void SetReportMetadata(ITestReport Report)
		{
			Report.SetMetadata("Platform", RoleReportData.Platform.ToString());
			Report.SetMetadata("BuildTarget", RoleReportData.RoleType);
			Report.SetMetadata("Configuration", RoleReportData.Configuration.ToString());

			if (!string.IsNullOrEmpty(RoleReportData.ProjectName))
			{
				Report.SetMetadata("Project", RoleReportData.ProjectName);
			}

			var TagFilter = Globals.Params.ParseValue("TagFilter", string.Empty);
			if (!string.IsNullOrEmpty(TagFilter))
			{
				Report.SetMetadata("TagFilter", TagFilter);
			}

			foreach (var Meta in Globals.Params.ParseValues("Metadata"))
			{
				var Entry = Meta.Split(":", 2);
				if (Entry.Count() > 1)
				{
					Report.SetMetadata(Entry[0], Entry[1]);
				}
			}
		}

		private void PopulateClosurePhase(AutomatedTestSessionData TestReport, ITestNode TestNode, string PhaseName = "Closure", DateTime? SessionTime = null, float TimeElapseSeconds = 0)
		{
			var ClosurePhase = TestReport.AddPhase(PhaseName);
			ClosurePhase.SetTiming(SessionTime ?? DateTime.UtcNow, TimeElapseSeconds);
			ClosurePhase.deviceKeys = TestReport.GetDevices().Select(D => D.key).ToHashSet();
		
			var MainStream = ClosurePhase.GetStream();
			TestResult Result = TestNode.GetTestResult();

			if (TestNode.GetTestResult() == TestResult.Passed)
			{
				ClosurePhase.SetOutcome(TestPhaseOutcome.Success);
			}
			else if (TestNode.GetTestResult() == TestResult.Cancelled)
			{
				ClosurePhase.SetOutcome(TestPhaseOutcome.NotRun);
			}
			else if (TestNode.GetTestStatus() == TestStatus.NotStarted)
			{
				ClosurePhase.SetOutcome(TestPhaseOutcome.Skipped);
			}
			else if (TestNode.GetTestResult() == TestResult.Failed ||
					 TestNode.GetTestResult() == TestResult.TimedOut ||
					 TestNode.GetTestResult() == TestResult.Invalid ||
					 TestNode.GetTestResult() == TestResult.OperationalException)
			{
				ClosurePhase.SetOutcome(TestPhaseOutcome.Failed);
			}

			if (RoleReportData.ClosureEvents != null)
			{
				foreach (var ClosureEvent in RoleReportData.ClosureEvents)
				{
					ClosurePhase.GetStream().AddEvent(ClosureEvent);
				}
			}
		}
		private TestDevice AddDeviceToSessionReport(AutomatedTestSessionData TestReport)
		{
			string DeviceName = RoleReportData.TargetDevice.Platform == BuildHostPlatform.Current.Platform ? Environment.MachineName : RoleReportData.TargetDevice.Name;
			string DeviceKey = RoleReportData.TargetDevice.Name.Replace(".", "-");
			TestDevice ReportDevice = TestReport.AddDevice(DeviceName, DeviceKey);
			ReportDevice.metadata.Add("Platform", RoleReportData.Platform.ToString());
			ReportDevice.metadata.Add("Role", RoleReportData.RoleType);

			return ReportDevice;
		}

		/// <summary>
		/// Generates a Horde report for a test node that has completed.
		/// </summary>
		public ITestReport GenerateReport(string TestKey, ITestNode TestNode, PopulateTestPhasesReport OnPopulateTestPhases)
		{
			return GenerateReport(TestKey, TestNode, string.Empty, OnPopulateTestPhases);
		}

		/// <summary>
		/// Generates a Horde report for a test node that has completed, based on existing Json test data report.
		/// </summary>
		public ITestReport GenerateReport(string TestKey, ITestNode TestNode, string JsonReportPath, PopulateTestPhasesReport OnPopulateTestPhases)
		{
			AutomatedTestSessionData TestReport = new AutomatedTestSessionData(TestKey, TestNode.Name);
			TestReport.SetOutputArtifactPath(string.IsNullOrEmpty(RoleReportData.HordeArtifactsDir) ? DefaultArtifactsDir : RoleReportData.HordeArtifactsDir);

			DateTime SessionTime = RoleReportData.SessionStartTime.ToUniversalTime();
			float TimeElapse = (float)(DateTime.Now - RoleReportData.SessionStartTime).TotalSeconds;
			TestReport.SetSessionTiming(SessionTime, TimeElapse);

			var InitPhase = TestReport.AddPhase("Initialization", "Initialization");

			if (RoleReportData.InitializationEvents != null)
			{
				foreach (var InitEvent in RoleReportData.InitializationEvents)
				{
					InitPhase.GetStream().AddEvent(InitEvent);
				}
			}

			InitPhase.SetTiming(SessionTime, 0);

			if (TestNode.GetTestResult() == TestResult.Passed)
			{
				InitPhase.SetOutcome(TestPhaseOutcome.Success);
			}
			else if (TestNode.GetTestResult() == TestResult.Cancelled)
			{
				InitPhase.SetOutcome(TestPhaseOutcome.Interrupted);
			}
			else if (TestNode.GetTestStatus() == TestStatus.NotStarted)
			{
				InitPhase.SetOutcome(TestPhaseOutcome.NotRun);
			}

			var ReportDevice = AddDeviceToSessionReport(TestReport);
			InitPhase.deviceKeys.Add(ReportDevice.key);

			if (!string.IsNullOrEmpty(JsonReportPath) && File.Exists(JsonReportPath))
			{
				Log.Verbose("Reading json test data report from {Path}", JsonReportPath);
				UnrealAutomatedTestPassResults JsonTestPassResults = UnrealAutomatedTestPassResults.LoadFromJson(JsonReportPath);
				TestReport.PopulateFromUnrealAutomatedTests(JsonTestPassResults, DefaultTestDataDir);
			}

			if (OnPopulateTestPhases != null)
			{
				// Some delegates have been setup
				OnPopulateTestPhases(TestReport);
				PopulateClosurePhase(TestReport, TestNode, "Closure", SessionTime, TimeElapse);
			}
			else
			{
				PopulateClosurePhase(TestReport, TestNode, "Main", SessionTime, TimeElapse);
			}

			SetReportMetadata(TestReport);

			return TestReport;
		}

		public void SubmitToHorde(ITestReport Report, string ArtifactName, string HordeTestDataPath, string HordeTestDataKey)
		{
			// write test data collection for Horde
			string FileName = FileUtils.SanitizeFilename(ArtifactName);
			string HordeTestDataFilePath = Path.Combine(
				string.IsNullOrEmpty(HordeTestDataPath) ? DefaultTestDataDir : HordeTestDataPath,
				FileName + ".TestData.json"
			);
			TestDataCollection HordeTestDataCollection = new TestDataCollection();
			HordeTestDataCollection.AddNewTestReport(Report, HordeTestDataKey);
			HordeTestDataCollection.WriteToJson(HordeTestDataFilePath, !AutomationTool.Automation.IsBuildMachine);
		}
	}
}
