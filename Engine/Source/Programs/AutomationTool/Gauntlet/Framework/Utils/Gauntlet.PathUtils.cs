// Copyright Epic Games, Inc. All Rights Reserved.
using System;
using System.Collections.Generic;
using System.Linq;
using System.IO;

namespace Gauntlet
{
	public class PathUtils
	{
		/// <summary>
		/// Converts a path to the long path format if it's not already.
		/// </summary>
		/// <param name="InPath">The path to convert.</param>
		/// <returns>The converted path.</returns>
		public static string MakeLongPath(string InPath)
		{
			string LongPathPrefix = @"\\?\";
			if (!InPath.StartsWith(LongPathPrefix))
			{
				InPath = LongPathPrefix + InPath;
			}
			return InPath;
		}

		/// <summary>
		/// Similar to Path.Combine but creates a long path instead.
		/// </summary>
		/// <param name="Paths">The parts of the path to combine.</param>
		/// <returns>The full long path.</returns>
		public static string CombineLongPath(params string[] Paths)
		{
			return MakeLongPath(Path.Combine(Paths));
		}

		/// <summary>
		/// Find a path that includes specific directories in the given order, but could also include unknown intermediate directories.
		/// </summary>
		/// <param name="BasePath">The absolute path to the root directory where we will begin our search</param>
		/// <param name="PathNodes">Directories names that will be searched for and combined into a path</param>
		/// <returns>If successful, returns the absolute found relevant path. Returns string.Empty if an issue occurs or the path cannot be found.</returns>
		public static string FindRelevantPath(string BasePath, params string[] PathNodes)
		{
			string[] TotalPathNodes = new string[PathNodes.Length + 1];
			TotalPathNodes[0] = BasePath;
			for (int i = 0; i < PathNodes.Length; i++)
			{
				TotalPathNodes[i + 1] = PathNodes[i];
			}
			return FindRelevantPath(TotalPathNodes);
		}

		/// <summary>
		/// Find a path that includes specific directories in the given order, but could also include unknown intermediate directories.
		/// </summary>
		/// <param name="PathNodes">Directories names that will be searched for and combined into a path</param>
		/// <returns>If successful, returns the absolute found relevant path. Returns string.Empty if an issue occurs or the path cannot be found.</returns>
		public static string FindRelevantPath(params string[] PathNodes)
		{
			if (PathNodes.Length < 2)
			{
				Gauntlet.Log.Warning("This function requires two or more directory names (path nodes) to search for a path.");
				return string.Empty;
			}

			if (!Directory.Exists(PathNodes[0]))
			{
				Gauntlet.Log.Warning("The first parameter in this function must be an existing directory path.");
				return string.Empty;
			}

			List<string> FoundPathNodes = new List<string>() { PathNodes[0] };
			string FoundPath = PathNodes[0];

			for (int NodeIndex = 1; NodeIndex < PathNodes.Length; NodeIndex++)
			{
				// Break at the final path node if it is a file name
				if (NodeIndex == PathNodes.Length - 1 && Path.HasExtension(PathNodes[NodeIndex]))
				{
					FoundPathNodes.Add(PathNodes[NodeIndex]);
					FoundPath = Path.Combine(FoundPathNodes.ToArray());
					break;
				}

				string FoundSubdirectory = FindRelevantSubdirectory(FoundPath, PathNodes[NodeIndex]);

				if (string.IsNullOrEmpty(FoundSubdirectory))
				{
					Gauntlet.Log.Info("Could not find complete relevant path. One or more path nodes could not be found among: " + string.Join(" \\ ", PathNodes));
					Gauntlet.Log.Info("Partially found relevant path: " + FoundPath);
					return string.Empty;
				}

				FoundSubdirectory = Path.GetRelativePath(FoundPath, FoundSubdirectory);
				FoundPathNodes.Add(FoundSubdirectory);
				FoundPath = Path.Combine(FoundPathNodes.ToArray());
			}

			return FoundPath;
		}

		private static string FindRelevantSubdirectory(string ParentDirectoryPath, string SubdirectoryName, bool bSearchWideFirst = true)
		{
			try
			{
				string RelevantSubdirectory = MatchChildDirectory(ParentDirectoryPath, SubdirectoryName);
				if (string.IsNullOrEmpty(RelevantSubdirectory))
				{
					string[] Subdirectories = Directory.GetDirectories(ParentDirectoryPath);

					// Most of the time, our target directory is only one extra level deep, so search that before deeply recursing
					if (bSearchWideFirst)
					{
						Gauntlet.Log.Info("Did not find child directory named \"{0}\" within {1}. Searching next level down.",
							SubdirectoryName, ParentDirectoryPath);
						foreach (string Subdirectory in Subdirectories)
						{
							if (string.IsNullOrEmpty(RelevantSubdirectory))
							{
								RelevantSubdirectory = MatchChildDirectory(Subdirectory, SubdirectoryName);
							}
							else
							{
								string AdditionalRelevantSubdirectory = MatchChildDirectory(Subdirectory, SubdirectoryName);
								if (!string.IsNullOrEmpty(AdditionalRelevantSubdirectory))
								{
									Gauntlet.Log.Info("Found additional match for \"{0}\" within {1}! Ignoring this match ({2}) in favor of {3}",
										SubdirectoryName, ParentDirectoryPath, AdditionalRelevantSubdirectory, RelevantSubdirectory);
								}
							}
						}

						if (!string.IsNullOrEmpty(RelevantSubdirectory))
						{
							return RelevantSubdirectory;
						}

						Gauntlet.Log.Info("Did not find child directory named \"{0}\" one level below {1}. Searching recursively through all subdirectories.",
							SubdirectoryName, ParentDirectoryPath);
					}

					foreach (string Subdirectory in Subdirectories)
					{
						string FoundDirectory = FindRelevantSubdirectory(Subdirectory, SubdirectoryName, false);
						if (!string.IsNullOrEmpty(FoundDirectory))
						{
							// Return the first match rather than searching the entire parent directory before analyzing for the best match, which could take a long time
							return FoundDirectory;
						}
					}
				}

				return RelevantSubdirectory;
			}
			catch (Exception Ex)
			{
				Gauntlet.Log.Error(Ex.ToString());
				return string.Empty;
			}
		}

		private static string MatchChildDirectory(string ParentDirectoryPath, string ChildDirectoryName)
		{
			try
			{
				string[] Subdirectories = Directory.GetDirectories(ParentDirectoryPath);
				string[] MatchingDirectories = Subdirectories.Where(directory =>
					new DirectoryInfo(directory).Name.Equals(ChildDirectoryName, StringComparison.InvariantCultureIgnoreCase)).ToArray();

				if (MatchingDirectories.Any())
				{
					if (MatchingDirectories.Count() > 1)
					{
						// If case-insensitive matched more than one result, match case-sensitive
						MatchingDirectories = MatchingDirectories.Where(directory =>
							new DirectoryInfo(directory).Name.Equals(ChildDirectoryName, StringComparison.InvariantCulture)).ToArray();

						if (MatchingDirectories.Count() != 1)
						{
							Gauntlet.Log.Warning("Target directory name \"{0}\" is ambiguously capitalized for a match within directory {1}", ChildDirectoryName, ParentDirectoryPath);
							return string.Empty;
						}
					}

					return MatchingDirectories.First();
				}
			}
			catch (Exception Ex)
			{
				Gauntlet.Log.Error(Ex.ToString());
				return string.Empty;
			}

			return string.Empty;
		}
	}
}
