// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Linq;
using System.Diagnostics;
using System.IO;
using AutomationTool;
using UnrealBuildTool;
using EpicGames.Core;
using System.Text.Json;
using Newtonsoft.Json;

namespace Gauntlet
{
	public class SummaryTableData
	{
		[JsonIgnore]
		public string CsvId { get; set; }
		[JsonIgnore]
		public string DeviceProfile { get; set; }
		public Dictionary<string, double> CsvStatAverage { get; set; }
		public Dictionary<string, string> CsvMetadata { get; set; }
		public Dictionary<string, double> SummaryTableMetric { get; set; }
		public Dictionary<string, string> ToolMetadata { get; set; }
	}

	public static class ReportGenUtils
	{
		private static string PythonExecutable = null;
		private static string BasePythonLocation;
		private static string DotnetPathLocation;
		public const string CsvBinExt = ".csv.bin";

		static ReportGenUtils()
		{
			if (BuildHostPlatform.Current.Platform.IsInGroup("Windows"))
			{
				DotnetPathLocation = "dotnet.exe";
				BasePythonLocation = @"Engine\Binaries\ThirdParty\Python3\Win64\python.exe";
			}
			else
			{
				BasePythonLocation = @"Engine/Binaries/ThirdParty/Python3/Mac/bin/python3";

				if (CommandUtils.IsBuildMachine)
				{
					DotnetPathLocation = "/usr/local/bin/dotnet";
					return;
				}
				else
				{
					BasePythonLocation = @"Engine/Binaries/ThirdParty/Python3/Mac/bin/python3";
				}
			}
			DotnetPathLocation = UnrealBuildBase.Unreal.DotnetPath.FullName;

			if (string.IsNullOrEmpty(DotnetPathLocation))
			{
				Log.Verbose("Could not locate dotnet directory from PATH! Running ReportGenUtils commands with .net dlls will throw exceptions.");
			}
		}

		public static int RunPerfReportTool(string Args, string ReportXmlBaseDir)
		{
			string PerfReportToolPath = GetCsvToolPath("PerfReportTool");
			string AllArgs = Args + " -reportxmlbasedir " + Quotify(ReportXmlBaseDir);
			return ReportGenUtils.RunCommandlineTool(PerfReportToolPath, AllArgs);
		}

		public static void DirCopy(string SourceDir, string TargetDir, bool bSkipExisting = false, string SearchPattern = "*.*")
		{
			Log.Info(string.Format("Copying Directory {0} to {1}", SourceDir, TargetDir));
			foreach (string dirPath in Directory.GetDirectories(SourceDir, "*", SearchOption.AllDirectories))
			{
				Directory.CreateDirectory(dirPath.Replace(SourceDir, TargetDir));
			}

			//Copy all the files & Replaces any files with the same name
			foreach (string newPath in Directory.GetFiles(SourceDir, SearchPattern, SearchOption.AllDirectories))
			{
				string DestFilename = newPath.Replace(SourceDir, TargetDir);
				if (bSkipExisting && File.Exists(DestFilename))
				{
					Log.Info("File " + DestFilename + " already exists. Skipping!");
				}
				else
				{
					File.Copy(newPath, DestFilename, true);
				}
			}
		}

		public static string GetCsvToolPath(string ToolName)
		{
			string Extension = BuildHostPlatform.Current.Platform.IsInGroup(UnrealPlatformGroup.Microsoft) ? ".exe" : ".dll";
			return Path.Combine(CommandUtils.CmdEnv.LocalRoot, "Engine", "Binaries", "DotNET", "CsvTools", ToolName) + Extension;
		}

		public static int RunPythonScript(string ScriptPath, string Args)
		{
			Process NewProcess = RunPythonScriptAsync(ScriptPath, Args);
			NewProcess.WaitForExit();
			if (NewProcess.ExitCode != 0)
			{
				Log.Error("Python script " + ScriptPath + " exited with error code " + NewProcess.ExitCode);
			}
			return NewProcess.ExitCode;
		}

		public static Process RunPythonScriptAsync(string ScriptPath, string Args, bool bRedirectStdOut = false)
		{
			// Lazy init the python executable
			if (PythonExecutable == null)
			{
				if (File.Exists(Path.Combine(CommandUtils.CmdEnv.LocalRoot, BasePythonLocation)))
				{
					Log.Info("Running Python from branch at" + Path.Combine(CommandUtils.CmdEnv.LocalRoot, BasePythonLocation));
					PythonExecutable = Path.Combine(CommandUtils.CmdEnv.LocalRoot, BasePythonLocation);
				}
				else
				{
					Log.Info("Unable to find python at " + Path.Combine(CommandUtils.CmdEnv.LocalRoot, BasePythonLocation) + " - using path instead");
					PythonExecutable = "python.exe";
				}
			}
			return RunCommandlineToolAsync(PythonExecutable, ScriptPath + " " + Args, bRedirectStdOut);
		}

		public static int RunCommandlineTool(string ToolPath, string Args)
		{
			Process ToolProcess = RunCommandlineToolAsync(ToolPath, Args, false);
			ToolProcess.WaitForExit();
			if (ToolProcess.ExitCode != 0)
			{
				Log.Error(ToolPath + " exited with error code " + ToolProcess.ExitCode);
			}
			return ToolProcess.ExitCode;
		}

		public static Process RunCommandlineToolAsync(string ToolPath, string Args, bool bRedirectStdOut = false)
		{
			Process ToolProcess = new Process();
			if (ToolPath.EndsWith(".dll"))
			{
				if (string.IsNullOrEmpty(DotnetPathLocation))
				{
					throw new AutomationException("Could not locate a dotnet executable in your PATH. Ensure it has been added to your environment variables.");
				}
				ToolProcess.StartInfo.FileName = DotnetPathLocation;
				ToolProcess.StartInfo.Arguments = ToolPath + ' ' + Args;
			}
			else
			{
				ToolProcess.StartInfo.FileName = ToolPath;
				ToolProcess.StartInfo.Arguments = Args;
			}

			ToolProcess.StartInfo.RedirectStandardOutput = bRedirectStdOut;
			ToolProcess.StartInfo.UseShellExecute = false;

			Log.Info("Running " + ToolProcess.StartInfo.FileName + " with args:\n" + ToolProcess.StartInfo.Arguments);
			ToolProcess.Start();
			return ToolProcess;
		}

		/// <summary>
		/// Converts the csv files in the given directory into binary.
		/// </summary>
		/// <param name="CsvDirectory">The directory to search for csv files in.</param>
		/// <returns>List of converted csv paths.</returns>
		public static List<FileInfo> ConvertCsvsToBinary(string CsvDirectory)
		{
			return ConvertCsvsToBinary(Directory.GetFiles(CsvDirectory, "*.csv", SearchOption.TopDirectoryOnly));
		}

		/// <summary>
		/// Converts the given csv files into binary format.
		/// </summary>
		/// <param name="CsvFilePaths">The csv file paths to convert.</param>
		/// <returns>List of converted csv paths.</returns>
		public static List<FileInfo> ConvertCsvsToBinary(IEnumerable<string> CsvFilePaths)
		{
			List<FileInfo> ConvertedPaths = new List<FileInfo>();
			string CsvConvertPath = GetCsvToolPath("CsvConvert");
			//Copy all the files & Replaces any files with the same name
			foreach (string CsvFilename in CsvFilePaths)
			{
				// Do a basic validity check to ensure there's metadata in the csv before we convert.
				if (!File.Exists(CsvFilename) || ReadCsvMetadata(CsvFilename) == null)
				{
					Log.Info($"Skipping {CsvFilename} for binary conversion as it's missing metadata or doesn't exist on disk.");
					continue;
				}

				string BinFilename = CsvFilename + ".bin";
				Log.Info("Converting CSV " + CsvFilename + " to binary");
				int ExitCode = RunCommandlineTool(CsvConvertPath,
					" -in " + Quotify(CsvFilename) +
					" -out " + Quotify(BinFilename) +
					" -outFormat bin" +
					" -binCompress 2" +
					" -verify");
				if (ExitCode != 0)
				{
					Log.Warning("Failed to convert csv " + CsvFilename + " to binary!");
					// -Verify doesn't delete the file if it fails, so make sure it's deleted now
					if (File.Exists(BinFilename))
					{
						File.Delete(BinFilename);
					}
				}
				else
				{
					// Delete the original
					File.Delete(CsvFilename);
					ConvertedPaths.Add(new FileInfo(BinFilename));
				}
			}
			return ConvertedPaths;
		}

		/// <summary>
		/// Converts a list of csv files into binary (.csv.bin).
		/// </summary>
		/// <param name="CsvFiles">The csv files to convert.</param>
		/// <returns>List of FileInfo for all the converted csvs.</returns>
		public static List<FileInfo> ConvertCsvsToBinary(IEnumerable<FileInfo> CsvFiles)
		{
			return ConvertCsvsToBinary(CsvFiles.Select(file => file.FullName));
		}

		/// <summary>
		/// Collects all valid csvs (with metadata) within a directory and converts them to binary (.csv.bin).
		/// Note: this will not return existing .csv.bin files, only newly converted ones.
		/// </summary>
		/// <param name="DirectoryPath">The directory to search in.</param>
		/// <param name="Search">The search method.</param>
		/// <returns>List of FileInfo for all the converted csvs.</returns>
		public static List<FileInfo> CollectAndConvertCsvFilesToBinary(string DirectoryPath, SearchOption Search = SearchOption.AllDirectories)
		{
			return ConvertCsvsToBinary(CollectValidCsvFiles(DirectoryPath, Search));
		}

		/// <summary>
		/// Collects all csv files that have valid metadata.
		/// </summary>
		/// <param name="DirectoryPath">The directory to search in.</param>
		/// <param name="Search">The search method.</param>
		/// <returns>List of FileInfo for all valid csvs.</returns>
		public static List<FileInfo> CollectValidCsvFiles(string DirectoryPath, SearchOption Search = SearchOption.AllDirectories)
		{
			DirectoryInfo CSVDirectory = new DirectoryInfo(DirectoryPath);
			return CSVDirectory.GetFiles("*.csv", Search)
				.Where(CsvFile => ReadCsvMetadata(CsvFile.FullName) != null)
				.ToList();
		}

		/// <summary>
		/// Attempts to create the PerfReportServerImporter. This importer is currently internal to Epic and such this function will return null when not used internally.
		/// </summary>
		/// <param name="DataSourceName">The datasource we're importing for.</param>
		/// <param name="BuildName">Name of the build, used to validate csvs are for the correct build.</param>
		/// <param name="bIsBuildMachine">If we're running on a build machine. For Epic internal builds, determines where the import root should be.</param>
		/// <param name="ImportDirOverride">Optional override directory to write the import batch to. Must be set is bIsBuildMachine is false.</param>
		/// <param name="CommonDataSourceFields">Common metadata to apply to all imports.</param>
		/// <returns>The importer if found, otherwise null.</returns>
		public static ICsvImporter CreatePerfReportServerImporter(string DataSourceName, string BuildName, bool bIsBuildMachine, string ImportDirOverride = null, Dictionary<string, dynamic> CommonDataSourceFields = null)
		{
			Type ImporterType = Type.GetType("Gauntlet.PerfReportServerImporter", false);
			if (ImporterType == null)
			{
				Log.Info("Couldn't find Gauntlet.PerfReportServerImporter in the assembly, skipping creation.");
				return null;
			}

			var ConstructorArgs = new object[]
			{
				DataSourceName,
				BuildName,
				CommonDataSourceFields,
				bIsBuildMachine,
				ImportDirOverride
			};

			return Activator.CreateInstance(ImporterType, ConstructorArgs) as ICsvImporter;
		}

		/// <summary>
		/// Tests if a build is a preflight.
		/// </summary>
		/// <param name="BuildName">The name of the build to test.</param>
		/// <returns>True if this is a preflight build.</returns>
		public static bool IsTestingPreflightBuild(string BuildName)
		{
			return BuildName.ToLower().Contains("-pf-");
		}

		public static void OrderFilenamesByDate(string FilePrefix, string SearchPattern, string Dir)
		{
			FileInfo[] FilesOrdered = new DirectoryInfo(Dir).GetFiles(SearchPattern).OrderBy(f => f.CreationTime).ToArray();

			int FileIndex = 1;
			foreach (FileInfo Info in FilesOrdered)
			{
				string NewFilename = Path.Combine(Dir, FilePrefix + "_" + FileIndex.ToString("D2") + Info.Extension);
				System.IO.File.Move(Info.FullName, NewFilename);
				FileIndex++;
			}
		}

		// Returns metadata from the first valid CSV file that has metadata in this directory
		public static Dictionary<string, string> ReadCsvMetadataFromDirectory(string CsvDirectoryPath, bool bRecurse = false, string SearchPattern = null)
		{
			Dictionary<string, string> Metadata = null;
			DirectoryInfo DirInfo = new DirectoryInfo(CsvDirectoryPath);
			SearchOption SearchOption = bRecurse ? SearchOption.AllDirectories : SearchOption.TopDirectoryOnly;
			List<FileInfo> Files;
			if (SearchPattern != null)
			{
				Files = DirInfo.GetFiles(SearchPattern, SearchOption).ToList();
			}
			else
			{
				Files = DirInfo.GetFiles("*.csv", SearchOption).ToList();
				Files.AddRange(DirInfo.GetFiles("*.csv.bin", SearchOption));
			}
			foreach (FileInfo File in Files)
			{
				Metadata = ReadCsvMetadata(File.FullName, false);
				if (Metadata != null)
				{
					break;
				}
			}
			return Metadata;
		}

		// Reads metadata from a CSV or CSV.bin. Returns null if there was no metadata
		public static Dictionary<string, string> ReadCsvMetadata(string CsvFilename, bool bDoLogging = true)
		{
			if (bDoLogging)
			{
				Log.Info("Reading metadata from CSV: " + CsvFilename);
			}
			Dictionary<string, string> DictOut = null;
			Process CsvInfoProcess = RunCommandlineToolAsync(ReportGenUtils.GetCsvToolPath("CsvInfo"), Quotify(CsvFilename), true);
			while (!CsvInfoProcess.StandardOutput.EndOfStream)
			{
				string Line = CsvInfoProcess.StandardOutput.ReadLine();
				if (DictOut != null)
				{
					int Idx = Line.IndexOf(":");
					if (Idx != -1)
					{
						string Key = Line.Substring(0, Idx).Trim();
						string Value = Line.Substring(Idx + 1).Trim();
						DictOut.Add(Key, Value);
					}
				}
				if (Line == "Metadata:")
				{
					DictOut = new Dictionary<string, string>();
				}
			}
			CsvInfoProcess.WaitForExit();
			return DictOut;
		}

		public static string ReadCsvMetadataValue(string CsvFilename, string Key)
		{
			Dictionary<string, string> Dict = ReadCsvMetadata(CsvFilename);
			if (Dict != null)
			{
				if (Dict.TryGetValue(Key.ToLower(), out string ValueOut))
				{
					return ValueOut;
				}
			}
			return null;
		}

		public static string Quotify(string inStr)
		{
			return "\"" + inStr + "\"";
		}

		public static bool BuildVersionValidationCheck(string InBuildVersion, string CsvDirectory, string SearchPattern = null)
		{
			Dictionary<string, string> Metadata = ReadCsvMetadataFromDirectory(CsvDirectory, false, SearchPattern);
			return BuildVersionValidationCheck(InBuildVersion, Metadata);
		}

		public static bool BuildVersionValidationCheckForCsv(string InBuildVersion, string CsvFilename)
		{
			Dictionary<string, string> Metadata = ReadCsvMetadata(CsvFilename);
			return BuildVersionValidationCheck(InBuildVersion, Metadata);
		}

		public static string GetMetadataQueryForLastNDays(int DurationInDays)
		{
			long CurrentTimestamp = DateTimeOffset.Now.ToUnixTimeSeconds();
			long StartTimestamp = CurrentTimestamp - (long)DurationInDays * 60 * 60 * 24;
			return "starttimestamp>=" + StartTimestamp.ToString();
		}

		public static bool BuildVersionValidationCheck(string InBuildVersion, Dictionary<string, string> CsvMetadata)
		{
			Gauntlet.Log.Info("Validating build version against CSV metadata");
			if (CsvMetadata == null || !CsvMetadata.ContainsKey("buildversion"))
			{
				Gauntlet.Log.Error("No valid CSV metadata found!");
				return false;
			}

			if (InBuildVersion.Contains(CsvMetadata["buildversion"], StringComparison.OrdinalIgnoreCase))
			{
				Gauntlet.Log.Info("Test build version matches CSV metadata: " + InBuildVersion);
				return true;
			}
			Gauntlet.Log.Error("Test build version (" + InBuildVersion + ") doesn't match CSV metadata (" + CsvMetadata["buildversion"] + "). Gauntlet likely failed to deploy the build correctly. Please report to automation team");
			return false;
		}

		// Replaces csv buildversion metadata in bulk. Use with caution!
		public static void OverwriteBuildVersionForCsvs(string CsvDirectory, string BuildVersion)
		{
			string CsvConvertPath = ReportGenUtils.GetCsvToolPath("CsvConvert");
			DirectoryInfo DirInfo = new DirectoryInfo(CsvDirectory);
			List<FileInfo> Files = DirInfo.GetFiles("*.csv", SearchOption.TopDirectoryOnly).ToList();
			Files.AddRange(DirInfo.GetFiles("*.csv.bin", SearchOption.TopDirectoryOnly));
			foreach (FileInfo File in Files)
			{
				int ExitCode = ReportGenUtils.RunCommandlineTool(CsvConvertPath,
					" -in " + Quotify(File.FullName) +
					" -setMetadata buildversion=" + BuildVersion +
					" -inplace");

				if (ExitCode != 0)
				{
					throw new AutomationException("Setting metadata failed for file " + Quotify(File.FullName) + ". CsvConvert exitcode: " + ExitCode);
				}
			}
		}

		// Runs PRT with -summaryTableToJson and returns a list of each csv's summary table information
		public static bool GetSummaryTableData(List<FileReference> CsvFiles, out List<SummaryTableData> SummaryTables, out string Error)
		{
			Error = string.Empty;
			SummaryTables = new List<SummaryTableData>();

			if (CsvFiles?.Count  == 0)
			{
				Error = "Could not get summary table data for an empty or null list of csv files";
				return false;
			}

			// Construct an empty temp folder
			string PrtResultDirectory = Path.Combine(Globals.TempDir, "PrtResults");
			string SummaryTableJson = Path.Combine(PrtResultDirectory, "SummaryTable.json");
			try
			{
				if (CommandUtils.DirectoryExists(PrtResultDirectory))
				{
					CommandUtils.DeleteDirectory(PrtResultDirectory);
				}
				CommandUtils.CreateDirectory(PrtResultDirectory);
			}
			catch (Exception Ex)
			{
				Error = $"Encountered an {Ex.GetType().Name} while trying to clean temp PRT directory: {Ex.Message}";
				return false;
			}

			// Run PRT on the results
			string ReportToolCmdLine = string.Format("-csvdir {0} -o {1} -summaryTableToJson {2}", CsvFiles[0].Directory.FullName, PrtResultDirectory, SummaryTableJson);
			int PrtExitCode = RunCommandlineTool(GetCsvToolPath("PerfReportTool"), ReportToolCmdLine);
			if (PrtExitCode != 0)
			{
				Error = "PerfReportTool exited with code {PrtExitCode}. See above for details";
				return false;
			}

			foreach (FileReference CsvFile in CsvFiles)
			{
				string CsvId = ReadCsvMetadataValue(CsvFile.FullName, "csvid");
				if (string.IsNullOrEmpty(CsvId))
				{
					Error = string.Format("Failed to locate CsvId for {0}", CsvFile.FullName);
					return false;
				}

				string DeviceProfile = ReadCsvMetadataValue(CsvFile.FullName, "deviceprofile");
				if (string.IsNullOrEmpty(DeviceProfile))
				{
					Error = string.Format("Failed to locate CsvId for {0}", CsvFile.FullName);
					return false;
				}

				try
				{
					JsonDocument CsvJson = JsonDocument.Parse(File.ReadAllText(SummaryTableJson), new JsonDocumentOptions { AllowTrailingCommas = true });
					JsonElement IdElement = CsvJson.RootElement.GetProperty(CsvId);

					SummaryTableData SummaryTable = new SummaryTableData()
					{
						CsvId = CsvId,
						DeviceProfile = DeviceProfile,
						CsvStatAverage = IdElement.GetProperty("CsvStatAverage").Deserialize<Dictionary<string, double>>(),
						CsvMetadata = IdElement.GetProperty("CsvMetadata").Deserialize<Dictionary<string, JsonElement>>().ToDictionary(KVP => KVP.Key, KVP => KVP.Value.ToString()),
						SummaryTableMetric = IdElement.GetProperty("SummaryTableMetric").Deserialize<Dictionary<string, double>>(),
						ToolMetadata = IdElement.GetProperty("ToolMetadata").Deserialize<Dictionary<string, JsonElement>>().ToDictionary(KVP => KVP.Key, KVP => KVP.Value.ToString())
					};
					

					SummaryTables.Add(SummaryTable);
				}
				catch (Exception Ex)
				{
					Error = $"Failed to parse {CsvFile.FullName} for relevant summary table information. {Ex.Message}";
					return false;
				}
			}

			return true;
		}
	}
}