// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.IO;
using System.IO.Compression;

namespace Gauntlet
{
	public sealed class ScreenshotComparisonOptions
	{
		/// <summary>Minimum similarity percentage to consider a match (0-100).</summary>
		public double Threshold { get; set; } = 95.0;

		/// <summary>Per-channel color difference tolerance (0-255).</summary>
		public int ColorTolerance { get; set; } = 10;

		public static ScreenshotComparisonOptions Default => new ScreenshotComparisonOptions();
	}

	public enum ScreenshotComparisonStatus
	{
		Match,
		NoMatch,
		GroundTruthMissing,
		Error
	}

	public sealed class ScreenshotComparisonResult
	{
		public ScreenshotComparisonStatus Status { get; }
		public bool IsMatch => Status == ScreenshotComparisonStatus.Match;
		public bool IsGroundTruthMissing => Status == ScreenshotComparisonStatus.GroundTruthMissing;
		public double SimilarityPercent { get; }
		public int DifferentPixels { get; }
		public int TotalPixels { get; }
		public string ActualPath { get; }
		public string ExpectedPath { get; }
		public string ErrorMessage { get; }

		private ScreenshotComparisonResult(
			ScreenshotComparisonStatus InStatus,
			double InSimilarityPercent = 0,
			int InDifferentPixels = 0,
			int InTotalPixels = 0,
			string InActualPath = null,
			string InExpectedPath = null,
			string InErrorMessage = null)
		{
			Status = InStatus;
			SimilarityPercent = InSimilarityPercent;
			DifferentPixels = InDifferentPixels;
			TotalPixels = InTotalPixels;
			ActualPath = InActualPath;
			ExpectedPath = InExpectedPath;
			ErrorMessage = InErrorMessage;
		}

		public override string ToString()
		{
			switch (Status)
			{
				case ScreenshotComparisonStatus.Match:
					return string.Format("MATCH: {0:F2}% similar", SimilarityPercent);
				case ScreenshotComparisonStatus.NoMatch:
					return string.Format("NO MATCH: {0:F2}% similar, {1} different pixels", SimilarityPercent, DifferentPixels);
				case ScreenshotComparisonStatus.GroundTruthMissing:
					return string.Format("GROUND TRUTH MISSING: Candidate saved to {0}", ActualPath);
				case ScreenshotComparisonStatus.Error:
					return string.Format("ERROR: {0}", ErrorMessage);
				default:
					return Status.ToString();
			}
		}

		public static ScreenshotComparisonResult Match(
			double InSimilarity, int InTotal, string InActualPath, string InExpectedPath)
		{
			return new ScreenshotComparisonResult(
				ScreenshotComparisonStatus.Match,
				InSimilarity,
				0,
				InTotal,
				InActualPath,
				InExpectedPath);
		}

		public static ScreenshotComparisonResult NoMatch(
			double InSimilarity, int InDifferent, int InTotal,
			string InActualPath, string InExpectedPath)
		{
			return new ScreenshotComparisonResult(
				ScreenshotComparisonStatus.NoMatch,
				InSimilarity,
				InDifferent,
				InTotal,
				InActualPath,
				InExpectedPath);
		}

		public static ScreenshotComparisonResult GroundTruthMissing(
			string InCandidatePath, string InExpectedGroundTruthPath)
		{
			return new ScreenshotComparisonResult(
				ScreenshotComparisonStatus.GroundTruthMissing,
				InActualPath: InCandidatePath,
				InExpectedPath: InExpectedGroundTruthPath);
		}

		public static ScreenshotComparisonResult Error(string InMessage)
		{
			return new ScreenshotComparisonResult(
				ScreenshotComparisonStatus.Error,
				InErrorMessage: InMessage);
		}
	}

	/// <summary>
	/// Pixel-level PNG screenshot comparison with no external dependencies.
	/// </summary>
	public class ScreenshotCompare
	{
		public static ScreenshotComparisonResult Compare(
			string InActualPath,
			string InExpectedPath,
			ScreenshotComparisonOptions InOptions = null)
		{
			InOptions = InOptions ?? ScreenshotComparisonOptions.Default;

			Log.Verbose("ScreenshotCompare: Actual={0}, Expected={1}", InActualPath, InExpectedPath);

			try
			{
				RawImage ActualImage = PngReader.Read(InActualPath);
				RawImage ExpectedImage = PngReader.Read(InExpectedPath);

				return ComparePixels(ActualImage, ExpectedImage, InActualPath, InExpectedPath, InOptions);
			}
			catch (Exception Ex)
			{
				Log.Error("ScreenshotCompare: Comparison failed: {0}", Ex.Message);
				return ScreenshotComparisonResult.Error(Ex.Message);
			}
		}

		private static ScreenshotComparisonResult ComparePixels(
			RawImage InActual,
			RawImage InExpected,
			string InActualPath,
			string InExpectedPath,
			ScreenshotComparisonOptions InOptions)
		{
			if (InActual.Width != InExpected.Width || InActual.Height != InExpected.Height)
			{
				Log.Verbose("ScreenshotCompare: Size mismatch (Actual={0}x{1}, Expected={2}x{3}). Resizing actual to match.",
					InActual.Width, InActual.Height, InExpected.Width, InExpected.Height);
				InActual = InActual.Resize(InExpected.Width, InExpected.Height);
			}

			int Width = InExpected.Width;
			int Height = InExpected.Height;
			int TotalPixels = Width * Height;
			int DifferentPixels = 0;

			for (int Y = 0; Y < Height; Y++)
			{
				for (int X = 0; X < Width; X++)
				{
					Pixel ActualPixel = InActual.GetPixel(X, Y);
					Pixel ExpectedPixel = InExpected.GetPixel(X, Y);

					if (!AreColorsWithinTolerance(ActualPixel, ExpectedPixel, InOptions.ColorTolerance))
					{
						DifferentPixels++;
					}
				}
			}

			double Similarity = (1.0 - (double)DifferentPixels / TotalPixels) * 100;
			bool bIsMatch = Similarity >= InOptions.Threshold;

			Log.Info("ScreenshotCompare: {0:F2}% similar ({1} different pixels out of {2}), Match={3}",
				Similarity, DifferentPixels, TotalPixels, bIsMatch);

			return bIsMatch
				? ScreenshotComparisonResult.Match(Similarity, TotalPixels, InActualPath, InExpectedPath)
				: ScreenshotComparisonResult.NoMatch(
					Similarity, DifferentPixels, TotalPixels, InActualPath, InExpectedPath);
		}

		private static bool AreColorsWithinTolerance(Pixel InA, Pixel InB, int InTolerance)
		{
			return Math.Abs(InA.R - InB.R) <= InTolerance
				&& Math.Abs(InA.G - InB.G) <= InTolerance
				&& Math.Abs(InA.B - InB.B) <= InTolerance
				&& Math.Abs(InA.A - InB.A) <= InTolerance;
		}
	}

	/// <summary>
	/// Minimal PNG reader. Supports grayscale, RGB, indexed, grayscale+alpha, and RGBA.
	/// </summary>
	internal static class PngReader
	{
		private static readonly byte[] PngSignature = { 0x89, 0x50, 0x4E, 0x47, 0x0D, 0x0A, 0x1A, 0x0A };

		public static RawImage Read(string InPath)
		{
			byte[] FileBytes = File.ReadAllBytes(InPath);
			return Parse(FileBytes);
		}

		private static RawImage Parse(byte[] InData)
		{
			for (int i = 0; i < 8; i++)
			{
				if (InData[i] != PngSignature[i])
				{
					throw new InvalidDataException("Not a valid PNG file");
				}
			}

			int Width = 0, Height = 0;
			int BitDepth = 0, ColorType = 0;
			using var CompressedData = new MemoryStream();

			int Offset = 8;
			while (Offset < InData.Length)
			{
				int ChunkLength = ReadInt32BigEndian(InData, Offset);
				string ChunkType = System.Text.Encoding.ASCII.GetString(InData, Offset + 4, 4);

				if (ChunkType == "IHDR")
				{
					Width = ReadInt32BigEndian(InData, Offset + 8);
					Height = ReadInt32BigEndian(InData, Offset + 12);
					BitDepth = InData[Offset + 16];
					ColorType = InData[Offset + 17];
				}
				else if (ChunkType == "IDAT")
				{
					CompressedData.Write(InData, Offset + 8, ChunkLength);
				}
				else if (ChunkType == "IEND")
				{
					break;
				}

				Offset += 12 + ChunkLength;
			}

			CompressedData.Position = 2; // Skip zlib header
			byte[] Pixels;
			using (var Deflate = new DeflateStream(CompressedData, CompressionMode.Decompress))
			using (var Output = new MemoryStream())
			{
				Deflate.CopyTo(Output);
				Pixels = Output.ToArray();
			}

			return new RawImage(Width, Height, Pixels, ColorType);
		}

		private static int ReadInt32BigEndian(byte[] InData, int InOffset)
		{
			return (InData[InOffset] << 24) | (InData[InOffset + 1] << 16) |
				   (InData[InOffset + 2] << 8) | InData[InOffset + 3];
		}
	}

	internal class RawImage
	{
		public int Width { get; }
		public int Height { get; }
		private readonly byte[] Pixels;
		private readonly int BytesPerPixel;
		private readonly int Stride;

		public RawImage(int InWidth, int InHeight, byte[] InPixels, int InColorType)
		{
			Width = InWidth;
			Height = InHeight;
			Pixels = InPixels;

			BytesPerPixel = InColorType switch
			{
				0 => 1,  // Grayscale
				2 => 3,  // RGB
				3 => 1,  // Indexed
				4 => 2,  // Grayscale + Alpha
				6 => 4,  // RGBA
				_ => 4
			};

			Stride = 1 + (InWidth * BytesPerPixel);
		}

		public Pixel GetPixel(int InX, int InY)
		{
			int RowStart = InY * Stride + 1;
			int PixelStart = RowStart + (InX * BytesPerPixel);

			if (PixelStart + BytesPerPixel > Pixels.Length)
			{
				return new Pixel(0, 0, 0, 0);
			}

			return BytesPerPixel switch
			{
				4 => new Pixel(Pixels[PixelStart], Pixels[PixelStart + 1],
							  Pixels[PixelStart + 2], Pixels[PixelStart + 3]),
				3 => new Pixel(Pixels[PixelStart], Pixels[PixelStart + 1],
							  Pixels[PixelStart + 2], 255),
				1 => new Pixel(Pixels[PixelStart], Pixels[PixelStart],
							  Pixels[PixelStart], 255),
				_ => new Pixel(0, 0, 0, 255)
			};
		}

		private Pixel GetPixelClamped(int InX, int InY)
		{
			InX = Math.Clamp(InX, 0, Width - 1);
			InY = Math.Clamp(InY, 0, Height - 1);
			return GetPixel(InX, InY);
		}

		public RawImage Resize(int InTargetWidth, int InTargetHeight)
		{
			int TargetBytesPerPixel = 4;
			int TargetStride = 1 + (InTargetWidth * TargetBytesPerPixel);
			byte[] TargetPixels = new byte[TargetStride * InTargetHeight];

			double ScaleX = (double)Width / InTargetWidth;
			double ScaleY = (double)Height / InTargetHeight;

			for (int TY = 0; TY < InTargetHeight; TY++)
			{
				int RowStart = TY * TargetStride;
				TargetPixels[RowStart] = 0;

				for (int TX = 0; TX < InTargetWidth; TX++)
				{
					double SrcX = (TX + 0.5) * ScaleX - 0.5;
					double SrcY = (TY + 0.5) * ScaleY - 0.5;

					int X0 = (int)Math.Floor(SrcX);
					int Y0 = (int)Math.Floor(SrcY);
					int X1 = X0 + 1;
					int Y1 = Y0 + 1;

					double FracX = SrcX - X0;
					double FracY = SrcY - Y0;

					Pixel P00 = GetPixelClamped(X0, Y0);
					Pixel P10 = GetPixelClamped(X1, Y0);
					Pixel P01 = GetPixelClamped(X0, Y1);
					Pixel P11 = GetPixelClamped(X1, Y1);

					double R = Bilerp(P00.R, P10.R, P01.R, P11.R, FracX, FracY);
					double G = Bilerp(P00.G, P10.G, P01.G, P11.G, FracX, FracY);
					double B = Bilerp(P00.B, P10.B, P01.B, P11.B, FracX, FracY);
					double A = Bilerp(P00.A, P10.A, P01.A, P11.A, FracX, FracY);

					int PixelOffset = RowStart + 1 + (TX * TargetBytesPerPixel);
					TargetPixels[PixelOffset]     = (byte)Math.Clamp((int)Math.Round(R), 0, 255);
					TargetPixels[PixelOffset + 1] = (byte)Math.Clamp((int)Math.Round(G), 0, 255);
					TargetPixels[PixelOffset + 2] = (byte)Math.Clamp((int)Math.Round(B), 0, 255);
					TargetPixels[PixelOffset + 3] = (byte)Math.Clamp((int)Math.Round(A), 0, 255);
				}
			}

			return new RawImage(InTargetWidth, InTargetHeight, TargetPixels, 6);
		}

		private static double Bilerp(double V00, double V10, double V01, double V11, double Fx, double Fy)
		{
			double Top = V00 + (V10 - V00) * Fx;
			double Bottom = V01 + (V11 - V01) * Fx;
			return Top + (Bottom - Top) * Fy;
		}
	}

	internal readonly struct Pixel
	{
		public byte R { get; }
		public byte G { get; }
		public byte B { get; }
		public byte A { get; }

		public Pixel(byte InR, byte InG, byte InB, byte InA)
		{
			R = InR; G = InG; B = InB; A = InA;
		}
	}
}
