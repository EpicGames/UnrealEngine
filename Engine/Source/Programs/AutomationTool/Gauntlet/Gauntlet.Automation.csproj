<Project Sdk="Microsoft.NET.Sdk">
  <Import Project="..\..\Shared\UnrealEngine.csproj.props" />
  
  <PropertyGroup>
    <OutputType>Library</OutputType>
    <GenerateAssemblyInfo>false</GenerateAssemblyInfo>
    <GenerateTargetFrameworkAttribute>false</GenerateTargetFrameworkAttribute>
    <RootNamespace>Gauntlet.Automation</RootNamespace>
    <AssemblyName>Gauntlet.Automation</AssemblyName>
    <WarningsNotAsErrors>612,618</WarningsNotAsErrors>
    <OutputPath Condition="'$(ArtifactsPath)' == ''">..\..\..\..\Binaries\DotNET\AutomationTool\AutomationScripts\Gauntlet</OutputPath>
    <TreatWarningsAsErrors>true</TreatWarningsAsErrors>
    <CopyLocalLockFileAssemblies>true</CopyLocalLockFileAssemblies>
    <SatelliteResourceLanguages>en</SatelliteResourceLanguages>    <!-- remove non english resource languages -->
  </PropertyGroup>
  <ItemGroup>
    <Reference Include="fastJSON">
      <HintPath>..\..\..\..\Binaries\ThirdParty\fastJSON\netstandard2.0\fastJSON.dll</HintPath>
    </Reference>
  </ItemGroup>

    <!-- The following will glob all platform extensions' Gauntlet source
  files and include them in the build. They will also appear in VisualStudio -->
  <ItemGroup>
    <Compile Include="../../../../Platforms/*/Source/Programs/AutomationTool/Gauntlet/**/*.cs">
      <!-- RecursiveDir metadata expands to [PlatformName]/Source/Programs/[etc]. The Replace() will
      replace everything from the first slash to the end leaving just the platform name. Should the
      Replace() fail then RecursiveDir is still an agreeable value -->
      <Link>Platform/$([System.Text.RegularExpressions.Regex]::Replace(%(Compile.RecursiveDir), [\\/].+$, ``))/%(Compile.FileName).cs</Link>
    </Compile>
    <Compile Include="../../../../Restricted/*/Source/Programs/AutomationTool/Gauntlet/**/*.cs">
      <!-- Restricted may or may not be in a Platforms dir, so just use the subdir directly, even tho it makes for a
       messier directory structure for these few files -->
      <Link>$([System.Text.RegularExpressions.Regex]::Replace(%(FullPath), '^.+?[\\/]AutomationTool[\\/]Gauntlet[\\/]', ''))</Link>
    </Compile>
  </ItemGroup>

  <!-- The following two ItemGroups will glob all UAT plugin's Gauntlet source
    files and include them in the build. They will also appear in VisualStudio -->

  <!-- Allow skipping including Gauntlet plugins behind a specific MSBuild property, as:
    1. Currently there are no files that actually get picked up by this include.
    2. The extensive disk scan causes MSBuild project evaluation to take between 8-12 seconds extra for this alone.
    3. UBT project parsing support exposes, but nothing accesses, the collection of compiled items, so there's no actual benefit to having the items.
    This is defaulted to still include items for now, in case licensees include plugins that depend on this, but should be considered obsolete as of UE 5.8. -->
  <ItemGroup Condition="'$(EpicGamesIncludeGauntletPlugins)' != 'false'">
    <Compile Include="../../../../Plugins/**/Build/Scripts/Gauntlet/**/*.cs">
      <!-- Shorten the path so that everything is under a Plugins folder but does not include the Build/Scripts/Gauntlet part -->
      <Link>$([System.Text.RegularExpressions.Regex]::Replace(%(FullPath), '^.*?(?=Plugins[\\/])|Build[\\/]Scripts[\\/]Gauntlet[\\/]',''))</Link>
    </Compile>
  </ItemGroup>

  <ItemGroup>
    <Compile Include="../../../../Platforms/*/Plugins/**/Build/Scripts/Gauntlet/**/*.cs">
      <!-- RecursiveDir metadata expands to [PlatformName]/Source/Programs/[etc]. The Replace() will
      replace everything from the first slash to the end leaving just the platform name. Should the
      Replace() fail then RecursiveDir is still an agreeable value -->
      <Link>Platform/$([System.Text.RegularExpressions.Regex]::Replace(%(Compile.RecursiveDir), [\\/].+$, ``))/%(Compile.FileName).cs</Link>
    </Compile>
  </ItemGroup>

  <Import Project="../../../../Platforms/*/Source/Programs/AutomationTool/Gauntlet/*.Gauntlet.targets" Condition="'$(OS)' == 'Windows_NT'" />

  <ItemGroup>
    <ProjectReference Include="..\..\Shared\EpicGames.Core\EpicGames.Core.csproj" PrivateAssets="All"><Private>false</Private></ProjectReference>
    <ProjectReference Include="..\..\Shared\EpicGames.Horde\EpicGames.Horde.csproj" />
    <ProjectReference Include="..\..\Shared\EpicGames.LogMatchers\EpicGames.LogMatchers.csproj" />
    <ProjectReference Include="..\..\UnrealBuildTool\UnrealBuildTool.csproj" PrivateAssets="All"><Private>false</Private></ProjectReference>
    <ProjectReference Include="..\AutomationUtils\AutomationUtils.Automation.csproj" PrivateAssets="All"><Private>false</Private></ProjectReference>
    <ProjectReference Include="..\Scripts\AutomationScripts.Automation.csproj" PrivateAssets="All"><Private>false</Private></ProjectReference>
    <ProjectReference Include="..\OneSkyLocalization\OneSkyLocalization.Automation.csproj" PrivateAssets="All"><Private>false</Private></ProjectReference>
    <ProjectReference Include="..\Win\Win.Automation.csproj" PrivateAssets="All"><Private>false</Private></ProjectReference>
    <ProjectReference Include="..\XLocLocalization\XLocLocalization.Automation.csproj" PrivateAssets="All"><Private>false</Private></ProjectReference>
    <ProjectReference Include="..\Localization\Localization.Automation.csproj" PrivateAssets="All"><Private>false</Private></ProjectReference>
    <ProjectReference Include="..\Android\Android.Automation.csproj" PrivateAssets="All"><Private>false</Private></ProjectReference>
  </ItemGroup>
  <ItemGroup>
    <PackageReference Include="Appium.WebDriver" Version="8.0.0" />
    <PackageReference Include="Magick.NET-Q16-HDRI-AnyCPU" Version="14.13.1" PrivateAssets="all" />
    <PackageReference Include="Microsoft.IdentityModel.Tokens" Version="8.17.0" />
    <PackageReference Include="MySql.Data" Version="6.10.9" PrivateAssets="all" />
    <PackageReference Include="Newtonsoft.Json" Version="13.0.4" PrivateAssets="all" />
  </ItemGroup>
</Project>