// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.IO;
using AutomationTool;
using UnrealBuildTool;
using System.Threading;
using System.Text.RegularExpressions;
using System.Linq;
using EpicGames.Core;

namespace Gauntlet
{
	public class AndroidBuild : IBuild
	{
		public int PreferenceOrder { get { return 0; } }

		public UnrealTargetConfiguration Configuration { get; protected set; }

		public string SourceApkPath;

		public Dictionary<string, string> FilesToInstall;

		public string AndroidPackageName;

		public BuildFlags Flags { get; protected set; }

		public string Flavor { get { return ""; } }

		public UnrealTargetPlatform Platform { get { return UnrealTargetPlatform.Android; } }

		public bool SupportsAdditionalFileCopy => true;

		public bool Is32Bit { get; protected set; }

		public bool UsesExternalFilesDir { get; protected set; }

		public bool UseAFSWithoutToken { get; protected set; }

		public string ParsedAFSToken { get; protected set; }

		public bool UsesPublicLogs { get; protected set; }

		public AndroidBuild(UnrealTargetConfiguration InConfig, string InAndroidPackageName, string InApkPath, Dictionary<string, string> InFilesToInstall, BuildFlags InFlags, bool InIs32Bit, bool bInUsesExternalFilesDir, bool bInUsesPublicLogs, bool bInUseAFSWithoutToken = false, string InParsedAFSToken = null)
		{
			Configuration = InConfig;
			AndroidPackageName = InAndroidPackageName;
			SourceApkPath = InApkPath;
			FilesToInstall = InFilesToInstall;
			Flags = InFlags;
			Is32Bit = InIs32Bit;
			UsesExternalFilesDir = bInUsesExternalFilesDir;
			UsesPublicLogs = bInUsesPublicLogs;
			UseAFSWithoutToken = bInUseAFSWithoutToken;
			ParsedAFSToken = InParsedAFSToken;
		}

		public bool CanSupportRole(UnrealTargetRole RoleType)
		{
			if (RoleType.IsClient())
			{
				return true;
			}

			return false;
		}

		public static IEnumerable<AndroidBuild> CreateFromPath(string InProjectName, string InPath)
		{
			string BuildPath = InPath;

			List<AndroidBuild> DiscoveredBuilds = new List<AndroidBuild>();

			DirectoryInfo Di = new DirectoryInfo(BuildPath);

			
			// find all install batchfiles
			FileInfo[] InstallFiles = Di.GetFiles("Install_*");

			foreach (FileInfo Fi in InstallFiles)
			{ 
				bool PackageIs32Bit = Fi.FullName.Contains("armv7");
	
				UnrealTargetConfiguration UnrealConfig = UnrealHelpers.GetConfigurationFromExecutableName(InProjectName, Fi.FullName);
				UnrealTargetRole UnrealRole = UnrealHelpers.GetRoleFromExecutableName(InProjectName, Fi.FullName);

				if (UnrealConfig == UnrealTargetConfiguration.Unknown)
				{
					Log.Info("Skipping unrecognized build {0}", Fi.FullName);
					continue;
				}

				bool TestInstall = Fi.Name.EndsWith("_TEST.bat", StringComparison.OrdinalIgnoreCase);
				bool PatchInstall = Fi.Name.EndsWith("_Patch.bat", StringComparison.OrdinalIgnoreCase);

				// filter out non-matching or test installation batch files
				// test installation scripts are intended to be manually invoked
				if (TestInstall || PatchInstall)
				{
					if (TestInstall || PatchInstall)
					{
						Log.Verbose("Ignoring {0} installation batch file {1}", TestInstall ? "test" : "patch", Fi.Name);
					}

					continue;
				}

				Log.Verbose("Pulling install data from {0}", Fi.FullName);

				string AbsPath = Fi.Directory.FullName;

				// read contents and replace linefeeds (regex doesn't stop on them :((
				string BatContents = File.ReadAllText(Fi.FullName).Replace(Environment.NewLine, "\n");

				// Replace .bat with .apk and strip up to and including the first _, that is then our APK name
				var SourceApkMatch = Regex.Match(BatContents, @" install\s+(.+\.apk)");
				if ( SourceApkMatch.Groups.Count <= 0)
				{
					Log.Warning(KnownLogEvents.Gauntlet_BuildDropEvent, "Could not parse install command from {File}", Fi.FullName);
					continue;
				}
				string SourceApkPath = Path.Combine(AbsPath,SourceApkMatch.Groups[1].ToString());
				
				bool bUsesExternalFilesDir = false;
				bool bUsesPublicLogs = false;
				// parse APK's metadata for some build details
				{
					AndroidPlatform.GetPackageInfo(SourceApkPath, false);
					// Establish remote directory usage
					string ApkUsesExternalFilesDir = AndroidPlatform.GetMetadataValue("bUseExternalFilesDir");
					string ApkUsesPublicLogs = AndroidPlatform.GetMetadataValue("bPublicLogFiles");
					bUsesExternalFilesDir = ApkUsesExternalFilesDir != null ? ApkUsesExternalFilesDir.Contains("1") : false;
					bUsesPublicLogs = ApkUsesPublicLogs != null ? ApkUsesPublicLogs.Contains("1") : false;
				}

				// save com.companyname.product
				string AndroidPackageName = Regex.Match(BatContents, @"uninstall\s+(com\..+)").Groups[1].ToString();

				// pull all OBBs (probably just one..)
				var OBBMatches = Regex.Matches(BatContents, @"push\s+(.+?)\s+(.+)");

				// save them as a dict of full paths as keys and dest paths as values
				Dictionary<string, string> FilesToInstall = OBBMatches.Cast<Match>().ToDictionary(M => Path.Combine(AbsPath, M.Groups[1].ToString()), M => M.Groups[2].ToString());
	
				if (string.IsNullOrEmpty(SourceApkPath))
				{
					Log.Warning(KnownLogEvents.Gauntlet_BuildDropEvent, "No APK found for build at {File}", Fi.FullName);
					continue;
				}

				if (!File.Exists(SourceApkPath))
				{
					Log.Warning(KnownLogEvents.Gauntlet_BuildDropEvent, "Resolved APK name but it doesn't exist {File}", SourceApkPath);
					continue;
				}

				if (string.IsNullOrEmpty(AndroidPackageName))
				{
					Log.Warning(KnownLogEvents.Gauntlet_BuildDropEvent, "No product name found for build at {File}", Fi.FullName);
					continue;
				}

				// Android builds are always packaged, and we can always replace the command line and executable
				BuildFlags Flags = BuildFlags.Packaged | BuildFlags.CanReplaceCommandLine | BuildFlags.CanReplaceExecutable;
				if (AbsPath.Contains("Bulk"))
				{
					Flags |= BuildFlags.Bulk;
				}
				else
				{
					Flags |= BuildFlags.NotBulk;
				}

				// Detect AFS token from install script for shipping builds with separate AFS APK.
				string ParsedAFSToken = null;
				if (Path.GetFileName(SourceApkPath).StartsWith("AFS_"))
				{
					var AFSTokenMatch = Regex.Match(BatContents, @"-k\s+([A-Fa-f0-9]{32})");
					if (AFSTokenMatch.Success)
					{
						ParsedAFSToken = AFSTokenMatch.Groups[1].ToString();
						Log.Info("Detected AFS build with token at {0}", AbsPath);
					}
				}

				AndroidBuild NewBuild = new AndroidBuild(UnrealConfig, AndroidPackageName, SourceApkPath, FilesToInstall, Flags, PackageIs32Bit, bUsesExternalFilesDir, bUsesPublicLogs, InParsedAFSToken: ParsedAFSToken);

				DiscoveredBuilds.Add(NewBuild);

				Log.Verbose("Found {0} {1} build at {2}", UnrealConfig, ((Flags & BuildFlags.Bulk) == BuildFlags.Bulk) ? "(bulk)" : "(not bulk)", AbsPath);
			}

			// If we have both 32 and 64-bit builds, prefer 64-bit
			if (DiscoveredBuilds.Where(B => B.Is32Bit == false).Any())
			{
				DiscoveredBuilds = DiscoveredBuilds.Where(B => !B.Is32Bit).ToList();
			}

			return DiscoveredBuilds;
		}
	}
	
	public class AndroidBuildSource : IFolderBuildSource
	{	
		public string BuildName { get { return "AndroidBuildSource";  } }

		public bool CanSupportPlatform(UnrealTargetPlatform InPlatform)
		{
			return InPlatform == UnrealTargetPlatform.Android;
		}

		public string ProjectName { get; protected set; }

		public AndroidBuildSource()
		{
		}

		public List<IBuild> GetBuildsAtPath(string InProjectName, string InPath, int MaxRecursion = 3)
		{

			List<DirectoryInfo> AllDirs = new List<DirectoryInfo>();

			//AndroidBuildSource BuildSource = null;
			List<IBuild> Builds = new List<IBuild>();

			// c:\path\to\build
			DirectoryInfo PathDI = new DirectoryInfo(InPath);

			if (PathDI.Exists)
			{
				if (PathDI.Name.IndexOf("Android", StringComparison.OrdinalIgnoreCase) >= 0)
				{
					AllDirs.Add(PathDI);
				}

				// find all directories that begin with Android
				DirectoryInfo[] AndroidDirs = PathDI.GetDirectories("Android*", SearchOption.TopDirectoryOnly);

				AllDirs.AddRange(AndroidDirs);

				List<DirectoryInfo> DirsToRecurse = AllDirs;

				// now get subdirs
				while (MaxRecursion-- > 0)
				{
					List<DirectoryInfo> DiscoveredDirs = new List<DirectoryInfo>();

					DirsToRecurse.ToList().ForEach((D) =>
					{
						DiscoveredDirs.AddRange(D.GetDirectories("*", SearchOption.TopDirectoryOnly));
					});

					AllDirs.AddRange(DiscoveredDirs);
					DirsToRecurse = DiscoveredDirs;
				}

				string AndroidBuildFilter = Globals.Params.ParseValue("AndroidBuildFilter", "");
				foreach (DirectoryInfo Di in AllDirs)
				{
					IEnumerable<AndroidBuild> FoundBuilds = AndroidBuild.CreateFromPath(InProjectName, Di.FullName);

					if (FoundBuilds != null)
					{
						if (!string.IsNullOrEmpty(AndroidBuildFilter))
						{
							//IndexOf used because Contains must be case-sensitive
							FoundBuilds = FoundBuilds.Where(B => B.SourceApkPath.IndexOf(AndroidBuildFilter, StringComparison.OrdinalIgnoreCase) >= 0);
						}
						Builds.AddRange(FoundBuilds);
					}
				}
			}

			return Builds;
		}

		/*public AndroidBuild GetBuild(UnrealTargetConfiguration InConfig, BuildFlags InFlags)
		{
			return Builds.Where((B) => {
				return B.Configuration == InConfig && (B.Flags & InFlags) > 0;
				}).FirstOrDefault();
		}*/
	}
}