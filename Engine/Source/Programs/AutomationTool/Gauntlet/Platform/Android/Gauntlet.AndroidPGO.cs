// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.IO;
using System.Text;
using System.Collections.Generic;
using System.Runtime.InteropServices;
using System.Diagnostics;
using AutomationTool;
using UnrealBuildTool;

namespace Gauntlet
{
	/// <summary>
	/// Skeleton Android PGO platform implementation (primarily used to test PGO locally with editor builds)
	/// </summary>
	internal class AndroidPGOPlatform : IPGOPlatform
	{
		private readonly string AndroidLlvmProfdataPath;
		string LocalOutputDirectory;
		string LocalProfDataFile;

		public AndroidPGOPlatform()
		{
			if (RuntimeInformation.IsOSPlatform(OSPlatform.Windows))
			{
				AndroidLlvmProfdataPath = Path.Combine(Environment.ExpandEnvironmentVariables("%NDKROOT%"), @"toolchains\llvm\prebuilt\windows-x86_64\bin\llvm-profdata.exe");
			}
			else
			{
				AndroidLlvmProfdataPath = Path.Combine(Environment.ExpandEnvironmentVariables("%NDKROOT%"), "toolchains/llvm/prebuilt/darwin-x86_64/bin/llvm-profdata");
			}
		}

		public UnrealTargetPlatform GetPlatform()
		{
			return UnrealTargetPlatform.Android;
		}

		public void GatherResults(string ArtifactPath)
		{
			var ProfRawFiles = Directory.GetFiles(ArtifactPath, "*.profraw");
			if (ProfRawFiles.Length == 0)
			{
				throw new AutomationException(string.Format("Process exited cleanly but no .profraw PGO files were found in the output directory \"{0}\".", ArtifactPath));
			}


			if (File.Exists(LocalProfDataFile))
			{
				new FileInfo(LocalProfDataFile).IsReadOnly = false;
			}

			StringBuilder MergeCommandBuilder = new StringBuilder();
			foreach (var ProfRawFile in ProfRawFiles)
			{
				MergeCommandBuilder.AppendFormat(" \"{0}\"", ProfRawFile);
			}

			int ReturnCode = UnrealBuildTool.Utils.RunLocalProcessAndLogOutput(new ProcessStartInfo(AndroidLlvmProfdataPath, string.Format("merge{0} -o \"{1}\"", MergeCommandBuilder, LocalProfDataFile)), EpicGames.Core.Log.Logger);
			if (ReturnCode != 0)
			{
				throw new AutomationException(string.Format("{0} failed to merge profraw data. Error code {1}. ({2} {3})", Path.GetFileName(AndroidLlvmProfdataPath), ReturnCode, MergeCommandBuilder, LocalProfDataFile));
			}

			// Check the profdata file exists
			if (!File.Exists(LocalProfDataFile))
			{
				throw new AutomationException(string.Format("Profraw data merging completed, but the profdata output file (\"{0}\") was not found.", LocalProfDataFile));
			}
		}

		public void ApplyConfiguration(PGOConfig Config)
		{
			UnrealTestRole ClientRole = Config.RequireRole(UnrealTargetRole.Client);

			// ensure all pgo output goes to our artifactpath.
//		ClientRole.CommandLine += string.Format(" -pgoprofileoutput=\"{0}\"", AndroidDevice.DeviceArtifactPath );

			if (!File.Exists(AndroidLlvmProfdataPath))
			{
				throw new AutomationException(string.Format("No llvm-profdata at \"{0}\".", AndroidLlvmProfdataPath));
			}

			LocalOutputDirectory = Config.ProfileOutputDirectory;
			LocalProfDataFile = Path.Combine(LocalOutputDirectory, "profile.profdata");
		}

		public bool TakeScreenshot(ITargetDevice Device, string ScreenshotDirectory, out string ImageFilename)
		{
			TargetDeviceAndroid AndroidDevice = Device as TargetDeviceAndroid;
			ImageFilename = @"fortpgoscreen.png";
			string RemoteScreenshotPath = string.Format("{0}/{1}",AndroidDevice.DeviceExternalStorageSavedPath, ImageFilename);
			string LocalScreenshotPath = Path.Combine(ScreenshotDirectory, ImageFilename);

			AndroidDevice.RunAdbDeviceCommand(string.Format("shell screencap {0}", RemoteScreenshotPath), true, false, true);
			AndroidDevice.RunAdbDeviceCommand(string.Format("pull {0} {1}", RemoteScreenshotPath, LocalScreenshotPath), true, false, true);
			AndroidDevice.RunAdbDeviceCommand(string.Format("shell rm {0}", RemoteScreenshotPath), true, false, true);
			return true;
		}
	}
}



