// Copyright Epic Games, Inc. All Rights Reserved.

using AutomationTool;
using EpicGames.Core;
using Microsoft.Build.Framework;
using OpenQA.Selenium;
using System;
using System.Collections.Generic;
using System.Diagnostics;
using System.IO;
using System.Linq;
using System.Text;
using System.Text.RegularExpressions;
using System.Threading;
using UnrealBuildBase;
using UnrealBuildTool;
using static AutomationTool.CommandUtils;
using static System.Runtime.InteropServices.JavaScript.JSType;

namespace Gauntlet
{
	/// <summary>
	/// Android implementation of a device that can run applications
	/// </summary>
	public class TargetDeviceAndroid : ITargetDevice
	{
		#region ITargetDevice
		public string Name { get; protected set; }

		public UnrealTargetPlatform? Platform => UnrealTargetPlatform.Android;

		public CommandUtils.ERunOptions RunOptions { get; set; }

		public bool IsConnected => IsAvailable;

		public bool IsOn
		{
			get
			{
				string CommandLine = "shell dumpsys power";
				IProcessResult OnAndUnlockedQuery = RunAdbDeviceCommand(CommandLine, bPauseErrorParsing: true);
				return OnAndUnlockedQuery.Output.Contains("mHoldingDisplaySuspendBlocker=true")
					&& OnAndUnlockedQuery.Output.Contains("mHoldingWakeLockSuspendBlocker=true");
			}
		}

		public bool IsAvailable
		{
			get
			{
				// ensure our device is present in 'adb devices' output.
				var AllDevices = GetAllConnectedDevices();

				if (AllDevices.Keys.Contains(DeviceName) == false)
				{
					return false;
				}

				if (AllDevices[DeviceName] != "device")
				{
					Log.Warning(KnownLogEvents.Gauntlet_DeviceEvent, "Device {Name} is '{State}'", DeviceName, AllDevices[DeviceName]);
					return false;
				}

				// any device will do, but only one at a time.
				return true;
			}
		}
		#endregion

		/// <summary>
		/// Low-level device name
		/// </summary>
		public string DeviceName { get; protected set; }

		/// <summary>
		/// Temp path we use to push/pull things from the device
		/// </summary>
		public string LocalCachePath { get; protected set; }

		/// <summary>
		/// Root storage path of the device, e.g. /sdcard/
		/// </summary>
		public string StoragePath { get; protected set; }

		/// <summary>
		/// External storage path on the device, e.g. /sdcard/UEGame/ etc..
		/// </summary>
		public string DeviceExternalStorageSavedPath { get; protected set; }

		/// <summary>
		/// External files path on the device, this is the app's own publically accessible data directory e.g. /sdcard/Android/data/[com.package.name]/files/UnrealGame etc..
		/// </summary>
		public string DeviceExternalFilesSavedPath { get; protected set; }

		/// <summary>
		/// Saved storage path, e.g. /sdcard/UEGame/Saved (bulk) or /sdcard/Android/data/[com.package.name]/files/UEGame/Saved (notbulk)
		/// </summary>
		public string DeviceArtifactPath { get; protected set; }

		/// <summary>
		/// Path to the log file. (This takes public logs setting of the package in to account.)
		/// </summary>
		public string DeviceLogPath { get; protected set; }

		[AutoParam((int)(60 * 15))]
		public int MaxInstallTime { get; protected set; }

		/// <summary>
		/// Path to a command line if installed
		/// </summary>
		protected string CommandLineFilePath { get; set; }

		protected bool IsExistingDevice = false;

		protected Dictionary<EIntendedBaseCopyDirectory, string> LocalDirectoryMappings;

		private string NoPlayProtectSetting = null;

		private AndroidAppInstall InstallCache = null;

		private bool bUseAFSWithoutToken;

		private string ParsedAFSToken;

		private Thread ScreenshotThread = null;

		private AutoResetEvent ScreenshotTrigger = null;

		public TargetDeviceAndroid(string InDeviceName = "", AndroidDeviceData DeviceData = null, string InCachePath = null)
		{
			AutoParam.ApplyParamsAndDefaults(this, Globals.Params.AllArguments);

			AdbCredentialCache.AddInstance(DeviceData);

			// If no device name or its 'default' then use the first default device
			if (string.IsNullOrEmpty(InDeviceName) || InDeviceName.Equals("default", StringComparison.OrdinalIgnoreCase))
			{
				var DefaultDevices = GetAllAvailableDevices();

				if (DefaultDevices.Count() == 0)
				{
					if (GetAllConnectedDevices().Count > 0)
					{
						throw new AutomationException("No default device available. One or more devices are connected but unauthorized or offline. See 'adb devices'");
					}
					else
					{
						throw new AutomationException("No default device available. See 'adb devices'");
					}
				}

				DeviceName = DefaultDevices.First();
			}
			else
			{
				DeviceName = InDeviceName;
			}

			if (Log.IsVerbose)
			{
				RunOptions = CommandUtils.ERunOptions.NoWaitForExit;
			}
			else
			{
				RunOptions = CommandUtils.ERunOptions.NoWaitForExit | CommandUtils.ERunOptions.NoLoggingOfRunCommand;
			}

			// if this is not a connected device then remove when done
			var ConnectedDevices = GetAllConnectedDevices();

			IsExistingDevice = ConnectedDevices.Keys.Contains(DeviceName);
			if (!IsExistingDevice && !DeviceName.Contains(":"))
			{
				// adb uses port 5555 by default
				DeviceName += ":5555";
				IsExistingDevice = ConnectedDevices.Keys.Contains(DeviceName);
			}
			if (!IsExistingDevice)
			{
				lock (Globals.MainLock)
				{
					using (var PauseEC = new ScopedSuspendECErrorParsing())
					{
						IProcessResult AdbResult = RunAdbGlobalCommand(string.Format("connect {0}", DeviceName));

						if (AdbResult.ExitCode != 0)
						{
							throw new DeviceException("adb failed to connect to {0}. {1}", DeviceName, AdbResult.Output);
						}
					}

					Log.Info("Connecting to {0}", DeviceName);

					// Need to sleep for adb service process to register, otherwise get an unauthorized (especially on parallel device use)
					Thread.Sleep(5000);
				}

				ConnectedDevices = GetAllConnectedDevices();

				// sanity check that it was now found
				if (!ConnectedDevices.Keys.Contains(DeviceName))
				{
					throw new DeviceException("Failed to find new device {0} in connection list", DeviceName);
				}
			}

			if (ConnectedDevices[DeviceName] != "device")
			{
				Dispose();
				throw new DeviceException("Device {0} is '{1}'.", DeviceName, ConnectedDevices[DeviceName]);
			}

			// Get the external storage path from the device (once the device is validated as connected)
			IProcessResult StorageQueryResult = RunAdbDeviceCommand(AndroidPlatform.GetStorageQueryCommand());
			if (StorageQueryResult.ExitCode != 0)
			{
				throw new DeviceException("Failed to query external storage setup on {0}", DeviceName);
			}
			StoragePath = StorageQueryResult.Output.Trim();

			LocalDirectoryMappings = new Dictionary<EIntendedBaseCopyDirectory, string>();

			// for IP devices need to sanitize this
			Name = DeviceName.Replace(":", "_");

			// Temp path used when pulling artifact
			LocalCachePath = InCachePath ?? Path.Combine(Globals.TempDir, "AndroidDevice_" + Name);
		}

		~TargetDeviceAndroid()
		{
			Dispose(false);
		}

		#region IDisposable Support
		private bool DisposedValue = false; // To detect redundant calls

		protected virtual void Dispose(bool disposing)
		{
			if (!DisposedValue)
			{
				try
				{
					if (!string.IsNullOrEmpty(NoPlayProtectSetting))
					{
						Log.Verbose("Restoring play protect for this session...");
						RunAdbDeviceCommand($"shell settings put global package_verifier_user_consent {NoPlayProtectSetting}");
					}

					if (!IsExistingDevice)
					{
						// disconnect
						RunAdbGlobalCommand(string.Format("disconnect {0}", DeviceName), true, false, true);

						Log.Info("Disconnected {0}", DeviceName);
					}

					if (ScreenshotThread != null)
					{
						ScreenshotTrigger.Set();
						ScreenshotThread.Join();
						ScreenshotThread = null;
						ScreenshotTrigger = null;
					}
				}
				catch (Exception Ex)
				{
					Log.Warning(KnownLogEvents.Gauntlet_DeviceEvent, "TargetDeviceAndroid.Dispose() threw: {Exception}", Ex.Message);
				}
				finally
				{
					DisposedValue = true;
					AdbCredentialCache.RemoveInstance();
				}

			}
		}

		// This code added to correctly implement the disposable pattern.
		public void Dispose()
		{
			// Do not change this code. Put cleanup code in Dispose(bool disposing) above.
			Dispose(true);
			// TODO: uncomment the following line if the finalizer is overridden above.
			// GC.SuppressFinalize(this);
		}

		public bool Disposed
		{
			get
			{
				return DisposedValue;
			}

		}

		#endregion

		#region ITargetDevice
		public bool Connect()
		{
			AllowDeviceSleepState(true);
			return true;
		}

		public bool Disconnect(bool bForce = false)
		{
			AllowDeviceSleepState(false);
			return true;
		}

		public bool PowerOn()
		{
			Log.Verbose("{0}: Powering on", ToString());
			string CommandLine = "shell \"input keyevent KEYCODE_WAKEUP && input keyevent KEYCODE_MENU\"";
			IProcessResult PowerOnQuery = RunAdbDeviceCommand(CommandLine, bPauseErrorParsing: true);
			return !PowerOnQuery.Output.Contains("error");
		}
		public bool PowerOff()
		{
			Log.Verbose("{0}: Powering off", ToString());
			string CommandLine = "shell \"input keyevent KEYCODE_SLEEP\"";
			IProcessResult PowerOffQuery = RunAdbDeviceCommand(CommandLine, bPauseErrorParsing: true);
			return !PowerOffQuery.Output.Contains("error");
		}

		public bool Reboot()
		{
			return true;
		}

		public Dictionary<EIntendedBaseCopyDirectory, string> GetPlatformDirectoryMappings()
		{
			return LocalDirectoryMappings;
		}

		public override string ToString()
		{
			// TODO: device id
			if (Name == DeviceName)
			{
				return Name;
			}
			return string.Format("{0} ({1})", Name, DeviceName);
		}

		public void FullClean()
		{
			string StorageLocation = RunAdbDeviceCommand(AndroidPlatform.GetStorageQueryCommand()).Output.Trim();

			// Clean up any basic UE/System related directories
			RunAdbDeviceCommand(string.Format("shell rm -r {0}/UnrealGame/*", StorageLocation));
			RunAdbDeviceCommand(string.Format("shell rm -r {0}/Download/*", StorageLocation));
			RunAdbDeviceCommand(string.Format("shell rm -r {0}/gdeps/*", StorageLocation));

			// Now find all third party packages/obbs and remove them
			string PackageOutput = RunAdbDeviceCommandAndGetOutput("shell pm list packages -3");

			// Ex. package:com.epicgames.lyra\r\n
			Regex PackageRegex = new Regex("(package:)(.*)(\\r\\n)");
			IEnumerable<string> Packages = PackageRegex.Matches(PackageOutput).Select(Match => Match.Groups[2].Value);

			foreach (string Package in Packages)
			{
				Log.Info("Uninstalling {PackageName}...", Package);
				RunAdbDeviceCommand(string.Format("uninstall {0}", Package));

				RunAdbDeviceCommand(string.Format("shell rm -r {0}/Android/data/{1}", StorageLocation, Package));
				RunAdbDeviceCommand(string.Format("shell rm -r {0}/Android/obb/{1}", StorageLocation, Package));
			}
		}

		public void CleanArtifacts(UnrealAppConfig AppConfig = null)
		{
			if (AppConfig == null)
			{
				Log.Warning("{Platform} expects an non-null AppConfig value to reliably determine which artifact path to clear! Skipping clean", Platform);
				return;
			}

			if (AppConfig.Build is not AndroidBuild Build)
			{
				return;
			}

			bUseAFSWithoutToken = Build.UseAFSWithoutToken;
			ParsedAFSToken = Build.ParsedAFSToken;

			Log.Info("Cleaning up artifacts for {PackageName}...", Build.AndroidPackageName);

			string PackageOutput = RunAdbDeviceCommandAndGetOutput("shell pm list packages -3");

			bool bIsPackageInstalled = PackageOutput.Contains($"package:{Build.AndroidPackageName}");

			if (bIsPackageInstalled && UsingAndroidFileServer(AppConfig.ProjectFile, AppConfig.Configuration, out _, out string AFSToken, out _, out _, out _, bUseAFSWithoutToken, ParsedAFSToken))
			{
				RunAFSDeviceCommand(string.Format("-p \"{0}\" -k \"{1}\" rmdir -r \"^saved/\"", Build.AndroidPackageName, AFSToken));
			}
			else
			{
				string ExternalStoragePath = StoragePath + "/UnrealGame/" + AppConfig.ProjectName;
				string ExternalFilesPath = StoragePath + "/Android/data/" + Build.AndroidPackageName + "/files/UnrealGame/" + AppConfig.ProjectName;

				string ExternalAppStorage = string.Format("{0}/{1}/Saved", ExternalStoragePath, AppConfig.ProjectName);
				string ExternalAppFiles = string.Format("{0}/{1}/Saved", ExternalFilesPath, AppConfig.ProjectName);

				RunAdbDeviceCommand(string.Format("shell rm -r {0}/*", ExternalAppStorage));
				RunAdbDeviceCommand(string.Format("shell rm -r {0}/*", ExternalAppFiles));
			}
		}

		public void InstallBuild(UnrealAppConfig AppConfig)
		{
			AndroidBuild Build = AppConfig.Build as AndroidBuild;
			if (Build == null)
			{
				throw new AutomationException("Unsupported build type {0} for Android!", AppConfig.Build.GetType());
			}

			bUseAFSWithoutToken = Build.UseAFSWithoutToken;
			ParsedAFSToken = Build.ParsedAFSToken;

			string Package = Build.AndroidPackageName;
			KillRunningProcess(Package);

			// Install apk
			string APK = Globals.IsRunningDev && AppConfig.OverlayExecutable.GetOverlay(Build.SourceApkPath, out string OverlayAPK)
				? OverlayAPK
				: Build.SourceApkPath;
			CopyFileToDevice(Package, APK, string.Empty);

			EnablePermissions(Package);

			// Copy obbs from bulk builds
			bool bSkipOBBInstall = Globals.Params.ParseParam("SkipOBBCopy"); // useful when iterating on dev executables
			if (Build.FilesToInstall != null && Build.FilesToInstall.Any() && !bSkipOBBInstall)
			{
				CopyOBBFiles(AppConfig, Build.FilesToInstall, Package);
			}
		}

		public IAppInstall CreateAppInstall(UnrealAppConfig AppConfig)
		{
			if (AppConfig.Build is not AndroidBuild Build)
			{
				throw new AutomationException("Unsupported build type {0} for Android!", AppConfig.Build.GetType());
			}

			bUseAFSWithoutToken = Build.UseAFSWithoutToken;
			ParsedAFSToken = Build.ParsedAFSToken;

			string ExternalStoragePath = StoragePath + "/UnrealGame/" + AppConfig.ProjectName;
			string ExternalFilesPath = StoragePath + "/Android/data/" + Build.AndroidPackageName + "/files/UnrealGame/" + AppConfig.ProjectName;

			DeviceExternalStorageSavedPath = string.Format("{0}/{1}/Saved", ExternalStoragePath, AppConfig.ProjectName);
			DeviceExternalFilesSavedPath = string.Format("{0}/{1}/Saved", ExternalFilesPath, AppConfig.ProjectName);
			DeviceLogPath = string.Format("{0}/Logs/{1}.log", Build.UsesPublicLogs ? DeviceExternalFilesSavedPath : DeviceExternalStorageSavedPath, AppConfig.ProjectName);
			DeviceArtifactPath = Build.UsesExternalFilesDir ? DeviceExternalFilesSavedPath : DeviceExternalStorageSavedPath;

			PopulateDirectoryMappings(AppConfig);

			InstallCache = new AndroidAppInstall(AppConfig, this, AppConfig.ProjectName, Build.AndroidPackageName, AppConfig.CommandLine, AppConfig.ProjectFile, AppConfig.Build.Configuration);

			return InstallCache;
		}

		public void CopyAdditionalFiles(IEnumerable<UnrealFileToCopy> FilesToCopy)
		{
			foreach (UnrealFileToCopy FileToCopy in FilesToCopy)
			{
				string DestinationFile = string.Format("{0}/{1}", LocalDirectoryMappings[FileToCopy.TargetBaseDirectory], FileToCopy.TargetRelativeLocation);
				if (File.Exists(FileToCopy.SourceFileLocation))
				{
					FileInfo SourceFile = new FileInfo(FileToCopy.SourceFileLocation);
					SourceFile.IsReadOnly = false;

					// InstallCache may be null so I'll add back the previous call in that case
					if (InstallCache != null)
					{
						CopyFileToDevice(InstallCache.AndroidPackageName, SourceFile.FullName, DestinationFile, InstallCache.ProjectFile, InstallCache.Configuration);
					}
					else
					{
						CopyFileToDevice(null, SourceFile.FullName, DestinationFile);
					}
				}
				else
				{
					Log.Warning(KnownLogEvents.Gauntlet_DeviceEvent, "File to copy {File} not found", FileToCopy);
				}
			}
		}

		public IAppInstance Run(IAppInstall App)
		{
			if (App is not AndroidAppInstall Install)
			{
				throw new AutomationException("AppInstall is of incorrect type {0}! Must be of type AndroidAppInstall", App.GetType().Name);
			}

			if (!IsOn)
			{
				PowerOn();
			}

			// Now, create and push the UECommandline.txt file
			bool bUseExternalFileDirectory = (Install.AppConfig.Build as AndroidBuild)?.UsesExternalFilesDir ?? false;
			bUseAFSWithoutToken = (Install.AppConfig.Build as AndroidBuild)?.UseAFSWithoutToken ?? false;
			ParsedAFSToken = (Install.AppConfig.Build as AndroidBuild)?.ParsedAFSToken;
			CopyCommandlineFile(Install.AppConfig, Install.AppConfig.CommandLine, Install.AndroidPackageName, Install.AppConfig.ProjectName, bUseExternalFileDirectory);

			// kill any currently running instance:
			KillRunningProcess(Install.AndroidPackageName);

			string LaunchActivity = AndroidPlatform.GetLaunchableActivityName(Install.ApkPath);

			Log.Info("Launching {0} on '{1}' ", Install.AndroidPackageName + "/" + LaunchActivity, ToString());
			Log.Verbose("\t{0}", Install.CommandLine);

			// Clear the device's logcat in preparation for the test..
			RunAdbDeviceCommand("logcat --clear");

			// Ensure artifact directories exist
			if (UsingAndroidFileServer(Install.ProjectFile, Install.Configuration, out _, out string AFSToken, out _, out _, out _, bUseAFSWithoutToken, ParsedAFSToken))
			{
				RunAFSDeviceCommand(string.Format("-p \"{0}\" -k \"{1}\" mkdir \"^saved\"", Install.AndroidPackageName, AFSToken));
			}
			else
			{
				RunAdbDeviceCommand(string.Format("shell mkdir -p {0}/", DeviceExternalStorageSavedPath));
				RunAdbDeviceCommand(string.Format("shell mkdir -p {0}/", DeviceExternalFilesSavedPath));
			}

			if (!Globals.Params.ParseParam("NoAndroidScreenshots"))
			{
				ScreenshotThread = new Thread(StartScreenshotLoop);
				ScreenshotThread.Start();
			}

			if (!Globals.IsRunningDev && UseAFS(Install.AppConfig))
			{
				AndroidBuild Build = Install.AppConfig.Build as AndroidBuild;
				if (Path.GetFileName(Build.SourceApkPath).StartsWith("AFS_"))
				{
					Log.Info("Installing non-AFS apk before Launching the {0}", Build.AndroidPackageName);

					string QuotedSourcePath = Build.SourceApkPath.Replace("AFS_", "");
					if (QuotedSourcePath.Contains(" "))
					{
						QuotedSourcePath = '"' + QuotedSourcePath + '"';
					}

					IProcessResult AdbResult = RunAdbDeviceCommand(string.Format("install -r {0}", QuotedSourcePath));
					if (AdbResult.ExitCode != 0)
					{
						throw new DeviceException("Failed to install non-AFS APK {0}. Error {1}", QuotedSourcePath, AdbResult.Output);
					}
				}
			}

			// disable samsung game optimisation service, when -enableSamsungGOS not present.
			// log the attempt but ignore failure if not found etc.
			// we disable this to prevent GOS's 'touch protection' from down clocking the device.
			if (Globals.Params.ParseParam("enableSamsungGOS"))
			{
				IProcessResult AdbResult = RunAdbDeviceCommand("shell pm enable --user 0 com.samsung.android.game.gos", true, true);
				if (AdbResult.ExitCode != 0)
				{
					Log.Info("Failed to enable Samsung GOS");
				}
			}
			else
			{
				IProcessResult AdbResult = RunAdbDeviceCommand("shell pm disable-user --user 0 com.samsung.android.game.gos", true, true);
				if (AdbResult.ExitCode != 0)
				{
					Log.Info("Failed to disable Samsung GOS");
				}
			}

			// disable other popups that take focus away from the running app.
			// GMS will display 'google play is required'
			if (Globals.Params.ParseParam("disableAndroidGMS"))
			{
				IProcessResult AdbResult = RunAdbDeviceCommand("shell pm disable-user --user 0 com.google.android.gms", true, true);
				if (AdbResult.ExitCode != 0)
				{
					Log.Info("Failed to disable Google Mobile Services");
				}
			}
			else
			{
				IProcessResult AdbResult = RunAdbDeviceCommand("shell pm enable --user 0 com.google.android.gms", true, true);
				if (AdbResult.ExitCode != 0)
				{
					Log.Info("Failed to enable Google Mobile Services");
				}
			}

			// start the app on device!
			string CommandLine = "shell am start -W -S -n " + Install.AndroidPackageName + "/" + LaunchActivity;
			IProcessResult Process = RunAdbDeviceCommand(CommandLine, false, true);

			return new AndroidAppInstance(this, Install, Process);
		}
		#endregion

		public void PopulateDirectoryMappings(UnrealAppConfig AppConfig)
		{
			LocalDirectoryMappings.Clear();

			if (UseAFS(AppConfig))
			{
				LocalDirectoryMappings.Add(EIntendedBaseCopyDirectory.Build, "^build");
				LocalDirectoryMappings.Add(EIntendedBaseCopyDirectory.Binaries, "^binaries");
				LocalDirectoryMappings.Add(EIntendedBaseCopyDirectory.Config, "^saved/Config");
				LocalDirectoryMappings.Add(EIntendedBaseCopyDirectory.Content, "^content");
				LocalDirectoryMappings.Add(EIntendedBaseCopyDirectory.Demos, "^saved/Demos");
				LocalDirectoryMappings.Add(EIntendedBaseCopyDirectory.PersistentDownloadDir, "^saved/PersistentDownloadDir");
				LocalDirectoryMappings.Add(EIntendedBaseCopyDirectory.Profiling, "^saved/Profiling");
				LocalDirectoryMappings.Add(EIntendedBaseCopyDirectory.Saved, "^saved");
			}
			else
			{
				string ProjectDir = Path.GetDirectoryName(DeviceArtifactPath);
				ProjectDir = ProjectDir.Replace('\\', '/');
				if (!ProjectDir.EndsWith('/'))
				{
					ProjectDir += '/';
				}

				LocalDirectoryMappings.Add(EIntendedBaseCopyDirectory.Build, ProjectDir + "Build");
				LocalDirectoryMappings.Add(EIntendedBaseCopyDirectory.Binaries, ProjectDir + "Binaries");
				LocalDirectoryMappings.Add(EIntendedBaseCopyDirectory.Config, ProjectDir + "Saved/Config");
				LocalDirectoryMappings.Add(EIntendedBaseCopyDirectory.Content, ProjectDir + "Content");
				LocalDirectoryMappings.Add(EIntendedBaseCopyDirectory.Demos, ProjectDir + "Saved/Demos");
				LocalDirectoryMappings.Add(EIntendedBaseCopyDirectory.PersistentDownloadDir, ProjectDir + "Saved/PersistentDownloadDir");
				LocalDirectoryMappings.Add(EIntendedBaseCopyDirectory.Profiling, ProjectDir + "Saved/Profiling");
				LocalDirectoryMappings.Add(EIntendedBaseCopyDirectory.Saved, ProjectDir + "Saved");
			}
		}

		public static bool UsingAndroidFileServer(FileReference RawProjectPath, UnrealTargetConfiguration TargetConfiguration, out bool bEnablePlugin, out string AFSToken, out bool bIsShipping, out bool bIncludeInShipping, out bool bAllowExternalStartInShipping, bool bUseAFSWithoutToken = false, string ParsedAFSToken = null)
		{
			if (bUseAFSWithoutToken)
			{
				bEnablePlugin = true;
				AFSToken = "";
				bIsShipping = false;
				bIncludeInShipping = false;
				bAllowExternalStartInShipping = false;
				return true;
			}
			if (!string.IsNullOrEmpty(ParsedAFSToken))
			{
				bEnablePlugin = true;
				AFSToken = ParsedAFSToken;
				bIsShipping = false;
				bIncludeInShipping = false;
				bAllowExternalStartInShipping = false;
				return true;
			}
			if (RawProjectPath == null || !FileReference.Exists(RawProjectPath) || TargetConfiguration == UnrealTargetConfiguration.Unknown)
			{
				bEnablePlugin = false;
				AFSToken = string.Empty;
				bIsShipping = false;
				bIncludeInShipping = false;
				bAllowExternalStartInShipping = false;
				return false;
			}

			UnrealTargetPlatform TargetPlatform = UnrealTargetPlatform.Android;
			bIsShipping = TargetConfiguration == UnrealTargetConfiguration.Shipping;

			ConfigHierarchy Ini = ConfigCache.ReadHierarchy(ConfigHierarchyType.Engine, DirectoryReference.FromFile(RawProjectPath), TargetPlatform);
			if (!Ini.GetBool("/Script/AndroidFileServerEditor.AndroidFileServerRuntimeSettings", "bEnablePlugin", out bEnablePlugin))
			{
				bEnablePlugin = true;
			}
			if (!Ini.GetString("/Script/AndroidFileServerEditor.AndroidFileServerRuntimeSettings", "SecurityToken", out AFSToken))
			{
				AFSToken = "";
			}
			if (!Ini.GetBool("/Script/AndroidFileServerEditor.AndroidFileServerRuntimeSettings", "bIncludeInShipping", out bIncludeInShipping))
			{
				bIncludeInShipping = false;
			}
			if (!Ini.GetBool("/Script/AndroidFileServerEditor.AndroidFileServerRuntimeSettings", "bAllowExternalStartInShipping", out bAllowExternalStartInShipping))
			{
				bAllowExternalStartInShipping = false;
			}

			if (bIsShipping && !(bIncludeInShipping && bAllowExternalStartInShipping))
			{
				return false;
			}
			return bEnablePlugin;
		}

		public bool UseAFS(UnrealAppConfig AppConfig)
		{
			AndroidBuild Build = AppConfig.Build as AndroidBuild;
			if (Build == null)
			{
				throw new AutomationException("Unsupported build type {0} for Android!", AppConfig.Build.GetType());
			}

			bUseAFSWithoutToken = Build.UseAFSWithoutToken;
			ParsedAFSToken = Build.ParsedAFSToken;

			return UsingAndroidFileServer(AppConfig.ProjectFile, Build.Configuration, out _, out _, out _, out _, out _, bUseAFSWithoutToken, ParsedAFSToken);
		}

		/// <summary>
		/// Runs an ADB command, automatically adding the name of the current device to
		/// the arguments sent to adb
		/// </summary>
		/// <param name="Args"></param>
		/// <param name="Wait"></param>
		/// <param name="bShouldLogCommand"></param>
		/// <param name="bPauseErrorParsing"></param>
		/// <returns></returns>
		public IProcessResult RunAdbDeviceCommand(string Args, bool Wait = true, bool bShouldLogCommand = false, bool bPauseErrorParsing = false, int TimeoutSec = 60 * 15)
		{
			if (string.IsNullOrEmpty(DeviceName) == false)
			{
				Args = string.Format("-s {0} {1}", DeviceName, Args);
			}

			return RunAdbGlobalCommand(Args, Wait, bShouldLogCommand, bPauseErrorParsing, TimeoutSec);
		}

		public IProcessResult RunAFSDeviceCommand(string Args, bool Wait = true, bool bShouldLogCommand = false, bool bPauseErrorParsing = false, int TimeoutSec = 60 * 15)
		{
			if (string.IsNullOrEmpty(DeviceName) == false)
			{
				Args = string.Format("-s {0} {1}", DeviceName, Args);
			}

			return RunAFSGlobalCommand(Args, Wait, bShouldLogCommand, bPauseErrorParsing, TimeoutSec);
		}

		/// <summary>
		/// Runs an ADB command, automatically adding the name of the current device to
		/// the arguments sent to adb
		/// </summary>
		/// <param name="Args"></param>
		/// <returns></returns>
		public string RunAdbDeviceCommandAndGetOutput(string Args)
		{
			if (string.IsNullOrEmpty(DeviceName) == false)
			{
				Args = string.Format("-s {0} {1}", DeviceName, Args);
			}

			IProcessResult Result = RunAdbGlobalCommand(Args);

			if (Result.ExitCode != 0)
			{
				throw new DeviceException("adb command {0} failed. {1}", Args, Result.Output);
			}

			return Result.Output;
		}

		/// <summary>
		/// Runs an ADB command at the global scope
		/// </summary>
		/// <param name="Args"></param>
		/// <param name="Wait"></param>
		/// <param name="bShouldLogCommand"></param>
		/// <param name="bPauseErrorParsing"></param>
		/// <returns></returns>
		public static IProcessResult RunAdbGlobalCommand(string Args, bool Wait = true, bool bShouldLogCommand = false, bool bPauseErrorParsing = false, int TimeoutSec = 60 * 15)
		{
			CommandUtils.ERunOptions RunOptions = CommandUtils.ERunOptions.AppMustExist | CommandUtils.ERunOptions.NoWaitForExit | CommandUtils.ERunOptions.SpewIsVerbose;
			RunOptions |= Log.IsVeryVerbose ? ERunOptions.AllowSpew : ERunOptions.NoLoggingOfRunCommand;

			if (bShouldLogCommand)
			{
				Log.Verbose("Running ADB Command: adb {0}", Args);
			}

			IProcessResult Process;

			using (bPauseErrorParsing ? new ScopedSuspendECErrorParsing() : null)
			{
				Process = AndroidPlatform.RunAdbCommand(null, null, Args, null, RunOptions);

				if (Wait)
				{
					Process.ProcessObject.WaitForExit(TimeoutSec * 1000);

					if (!Process.HasExited)
					{
						Log.Info("ADB Command 'adb {Args}' timed out after {Timeout} sec.", Args, TimeoutSec);

						Process.ProcessObject?.Kill();
						Process.ProcessObject?.WaitForExit(15000);
					}
				}
			}

			return Process;
		}

		/// <summary>
		/// Run Adb command without waiting for exit
		/// </summary>
		/// <param name="Args"></param>
		/// <returns></returns>
		public ILongProcessResult RunAdbCommandNoWait(string Args, string LocalCache)
		{
			if (string.IsNullOrEmpty(DeviceName) == false)
			{
				Args = string.Format("-s {0} {1}", DeviceName, Args);
			}
			CommandUtils.ERunOptions RunOptions = CommandUtils.ERunOptions.AppMustExist | CommandUtils.ERunOptions.NoWaitForExit | CommandUtils.ERunOptions.SpewIsVerbose;
			RunOptions |= Log.IsVeryVerbose ? ERunOptions.AllowSpew : ERunOptions.NoLoggingOfRunCommand;

			string AdbCommand = Environment.ExpandEnvironmentVariables("%ANDROID_HOME%/platform-tools/adb" + (OperatingSystem.IsWindows() ? ".exe" : ""));
			return new LongProcessResult(AdbCommand, Args, RunOptions, LocalCache: LocalCache);
		}

		public static IProcessResult RunAFSGlobalCommand(string Args, bool Wait = true, bool bShouldLogCommand = false, bool bPauseErrorParsing = false, int TimeoutSec = 60 * 15)
		{
			CommandUtils.ERunOptions RunOptions = CommandUtils.ERunOptions.AppMustExist | CommandUtils.ERunOptions.NoWaitForExit | CommandUtils.ERunOptions.SpewIsVerbose;

			if (Log.IsVeryVerbose)
			{
				RunOptions |= CommandUtils.ERunOptions.AllowSpew;
			}
			else
			{
				RunOptions |= CommandUtils.ERunOptions.NoLoggingOfRunCommand;
			}

			if (bShouldLogCommand)
			{
				Log.Verbose("Running AFS Command: UnrealAndroidFileTool {0}", Args);
			}

			IProcessResult Process;

			using (bPauseErrorParsing ? new ScopedSuspendECErrorParsing() : null)
			{
				Process = AndroidPlatform.RunAFSCommand(null, null, Args, null, RunOptions);

				if (Wait)
				{
					Process.ProcessObject.WaitForExit(TimeoutSec * 1000);

					if (!Process.HasExited)
					{
						Log.Info("AFS Command 'UnrealAndroidFileTool {Args}' timed out after {Timeout} sec.", Args, TimeoutSec);

						Process.ProcessObject?.Kill();
						Process.ProcessObject?.WaitForExit(15000);
					}
				}
			}

			return Process;
		}


		public static ITargetDevice[] GetDefaultDevices()
		{
			return GetAllAvailableDevices().Select(Device => new TargetDeviceAndroid(Device)).ToArray();
		}

		public void PostRunCleanup(UnrealAppConfig AppConfig, string Package)
		{
			// Delete the commandline file, if someone installs an APK on top of ours
			// they will get very confusing behavior...
			if (string.IsNullOrEmpty(CommandLineFilePath) == false)
			{
				Log.Verbose("Removing {0}", CommandLineFilePath);
				if (UseAFS(AppConfig))
				{
					DeleteFileFromDevice(Package, "^commandfile", AppConfig.ProjectFile, AppConfig.Configuration);
				}
				else
				{
					DeleteFileFromDevice(Package, CommandLineFilePath);
				}
			}
		}

		public string AFSGetFileInfo(string PackageName, string AFSToken, string FileName)
		{
			IProcessResult result;
			if (FileName.StartsWith("^"))
			{
				// Query to get which file this actually is.
				string ActualFileName = "";
				result = RunAFSDeviceCommand(string.Format("-p \"{0}\" -k \"{1}\" query", PackageName, AFSToken));
				string[] mappings = result.Output.Split(new[] { "\r\n" }, StringSplitOptions.RemoveEmptyEntries).Select(S => S.Trim()).ToArray();
				foreach (string mapping in mappings)
				{
					string[] FromTo = mapping.Split(new[] { " " }, StringSplitOptions.RemoveEmptyEntries).Select(S => S.Trim()).ToArray();
					if (FromTo.Length != 2)
					{
						continue;
					}
					if (FromTo[0] == FileName)
					{
						ActualFileName = Path.GetFileName(FromTo[1]);
						break;
					}
				}

				if (string.IsNullOrEmpty(ActualFileName))
				{
					return "";
					//throw new AutomationException("Failed to find file {0} in AFS mappings", FileName);
				}

				// Now List all the files in ^obb and find our file
				result = RunAFSDeviceCommand(string.Format("-p \"{0}\" -k \"{1}\" ls -ls \"^obb\"", PackageName, AFSToken));
				string[] ObbFiles = result.Output.Split(new[] { "\r\n" }, StringSplitOptions.RemoveEmptyEntries).Select(S => S.Trim()).ToArray();
				foreach (string file in ObbFiles)
				{
					if (file.Contains(ActualFileName))
					{
						return file;
					}
				}

				// Now List all the files in ^project and find our file
				result = RunAFSDeviceCommand(string.Format("-p \"{0}\" -k \"{1}\" ls -ls \"^project\"", PackageName, AFSToken));
				string[] ProjectFiles = result.Output.Split(new[] { "\r\n" }, StringSplitOptions.RemoveEmptyEntries).Select(S => S.Trim()).ToArray();
				foreach (string file in ProjectFiles)
				{
					if (file.Contains(ActualFileName))
					{
						return file;
					}
				}

				//throw new AutomationException("Failed to find file {0} in AFS ^obb", FileName);
				return "";
			}
			else
			{
				//throw new AutomationException("Unsupported file {0} for AFS", FileName);
				return "";
			}
		}

		public bool CopyFileToDevice(string PackageName, string SourcePath, string DestPath,
			FileReference ProjectFile = null, UnrealTargetConfiguration Configuration = UnrealTargetConfiguration.Unknown, bool IgnoreDependencies = false)
		{
			string AFSToken;
			bool UseAFS = UsingAndroidFileServer(ProjectFile, Configuration, out _, out AFSToken, out _, out _, out _, bUseAFSWithoutToken, ParsedAFSToken);
			bool IsAPK = string.Equals(Path.GetExtension(SourcePath), ".apk", StringComparison.OrdinalIgnoreCase);

			// for the APK there's no easy/reliable way to get the date of the version installed, so
			// we write this out to a dependency file in the demote dir and check it each time.
			// current file time
			DateTime LocalModifiedTime = File.GetLastWriteTime(SourcePath);

			// If not using AFS and the APK starts with AFS_, rename to use the normal one
			if (!Globals.IsRunningDev && IsAPK && !UseAFS)
			{
				if (Path.GetFileName(SourcePath).StartsWith("AFS_"))
				{
					SourcePath = SourcePath.Replace("AFS_", "");
				}
			}

			string QuotedSourcePath = SourcePath;
			if (SourcePath.Contains(" "))
			{
				QuotedSourcePath = '"' + SourcePath + '"';
			}

			// dependency info is a hash of the destination name, saved under a folder on /sdcard
			string DestHash = ContentHash.MD5(DestPath).ToString();
			string DependencyCacheDir = "/sdcard/gdeps";
			string DepFile = string.Format("{0}/{1}", DependencyCacheDir, DestHash);

			IProcessResult AdbResult = null;


			// get info from the device about this file
			string CurrentFileInfo = null;

			if (IsAPK)
			{
				// for APK query the package info and get the update time
				AdbResult = RunAdbDeviceCommand(string.Format("shell dumpsys package {0} | grep lastUpdateTime", PackageName));

				if (AdbResult.ExitCode == 0)
				{
					CurrentFileInfo = AdbResult.Output.ToString().Trim();
				}
			}
			else
			{
				// for other files get the file info
				if (UseAFS)
				{
					CurrentFileInfo = AFSGetFileInfo(PackageName, AFSToken, DestPath);
				}
				else
				{
					AdbResult = RunAdbDeviceCommand(string.Format("shell ls -l {0}", DestPath));
					if (AdbResult.ExitCode == 0)
					{
						CurrentFileInfo = AdbResult.Output.ToString().Trim();
					}
				}

			}

			bool SkipInstall = false;

			// If this is valid then there is some form of that file on the device, now figure out if it matches the 
			if (string.IsNullOrEmpty(CurrentFileInfo) == false)
			{
				// read the dep file
				AdbResult = RunAdbDeviceCommand(string.Format("shell cat {0}", DepFile));

				if (AdbResult.ExitCode == 0)
				{
					// Dependency info is the modified time of the source, and the post-copy file stats of the installed file, separated by ###
					string[] DepLines = AdbResult.Output.ToString().Split(new[] { "###" }, StringSplitOptions.RemoveEmptyEntries).Select(S => S.Trim()).ToArray();

					if (DepLines.Length >= 2)
					{
						string InstalledSourceModifiedTime = DepLines[0];
						string InstalledFileInfo = DepLines[1];

						if (InstalledSourceModifiedTime == LocalModifiedTime.ToString()
							&& CurrentFileInfo == InstalledFileInfo)
						{
							SkipInstall = true;
						}
					}
				}
			}

			if (SkipInstall && IgnoreDependencies == false)
			{
				Log.Info("Skipping install of {0} - remote file up to date", Path.GetFileName(SourcePath));
			}
			else
			{
				if (IsAPK)
				{
					// we need to ununstall then install the apk - don't care if it fails, may have been deleted
					string AdbCommand = string.Format("uninstall {0}", PackageName);
					AdbResult = RunAdbDeviceCommand(AdbCommand);

					Log.Info("Installing {0} to {1}", SourcePath, Name);

					AdbCommand = string.Format("install {0}", QuotedSourcePath);
					AdbResult = RunAdbDeviceCommand(AdbCommand);

					if (AdbResult.ExitCode != 0)
					{
						throw new DeviceException("Failed to install {0}. Error {1}", SourcePath, AdbResult.Output);
					}

					// for APK query the package info and get the update time
					AdbResult = RunAdbDeviceCommand(string.Format("shell dumpsys package {0} | grep lastUpdateTime", PackageName));
					CurrentFileInfo = AdbResult.Output.ToString().Trim();
				}
				else
				{
					string FileDirectory = Path.GetDirectoryName(DestPath).Replace('\\', '/');

					if (UseAFS)
					{
						Log.Info("Copying {0} to {1} via AFS", QuotedSourcePath, DestPath);
						AdbResult = RunAFSDeviceCommand(string.Format("-p \"{0}\" -k \"{1}\" push \"{2}\" \"{3}\"", PackageName, AFSToken, SourcePath, DestPath), TimeoutSec: MaxInstallTime);
					}
					else
					{
						RunAdbDeviceCommand(string.Format("shell mkdir -p {0}/", FileDirectory));

						Log.Info("Copying {0} to {1} via adb push", QuotedSourcePath, DestPath);
						AdbResult = RunAdbDeviceCommand(string.Format("push {0} {1}", QuotedSourcePath, DestPath), TimeoutSec: MaxInstallTime);
					}

					// Note: Presently, AdbResult NEVER reports failures to push when using AFS. Should be fixed at some point.

					if (AdbResult.ExitCode != 0)
					{
						if ((AdbResult.Output.Contains("couldn't read from device") || AdbResult.Output.Contains("offline")) && !IsConnected)
						{
							Log.Info("Lost connection with device '{Name}'.", Name);
							// Disconnection occurred. Let's retry a second time before given up
							// Try to reconnect
							if (Connect())
							{
								if (UseAFS)
								{
									Log.Info("Retrying to copy via AFS...");
									AdbResult = RunAFSDeviceCommand(string.Format("-p {0} -k {1} push \"{2}\" \"{3}\"", PackageName, AFSToken, SourcePath, DestPath), TimeoutSec: MaxInstallTime);
								}
								else
								{
									Log.Info("Retrying to copy via adb push...");
									AdbResult = RunAdbDeviceCommand(string.Format("push {0} {1}", QuotedSourcePath, DestPath), TimeoutSec: MaxInstallTime);
								}
								if (AdbResult.ExitCode != 0)
								{
									throw new DeviceException("Failed to push {0} to device. Error {1}", SourcePath, AdbResult.Output);
								}
							}
							else
							{
								throw new DeviceException("Failed to reconnect {0}", Name);
							}
						}
						else
						{
							throw new DeviceException("Failed to push {0} to device. Error {1}", SourcePath, AdbResult.Output);
						}
					}

					// Now pull info about the file which we'll write as a dep
					if (UseAFS)
					{
						CurrentFileInfo = AFSGetFileInfo(PackageName, AFSToken, DestPath);
					}
					else
					{
						AdbResult = RunAdbDeviceCommand(string.Format("shell ls -l {0}", DestPath));
						CurrentFileInfo = AdbResult.Output.ToString().Trim();
					}
				}

				// write the actual dependency info
				string DepContents = LocalModifiedTime + "###" + CurrentFileInfo;

				// save last modified time to remote deps after success
				RunAdbDeviceCommand(string.Format("shell mkdir -p {0}", DependencyCacheDir));
				AdbResult = RunAdbDeviceCommand(string.Format("shell echo \"{0}\" > {1}", DepContents, DepFile));

				if (AdbResult.ExitCode != 0)
				{
					Log.Warning(KnownLogEvents.Gauntlet_DeviceEvent, "Failed to write dependency file {File}", DepFile);
				}
			}

			return true;
		}

		public void AllowDeviceSleepState(bool bAllowSleep)
		{
			string CommandLine = "shell svc power stayon " + (bAllowSleep ? "false" : "usb");
			RunAdbDeviceCommand(CommandLine, true, false, true);
		}
		/// <summary>
		/// Enable Android permissions which would otherwise block automation with permission requests
		/// </summary>
		public void EnablePermissions(string AndroidPackageName)
		{
			List<string> Permissions = new List<string> { "POST_NOTIFICATIONS", "WRITE_EXTERNAL_STORAGE", "READ_EXTERNAL_STORAGE", "GET_ACCOUNTS", "RECORD_AUDIO" };
			Permissions.ForEach(Permission =>
			{
				string CommandLine = string.Format("shell pm grant {0} android.permission.{1}", AndroidPackageName, Permission);
				Log.Verbose(string.Format("Enabling permission: {0} {1}", AndroidPackageName, Permission));
				RunAdbDeviceCommand(CommandLine, true, false, true);
			});
			Log.Verbose($"Enabling permission: {AndroidPackageName} MANAGE_EXTERNAL_STORAGE");
			RunAdbDeviceCommand($"shell appops set {AndroidPackageName} MANAGE_EXTERNAL_STORAGE allow");
		}

		public void KillRunningProcess(string AndroidPackageName)
		{
			Log.Verbose("{0}: Killing process '{1}' ", ToString(), AndroidPackageName);
			string KillProcessCommand = string.Format("shell am force-stop {0}", AndroidPackageName);
			RunAdbDeviceCommand(KillProcessCommand);
		}

		protected void CopyOBBFiles(UnrealAppConfig AppConfig, Dictionary<string, string> OBBFiles, string Package)
		{
			bool bAFSEnablePlugin;
			string AFSToken = "";
			bool bIsShipping;
			bool bAFSIncludeInShipping;
			bool bAFSAllowExternalStartInShipping;
			bool useADB = true;
			if (AppConfig != null)
			{
				AndroidBuild Build = AppConfig.Build as AndroidBuild;
				if (Build == null)
				{
					throw new AutomationException("Unsupported build type {0} for Android!", AppConfig.Build.GetType());
				}

				bUseAFSWithoutToken = Build.UseAFSWithoutToken;
				ParsedAFSToken = Build.ParsedAFSToken;

				if (UsingAndroidFileServer(AppConfig.ProjectFile, Build.Configuration, out bAFSEnablePlugin, out AFSToken, out bIsShipping, out bAFSIncludeInShipping, out bAFSAllowExternalStartInShipping, bUseAFSWithoutToken, ParsedAFSToken))
				{
					useADB = false;
				}
			}

			if (useADB)
			{
				string OBBRemoteDestination = string.Format("{0}/Android/obb/{1}", StoragePath, Package);

				// Remove all existing obbs
				RunAdbDeviceCommand(string.Format("shell rm {0}", OBBRemoteDestination));
				RunAdbDeviceCommand(string.Format("shell mkdir -p {0}", OBBRemoteDestination));
			}
			else
			{
				RunAFSDeviceCommand(string.Format("-p {0} -k {1} rm {2}", Package, AFSToken, "^mainobb"));
				RunAFSDeviceCommand(string.Format("-p {0} -k {1} rm {2}", Package, AFSToken, "^patchobb"));
				for (int i = 1; i <= 32; i++)
				{
					string overflowobb = string.Format("^overflow{0}obb", i);
					RunAFSDeviceCommand(string.Format("-p {0} -k {1} rm {2}", Package, AFSToken, overflowobb));
				}
			}

			// obb files need to be named based on APK version (grrr), so find that out. This should return something like
			// versionCode=2 minSdk=21 targetSdk=21
			string PackageInfo = RunAdbDeviceCommand(string.Format("shell dumpsys package {0} | grep versionCode", Package)).Output;
			var Match = Regex.Match(PackageInfo, @"versionCode=([\d\.]+)\s");
			if (Match.Success == false)
			{
				throw new AutomationException("Failed to find version info for APK!");
			}
			string PackageVersion = Match.Groups[1].ToString();

			foreach (KeyValuePair<string, string> Pair in OBBFiles)
			{
				string SourceFile = Pair.Key;
				string DestinationFile = Pair.Value;

				if (useADB)
				{
					// If we installed a new APK we need to change the package version
					Match OBBMatch = Regex.Match(SourceFile, @"\.(\d+)\.com.*\.obb");
					if (OBBMatch.Success)
					{
						string StrippedObb = Path.GetFileName(SourceFile.Replace(".Client.obb", ".obb").Replace(OBBMatch.Groups[1].ToString(), PackageVersion));
						DestinationFile = StoragePath + "/Android/obb/" + Package + "/" + StrippedObb;
					}

					DestinationFile = Regex.Replace(DestinationFile, "%STORAGE%", StoragePath, RegexOptions.IgnoreCase);
					CopyFileToDevice(null, SourceFile, DestinationFile, AppConfig.ProjectFile, AppConfig.Configuration);
				}
				else
				{
					string Filename = Path.GetFileName(SourceFile);
					string[] parts = Filename.Split(new[] { "." }, StringSplitOptions.RemoveEmptyEntries).Select(S => S.Trim()).ToArray();
					// Make sure last part is obb
					if (parts[parts.Length - 1] == "obb")
					{
						string obbname = string.Format("^{0}obb", parts[0]);
						CopyFileToDevice(Package, SourceFile, obbname, AppConfig.ProjectFile, AppConfig.Configuration);
					}
					else
					{
						DestinationFile = Regex.Replace(DestinationFile, "%STORAGE%", StoragePath, RegexOptions.IgnoreCase);
						CopyFileToDevice(Package, SourceFile, DestinationFile, AppConfig.ProjectFile, AppConfig.Configuration);
					}
				}
			}
		}

		protected void CopyCommandlineFile(UnrealAppConfig AppConfig, string Commandline, string Package, string Project, bool bUseExternalFileDirectory)
		{
			string ExternalStoragePath = StoragePath + "/UnrealGame/" + Project;
			string ExternalFilesPath = StoragePath + "/Android/data/" + Package + "/files/UnrealGame/" + Project;
			string DeviceBaseDir = bUseExternalFileDirectory ? ExternalFilesPath : ExternalStoragePath;
			CommandLineFilePath = string.Format("{0}/UECommandLine.txt", DeviceBaseDir);

			PostRunCleanup(AppConfig, Package); // function needs a rename in this context, just delete any old UECommandlines

			// Create a tempfile, insert the command line, and push it over
			string CommandLineTmpFile = Path.GetTempFileName();

			// I've seen a weird thing where adb push truncates by a byte, so add some padding...
			File.WriteAllText(CommandLineTmpFile, Commandline + "    ");

			if (UseAFS(AppConfig))
			{
				CopyFileToDevice(Package, CommandLineTmpFile, "^commandfile", AppConfig.ProjectFile, AppConfig.Configuration);
			}
			else
			{
				CopyFileToDevice(Package, CommandLineTmpFile, CommandLineFilePath);
			}

			File.Delete(CommandLineTmpFile);
		}

		protected bool DeleteFileFromDevice(string Package, string DestPath,
			FileReference ProjectFile = null, UnrealTargetConfiguration Configuration = UnrealTargetConfiguration.Unknown)
		{
			string AFSToken;
			bool UseAFS = UsingAndroidFileServer(ProjectFile, Configuration, out _, out AFSToken, out _, out _, out _, bUseAFSWithoutToken, ParsedAFSToken);

			IProcessResult AdbResult;
			if (UseAFS)
			{
				AdbResult = RunAFSDeviceCommand(string.Format("-p \"{0}\" -k \"{1}\" rm \"{2}\"", Package, AFSToken, DestPath));
			}
			else
			{
				AdbResult = RunAdbDeviceCommand(string.Format("shell rm -f {0}", DestPath));
			}
			return AdbResult.ExitCode == 0;
		}

		/// <summary>
		/// Returns a list of locally connected devices (e.g. 'adb devices').
		/// </summary>
		/// <returns></returns>
		private static Dictionary<string, string> GetAllConnectedDevices()
		{
			var Result = RunAdbGlobalCommand("devices");

			MatchCollection DeviceMatches = Regex.Matches(Result.Output, @"^([\d\w\.\:\-]{6,32})\s+(\w+)", RegexOptions.Multiline);

			var DeviceList = DeviceMatches.Cast<Match>().ToDictionary(
				M => M.Groups[1].ToString(),
				M => M.Groups[2].ToString().ToLower()
			);

			return DeviceList;
		}

		private static IEnumerable<string> GetAllAvailableDevices()
		{
			var AllDevices = GetAllConnectedDevices();
			return AllDevices.Keys.Where(D => AllDevices[D] == "device");
		}

		private void StartScreenshotLoop()
		{
			TimeSpan Period = TimeSpan.FromSeconds(30);
			ScreenshotTrigger = new AutoResetEvent(false);

			while (!ScreenshotTrigger.WaitOne(Period))
			{
				TakeScreenshot(string.Empty);
			}
		}
		/// <summary>
		/// Stops Screenshot thread, pulls screenshots from sdcard, and cleans up temp dir on the phone. 
		/// </summary>
		/// <param name="OutputDirectory"></param>
		public void StopScreenshotLoop(string OutputDirectory = "")
		{
			if (ScreenshotThread == null || ScreenshotTrigger == null)
			{
				Log.Verbose("Attempted to end the screenshot loop before it started. Ignoring.");
				return;
			}

			ScreenshotTrigger.Set();
			ScreenshotThread.Join();
			ScreenshotThread = null;
			ScreenshotTrigger = null;

			if (!string.IsNullOrEmpty(OutputDirectory))
			{
				try
				{
					if (!DirectoryExists(OutputDirectory))
					{
						CreateDirectory(OutputDirectory);
					}

					RunAdbDeviceCommand(string.Format($"pull /sdcard/Screenshots {OutputDirectory}"));
					var files = Directory.GetFiles(OutputDirectory);
					foreach (var file in files)
					{
						Log.Info("{0}", file);
					}
					Log.Info("Pulling Screenshots from phone to {OutputDirectory}", OutputDirectory);
					RunAdbDeviceCommand("shell rm -r /sdcard/Screenshots");

				}
				catch (Exception Ex)
				{
					Log.Info("Encountered an {ExceptionType} when trying to copy screenshots to requested output directory. " +
						"Screenshots will not be available: {ExceptionMessage}", Ex.GetType().Name, Ex.Message);
				}
			}
		}
		/// <summary>
		/// Takes a screenshot using adb and saves to the phone's sdcard in a temp directory. 
		/// </summary>
		public void TakeScreenshot(string ScreenshotName)
		{
			if (string.IsNullOrEmpty(ScreenshotName))
			{
				ScreenshotName = $"{DateTime.Now.ToString("yy-MM-dd-HH-mm-ss-ff")}.png";
			}
			try
			{
				RunAdbDeviceCommand("shell mkdir -p /sdcard/Screenshots/");
				RunAdbDeviceCommand(string.Format($"shell screencap -p /sdcard/Screenshots/{ScreenshotName}"));

				Log.Verbose("Saved screenshot: {ScreenshotFile}", ScreenshotName);
			}
			catch (Exception Ex)
			{
				Log.Info("Encountered an {ExceptionType} when trying to capture a screenshot: {ExceptionMessage}", Ex.GetType().Name, Ex.Message);
			}
		}

		#region Legacy Implementations
		public IAppInstall InstallApplication(UnrealAppConfig AppConfig)
		{
			// todo - pass this through
			AndroidBuild Build = AppConfig.Build as AndroidBuild;

			// Ensure APK exists
			if (Build == null)
			{
				throw new AutomationException("Invalid build for Android!");
			}

			bool bAFSEnablePlugin;
			string AFSToken = "";
			bool bIsShipping;
			bool bAFSIncludeInShipping;
			bool bAFSAllowExternalStartInShipping;
			bool useADB = true;

			bUseAFSWithoutToken = Build.UseAFSWithoutToken;
			ParsedAFSToken = Build.ParsedAFSToken;
			if (AppConfig != null)
			{
				if (UsingAndroidFileServer(AppConfig.ProjectFile, Build.Configuration, out bAFSEnablePlugin, out AFSToken, out bIsShipping, out bAFSIncludeInShipping, out bAFSAllowExternalStartInShipping, bUseAFSWithoutToken, ParsedAFSToken))
				{
					useADB = false;
				}
			}

			// kill any currently running instance:
			KillRunningProcess(Build.AndroidPackageName);

			// "/mnt/sdcard";
			// remote dir used to save things
			string ExternalStoragePath = StoragePath + "/UnrealGame/" + AppConfig.ProjectName;
			string ExternalFilesPath = StoragePath + "/Android/data/" + Build.AndroidPackageName + "/files/UnrealGame/" + AppConfig.ProjectName;
			string DeviceBaseDir = Build.UsesExternalFilesDir ? ExternalFilesPath : ExternalStoragePath;

			// path to the APK to install.
			string ApkPath = Build.SourceApkPath;

			// get the device's external file paths, always clear between runs
			DeviceExternalStorageSavedPath = string.Format("{0}/{1}/Saved", ExternalStoragePath, AppConfig.ProjectName);
			DeviceExternalFilesSavedPath = string.Format("{0}/{1}/Saved", ExternalFilesPath, AppConfig.ProjectName);
			DeviceLogPath = string.Format("{0}/Logs/{1}.log", Build.UsesPublicLogs ? DeviceExternalFilesSavedPath : DeviceExternalStorageSavedPath, AppConfig.ProjectName);
			DeviceArtifactPath = Build.UsesExternalFilesDir ? DeviceExternalFilesSavedPath : DeviceExternalStorageSavedPath;

			// path for OBB files
			string OBBRemoteDestination = string.Format("{0}/Android/obb/{1}", StoragePath, Build.AndroidPackageName);

			Log.Info("DeviceBaseDir: " + DeviceBaseDir);
			Log.Info("DeviceExternalStorageSavedPath: " + DeviceExternalStorageSavedPath);
			Log.Info("DeviceExternalFilesSavedPath: " + DeviceExternalFilesSavedPath);
			Log.Info("DeviceLogPath: " + DeviceLogPath);
			Log.Info("DeviceArtifactPath: " + DeviceArtifactPath);

			// clear all file store paths between installs:
			RunAdbDeviceCommand(string.Format("shell rm -r {0}", DeviceExternalStorageSavedPath));
			RunAdbDeviceCommand(string.Format("shell rm -r {0}", DeviceExternalFilesSavedPath));

			if (AppConfig.FullClean)
			{
				Log.Info("Fully cleaning console before install...");
				RunAdbDeviceCommand(string.Format("shell rm -r {0}/UnrealGame/*", StoragePath));
				RunAdbDeviceCommand(string.Format("shell rm -r {0}/Android/data/{1}/files/*", StoragePath, Build.AndroidPackageName));
				RunAdbDeviceCommand(string.Format("shell rm -r {0}/obb/{1}/*", StoragePath, Build.AndroidPackageName));
				RunAdbDeviceCommand(string.Format("shell rm -r {0}/Android/obb/{1}/*", StoragePath, Build.AndroidPackageName));
				RunAdbDeviceCommand(string.Format("shell rm -r {0}/Download/*", StoragePath));
			}

			if (!AppConfig.SkipInstall)
			{
				if (Globals.Params.ParseParam("cleandevice")
					|| AppConfig.FullClean)
				{
					Log.Info("Cleaning previous builds due to presence of -cleandevice");

					// we need to ununstall then install the apk - don't care if it fails, may have been deleted
					Log.Info("Uninstalling {0}", Build.AndroidPackageName);
					RunAdbDeviceCommand(string.Format("uninstall {0}", Build.AndroidPackageName));

					// delete DeviceExternalStorageSavedPath, note: DeviceExternalFilesSavedPath is removed with package uninstall.
					Log.Info("Removing {0}", DeviceExternalStorageSavedPath);
					RunAdbDeviceCommand(string.Format("shell rm -r {0}", DeviceExternalStorageSavedPath));

					Log.Info("Removing {0}", OBBRemoteDestination);
					RunAdbDeviceCommand(string.Format("shell rm -r {0}", OBBRemoteDestination));
				}

				// check for a local newer executable
				if (Globals.Params.ParseParam("dev"))
				{
					//string ApkFileName = Path.GetFileName(ApkPath);

					string ApkFileName2 = UnrealHelpers.GetExecutableName(AppConfig.ProjectName, UnrealTargetPlatform.Android, AppConfig.Configuration, AppConfig.ProcessType, "apk");

					string LocalAPK = Path.Combine(Environment.CurrentDirectory, AppConfig.ProjectName, "Binaries/Android", ApkFileName2);

					bool LocalFileExists = File.Exists(LocalAPK);
					bool LocalFileNewer = LocalFileExists && File.GetLastWriteTime(LocalAPK) > File.GetLastWriteTime(ApkPath);

					Log.Verbose("Checking for newer binary at {0}", LocalAPK);
					Log.Verbose("LocalFile exists: {0}. Newer: {1}", LocalFileExists, LocalFileNewer);

					if (LocalFileExists && LocalFileNewer)
					{
						ApkPath = LocalAPK;
					}
				}

				bool NoPlayProtect = Globals.Params.ParseParam("no-play-protect");
				if (NoPlayProtect)
				{
					NoPlayProtectSetting = RunAdbDeviceCommandAndGetOutput("shell settings get global package_verifier_user_consent");
					if (NoPlayProtectSetting != "-1")
					{
						Log.Verbose("Removing play protect for this session...");
						RunAdbDeviceCommand("shell settings put global package_verifier_user_consent -1");
					}
				}

				// first install the APK
				CopyFileToDevice(Build.AndroidPackageName, ApkPath, string.Empty, AppConfig.ProjectFile, AppConfig.Configuration);

				// remote dir on the device, create it if it doesn't exist
				if (useADB)
				{
					RunAdbDeviceCommand(string.Format("shell mkdir -p {0}/", DeviceExternalStorageSavedPath));
					RunAdbDeviceCommand(string.Format("shell mkdir -p {0}/", DeviceExternalFilesSavedPath));
				}
				else
				{
					RunAFSDeviceCommand(string.Format("-p {0} -k {1} mkdir \"^saved\"", Build.AndroidPackageName, AFSToken));
				}
			}

			// Convert the files from the source to final destination names
			Dictionary<string, string> FilesToInstall = new Dictionary<string, string>();

			Console.WriteLine("trying to copy files over.");
			if (AppConfig.FilesToCopy != null)
			{
				if (LocalDirectoryMappings.Count == 0)
				{
					Console.WriteLine("Populating Directory");
					PopulateDirectoryMappings(AppConfig);
				}
				foreach (UnrealFileToCopy FileToCopy in AppConfig.FilesToCopy)
				{
					string PathToCopyTo = Path.Combine(LocalDirectoryMappings[FileToCopy.TargetBaseDirectory], FileToCopy.TargetRelativeLocation);
					if (File.Exists(FileToCopy.SourceFileLocation))
					{
						FileInfo SrcInfo = new FileInfo(FileToCopy.SourceFileLocation);
						SrcInfo.IsReadOnly = false;
						FilesToInstall.Add(FileToCopy.SourceFileLocation, PathToCopyTo.Replace("\\", "/"));
					}

					else
					{
						Log.Warning(KnownLogEvents.Gauntlet_DeviceEvent, "File to copy {File} not found", FileToCopy);
					}
				}
			}

			if (!AppConfig.SkipInstall)
			{
				// obb files need to be named based on APK version (grrr), so find that out. This should return something like
				// versionCode=2 minSdk=21 targetSdk=21
				string PackageInfo = RunAdbDeviceCommand(string.Format("shell dumpsys package {0} | grep versionCode", Build.AndroidPackageName)).Output;
				var Match = Regex.Match(PackageInfo, @"versionCode=([\d\.]+)\s");
				if (Match.Success == false)
				{
					throw new AutomationException("Failed to find version info for APK!");
				}
				string PackageVersion = Match.Groups[1].ToString();

				Build.FilesToInstall.Keys.ToList().ForEach(K =>
				{
					string SrcPath = K;
					string SrcFile = Path.GetFileName(SrcPath);

					string DestPath = Build.FilesToInstall[K];
					string DestFile = Path.GetFileName(DestPath);

					// If we installed a new APK we need to change the package version
					Match OBBMatch = Regex.Match(SrcFile, @"\.(\d+)\.com.*\.obb");
					if (OBBMatch.Success)
					{
						DestPath = StoragePath + "/Android/obb/" + Build.AndroidPackageName + "/" + SrcFile.Replace(".Client.obb", ".obb").Replace(OBBMatch.Groups[1].ToString(), PackageVersion);
					}

					DestPath = Regex.Replace(DestPath, "%STORAGE%", StoragePath, RegexOptions.IgnoreCase);

					FilesToInstall.Add(SrcPath, DestPath);
				});

				// get a list of files in the destination OBB directory
				IProcessResult AdbResult;
				AdbResult = RunAdbDeviceCommand(string.Format("shell ls {0}", OBBRemoteDestination));

				// if != 0 then no folder exists
				if (AdbResult.ExitCode == 0)
				{
					string[] Delimiters = { "\r\n", "\n" };
					string[] CurrentRemoteFileList = AdbResult.Output.Split(Delimiters, StringSplitOptions.RemoveEmptyEntries);
					for (int i = 0; i < CurrentRemoteFileList.Length; ++i)
					{
						CurrentRemoteFileList[i] = CurrentRemoteFileList[i].Trim();
					}

					IEnumerable<string> NewRemoteFileList = FilesToInstall.Values.Select(F => Path.GetFileName(F));

					// delete any files that should not be there
					foreach (string FileName in CurrentRemoteFileList)
					{
						if (FileName.StartsWith(".") || FileName.Length == 0)
						{
							continue;
						}

						if (NewRemoteFileList.Contains(FileName) == false)
						{
							RunAdbDeviceCommand(string.Format("shell rm {0}/{1}", OBBRemoteDestination, FileName));
						}
					}
				}

				EnablePermissions(Build.AndroidPackageName);

				// Copy other file dependencies (including OBB files)
				foreach (var KV in FilesToInstall)
				{
					string LocalFile = KV.Key;
					string RemoteFile = KV.Value;

					Console.WriteLine("Copying {0} to {1}", LocalFile, RemoteFile);
					if (useADB)
					{
						CopyFileToDevice(Build.AndroidPackageName, LocalFile, RemoteFile, AppConfig.ProjectFile, AppConfig.Configuration);
					}
					else
					{
						string Filename = Path.GetFileName(LocalFile);
						string[] parts = Filename.ToString().Split(new[] { "." }, StringSplitOptions.RemoveEmptyEntries).Select(S => S.Trim()).ToArray();
						// Make sure last part is obb
						if (parts[parts.Length - 1] == "obb")
						{
							string obbname = string.Format("^{0}obb", parts[0]);
							CopyFileToDevice(Build.AndroidPackageName, LocalFile, obbname, AppConfig.ProjectFile, AppConfig.Configuration);
						}
						else
						{
							CopyFileToDevice(Build.AndroidPackageName, LocalFile, RemoteFile, AppConfig.ProjectFile, AppConfig.Configuration);
						}
					}
				}
			}
			else
			{
				Log.Info("Skipping install of {0} (-skipdeploy)", Build.AndroidPackageName);
			}

			CopyCommandlineFile(AppConfig, AppConfig.CommandLine, Build.AndroidPackageName, AppConfig.ProjectName, Build.UsesExternalFilesDir);

			AndroidAppInstall AppInstall = new AndroidAppInstall(AppConfig, this, InApkPath: ApkPath, AppConfig.ProjectName, Build.AndroidPackageName, AppConfig.CommandLine, AppConfig.ProjectFile, Build.Configuration);

			return AppInstall;
		}
		#endregion
	}

	class AndroidAppInstall : IAppInstall
	{
		public string Name { get; protected set; }

		public TargetDeviceAndroid AndroidDevice { get; protected set; }

		public ITargetDevice Device => AndroidDevice;

		public string AndroidPackageName { get; protected set; }

		public string CommandLine { get; protected set; }

		public string AppTag { get; set; }

		public string ApkPath { get; protected set; }

		public FileReference ProjectFile { get; protected set; }

		public UnrealTargetConfiguration Configuration { get; protected set; }

		public UnrealAppConfig AppConfig { get; protected set; }

		public AndroidAppInstall(UnrealAppConfig InAppConfig, TargetDeviceAndroid InDevice, string InName, string InAndroidPackageName, string InCommandLine, FileReference InProjectFile, UnrealTargetConfiguration InConfiguration, string InAppTag = "UE")
		{
			AppConfig = InAppConfig;
			AndroidDevice = InDevice;
			Name = InName;
			AndroidPackageName = InAndroidPackageName;
			CommandLine = InCommandLine;
			AppTag = InAppTag;
			ProjectFile = InProjectFile;
			Configuration = InConfiguration;
		}

		public AndroidAppInstall(UnrealAppConfig InAppConfig, TargetDeviceAndroid InDevice, string InApkPath, string InName, string InAndroidPackageName, string InCommandLine, FileReference InProjectFile, UnrealTargetConfiguration InConfiguration, string InAppTag = "UE")
		{
			AppConfig = InAppConfig;
			AndroidDevice = InDevice;
			ApkPath = InApkPath;
			Name = InName;
			AndroidPackageName = InAndroidPackageName;
			CommandLine = InCommandLine;
			AppTag = InAppTag;
			ProjectFile = InProjectFile;
			Configuration = InConfiguration;
		}

		public IAppInstance Run()
		{
			return AndroidDevice.Run(this);
		}
	}

	public class DefaultAndroidDevices : IDefaultDeviceSource
	{
		public bool CanSupportPlatform(UnrealTargetPlatform? Platform)
		{
			return Platform == UnrealTargetPlatform.Android;
		}

		public ITargetDevice[] GetDefaultDevices()
		{
			return TargetDeviceAndroid.GetDefaultDevices();
		}
	}

	// become IAppInstance when implemented enough
	class AndroidAppInstance : IAppInstance
	{
		public string ArtifactPath
		{
			get
			{
				if (HasExited && !bHaveSavedArtifacts)
				{
					bHaveSavedArtifacts = true;
					SaveArtifacts();
				}

				return Path.Combine(AndroidDevice.LocalCachePath, "Saved");
			}
		}

		public bool HasExited
		{
			get
			{
				try
				{
					if (!LaunchProcess.HasExited)
					{
						return false;
					}
				}
				catch (InvalidOperationException)
				{
					return true;
				}

				return !IsActivityRunning();
			}
		}

		public string StdOut
		{
			get
			{
				CheckGeneratedCrashLog();
				return LogProcess.Output;
			}
		}

		public ILogStreamReader GetLogReader()
		{
			CheckGeneratedCrashLog();
			return LogProcess.GetLogReader();
		}

		public ILogStreamReader GetLogBufferReader() => LogProcess.GetLogBufferReader();

		public bool WriteOutputToFile(string FilePath) => LogProcess.WriteOutputToFile(FilePath) != null;

		public string CommandLine => Install.CommandLine;

		public ITargetDevice Device => AndroidDevice;

		public int ExitCode => LaunchProcess.ExitCode;

		public bool WasKilled { get; protected set; }

		public IProcessResult LaunchProcess;

		protected ILongProcessResult LogProcess;

		protected TargetDeviceAndroid AndroidDevice;

		protected AndroidAppInstall Install;

		protected bool bHaveSavedArtifacts;
		protected bool bWasCheckedForCrash = false;

		private bool ActivityExited = false;

		public AdbScreenRecorder Recorder;

		private System.Threading.Timer DiagnosticPingTimer;


		public AndroidAppInstance(TargetDeviceAndroid InDevice, AndroidAppInstall InInstall, IProcessResult InProcess)
		{
			AndroidDevice = InDevice;
			Install = InInstall;
			LaunchProcess = InProcess;
			LogProcess = AndroidDevice.RunAdbCommandNoWait($"logcat -s {Install.AppTag} debug Debug DEBUG -v raw", Install.Device.LocalCachePath);

			if (Globals.Params.ParseParam("screenrecord"))
			{
				StartRecording();
			}

			// Start ping timer - runs every 5 seconds to monitor network connectivity
			if (Globals.Params.ParseParam("androiddiagnostics"))
			{
				DiagnosticPingTimer = new Timer(PingGoogle, null, TimeSpan.FromSeconds(5), TimeSpan.FromSeconds(5));
			}
		}

		public int WaitForExit()
		{
			if (!HasExited)
			{
				LaunchProcess.WaitForExit();
				LogProcess.StopProcess();
			}

			return ExitCode;
		}

		public void Kill(bool GenerateDump)
		{
			if (!HasExited && !AndroidDevice.Disposed)
			{
				WasKilled = true;
				Install.AndroidDevice.KillRunningProcess(Install.AndroidPackageName);
				LogProcess.StopProcess();
				StopRecording();

				if (DiagnosticPingTimer != null)
				{
					DiagnosticPingTimer.Dispose();
					DiagnosticPingTimer = null;
				}
			}
		}
		public void StartRecording()
		{
			bool bAlwaysUseAFS = (Install.AppConfig.Build as AndroidBuild)?.UseAFSWithoutToken ?? false;
			string RecordingParsedAFSToken = (Install.AppConfig.Build as AndroidBuild)?.ParsedAFSToken;
			// Ensure artifact directories exist
			if (TargetDeviceAndroid.UsingAndroidFileServer(Install.ProjectFile, Install.Configuration, out _, out string AFSToken, out _, out _, out _, bAlwaysUseAFS, RecordingParsedAFSToken))
			{
				Install.AndroidDevice.RunAFSDeviceCommand(string.Format("-p \"{0}\" -k \"{1}\" mkdir \"^saved/Logs\"", Install.AndroidPackageName, AFSToken));
			}
			else
			{
				Install.AndroidDevice.RunAdbDeviceCommand($"shell mkdir -p {Install.AndroidDevice.DeviceArtifactPath}/Logs/");
			}
			Recorder = new AdbScreenRecorder();
			Recorder.StartRecording(Install.AndroidDevice.DeviceName, $"{Install.AndroidDevice.DeviceArtifactPath}/Logs/screen_recording.mp4");
		}

		public void StopRecording()
		{
			if (Recorder != null)
			{
				Recorder.StopRecording();
				Recorder.Dispose();
				Recorder = null;
			}
		}

		protected void SaveArtifacts()
		{
			StopRecording();

			if (DiagnosticPingTimer != null)
			{
				DiagnosticPingTimer.Dispose();
				DiagnosticPingTimer = null;
			}

			// Pull all the artifacts
			string ArtifactPullCommand = string.Format("pull {0} \"{1}\"", Install.AndroidDevice.DeviceArtifactPath, Install.AndroidDevice.LocalCachePath);
			IProcessResult PullCmd = Install.AndroidDevice.RunAdbDeviceCommand(ArtifactPullCommand, bShouldLogCommand: Log.IsVerbose);

			if (PullCmd.ExitCode != 0)
			{
				Log.Warning(KnownLogEvents.Gauntlet_DeviceEvent, "Failed to retrieve artifacts. {Output}", PullCmd.Output);
			}

			// pull the logcat over from device.
			IProcessResult LogcatResult = Install.AndroidDevice.RunAdbDeviceCommand("logcat -d", bShouldLogCommand: Log.IsVerbose);

			string SavedDirectory = Path.Combine(Install.AndroidDevice.LocalCachePath, "Saved");
			string LogcatFilename = "Logcat.log";
			// Save logcat dump to local artifact path.
			if (!Directory.Exists(SavedDirectory))
			{
				Directory.CreateDirectory(SavedDirectory);
			}
			File.WriteAllText(Path.Combine(SavedDirectory, LogcatFilename), LogcatResult.Output);
			bHaveSavedArtifacts = true;
			if (!Globals.Params.ParseParam("NoAndroidScreenshots"))
			{
				AndroidDevice.StopScreenshotLoop(SavedDirectory);
			}
		}

		private void CheckGeneratedCrashLog()
		{
			// Parse Android native crash dumps from logcat and convert to Unreal crash format
			if (HasExited && !bWasCheckedForCrash)
			{
				bWasCheckedForCrash = true;
				string CrashLog = AndroidCrashParser.GenerateCrashLog(LogProcess.GetLogReader());
				if (!string.IsNullOrEmpty(CrashLog))
				{
					LogProcess.AppendToOutput(CrashLog, false);
				}
			}
		}

		/// <summary>
		/// Checks on device whether the activity is running
		/// </summary>
		private bool IsActivityRunning()
		{
			if (ActivityExited || AndroidDevice.Disposed)
			{
				return false;
			}

			// get activities filtered by our package name
			IProcessResult ActivityQuery = AndroidDevice.RunAdbDeviceCommand("shell dumpsys activity -p " + Install.AndroidPackageName + " a");

			// We have exited if our activity doesn't appear in the activity query or is not the focused activity.
			bool bActivityPresent = ActivityQuery.Output.Contains(Install.AndroidPackageName);
			bool bActivityInForeground = ActivityQuery.Output.Contains("ResumedActivity");
			bool bHasExited = !bActivityPresent || !bActivityInForeground;
			if (bHasExited)
			{
				ActivityExited = true;
				if (!LogProcess.HasExited)
				{
					// The activity has exited, make sure entire activity log has been captured, sleep to allow time for the log to flush
					Thread.Sleep(5000);
					LogProcess.StopProcess();
				}
				Log.VeryVerbose("{0}: process exited, Activity running={1}, Activity in foreground={2} ", ToString(), bActivityPresent.ToString(), bActivityInForeground.ToString());
			}

			return !bHasExited;
		}
		private void PingGoogle(object State)
		{
			try
			{
				IProcessResult PingResult = AndroidDevice.RunAdbDeviceCommand("shell ping -c 1 -W 3 www.google.com", bShouldLogCommand: false);
				if (PingResult.ExitCode == 0 && !string.IsNullOrWhiteSpace(PingResult.Output))
				{
					int TimeIndex = PingResult.Output.IndexOf("time=", StringComparison.Ordinal);
					if (TimeIndex >= 0)
					{
						string AfterTime = PingResult.Output.Substring(TimeIndex + 5);
						int EndIndex = AfterTime.IndexOf(' ');
						string Rtt = EndIndex >= 0 ? AfterTime.Substring(0, EndIndex) : AfterTime.Trim();
						Log.Info("Android Diagnostics: Ping google.com: {0}ms", Rtt);
					}
					else
					{
						Log.Info("Android Diagnostics: Ping google.com: no response");
					}
				}
				else
				{
					Log.Info("Android Diagnostics: Ping google.com: FAILED (exit={0})", PingResult.ExitCode);
				}
			}
			catch (Exception Ex)
			{
				Log.Verbose("Android Diagnostics: Exception in ping: {0}", Ex.Message);
			}
		}
	}

	/// <summary>
	/// Helper class to parse Android native crash tombstones and generate Unreal compatible log callstack
	/// </summary>
	static class AndroidCrashParser
	{
		/// <summary>
		/// Parse Android native crash tombstone format to Unreal log format
		/// </summary>
		public static string GenerateCrashLog(ILogStreamReader LogReader)
		{
			try
			{
				string SignalInfo;
				List<string> Callstack = ParseTombstone(LogReader, out SignalInfo);
				if (Callstack == null || Callstack.Count == 0)
				{
					return null;
				}

				StringBuilder CrashLog = new StringBuilder();
				CrashLog.Append(string.Format("LogCore: Error: === Fatal Error: ===\n"));
				CrashLog.Append(string.Format("LogCore: Error: {0}\n", SignalInfo));
				CrashLog.Append(string.Join("\n", Callstack));

				return CrashLog.ToString();
			}
			catch (Exception Ex)
			{
				Log.Warning(KnownLogEvents.Gauntlet_DeviceEvent, "Exception parsing Android crash tombstone {Exception}", Ex.Message);
			}

			return null;
		}

		private static List<string> ParseTombstone(ILogStreamReader LogReader, out string SignalInfo)
		{
			SignalInfo = "Android Native Crash";

			Regex TombstonePattern = new Regex(@"\*\*\* \*\*\* \*\*\* \*\*\* \*\*\* \*\*\* \*\*\* \*\*\*");
			Regex SignalPattern = new Regex(@"signal (\d+) \((\w+)\)");
			Regex BacktraceStartPattern = new Regex(@"^\s*backtrace:\s*$", RegexOptions.IgnoreCase);
			Regex BacktraceLinePattern = new Regex(@"^\s*#(\d+)\s+pc\s+([0-9a-fA-F]+)\s+(.+?)(?:\s+\((.+?)\))?(?:\s+\(BuildId:\s*([0-9a-fA-F]+)\))?$");

			List<string> Callstack = null;
			bool InCrash = false;
			bool InBacktrace = false;

			LogReader.SetLineIndex(0);
			foreach (string Line in LogReader.EnumerateNextLines())
			{
				// Look for tombstone start marker
				if (!InCrash && TombstonePattern.IsMatch(Line))
				{
					InCrash = true;
					Callstack = new List<string>();
					continue;
				}

				if (!InCrash) continue;

				// Extract signal information
				if (!InBacktrace)
				{
					Match SignalMatch = SignalPattern.Match(Line);
					if (SignalMatch.Success)
					{
						string SignalNumber = SignalMatch.Groups[1].Value;
						string SignalName = SignalMatch.Groups[2].Value;
						SignalInfo = string.Format("Android Native Crash: signal {0} ({1})", SignalNumber, SignalName);
					}

					// Check for backtrace start
					if (BacktraceStartPattern.IsMatch(Line))
					{
						InBacktrace = true;
					}
				}
				else
				{
					// Parse backtrace lines
					Match BacktraceMatch = BacktraceLinePattern.Match(Line);
					if (BacktraceMatch.Success)
					{
						string Address = BacktraceMatch.Groups[2].Value;
						string Path = BacktraceMatch.Groups[3].Value;
						string Function = BacktraceMatch.Groups[4].Success ? BacktraceMatch.Groups[4].Value : "";

						// Extract module name from path
						string Module = System.IO.Path.GetFileName(Path.Split('!')[0].Trim());

						// Format as Unreal callstack: LogCore: Error: [Callstack] 0xaddress module!function [file]
						string FormattedFrame;
						if (!string.IsNullOrEmpty(Function))
						{
							FormattedFrame = string.Format("LogCore: Error: [Callstack] 0x{0} {1}!{2} [{3}]", Address, Module, Function, Path);
						}
						else
						{
							FormattedFrame = string.Format("LogCore: Error: [Callstack] 0x{0} {1} [{2}]", Address, Module, Path);
						}

						Callstack.Add(FormattedFrame);
					}
					else if (!string.IsNullOrWhiteSpace(Line) && !Line.StartsWith(" ") && !Line.StartsWith("\t"))
					{
						// End of backtrace - non-indented line found
						break;
					}
				}
			}

			return Callstack;
		}
	}

	public class AdbScreenRecorder : IDisposable
	{
		private Process ADBProcess;
		private bool Disposed;

		public void StartRecording(string Device, string OutputFilePath)
		{
			if (ADBProcess != null && !ADBProcess.HasExited)
			{
				throw new InvalidOperationException("A screen recording session is already in progress.");
			}

			string AdbCommand = Environment.ExpandEnvironmentVariables("%ANDROID_HOME%/platform-tools/adb" + (OperatingSystem.IsWindows() ? ".exe" : ""));

			ADBProcess = new Process
			{
				StartInfo = new ProcessStartInfo
				{
					FileName = AdbCommand,
					Arguments = $"-s {Device} shell screenrecord {OutputFilePath}",
					RedirectStandardOutput = true,
					RedirectStandardInput = true,
					UseShellExecute = false,
					CreateNoWindow = true
				}
			};

			ADBProcess.Start();
		}
		public void StopRecording(int timeout = 500)
		{
			if (ADBProcess != null && !ADBProcess.HasExited)
			{
				try
				{
					ADBProcess.StandardInput.WriteLine("\x03"); // Send Ctrl+C to stop the recording
					if (!ADBProcess.WaitForExit(timeout))
					{
						// Process did not exit within the specified timeout
						KillAdbProcess();
					}
				}
				catch (InvalidOperationException)
				{
					// Handle the exception if StandardInput is not available
					KillAdbProcess();
				}
			}
		}
		private void KillAdbProcess()
		{
			try
			{
				if (ADBProcess != null && !ADBProcess.HasExited)
				{
					ADBProcess.Kill();
				}
			}
			catch (InvalidOperationException)
			{
				// Process has already exited, do nothing
			}
		}

		public void Dispose()
		{
			Dispose(true);
			GC.SuppressFinalize(this);
		}

		protected virtual void Dispose(bool disposing)
		{
			if (!Disposed)
			{
				if (disposing)
				{
					// Release managed resources
					StopRecording();
				}

				// Release unmanaged resources
				if (ADBProcess != null)
				{
					ADBProcess.Dispose();
					ADBProcess = null;
				}

				Disposed = true;
			}
		}

		~AdbScreenRecorder()
		{
			Dispose(false);
		}
	}

	// Device data from json
	public sealed class AndroidDeviceData
	{
		// Host of PC which is tethered
		public string HostIP { get; set; }

		// public key
		public string PublicKey { get; set; }

		// private key
		public string PrivateKey { get; set; }
	}

	public class AndroidBuildSupport : BaseBuildSupport
	{
		protected override BuildFlags SupportedBuildTypes => BuildFlags.Packaged | BuildFlags.CanReplaceCommandLine | BuildFlags.CanReplaceExecutable | BuildFlags.Bulk | BuildFlags.NotBulk;
		protected override UnrealTargetPlatform? Platform => UnrealTargetPlatform.Android;
	}

	public class AndroidDeviceFactory : IDeviceFactory
	{
		public bool CanSupportPlatform(UnrealTargetPlatform? Platform)
		{
			return Platform == UnrealTargetPlatform.Android;
		}

		public ITargetDevice CreateDevice(string InRef, string InCachePath, string InParam = null)
		{
			AndroidDeviceData DeviceData = null;

			if (!string.IsNullOrEmpty(InParam))
			{
				DeviceData = fastJSON.JSON.Instance.ToObject<AndroidDeviceData>(InParam);
			}

			return new TargetDeviceAndroid(InRef, DeviceData, InCachePath);
		}
	}

	/// <summary>
	/// ADB key credentials, running adb-server commands (must) use same pub/private key store
	/// </summary>
	internal static class AdbCredentialCache
	{
		private static int InstanceCount = 0;
		private static bool bUsingCustomKeys = false;

		private static string PrivateKey;
		private static string PublicKey;

		private const string KeyBackupExt = ".gauntlet.bak";

		static AdbCredentialCache()
		{
			Reset();
		}

		public static void RemoveInstance()
		{
			lock (Globals.MainLock)
			{
				InstanceCount--;

				if (InstanceCount == 0 && bUsingCustomKeys)
				{
					Reset();
					KillAdbServer();

					// Kill ADB server, just as a safety measure to ensure it closes
					IEnumerable<Process> ADBProcesses = Process.GetProcesses().Where(p => p.ProcessName.Equals("adb"));
					if (ADBProcesses.Count() > 0)
					{
						Log.Info("Terminating {0} ADB Process(es)", ADBProcesses.Count());
						foreach (Process ADBProcess in ADBProcesses)
						{
							Log.Info("Killing ADB process {0}", ADBProcess.Id);
							ADBProcess.Kill();
						}
					}
				}
			}
		}

		public static void RestoreBackupKeys()
		{
			string LocalKeyPath = Path.Combine(Environment.GetFolderPath(Environment.SpecialFolder.UserProfile), ".android");
			string LocalKeyFile = Path.Combine(LocalKeyPath, "adbkey");
			string LocalPubKeyFile = Path.Combine(LocalKeyPath, "adbkey.pub");
			string BackupSentry = Path.Combine(LocalKeyPath, "gauntlet.inuse");

			if (File.Exists(BackupSentry))
			{
				Log.Info("Restoring original adb keys");

				if (File.Exists(LocalKeyFile + KeyBackupExt))
				{
					File.Copy(LocalKeyFile + KeyBackupExt, LocalKeyFile, true);
					File.Delete(LocalKeyFile + KeyBackupExt);
				}
				else
				{
					File.Delete(LocalKeyFile);
				}

				if (File.Exists(LocalPubKeyFile + KeyBackupExt))
				{
					File.Copy(LocalPubKeyFile + KeyBackupExt, LocalPubKeyFile, true);
					File.Delete(LocalPubKeyFile + KeyBackupExt);
				}
				else
				{
					File.Delete(LocalPubKeyFile);
				}

				File.Delete(BackupSentry);
			}

		}

		public static void AddInstance(AndroidDeviceData DeviceData = null)
		{
			lock (Globals.MainLock)
			{
				string KeyPath = Globals.Params.ParseValue("adbkeys", null);

				// setup key store from device data
				if (string.IsNullOrEmpty(KeyPath) && DeviceData != null)
				{
					// checked that cached keys are the same
					if (!string.IsNullOrEmpty(PrivateKey))
					{
						if (PrivateKey != DeviceData.PrivateKey)
						{
							throw new AutomationException("ADB device private keys must match");
						}
					}

					if (!string.IsNullOrEmpty(PublicKey))
					{
						if (PublicKey != DeviceData.PublicKey)
						{
							throw new AutomationException("ADB device public keys must match");
						}
					}

					PrivateKey = DeviceData.PrivateKey;
					PublicKey = DeviceData.PublicKey;

					if (string.IsNullOrEmpty(PublicKey) || string.IsNullOrEmpty(PrivateKey))
					{
						throw new AutomationException("Invalid key in device data");
					}

					KeyPath = Path.Combine(Globals.TempDir, "AndroidADBKeys");

					if (!Directory.Exists(KeyPath))
					{
						Directory.CreateDirectory(KeyPath);
					}

					if (InstanceCount == 0)
					{
						byte[] data = Convert.FromBase64String(PrivateKey);
						File.WriteAllText(KeyPath + "/adbkey", Encoding.UTF8.GetString(data));

						data = Convert.FromBase64String(PublicKey);
						File.WriteAllText(KeyPath + "/adbkey.pub", Encoding.UTF8.GetString(data));
					}

				}

				if (InstanceCount == 0 && !System.String.IsNullOrEmpty(KeyPath))
				{

					Log.Info("Using adb keys at {0}", KeyPath);

					string LocalKeyPath = Path.Combine(Environment.GetFolderPath(Environment.SpecialFolder.UserProfile), ".android");
					if (!Directory.Exists(LocalKeyPath))
					{
						Directory.CreateDirectory(LocalKeyPath);
					}

					string RemoteKeyFile = Path.Combine(KeyPath, "adbkey");
					string RemotePubKeyFile = Path.Combine(KeyPath, "adbkey.pub");
					string LocalKeyFile = Path.Combine(LocalKeyPath, "adbkey");
					string LocalPubKeyFile = Path.Combine(LocalKeyPath, "adbkey.pub");
					string BackupSentry = Path.Combine(LocalKeyPath, "gauntlet.inuse");

					if (File.Exists(RemoteKeyFile) == false)
					{
						throw new AutomationException("adbkey at {0} does not exist", KeyPath);
					}

					if (File.Exists(RemotePubKeyFile) == false)
					{
						throw new AutomationException("adbkey.pub at {0} does not exist", KeyPath);
					}

					if (File.Exists(BackupSentry) == false)
					{
						if (File.Exists(LocalKeyFile))
						{
							File.Copy(LocalKeyFile, LocalKeyFile + KeyBackupExt, true);
						}

						if (File.Exists(LocalPubKeyFile))
						{
							File.Copy(LocalPubKeyFile, LocalPubKeyFile + KeyBackupExt, true);
						}
						File.WriteAllText(BackupSentry, "placeholder");
					}

					File.Copy(RemoteKeyFile, LocalKeyFile, true);
					File.Copy(RemotePubKeyFile, LocalPubKeyFile, true);

					bUsingCustomKeys = true;

					KillAdbServer();
				}

				InstanceCount++;
			}

		}

		private static void Reset()
		{
			if (InstanceCount != 0)
			{
				throw new AutomationException("AdbCredentialCache.Reset() called with outstanding instances");
			}

			PrivateKey = PublicKey = string.Empty;
			bUsingCustomKeys = false;

			RestoreBackupKeys();
		}

		private static void KillAdbServer()
		{
			using (new ScopedSuspendECErrorParsing())
			{
				Log.Info("Running adb kill-server to refresh credentials");
				TargetDeviceAndroid.RunAdbGlobalCommand("kill-server");

				// Killing the adb server restarts it and can surface superfluous device errors
				int SleepTime = CommandUtils.IsBuildMachine ? 15000 : 5000;
				Thread.Sleep(SleepTime);
			}
		}
	}

	class AndroidPlatformSupport : TargetPlatformSupportBase
	{
		public override UnrealTargetPlatform? Platform => UnrealTargetPlatform.Android;
		public override bool IsHostMountingSupported() => false;
	}
}
