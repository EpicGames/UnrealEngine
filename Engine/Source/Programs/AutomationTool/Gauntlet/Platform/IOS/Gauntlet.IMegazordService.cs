// Copyright Epic Games, Inc. All Rights Reserved.

using System.Threading.Tasks;

#nullable enable

namespace Gauntlet
{
	/// <summary>
	/// iOS-specific portion of the <see cref="IMegazordService"/> contract.
	/// </summary>
	public partial interface IMegazordService
	{
		/// <summary>
		/// Get the base64-encoded lockdown pair record plist for an iOS device by its UUID/serial.
		/// Returns null if the device is not found or has no lockdown key.
		/// </summary>
		/// <param name="Serial">iOS device UDID</param>
		Task<string?> GetDeviceLockdownKeyAsync(string Serial);

		/// <summary>
		/// Get the base64-encoded SystemConfiguration plist for the host that paired the iOS device.
		/// Decoded contents are written to <c>/var/db/lockdown/SystemConfiguration.plist</c>.
		/// Returns null if the device is not found or has no system_config recorded.
		/// </summary>
		/// <param name="Serial">iOS device UDID</param>
		Task<string?> GetDeviceSystemConfigAsync(string Serial);
	}
}
