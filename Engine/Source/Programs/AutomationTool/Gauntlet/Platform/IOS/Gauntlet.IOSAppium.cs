// Copyright Epic Games, Inc. All Rights Reserved.
using AutomationTool;
using Newtonsoft.Json;
using OpenQA.Selenium;
using OpenQA.Selenium.Appium;
using OpenQA.Selenium.Appium.Enums;
using OpenQA.Selenium.Appium.iOS;
using OpenQA.Selenium.Appium.Service;
using OpenQA.Selenium.Support.UI;
using System;
using System.IO;
using System.Text.RegularExpressions;
using System.Threading;
using UnrealBuildBase;
using static AutomationTool.CommandUtils;

namespace Gauntlet
{
	/// <summary>
	/// AppiumContainer is a wrapper for both an Appium server instance and an AppiumDriver.
	/// It's primary use in Gauntlet is for automating dismissal of blocking system notifications that cannot be managed by an MDM profile.
	/// Provided you configure your environment correctly, the container will automatically initialize as part of an IOS app instance execution.
	///
	/// Steps for configuring your environment:
	///		1.	Download appium on your host. It's recommended you use the global npm installation 'npm install -g appium'.
	///		2.  For real device testing, you will need a WebDriverAgentRunner.app which is signed with a mobile provision that includes your device.
	///			For this you have two options:
	///				- Build the app from source. This lets you configure bundle.id's if your signing cert only allows for certain identifiers.
	///				- Download the precompiled app and re-sign after replacing the embedded mobileprovision.
	///			In either case, you can find both the source and precompiled app on this page https://github.com/appium/WebDriverAgent/releases
	///		3.	Create a JSON file that can be de-serialized to the 'AppiumContainer.Config' type. This file is used to configure the driver with information
	///			that is specific to your team. Place this file in a location that can be read by your host.
	///		4.	Point the container to the location of the json file you created in step 4 by doing one of the following:
	///				- Setting the UE-AppiumConfigPath EnvVar to a qualified path or a relative path to your UE root.
	///				- Run UAT with -AppiumConfigPath=/path set to a qualified path or a relative path to your UE root.
	///
	/// Once all these steps are completed, before TargetDeviceIOS.Run starts the app process, it will execute these actions:
	///		1. Start an appium server on an available loopback port
	///		2. Install the WebDriverAgent app
	///		3. Start the driver with your configured settings
	///
	/// From there appium will automatically accept/dismiss any system prompts it encounters.
	/// </summary>
	public class AppiumContainer : IDisposable
	{
		public class Config
		{
			/// <summary>
			/// Name of the automation driver to use. Defaults to XCUITest if not specified
			/// </summary>
			public string AutomationName { get; set; } = "xcuitest";

			/// <summary>
			/// Your company's apple developer team ID
			/// </summary>
			public string OrgId { get; set; }

			/// <summary>
			/// Identity of your signing cert. Usually just 'Apple Development'
			/// </summary>
			public string SigningId { get; set; }

			/// <summary>
			/// Path to a precompiled WebDriverAgentRunner app
			/// </summary>
			public string WdaAppPath { get; set; }

			/// <summary>
			/// Bundle ID of the WebDriverAgent app. Ex: 'com.epicgames.WebDriverAgent'
			/// </summary>
			public string WdaBundleId { get; set; }

			/// <summary>
			/// Launch Timeout for WebDriverAgent app in ms. Default: 120000ms (increased for VM environments)
			/// </summary>
			public int WdaLaunchTimeoutMs { get; set; } = 120000;

			/// <summary>
			/// How often to take screenshots in seconds. Disabled if <= 0
			/// </summary>
			public float ScreenshotPeriod { get; set; } = 0;

			/// <summary>
			/// Path screenshots are output to
			/// </summary>
			public string ScreenshotDirectory { get; set; }

			/// <summary>
			/// Optional - Allow you to override the location of the appium executable.
			/// Useful if you opt not to install appium to the global npm root
			/// </summary>
			public string AppiumLocation { get; set; }

			public AppiumOptions GetCapabilities(string UUID, string PackageName)
			{
				AppiumOptions Capabilities = new AppiumOptions();
				Capabilities.PlatformName = "iOS";
				Capabilities.AutomationName = AutomationName;
				Capabilities.AddAdditionalAppiumOption("udid", UUID);
				Capabilities.AddAdditionalAppiumOption("bundleId", PackageName);
				Capabilities.AddAdditionalAppiumOption("autoAcceptAlerts", true);
				Capabilities.AddAdditionalAppiumOption("xcodeOrgId", OrgId);
				Capabilities.AddAdditionalAppiumOption("xcodeSigningId", SigningId);
				Capabilities.AddAdditionalAppiumOption("autoLaunch", false);
				Capabilities.AddAdditionalAppiumOption("usePreinstalledWDA", true);
				Capabilities.AddAdditionalAppiumOption("updatedWDABundleId", WdaBundleId);
				Capabilities.AddAdditionalAppiumOption("wdaLaunchTimeout", WdaLaunchTimeoutMs);

				if (Log.Level == LogLevel.Verbose)
				{
					Capabilities.AddAdditionalAppiumOption("showXcodeLog", true);
				}

				return Capabilities;
			}
		}

		private const string AppiumConfigEnvVar = "UE-AppiumConfigPath";

		private const string AppiumConfigArg = "AppiumConfigPath";

		/// <summary>
		/// **REQUIRED**
		/// Path to a json file that can be deserialized into the AppiumContainer.Config type
		/// This allows users to specify things such as the org id, signing identity, etc.
		/// Can be overriden by setting the UE-AppiumConfigPath envvar, or by running with -AppiumConfigPath=/path
		/// Path can be relative or absolute
		/// </summary>
		public static string AppiumConfigPath = "Engine/Restricted/NotForLicensees/Extras/ThirdPartyNotUE/WebDriverAgent/AppiumConfig.json";

		/// <summary>
		/// Whether or not the appium container is configured properly for use.
		/// This requires the appium config path and the appium server path to point to existing files
		/// </summary>
		public static bool Enabled => AppiumConfig != null;

		/// <summary>
		/// Lock object
		/// </summary>
		private static object Mutex = new();

		/// <summary>
		/// Config file used for the Driver
		/// </summary>
		public static Config AppiumConfig { get; private set; } = null;

		/// <summary>
		/// Driver handle
		/// </summary>
		private IOSDriver Driver = null;

		/// <summary>
		/// Server handle
		/// </summary>
		private AppiumLocalService Server = null;

		private Thread MonitorThread = null;

		private AutoResetEvent MonitorTrigger = null;

		/// <summary>
		/// UUID of the device being tests
		/// </summary>
		private string UUID = null;

		/// <summary>
		/// Package name of the app being tested
		/// </summary>
		private string PackageName = null;
		static AppiumContainer()
		{
			if (!Globals.Params.ParseParam("UseAppium"))
			{
				return;
			}

			string ConfigEnvVar = Environment.GetEnvironmentVariable(AppiumConfigEnvVar);
			if (!string.IsNullOrEmpty(ConfigEnvVar))
			{
				AppiumConfigPath = ConfigEnvVar;
			}
			AppiumConfigPath = Globals.Params.ParseValue(AppiumConfigArg, AppiumConfigPath);

			if (FileExists(AppiumConfigPath))
			{
				try
				{
					AppiumConfig = JsonConvert.DeserializeObject<Config>(ReadAllText(AppiumConfigPath));
				}
				catch (Exception Ex)
				{
					throw new AutomationException(Ex, "Failed to derserialize AppiumConfig at {0}", AppiumConfigPath);
				}
			}
		}

		public AppiumContainer(string UUID)
		{
			this.UUID = UUID;
			ConfigureDrivers();
		}

		#region IDisposable Support
		private bool Disposed = false;
		~AppiumContainer()
		{
			Dispose(false);
		}
		public void Dispose()
		{
			Dispose(true);
			GC.SuppressFinalize(this);
		}

		private void Dispose(bool bDisposing)
		{
			if (!Disposed)
			{
				if (bDisposing)
				{

				}
				Stop();

				// Ensure monitor thread is stopped (fallback if SaveArtifacts didn't call it)
				if (MonitorThread != null)
				{
					Log.Verbose("Stopping monitor thread during disposal (screenshots will not be saved)");
					StopMonitorLoop(); // No output directory - screenshots discarded
				}

				Disposed = true;
			}
		}
		#endregion

		public void Start(string PackageName)
		{
			Stop();

			this.PackageName = PackageName;

			// Install WebdriverAgent
			try
			{
				Log.Info("Installing WebDriverAgent ({0}) on device {1}...", AppiumConfig.WdaBundleId, UUID);
				IProcessResult InstallResult;
				using (ScopedSuspendECErrorParsing ErrorSuspension = new())
				{
					if (TargetDeviceIOS.UseDeviceCtl)
					{
						InstallResult = TargetDeviceIOS.ExecuteDevicectlCommand(
							string.Format("device install app \"{0}\" -v", AppiumConfig.WdaAppPath), UUID, 60, AdditionalOptions: ERunOptions.NoStdOutRedirect);
					}
					else
					{
						InstallResult = TargetDeviceIOS.ExecuteIOSDeployCommand(
							string.Format("-b \"{0}\"", AppiumConfig.WdaAppPath), UUID, 60, AdditionalOptions: ERunOptions.NoStdOutRedirect);
					}
				}
				if (InstallResult.ExitCode != 0)
				{
					throw new AutomationException("Failed to install WDA app ({0}): {1}", InstallResult.ExitCode, InstallResult.Output);
				}
				Log.Info("WebDriverAgent installed successfully on device {0}", UUID);
			}
			catch (Exception Ex)
			{
				throw new AutomationException($"Exception during WDA Install {Ex.Message}");
			}

			lock (Mutex)
			{
				// Retry logic for both server start and WebDriver creation
				// Sometimes servers fail to start or devices appear temporarily unavailable
				const int MaxRetries = 3;
				int attemptCount = 0;
				Exception lastException = null;

				while (attemptCount < MaxRetries)
				{
					attemptCount++;

					try
					{
						// Clean up any previous server instance
						if (Server != null)
						{
							Log.Verbose("Disposing previous server instance before retry");
							try
							{
								Server.Dispose();
							}
							catch (Exception ex)
							{
								Log.Verbose("Error disposing previous server: {0}", ex.Message);
							}
							Server = null;
						}

						// Refresh device tunnel to ensure it's active BEFORE starting Appium (critical for VM environments)
						// This helps prevent "Unknown device" errors when Appium tries to connect
						if (TargetDeviceIOS.UseDeviceCtl)
						{
							try
							{
								Log.Info("Refreshing device tunnel for {0} before Appium server starts...", UUID);
								IProcessResult pairResult = TargetDeviceIOS.ExecuteDevicectlCommand("manage pair", UUID, 30, AdditionalOptions: ERunOptions.NoLoggingOfRunCommand);
								Log.Verbose("Waiting 5 seconds for device tunnel to stabilize...");
								Thread.Sleep(5000); // Wait for tunnel to stabilize before Appium queries devices
							}
							catch (Exception ex)
							{
								Log.Verbose("Device pair refresh warning: {0}", ex.Message);
							}
						}

						// Start Appium Server
						Log.Info("Starting Appium server for device {0} (attempt {1}/{2})", UUID, attemptCount, MaxRetries);
						AppiumServiceBuilder ServerBuilder = new AppiumServiceBuilder()
							.UsingAnyFreePort()
							.WithIPAddress("127.0.0.1");

						if (!string.IsNullOrEmpty(AppiumConfig.AppiumLocation) && FileExists(AppiumConfig.AppiumLocation))
						{
							ServerBuilder.WithAppiumJS(new System.IO.FileInfo(AppiumConfig.AppiumLocation));
						}

						Server = ServerBuilder.Build();
						Server.Start();
						Log.Verbose("Appium server started successfully");

						// Create Webdriver instance
						Log.Info("Creating WebDriver for device {0} (attempt {1}/{2})", UUID, attemptCount, MaxRetries);
						Driver = new IOSDriver(Server, AppiumConfig.GetCapabilities(UUID, PackageName));
						Log.Info("Successfully created WebDriver for device {0}", UUID);
						break; // Success - exit retry loop
					}
					catch (Exception Ex)
					{
						lastException = Ex;
						Log.Warning("Appium start attempt {0} failed: {1}", attemptCount, Ex.Message);

						// Clean up failed server
						if (Server != null)
						{
							try
							{
								Server.Dispose();
							}
							catch { }
							Server = null;
						}

						if (attemptCount < MaxRetries)
						{
							int waitSeconds = attemptCount * 5; // Progressive backoff: 5s, 10s, 15s
							Log.Info("Waiting {0} seconds before retry...", waitSeconds);
							Thread.Sleep(waitSeconds * 1000);
						}
					}
				}

				if (Driver == null || Server == null)
				{
					throw new DeviceException($"Failed to start Appium and create WebDriver after {MaxRetries} attempts: {lastException?.Message}");
				}
			}
		}

		/// <summary>
		/// Starts the monitor thread which checks for alerts and takes screenshots
		/// Should be called after the app has launched on the device
		/// </summary>
		public void StartMonitorThread()
		{
			if (!Globals.Params.ParseParam("NoAppiumScreenshots") && AppiumConfig.ScreenshotPeriod > 0)
			{
				MonitorThread = new Thread(StartMonitorLoop);
				MonitorThread.Start();
			}
		}

		public void Stop()
		{
			if (Driver != null)
			{
				Driver.Dispose();
				Driver = null;
			}

			if (Server != null)
			{
				Server.Dispose();
				Server = null;
			}

			// Note: MonitorThread cleanup is handled by StopMonitorLoop() which should be called
			// with an output directory parameter to save screenshots. See SaveArtifacts() in TargetDeviceIOS.

			// TODO: Terminate WDA app - parse devicectl for pid, then kill?
		}

		/// <summary>
		/// Checks if WebDriverAgent is already installed on the device
		/// </summary>
		/// <param name="deviceUUID">UUID of the device to check</param>
		/// <param name="bundleId">Bundle ID of WebDriverAgent (e.g., com.epicgames.WebDriverAgent)</param>
		/// <returns>True if WDA is installed, false otherwise</returns>
		private bool CheckIfWdaInstalled(string deviceUUID, string bundleId)
		{
			try
			{
				IProcessResult checkResult;

				if (TargetDeviceIOS.UseDeviceCtl)
				{
					// Use devicectl to list installed apps
					checkResult = TargetDeviceIOS.ExecuteDevicectlCommand("device info apps", deviceUUID, 30);
				}
				else
				{
					// Use ios-deploy to list bundle IDs
					checkResult = TargetDeviceIOS.ExecuteIOSDeployCommand("--list_bundle_id", deviceUUID, 30);
				}

				if (checkResult.ExitCode == 0)
				{
					// Check for both the base bundle ID and the .xctrunner variant
					// (actual installed app often has .xctrunner suffix)
					bool isInstalled = checkResult.Output.Contains(bundleId) ||
									   checkResult.Output.Contains(bundleId + ".xctrunner");
					Log.Verbose("Checked for {0} (or .xctrunner variant) on device {1}: {2}", bundleId, deviceUUID, isInstalled ? "found" : "not found");
					return isInstalled;
				}

				Log.Verbose("Failed to check installed apps on device {0}, assuming WDA not installed", deviceUUID);
				return false;
			}
			catch (Exception ex)
			{
				Log.Verbose("Exception checking for installed WDA: {0}. Assuming not installed.", ex.Message);
				return false; // If check fails, assume not installed and proceed with install
			}
		}

		private void ConfigureDrivers()
		{
			// Query installed drivers
			IProcessResult Result = Run("appium", "driver list");
			if (Result.ExitCode != 0)
			{
				throw new AutomationException("Failed to query appium driver list ({0}): {1}", Result.ExitCode, Result.Output);
			}

			// Trim ansii escape codes
			string SanitizedOutput = Regex.Replace(Result.Output, "\\x1B\\[[0-9;]*[a-zA-Z]", string.Empty);

			// Try to find the driver for for this container's config
			bool bFoundDriver = false;
			Regex DriverMatch = new Regex("(- )(.*?)(@)(\\d+((\\.\\d+)+)?)", RegexOptions.IgnoreCase | RegexOptions.Multiline);
			foreach (Match Match in DriverMatch.Matches(SanitizedOutput))
			{
				string Driver = Match.Groups[2].Value;
				string Version = Match.Groups[4].Value;

				if (Driver.Equals(AppiumConfig.AutomationName, StringComparison.OrdinalIgnoreCase))
				{
					bFoundDriver = true;
					Log.Verbose("Using {AppiumDriverName} appium driver version {AppiumDriverVersion}", Driver, Version);
					break;
				}
			}

			// Install the driver if the missing
			if (!bFoundDriver)
			{
				Log.Info("Could not find appium driver {AppiumDriverName}. Attempting to install...", AppiumConfig.AutomationName);
				Result = Run("appium", $"driver install {AppiumConfig.AutomationName.ToLower()}");
				if (Result.ExitCode != 0)
				{
					throw new AutomationException("Failed to install {0} appium driver. ({1}): {2}", AppiumConfig.AutomationName, Result.ExitCode, Result.Output);
				}
			}
		}

		public void VerifyAppInForeground(string packageName, int retries = 5)
		{

			bool IsAppInForeground = false;
			while (!IsAppInForeground && retries > 0)
			{
				IsAppInForeground = CheckAppInForeground(packageName);
				if (!IsAppInForeground)
				{
					Log.Info("{0} in background, trying to foreground Retries remaining:{1}", packageName, retries);
					ForegroundApp(packageName);
					Thread.Sleep(500);
					retries--;
				}
			}
			if (retries == 0)
			{
				throw new DeviceException("Unable to foreground {0} after {1} attempts", packageName, retries);
			}
		}

		public bool CheckAppInForeground(string appID, int retries = 5)
		{
			bool bInForeground = false;
			var appState = Driver.GetAppState(appID);

			switch (appState)
			{
				case AppState.RunningInForeground:
					bInForeground = true;
					Log.Verbose("{0} app in Foreground", appID);
					break;
				case AppState.RunningInBackground:
					Log.Verbose("{0} app in Background", appID);
					break;
				case AppState.RunningInBackgroundOrSuspended:
					Log.Verbose("{0} app in Background or Suspended", appID);
					break;
				case AppState.NotRunning:
					Log.Verbose("{0} app not running", appID);
					break;
			}

			return bInForeground;
		}

		public void ForegroundApp(string appID)
		{
			Driver.ActivateApp(appID);
		}

		/// <summary>
		/// Checks for and dismisses Apple Sign-On alerts. Used by the monitor loop.
		/// Takes a screenshot if an alert is found before dismissing it.
		/// Only performs 1 retry attempt.
		/// </summary>
		private void CheckAndDismissAlert(string ScreenshotDirectory)
		{
			var WebDriverWait = new WebDriverWait(new SystemClock(), Driver, TimeSpan.FromMilliseconds(5000), TimeSpan.FromMilliseconds(250));

			try
			{
				var alert = WebDriverWait.Until(drv =>
				{
					try
					{
						return drv.SwitchTo().Alert();
					}
					catch (NoAlertPresentException)
					{
						return null;
					}
					catch (Exception ex)
					{
						Log.Verbose("Alert switch attempt failed: {0}", ex.GetType().Name);
						return null;  // Return null to keep polling
					}
				});

				if (alert != null)
				{
					Log.Info("Found Alert, trying to dismiss");

					// Take screenshot before dismissing
					try
					{
						if (!string.IsNullOrEmpty(ScreenshotDirectory))
						{
							if (!DirectoryExists(ScreenshotDirectory))
							{
								CreateDirectory(ScreenshotDirectory);
							}

							string FileName = $"alert-{DateTime.Now.ToString("yy-MM-dd-HH-mm-ss-ff")}.png";
							string FilePath = Path.Combine(ScreenshotDirectory, FileName);
							(Driver as ITakesScreenshot).GetScreenshot().SaveAsFile(FilePath);
							Log.Info("Saved alert screenshot to: {0}", FilePath);
						}
					}
					catch (Exception Ex)
					{
						Log.Verbose("Failed to capture alert screenshot: {0}", Ex.Message);
					}

					// Now dismiss the alert
					// Retry find+click operation to handle stale elements
					bool clicked = false;
					for (int attempt = 0; attempt < 3 && !clicked; attempt++)
					{
						try
						{
							var cancelButton = Driver.FindElement(By.XPath("//XCUIElementTypeButton[@name='Cancel']"));
							Log.Info("Found Cancel Button, clicking (attempt {0})", attempt + 1);
							cancelButton.Click();
							clicked = true;
							Log.Info("Successfully clicked Cancel button");
						}
						catch (NoSuchElementException)
						{
							Log.Info("Cancel button not found on attempt {0}", attempt + 1);
							break; // Button doesn't exist, don't retry
						}
						catch (StaleElementReferenceException)
						{
							Log.Verbose("Button became stale on attempt {0}, retrying...", attempt + 1);
							Thread.Sleep(100); // Brief delay before retry
						}
					}

					if (clicked)
					{
						Log.Info("Monitor thread dismissed an alert.");
						if (!string.IsNullOrEmpty(PackageName))
						{
							Thread.Sleep(300);
							Driver.ActivateApp(PackageName);
						}
					}
				}
			}
			catch (WebDriverTimeoutException)
			{
				// No alert present, this is normal
			}
			catch (UnhandledAlertException)
			{
				// Alert handling already in progress
			}
			catch (Exception ex)
			{
				Log.Info("Alert check exception: {0}", ex.Message);
			}
		}

		/// <summary>
		/// Checks for view-based popups that aren't detected as system alerts.
		/// Looks for common dismiss buttons like Cancel, Try Again, etc.
		/// </summary>
		private bool CheckAndDismissViewPopup(string ScreenshotDirectory)
		{
			try
			{
				// Look for common popup button combinations
				// This catches popups that aren't system alerts but are view elements
				// NOTE: Intentionally excludes 'OK' buttons to avoid accidentally confirming dialogs
				var popupButtons = Driver.FindElements(By.XPath(
					"//XCUIElementTypeButton[@name='Cancel' or @name='Try Again' or @name='Close' or @name='Dismiss']"
				));

				if (popupButtons.Count >= 2) // Likely a popup if multiple buttons present
				{
					Log.Info("Found view-based popup with {0} buttons", popupButtons.Count);

					// Take screenshot before dismissing
					try
					{
						if (!string.IsNullOrEmpty(ScreenshotDirectory) && DirectoryExists(ScreenshotDirectory))
						{
							string FileName = $"popup-{DateTime.Now.ToString("yy-MM-dd-HH-mm-ss-ff")}.png";
							string FilePath = Path.Combine(ScreenshotDirectory, FileName);
							(Driver as ITakesScreenshot).GetScreenshot().SaveAsFile(FilePath);
							Log.Info("Saved popup screenshot to: {0}", FilePath);
						}
					}
					catch (Exception Ex)
					{
						Log.Verbose("Failed to capture popup screenshot: {0}", Ex.Message);
					}

					// Click Cancel/Dismiss button if present
					foreach (var button in popupButtons)
					{
						string buttonName = button.GetAttribute("name");
						// Use case-insensitive comparison and trim whitespace for robustness
						if (!string.IsNullOrEmpty(buttonName))
						{
							string trimmedName = buttonName.Trim();
							if (trimmedName.Equals("Cancel", StringComparison.OrdinalIgnoreCase) ||
								trimmedName.Equals("Dismiss", StringComparison.OrdinalIgnoreCase) ||
								trimmedName.Equals("Close", StringComparison.OrdinalIgnoreCase))
							{
								Log.Info("Clicking dismiss button on popup: {0}", buttonName);
								button.Click();
								if (!string.IsNullOrEmpty(PackageName))
								{
									Thread.Sleep(300);
									Driver.ActivateApp(PackageName);
								}
								return true;
							}
						}
					}

					// No safe dismiss button found - log and don't click anything
					Log.Warning("Found popup with {0} buttons but no Cancel/Dismiss/Close button. Not clicking to avoid confirming dialog.", popupButtons.Count);
					return false;
				}
			}
			catch (Exception ex)
			{
				Log.Verbose("View popup check failed: {0}", ex.Message);
			}

			return false;
		}

		/// <summary>
		/// Uses appium WebDriver to take a screenshot. 
		/// Filename is saved as png in the format of: yy-MM-dd-HH-mm-ss-ff
		/// </summary>
		/// <param name="OutputDirectory">Folder to copy screenshots to</param>
		public void TakeScreenshot(string OutputDirectory, string ScreenshotName)
		{
			if (string.IsNullOrEmpty(ScreenshotName))
			{
				ScreenshotName = $"{DateTime.Now.ToString("yy-MM-dd-HH-mm-ss-ff")}.png";
			}

			if (string.IsNullOrEmpty(OutputDirectory))
			{
				throw new AutomationException("Screenshot output directory must not be null or empty!");
			}

			if (!DirectoryExists(OutputDirectory))
			{
				CreateDirectory(OutputDirectory);
			}

			string FilePath = Path.Combine(OutputDirectory, ScreenshotName);

			// Retry logic for screenshot capture (helps with socket hang up issues on VMs)
			const int maxRetries = 3;
			for (int attempt = 0; attempt < maxRetries; attempt++)
			{
				try
				{
					(Driver as ITakesScreenshot).GetScreenshot().SaveAsFile(FilePath);
					Log.Verbose("Saved screenshot to: {ScreenshotFile}", FilePath);
					break; // Success - exit retry loop
				}
				catch (Exception Ex)
				{
					if (Ex.Message.Contains("socket hang up") && attempt < maxRetries - 1)
					{
						Log.Verbose("Screenshot attempt {0} failed with socket hang up, retrying...", attempt + 1);
						Thread.Sleep(1000); // Wait 1 second before retry
					}
					else
					{
						Log.Info("Encountered an {ExceptionType} when trying to capture a screenshot: {ExceptionMessage}", Ex.GetType().Name, Ex.Message);
						break; // Non-retryable error or final attempt
					}
				}
			}
		}

		private void StartMonitorLoop()
		{
			// Alert check runs frequently (every 10 seconds)
			TimeSpan AlertCheckPeriod = TimeSpan.FromSeconds(10);

			// Calculate how many alert checks happen per screenshot
			// e.g., if ScreenshotPeriod = 30 seconds, and AlertCheckPeriod = 10 seconds
			// then screenshotCheckCount = 30 / 10 = 3 iterations
			int screenshotCheckCount = (int)Math.Max(1, AppiumConfig.ScreenshotPeriod / AlertCheckPeriod.TotalSeconds);
			int iterationCounter = 0;

			MonitorTrigger = new AutoResetEvent(false);
			string ScreenshotDirectory = GetScreenshotDirectory();

			if (DirectoryExists(ScreenshotDirectory))
			{
				DeleteDirectory_NoExceptions(ScreenshotDirectory);
			}

			Log.Verbose("Monitor loop: Alert checks every {0}s, Screenshots every {1}s ({2} iterations)",
				AlertCheckPeriod.TotalSeconds, AppiumConfig.ScreenshotPeriod, screenshotCheckCount);

			while (!MonitorTrigger.WaitOne(AlertCheckPeriod))
			{
				// Check for system alerts first
				try
				{
					CheckAndDismissAlert(ScreenshotDirectory);
				}
				catch (Exception Ex)
				{
					Log.Verbose("Alert check failed: {0}", Ex.Message);
				}

				// Fallback: Check for view-based popups
				try
				{
					CheckAndDismissViewPopup(ScreenshotDirectory);
				}
				catch (Exception Ex)
				{
					Log.Verbose("Popup check failed: {0}", Ex.Message);
				}

				// Only take screenshot every N iterations
				iterationCounter++;
				if (iterationCounter >= screenshotCheckCount)
				{
					TakeScreenshot(ScreenshotDirectory, string.Empty);
					iterationCounter = 0;
				}
			}
		}

		/// <summary>
		/// Stops the monitoring loop (screenshots and alerts), optionally moving all captured images to an output directory
		/// </summary>
		/// <param name="OutputDirectory">Folder to copy screenshots to</param>
		public void StopMonitorLoop(string OutputDirectory = "")
		{
			if (MonitorThread == null || MonitorTrigger == null)
			{
				Log.Verbose("Attempted to end the monitor loop before it started. Ignoring.");
				return;
			}

			MonitorTrigger.Set();
			MonitorThread.Join();
			MonitorThread = null;
			MonitorTrigger = null;

			if (!string.IsNullOrEmpty(OutputDirectory))
			{
				try
				{
					if (!DirectoryExists(OutputDirectory))
					{
						CreateDirectory(OutputDirectory);
					}

					string ScreenshotDirectory = GetScreenshotDirectory();

					Log.Verbose("Copying {SourceScreenshotDirectory} to {DestinationScreenshotDirectory}...", ScreenshotDirectory, OutputDirectory);
					CopyDirectory_NoExceptions(ScreenshotDirectory, OutputDirectory);
				}
				catch (Exception Ex)
				{
					Log.Info("Encountered an {ExceptionType} when trying to copy screenshots to requested output directory. " +
						"Screenshots will not be available: {ExceptionMessage}", Ex.GetType().Name, Ex.Message);
				}
			}
		}

		public string GetScreenshotDirectory()
		{
			string ScreenshotDirectory = AppiumConfig.ScreenshotDirectory;
			if (string.IsNullOrEmpty(ScreenshotDirectory))
			{
				ScreenshotDirectory = Path.Combine(Unreal.RootDirectory.FullName, "GauntletTemp", "IOSScreenshots");
			}
			return ScreenshotDirectory;
		}
	}
}
