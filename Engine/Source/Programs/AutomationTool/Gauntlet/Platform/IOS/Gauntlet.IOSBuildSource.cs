// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.IO;
using AutomationTool;
using UnrealBuildTool;
using System.Xml.Linq;
using System.Text.RegularExpressions;
using System.Linq;
using EpicGames.Core;

namespace Gauntlet
{
	public class AppleBuild : IBuild
	{
		public int PreferenceOrder => 0;

		public UnrealTargetConfiguration Configuration { get; protected set; }

		public string SourcePath;

		public bool IsIPAFile;

		public string PackageName;

		public BuildFlags Flags { get; protected set; }

		public string Flavor => string.Empty;

		public virtual UnrealTargetPlatform Platform { get; }

		public bool SupportsAdditionalFileCopy { get; }

		public Dictionary<string, string> BulkContents { get; protected set; }

		private const int BulkContentSearchDepth = 3;

		public AppleBuild(UnrealTargetConfiguration InConfig, string InPackageName, string InSourcePath, BuildFlags InFlags, Dictionary<string, string> InBulkContents = null)
		{
			Configuration = InConfig;
			PackageName = InPackageName;
			SourcePath = InSourcePath;
			Flags = InFlags;
			SupportsAdditionalFileCopy = true;
			IsIPAFile = Path.GetExtension(InSourcePath).Equals(".ipa", StringComparison.OrdinalIgnoreCase);
			BulkContents = InBulkContents;
		}

		public bool CanSupportRole(UnrealTargetRole RoleType)
		{
			if (RoleType.IsClient())
			{
				return true;
			}

			return false;
		}

		internal static IProcessResult ExecuteCommand(String Command, String Arguments)
		{
			CommandUtils.ERunOptions RunOptions = CommandUtils.ERunOptions.AppMustExist;

			if (Log.IsVeryVerbose)
			{
				RunOptions |= CommandUtils.ERunOptions.AllowSpew;
			}
			else
			{
				RunOptions |= CommandUtils.ERunOptions.NoLoggingOfRunCommand;
			}

			Log.Verbose("Executing '{0} {1}'", Command, Arguments);

			IProcessResult Result = CommandUtils.Run(Command, Arguments, Options: RunOptions);

			return Result;
		}

		internal static bool ExecuteIPAZipCommand(String Arguments, out String Output, String ShouldExist = "")
		{
			using (new ScopedSuspendECErrorParsing())
			{
				IProcessResult Result = ExecuteCommand("unzip", Arguments);
				Output = Result.Output;

				if (Result.ExitCode != 0)
				{
					if (!String.IsNullOrEmpty(ShouldExist))
					{
						if (!File.Exists(ShouldExist) && !Directory.Exists(ShouldExist))
						{
							Log.Error(KnownLogEvents.Gauntlet_BuildDropEvent, "unzip encountered an error or warning procesing IPA, possibly due to Zip64 issue, {File} missing", ShouldExist);
							return false;
						}
					}

					Log.Info("unzip encountered an issue procesing IPA, possibly due to Zip64. Future steps may fail.");
				}
			}

			return true;
		}

		// IPA handling using ditto command, which is capable of handling IPA's > 4GB/Zip64
		internal static bool ExecuteIPADittoCommand(String Arguments, out String Output, String ShouldExist = "")
		{
			using (new ScopedSuspendECErrorParsing())
			{
				IProcessResult Result = ExecuteCommand("ditto", Arguments);
				Output = Result.Output;

				if (Result.ExitCode != 0)
				{
					if (!string.IsNullOrEmpty(ShouldExist))
					{
						if (!File.Exists(ShouldExist) && !Directory.Exists(ShouldExist))
						{
							Log.Error("ditto encountered an error or warning procesing IPA, {ShouldExist} missing", ShouldExist);
							return false;
						}
					}

					Log.Error("ditto encountered an issue procesing IPA");
					return false;

				}
			}

			return true;
		}

		private static PlistInfo GetPlistInfo(string Source)
		{
			bool IsIPAFile = Path.GetExtension(Source).Equals(".ipa", StringComparison.OrdinalIgnoreCase);
			if (IsIPAFile)
			{
				// Get a list of files in the IPA
				if (!ExecuteIPAZipCommand(string.Format("-Z1 {0}", Source), out string Output))
				{
					Log.Info("Unable to list files for IPA {IPAPath}", Source);
					return null;
				}

				string[] Filenames = Regex.Split(Output, "\r\n|\r|\n");
				string PListFile = Filenames.Where(F => Regex.IsMatch(F.ToLower().Trim(), @"(payload\/)([^\/]+)(\/info\.plist)")).FirstOrDefault();
				if (string.IsNullOrEmpty(PListFile))
				{
					Log.Info("Unable to find plist for IPA {IPAPath}", Source);
					return null;
				}

				// Get the plist info
				if (!ExecuteIPAZipCommand(string.Format("-p '{0}' '{1}'", Source, PListFile), out Output))
				{
					Log.Info("Unable to extract plist data for IPA {IPAPath}", Source);
					return null;
				}

				return new PlistInfo(Output);
			}
			else
			{
				string PlistFile = Path.Combine(Source, "Info.plist");
				if (!File.Exists(PlistFile))
				{
					Log.Info("Unable to find plist from {IPAPath}. Skipping.", Source);
					return null;
				}

				return new PlistInfo(File.ReadAllText(PlistFile));
			}
		}

		private class PlistInfo
		{
			private XDocument Document;

			public PlistInfo(string InContent)
			{
				try
				{
					Document = XDocument.Parse(InContent);
				}
				catch (Exception Ex)
				{
					// Ignore errors
					Log.Warning(KnownLogEvents.Gauntlet_BuildDropEvent, "Fail to parse PlistInfo:\n{Exception}", Ex);
					Document = new XDocument();
				}
			}

			/// <summary>
			/// Get first value from corresponding key
			/// </summary>
			/// <param name="Key"></param>
			/// <returns></returns>
			public string GetFirstValue(string Key)
			{
				foreach (XElement element in Document.Descendants("key"))
				{
					if (element.Value == Key)
					{
						XElement NextElement = element.ElementsAfterSelf().FirstOrDefault();
						if (NextElement != null)
						{
							if (NextElement.Name == "string")
							{
								return NextElement.Value;
							}
							else if (NextElement.Name == "array")
							{
								return NextElement.Descendants("string").Select(e => e.Value).FirstOrDefault();
							}
						}
					}
				}

				return null;
			}

			/// <summary>
			/// Get all values from corresponding key
			/// </summary>
			/// <param name="Key"></param>
			/// <returns></returns>
			public IEnumerable<string> GetAllValues(string Key)
			{
				foreach (XElement element in Document.Descendants("key"))
				{
					if (element.Value == Key)
					{
						XElement NextElement = element.ElementsAfterSelf().FirstOrDefault();
						if (NextElement != null)
						{
							if (NextElement.Name == "array")
							{
								return NextElement.Descendants("string").Select(e => e.Value);
							}
							else if (NextElement.Name == "string")
							{
								return new List<string> { NextElement.Value };
							}
						}
					}
				}

				return null;
			}
		}

		public static T CreateFromPath<T>(string InProjectName, string InRootPath, string InBuildPath, AppleBuildSource<T> BuildSource)
			where T : AppleBuild
		{
			FileSystemInfo BuildPath;
			if (Directory.Exists(InBuildPath))
			{
				BuildPath = new DirectoryInfo(InBuildPath);
			}
			else if (File.Exists(InBuildPath))
			{
				BuildPath = new FileInfo(InBuildPath);
			}
			else
			{
				Log.Verbose("Build path does not exist! Skipping.");
				return null;
			}

			// Check there's an executable with the right name
			string AppShortName = Regex.Replace(InProjectName, "Game", string.Empty, RegexOptions.IgnoreCase);
			UnrealTargetConfiguration Configuration = UnrealTargetConfiguration.Unknown;

			if (BuildPath is DirectoryInfo App)
			{
				Configuration = UnrealHelpers.GetConfigurationFromExecutableName(InProjectName, InBuildPath);
				if (Configuration == UnrealTargetConfiguration.Unknown)
				{
					Log.Verbose("Could not deduce iOS build configuration from build at path {BuildPath}. Skipping.", InBuildPath);
					return null;
				}

				if (App.GetFiles(AppShortName + '*', SearchOption.TopDirectoryOnly).FirstOrDefault() == null)
				{
					Log.Verbose("Could not find an executable within build path {BuildPath}. Skipping", InBuildPath);
					return null;
				}
			}
			else
			{
				// Get a list of files in the IPA
				if (!ExecuteIPAZipCommand(string.Format("-Z1 {0}", InBuildPath), out string Output))
				{
					Log.Info("Unable to list files for IPA {IPAPath}", InBuildPath);
					return null;
				}

				bool bFoundExecutable = false;
				IEnumerable<string> FileNames = Regex.Split(Output, "\r\n|\r|\n").Select(File => Path.GetFileName(File));

				foreach(string File in FileNames)
				{
					if(File.Contains(AppShortName, StringComparison.OrdinalIgnoreCase))
					{
						Configuration = UnrealHelpers.GetConfigurationFromExecutableName(InProjectName, File);
						if (Configuration == UnrealTargetConfiguration.Unknown)
						{
							Log.Verbose("Could not deduce iOS build configuration from build at path {BuildPath}. Skipping.", InBuildPath);
							return null;
						}
						bFoundExecutable = true;
						break;
					}
				}

				if (!bFoundExecutable)
				{
					Log.Verbose("Could not find an executable within build path {BuildPath}. Skipping", InBuildPath);
					return null;
				}
			}

			PlistInfo Info = GetPlistInfo(BuildPath.FullName);
			if(Info == null)
			{
				Log.Info("Unable to parse PlistInfo for '{BuildPath}'. Skipping", BuildPath.FullName);
			}

			IEnumerable<string> CFBundlePlatformNames = Info.GetAllValues("CFBundleSupportedPlatforms");
			if(CFBundlePlatformNames == null || !CFBundlePlatformNames.Contains(BuildSource.CFBundlePlatformName))
			{
				Log.Info("Unable to find matching platform '{BundlePlatform}' for CFBundleSupportedPlatforms in PlistInfo for App {BuildPath}. Skipping", BuildSource.CFBundlePlatformName, BuildPath.FullName);
				return null;
			}

			string PackageName = Info.GetFirstValue("CFBundleIdentifier");
			if (string.IsNullOrEmpty(PackageName))
			{
				Log.Info("Unable to find CFBundleIdentifier in PlistInfo for App {BuildPath}. Skipping.", BuildPath);
				return null;
			}

			// IOS builds are always packaged, and can always replace the command line and executable (even as IPAs because we cache the unzipped app)
			BuildFlags Flags = BuildFlags.Packaged | BuildFlags.CanReplaceCommandLine | BuildFlags.CanReplaceExecutable;
			Dictionary<string, string> BulkContents = new Dictionary<string, string>();
			if (BuildPath.FullName.Contains("Bulk"))
			{
				if (BuildPath is FileInfo)
				{
					Log.Info("Bulk builds cannot be contained in an IPA file. Skipping.");
					return null;
				}

				bool bInstallOptionalContent = Globals.Params.ParseParam("InstallOptionalContent");

				// From the build, work backwards until we find the required content file
				int SearchDepth = 0;
				DirectoryInfo CurrentDirectory = Directory.GetParent(BuildPath.FullName);
				FileInfo RequiredContentFile = null;
				FileInfo OptionalContentFile = null;

				while (SearchDepth < BulkContentSearchDepth)
				{
					foreach (FileInfo File in CurrentDirectory.EnumerateFiles())
					{
						if(File.Name.Equals("RequiredContent.txt"))
						{
							RequiredContentFile = File;
						}

						if(File.Name.Equals("OptionalContent.txt"))
						{
							OptionalContentFile = File;
						}
					}

					if(RequiredContentFile != null && OptionalContentFile != null)
					{
						break;
					}

					CurrentDirectory = CurrentDirectory.Parent;
					++SearchDepth;
				}

				if (RequiredContentFile == null)
				{
					Log.Info("Could not locate RequiredContent file for bulk build at path {BuildPath}. Skipping.", BuildPath);
					return null;
				}

				if (bInstallOptionalContent && OptionalContentFile == null)
				{
					Log.Info("Could not locate OptionalContent file for bulk build ath path {BuildPath}. Skipping.", BuildPath);
					return null;
				}

				string[] RequiredContents = File.ReadAllLines(RequiredContentFile.FullName);
				string[] OptionalContents = File.ReadAllLines(OptionalContentFile.FullName);
				
				foreach(string RequiredContent in RequiredContents)
				{
					string[] Split = RequiredContent.Split(',');
					string Source = Split[0];
					string Target = Split[1];
					BulkContents.Add(Source, Target);
				}

				if (bInstallOptionalContent)
				{
					foreach (string OptionalContent in OptionalContents)
					{
						string[] Split = OptionalContent.Split(',');
						string Source = Split[0];
						string Target = Split[1];
						BulkContents.Add(Source, Target);
					}
				}

				foreach (string SourceFile in BulkContents.Keys)
				{
					string QualifiedSourceFile = Path.Combine(BuildPath.FullName, SourceFile);
					if(!File.Exists(QualifiedSourceFile))
					{
						Log.Info("Failed to find required content file {QualifiedSourceFile} for build at path {BuildPath}. Skipping", QualifiedSourceFile, BuildPath);
						return null;
					}
				}

				Flags |= BuildFlags.Bulk;
			}
			else
			{
				Flags |= BuildFlags.NotBulk;
			}

			Log.Verbose("Found bundle id: {PackageName}", PackageName);
			Log.Verbose("Found {Configuration} {Flags} build at {BuildPath}", Configuration, ((Flags & BuildFlags.Bulk) == BuildFlags.Bulk) ? "(bulk)" : "(not bulk)", BuildPath);

			return Activator.CreateInstance(typeof(T), [Configuration, PackageName, BuildPath.FullName, Flags, BulkContents]) as T;
		}
	}

	public abstract class AppleBuildSource<T> : IFolderBuildSource
		where T : AppleBuild
	{
		public string ProjectName { get; protected set; }

		public abstract string CFBundlePlatformName { get; }

		public string BuildName => Platform.ToString() + "BuildSource";

		protected abstract UnrealTargetPlatform Platform { get; }

		protected string BuildFilter;

		public AppleBuildSource()
		{
			BuildFilter = Globals.Params.ParseValue(Platform.ToString() + "BuildFilter", null);
		}

		public bool CanSupportPlatform(UnrealTargetPlatform InPlatform)
		{
			return InPlatform == Platform;
		}

		public virtual List<IBuild> GetBuildsAtPath(string InProjectName, string InPath, int MaxRecursion = 3)
		{
			// We only want iOS builds on Mac host
			List<IBuild> Builds = new List<IBuild>();
			if (BuildHostPlatform.Current.Platform != UnrealTargetPlatform.Mac)
			{
				return new List<IBuild>();
			}

			// Interface default parameters don't let us modify the default if calling from an interface cast...
			// IOS builds are often located deeper within a client directory, so increase the depth here
			MaxRecursion = MaxRecursion > 7 ? MaxRecursion : 7;

			DirectoryInfo BuildDirectory = new DirectoryInfo(InPath);
			if (BuildDirectory.Exists)
			{
				List<DirectoryInfo> SearchDirs = new List<DirectoryInfo>();

				// Find the first folder in the build directory containing the platform name
				DirectoryInfo[] SubDirectories = BuildDirectory.Name.Contains(Platform.ToString(), StringComparison.OrdinalIgnoreCase)
					? new[] { BuildDirectory }
					: BuildDirectory.GetDirectories(Platform.ToString() + '*', SearchOption.TopDirectoryOnly).ToArray();

				// Now, recursively search the IOS directory for any .app folders
				// We also discard any apps that don't include the IOSBuildFilter
				List<FileSystemInfo> Apps = new();
				while (MaxRecursion-- > 0)
				{
					IEnumerable<DirectoryInfo> ValidApps = SubDirectories
						.Where(Directory => Directory.Extension.Equals(".app", StringComparison.OrdinalIgnoreCase) || Directory.Extension.Equals(".ipa", StringComparison.OrdinalIgnoreCase))
						.Where(Directory => string.IsNullOrEmpty(BuildFilter) || Directory.FullName.Contains(BuildFilter, StringComparison.OrdinalIgnoreCase));

					IEnumerable<FileInfo> ValidIPAs = SubDirectories
						.SelectMany(Directory => Directory.GetFiles("*.ipa", SearchOption.TopDirectoryOnly))
						.Where(File => string.IsNullOrEmpty(BuildFilter) || File.FullName.Contains(BuildFilter, StringComparison.OrdinalIgnoreCase));

					Apps.AddRange(ValidApps);
					Apps.AddRange(ValidIPAs);
					SubDirectories = SubDirectories.SelectMany(Directory => Directory.GetDirectories("*", SearchOption.TopDirectoryOnly)).ToArray();

				}

				foreach (FileSystemInfo App in Apps)
				{
					AppleBuild Build = AppleBuild.CreateFromPath(InProjectName, InPath, App.FullName, this);
					if (Build != null)
					{
						Builds.Add(Build);
					}
				}
			}

			return Builds;
		}
	}

	public class IOSBuild : AppleBuild
	{
		public IOSBuild(UnrealTargetConfiguration InConfig, string InPackageName, string InSourcePath, BuildFlags InFlags, Dictionary<string, string> InBulkContents = null)
			: base(InConfig, InPackageName, InSourcePath, InFlags, InBulkContents)
		{ }

		public override UnrealTargetPlatform Platform => UnrealTargetPlatform.IOS;
	}

	public class IOSBuildSource : AppleBuildSource<IOSBuild>
	{
		protected override UnrealTargetPlatform Platform => UnrealTargetPlatform.IOS;
		public override string CFBundlePlatformName => "iPhoneOS";
	}

	public class TVOSBuild : AppleBuild
	{
		public TVOSBuild(UnrealTargetConfiguration InConfig, string InPackageName, string InSourcePath, BuildFlags InFlags, Dictionary<string, string> InBulkContents = null)
			: base(InConfig, InPackageName, InSourcePath, InFlags, InBulkContents)
		{ }

		public override UnrealTargetPlatform Platform => UnrealTargetPlatform.TVOS;
	}

	public class TVOSBuildSource : AppleBuildSource<TVOSBuild>
	{
		protected override UnrealTargetPlatform Platform => UnrealTargetPlatform.TVOS;
		public override string CFBundlePlatformName => "AppleTVOS";
	}
}