// Copyright Epic Games, Inc. All Rights Reserved.

using System.IO;
using System.Text;
using System.Diagnostics;
using AutomationTool;
using UnrealBuildTool;

namespace Gauntlet
{
	/// <summary>
	/// IOS PGO platform implementation (primarily used to test PGO locally with editor builds)
	/// </summary>
	internal sealed class IOSPGOPlatform : IPGOPlatform
	{
		string LocalOutputDirectory;
		string LocalProfDataFile;

		public UnrealTargetPlatform GetPlatform() => UnrealTargetPlatform.IOS;

		public void GatherResults(string ArtifactPath)
		{
			var ProfRawFiles = Directory.GetFiles(ArtifactPath, "*.profraw");
			if (ProfRawFiles.Length == 0)
			{
				throw new AutomationException(string.Format("Process exited cleanly but no .profraw PGO files were found in the output directory \"{0}\".", ArtifactPath));
			}

			if (File.Exists(LocalProfDataFile))
			{
				new FileInfo(LocalProfDataFile).IsReadOnly = false;
			}

			StringBuilder MergeCommandBuilder = new StringBuilder();
			foreach (var ProfRawFile in ProfRawFiles)
			{
				MergeCommandBuilder.AppendFormat(" \"{0}\"", ProfRawFile);
			}

			int ReturnCode = UnrealBuildTool.Utils.RunLocalProcessAndLogOutput(new ProcessStartInfo("xcrun", string.Format("llvm-profdata merge{0} -o \"{1}\"", MergeCommandBuilder, LocalProfDataFile)), EpicGames.Core.Log.Logger);
			if (ReturnCode != 0)
			{
				throw new AutomationException(string.Format("llvm-profdata failed to merge profraw data. Error code {0}. ({1} {2})", ReturnCode, MergeCommandBuilder, LocalProfDataFile));
			}

			// Check the profdata file exists
			if (!File.Exists(LocalProfDataFile))
			{
				throw new AutomationException(string.Format("Profraw data merging completed, but the profdata output file (\"{0}\") was not found.", LocalProfDataFile));
			}
		}

		public void ApplyConfiguration(PGOConfig Config)
		{
			LocalOutputDirectory = Config.ProfileOutputDirectory;
			LocalProfDataFile = Path.Combine(LocalOutputDirectory, "profile.profdata");

			UnrealTestRole ClientRole = Config.RequireRole(UnrealTargetRole.Client);
		}

		public bool TakeScreenshot(ITargetDevice Device, string ScreenshotDirectory, out string ImageFilename)
		{
			ImageFilename = string.Empty;
			return false;
		}
	}
}



