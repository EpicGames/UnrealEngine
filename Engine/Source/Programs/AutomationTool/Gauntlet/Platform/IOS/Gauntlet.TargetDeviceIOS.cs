// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.IO;
using System.Linq;
using System.Data;
using System.Text;
using System.Text.RegularExpressions;
using System.Collections.Generic;
using EpicGames.Core;
using Gauntlet.Utils;
using AutomationTool;
using UnrealBuildTool;
using static AutomationTool.CommandUtils;
using System.Threading;
using System.Threading.Tasks;

/** IOS General Notes:
 *
 *	Functionality:
 *
 *		Tooling for iOS automation is in a tumultuous state.
 *		Historically, ios-deploy was used to manage all aspects of automation.
 *		However, recent changes in Apple's system framework have made ios-deploy obsolete for launching apps for iOS versions 17+ (See for details: https://github.com/ios-control/ios-deploy/issues/588)
 *		Alternative tools offer most - but not all - of the features needed to install, launch, monitor, and debug an app.
 *		Because of this, Gauntlet iOS automation is restricted to running on a Mac host and mixes and matches which tools are used for a given operation.
 *
 *		For now, TargetDeviceIOS uses the following tools to support automation:
 *			ios-deploy			- Third party CLI tool, still used for legacy operations and copying files to/from device - https://github.com/ios-control/ios-deploy
 *			devicectl			- Apple's core device framework. Has issues with copying files FROM the device, but handles just about everything else - run 'xcrun devicectl' for details
 *
 *		In general:
 *			IF you are using xcode versions 15 or below AND iOS version 16 or below, ios-deploy will be primarily used
 *			IF you are using xcode version 16+, devicectl will be primarily used
 *			IF you are using trying to test on iOS versions 17+, you MUST upgrade to XCode 16 which contains critical app monitoring features. (See https://forums.developer.apple.com/forums/thread/756393)

		*** IMPORTANT ****
		Testing iOS 17+ and iOS 16 or lower in a single pass is not supported. Doing so can result in undefined behavior.
		If you need to do this, separate the testing into separate passes targeting each device.
 *
 *	Other:
 *
 *		- Builds (including local developer builds used with -dev) must already be signed. If built with UBT, signing is already part of the process. Automated re-signing may be added in the future.
 *		- Installing builds requires your device to be added to a mobile provision, and for your build to have been signed with an embedded mobile provision containing the device (See https://developer.apple.com/documentation/xcode/distributing-your-app-to-registered-devices)
 *		- If your app installs content after launch, consider using a bulk build to prevent timeouts. This also lets you more rapidly test -dev apps because code and content are separated
 *		- If your app requires additional permissions, consider using MDM profiles to prevent permission pop-ups from blocking your tests
 */

namespace Gauntlet
{
	/// <summary>
	/// iOS implementation of a device to run applications
	/// </summary>
	public class TargetDeviceIOS : ITargetDevice
	{
		public class ConnectedDevice
		{
			public string UUID;
			public int IOSMajorVersion;
			public bool bConnected;
			public string IPAddress;
			public ITargetDevice AssignedDevice;
			public bool IsAssigned => AssignedDevice != null;
			public ConnectedDevice(string UUID, int IOSMajorVersion, bool bConnected, string IPAddress = null)
			{
				this.UUID = UUID;
				this.IOSMajorVersion = IOSMajorVersion;
				this.bConnected = bConnected;
				this.IPAddress = IPAddress;
			}
		}

		/// <summary>
		/// Whether or not devicectl is used to run most device operations
		/// ios-deploy will be used when this is false.
		/// This property is set based on the installed version of xcode and the ios version of the device
		/// </summary>
		public static bool UseDeviceCtl { get; private set; }

		/// <summary>
		/// Collection of devices visible to the UAT host
		/// </summary>
		private static Dictionary<string, ConnectedDevice> ConnectedDevices;

		/// <summary>
		/// Currently active version of XCode. This can be changed between installed versions using 'xcodes select'
		/// </summary>
		private static int XCodeMajorVersion;

		/// <summary>
		/// Creating an iOS device requires several queries to be performed first.
		/// These results are cached and gated from being refreshed behind this bool
		/// </summary>
		private static bool bStaticInitialized;

		/// <summary>
		/// Lock object
		/// </summary>
		private static object Mutex = new object();

		/// <summary>
		/// Used to prevent re-copies of an entire network share build in the event of a device failure (bulk only)
		/// </summary>
		private static bool HasCopiedNetworkBuild = false;

		#region ITargetDevice
		public string Name { get; protected set; }

		public string IPAddress { get; protected set; }

		public UnrealTargetPlatform? Platform => UnrealTargetPlatform.IOS;

		public ERunOptions RunOptions { get; set; }

		public bool IsConnected => ConnectedDevices.ContainsKey(UUID) && ConnectedDevices[UUID].bConnected;

		public bool IsOn => true;

		public bool IsAvailable => true;
		#endregion

		[AutoParam((int)(60 * 60 * 2.5))] // iOS installs can be slow... 2.5 hour default
		public int MaxInstallTime { get; protected set; }

		[AutoParam(60 * 15)]
		public int MaxPerFileInstallTime { get; protected set; }

		/// <summary>
		/// Temp path we use to push/pull things from the device
		/// </summary>
		public string LocalCachePath { get; protected set; }

		/// <summary>
		/// Low-level device name (uuid)
		/// </summary>
		public string UUID { get; protected set; }

		/// <summary>
		/// Temporary opt-in flag while we triage devicectl copy hangs
		/// </summary>
		[AutoParam]
		public bool UseIOSDeployForCopy { get; protected set; }

		protected Dictionary<EIntendedBaseCopyDirectory, string> LocalDirectoryMappings { get; set; } = new();

		private bool IsDefaultDevice = false;

		/// <summary>
		/// Handle to the device's Install
		/// </summary>
		private IOSAppInstall InstallCache = null;

		public AppiumContainer AppiumContainer { get; private set; } = null;

		private List<string> LockdownPlistPaths = new();

		private readonly IMegazordService MegazordService =
			InterfaceHelpers.FindImplementations<IMegazordService>(true).FirstOrDefault();

		public TargetDeviceIOS(string InName, string InCachePath = null)
		{
			AutoParam.ApplyParamsAndDefaults(this, Globals.Params.AllArguments);

			if (BuildHostPlatform.Current.Platform != UnrealTargetPlatform.Mac)
			{
				throw new AutomationException("Creation of an iOS target device is only supported on Mac at this time.");
			}

			InitializeDevices();

			IsDefaultDevice = string.IsNullOrEmpty(InName) || InName.Equals("default", StringComparison.OrdinalIgnoreCase);
			Name = IsDefaultDevice ? "default" : InName;

			if (IsDefaultDevice)
			{
				IEnumerable<ConnectedDevice> AvailableDevices = ConnectedDevices.Values.Where(Device => !Device.IsAssigned);
				if (!AvailableDevices.Any())
				{
					throw new AutomationException("Ran out of unassigned default devices. Ensure all devices are properly connected to the host.");
				}

				if (!AvailableDevices.Where(Device => Device.bConnected).Any())
				{
					// If some devices are detected, but none connected, attempt connection
					UUID = AvailableDevices.First().UUID;

					Log.Info("Attempting to connect to {0}", UUID);
					if (!Connect())
					{
						throw new DeviceException("Failed to connect to {0}. Ensure the device is properly connected to the host.", UUID);
					}
				}
				else
				{
					// Just select the first available device
					UUID = AvailableDevices.Where(Device => Device.bConnected).First().UUID;

				}

				Log.Verbose("Selected device {DeviceName} as default device", UUID);
			}
			else
			{
				UUID = Name;
				if (!ConnectedDevices.ContainsKey(Name))
				{
					if (!ConnectedDevices.ContainsKey(Name))
					{
						throw new DeviceException("Device with UUID {0} not found in device list", UUID);
					}
				}

				// Connect to device (or reconnect after reboot)
				Log.Info("Attempting to connect to {0}", Name);
				if (!Connect())
				{
					throw new DeviceException("Failed to connect to {0}", UUID);
				}
			}

			ConnectedDevices[UUID].AssignedDevice = this;
			IPAddress = ConnectedDevices[UUID].IPAddress;

			// If IP address is not available from device list, try to query it directly
			if (string.IsNullOrEmpty(IPAddress) && UseDeviceCtl)
			{
				Log.Info("Attempting to query device {0} for WiFi IP address...", UUID);
				IPAddress = QueryDeviceIPAddress(UUID);
				if (!string.IsNullOrEmpty(IPAddress))
				{
					Log.Info("Found WiFi IP address for device {0}: {1}", UUID, IPAddress);
					ConnectedDevices[UUID].IPAddress = IPAddress;
				}
				else
				{
					Log.Verbose("Could not determine WiFi IP address for device {0}", UUID);
				}
			}

			if (XCodeMajorVersion < 15)
			{
				UseDeviceCtl = false;
			}
			else if (XCodeMajorVersion == 15)
			{
				if (ConnectedDevices[UUID].IOSMajorVersion < 17)
				{
					UseDeviceCtl = false;
				}
				else
				{
					string Message = "As of XCode 15, Apple has removed DeveloperDiskImages for iOS 17 and beyond. " +
						"Because of this, ios-deploy cannot be used to run and monitor apps within Gauntlet. " +
						"Instead, DeviceCtl replaces this functionality which is fully available as of XCode 16. " +
						"Update to XCode 16 to prevent this error.";
					throw new AutomationException(Message);
				}
			}
			else
			{
				UseDeviceCtl = true;
			}

			if (AppiumContainer.Enabled)
			{
				AppiumContainer = new AppiumContainer(UUID);
			}

			LocalCachePath = InCachePath ?? Path.Combine(Globals.TempDir, "IOSDevice_" + UUID);
		}

		#region IDisposable Support
		private bool DisposedValue = false; // To detect redundant calls
		~TargetDeviceIOS()
		{
			Dispose(false);
		}

		protected virtual void Dispose(bool disposing)
		{
			if (!DisposedValue)
			{
				if (AppiumContainer != null)
				{
					AppiumContainer.Dispose();
				}

				CleanupLockdownPairRecord();

				DisposedValue = true;
			}
		}

		public void Dispose()
		{
			Dispose(true);
			GC.SuppressFinalize(this);
		}
		#endregion

		#region ITargetDevice
		public bool Connect()
		{
			if (UseDeviceCtl)
			{
				if (ConnectedDevices.ContainsKey(UUID) && ConnectedDevices[UUID].bConnected)
				{
					return true; // already connected
				}

				WriteLockdownPairRecord();

				IProcessResult ConnectResult = ExecuteDevicectlCommand("manage pair");
				ConnectResult.WaitForExit();
				if (ConnectResult.ExitCode != 0)
				{
					Log.Info("Connecting to {DeviceName} exited with code {ExitCode}:\n{Output}", UUID, ConnectResult.ExitCode, ConnectResult.Output);
					return false;
				}

				// Wait after pair for device tunnel to establish before querying details
				Log.Verbose("Waiting 20 seconds after pair for device tunnel to establish...");
				Thread.Sleep(20000);

				// Get device details with retry logic
				// Note: ValidateDeviceConnection() will check tunnel state from device info details response
				JsonObject JsonDevice = null;
				const int MaxRetries = 2;
				for (int attempt = 1; attempt <= MaxRetries; attempt++)
				{
					string DetailsFile = Path.GetTempFileName();
					string DetailsCommand = string.Format("device info details -j {0}", DetailsFile);
					IProcessResult DetailResult = ExecuteDevicectlCommand(DetailsCommand, 180);
					DetailResult.WaitForExit();

					if (DetailResult.ExitCode != 0)
					{
						Log.Warning("Detailing {DeviceName} attempt {Attempt} exited with code {ExitCode}:\n{Output}",
							UUID, attempt, DetailResult.ExitCode, DetailResult.Output);

						if (attempt < MaxRetries)
						{
							Log.Info("Retrying device info details after 10 second wait...");
							Thread.Sleep(10000);
							continue;
						}
						else
						{
							Log.Warning("Device info details failed after {0} attempts", MaxRetries);
							return false;
						}
					}

					// Validate file exists and has content before parsing
					if (!File.Exists(DetailsFile))
					{
						Log.Warning("Device info details file not found at {0} (attempt {1})", DetailsFile, attempt);
						if (attempt < MaxRetries)
						{
							Thread.Sleep(10000);
							continue;
						}
						return false;
					}

					FileInfo fileInfo = new FileInfo(DetailsFile);
					if (fileInfo.Length == 0)
					{
						Log.Warning("Device info details file is empty for {0} (attempt {1})", UUID, attempt);
						if (attempt < MaxRetries)
						{
							Thread.Sleep(10000);
							continue;
						}
						return false;
					}

					// Parse JSON and validate connection immediately with fresh data
					try
					{
						JsonDevice = JsonObject.Parse(File.ReadAllText(DetailsFile)).GetObjectField("result");
						Log.Verbose("Successfully retrieved device details for {0} on attempt {1}", UUID, attempt);

						// Validate connection state immediately while data is fresh
						bool bIsConnected = ValidateDeviceConnection(JsonDevice, true);

						if (bIsConnected)
						{
							Log.Verbose("Device {0} connection validated successfully on attempt {1}", UUID, attempt);
							break; // Success - device is connected
						}
						else
						{
							// Connection validation failed - retry if we have attempts left
							if (attempt < MaxRetries)
							{
								Log.Info("Device {0} not connected (attempt {1}), waiting 10s before retry...", UUID, attempt);
								Thread.Sleep(10000);
								continue;
							}
							else
							{
								Log.Warning("Device {0} connection validation failed after {1} attempts", UUID, MaxRetries);
								return false;
							}
						}
					}
					catch (Exception ex)
					{
						Log.Warning("Failed to parse device details JSON for {0} (attempt {1}): {2}", UUID, attempt, ex.Message);
						if (attempt < MaxRetries)
						{
							Thread.Sleep(10000);
							continue;
						}
						throw;
					}
				}

				if (JsonDevice == null)
				{
					Log.Warning("Failed to retrieve valid device details for {0} after {1} attempts", UUID, MaxRetries);
					return false;
				}

				if (ConnectedDevices.ContainsKey(UUID))
				{
					ConnectedDevices[UUID].bConnected = true;
				}
				else
				{
					string OSVersionString = JsonDevice.GetObjectField("deviceProperties").GetStringField("osVersionNumber");
					int IOSMajorVersion = int.Parse(OSVersionString.Substring(0, OSVersionString.IndexOf('.')));

					ConnectedDevice Device = new ConnectedDevice(UUID, IOSMajorVersion, true);
					ConnectedDevices.Add(UUID, Device);
				}
			}

			return true;
		}

		public bool Disconnect(bool bForce = false)
		{
			// Disconnecting from an iOS device causes a lot of problems for device farm automation
			// Namely, re-pairing a device requires someone to re-trust the host PC manually
			return true;
		}

		public bool PowerOn()
		{
			return true;
		}

		public bool PowerOff()
		{
			return true;
		}

		public bool Reboot()
		{
			// Rebooting tends to lock the device and break connections requiring manual intervention to unlock the device
			return true;
		}

		public Dictionary<EIntendedBaseCopyDirectory, string> GetPlatformDirectoryMappings()
		{
			return LocalDirectoryMappings;
		}
		public override string ToString()
		{
			return Name;
		}

		public void TakeScreenshot(string ScreenshotName)
		{
			if (AppiumContainer is null)
			{
				Log.Warning(KnownLogEvents.Gauntlet_TestEvent, "Cannot perform screenshot, AppiumContainer is null");
				return;
			}

			AppiumContainer.TakeScreenshot(AppiumContainer.GetScreenshotDirectory(), ScreenshotName);
		}
		#endregion

		public void FullClean()
		{
			// Patterns for apps that should NOT be uninstalled (supports wildcard matching via EndsWith)
			List<string> AppAllowListPatterns = new List<string>
			{
				"WebDriverAgent.xctrunner",
				"deviceControl.xctrunner",
				"deviceControl"
			};
			string AppOutput;
			if (UseDeviceCtl)
			{
				IProcessResult AppListResult = ExecuteDevicectlCommand("device info apps", 180);
				if (AppListResult.ExitCode != 0)
				{
					throw new DeviceException("App list exited with code {0}: {1}", AppListResult.ExitCode, AppListResult.Output);
				}

				AppOutput = AppListResult.Output;
			}
			else
			{
				IProcessResult AppListResult = ExecuteIOSDeployCommand("-B");
				if (AppListResult.ExitCode != 0)
				{
					throw new DeviceException("App list exited with code {0}: {1}", AppListResult.ExitCode, AppListResult.Output);
				}

				AppOutput = AppListResult.Output;
			}

			// Match bundle IDs - handle both raw format and "bundleID: com.xxx" format
			Regex AppRegex = new Regex(@"(?:bundleID:\s*)?(com\.[^\s,\)]+)", RegexOptions.IgnoreCase);
			var allApps = AppRegex.Matches(AppOutput)
				.Where(Match => Match.Success)
				.Select(Match => Match.Groups[1].Value) // Get the captured group (the bundle ID)
				.Where(Match => !Match.Contains("com.apple.", StringComparison.OrdinalIgnoreCase))
				.ToList();

			IEnumerable<string> AppList = allApps
				.Where(app =>
				{
					bool isProtected = AppAllowListPatterns.Any(pattern => app.EndsWith(pattern, StringComparison.OrdinalIgnoreCase));
					if (isProtected)
					{
						Log.Verbose("Skipping protected app: {0}", app);
					}
					return !isProtected;
				});

			foreach (string App in AppList)
			{
				Log.Info("Uninstalling {App}...", App);

				IProcessResult UninstallResult;
				if (UseDeviceCtl)
				{
					UninstallResult = ExecuteDevicectlCommand(string.Format("device uninstall app {0}", App));
				}
				else
				{
					UninstallResult = ExecuteIOSDeployCommand(string.Format("--bundle_id {0} --uninstall_only", App));
				}

				UninstallResult.WaitForExit();
				if (UninstallResult.ExitCode != 0)
				{
					throw new DeviceException("Uninstalling {0} exited with code {1}: {2}", App, UninstallResult.ExitCode, UninstallResult.Output);
				}
			}
		}

		public void CleanArtifacts(UnrealAppConfig AppConfig = null)
		{
			if (AppConfig == null)
			{
				Log.Warning("{Platform} expects an non-null AppConfig value to reliably determine which artifact path to clear! Skipping clean", Platform);
				return;
			}

			if (AppConfig.Build is not IOSBuild Build)
			{
				return;
			}

			CleanArtifacts(Build.PackageName, AppConfig.ProjectName);
		}

		public void CleanArtifacts(string BundleID, string ProjectName)
		{
			if (UseDeviceCtl)
			{
				// devicectl doesn't support direct file deletion, so we use an empty directory replacement approach
				// Create empty local directories and copy them to the device to clear the Saved folders

				string DevicectlEngineSaved = "Documents/Engine/Saved";
				string DevicectlProjectSaved = string.Format("Documents/{0}/Saved", ProjectName);

				// Create temporary empty directories
				string TempCleanupPath = Path.Combine(LocalCachePath, "CleanupTemp");
				string EmptyEngineSaved = Path.Combine(TempCleanupPath, "Engine", "Saved");
				string EmptyProjectSaved = Path.Combine(TempCleanupPath, ProjectName, "Saved");

				try
				{
					// Clean up any existing temp cleanup directory
					if (Directory.Exists(TempCleanupPath))
					{
						Directory.Delete(TempCleanupPath, true);
					}

					// Create empty directory structures with marker files
					// devicectl doesn't handle truly empty directories well, so we add a small marker file
					Directory.CreateDirectory(EmptyEngineSaved);
					Directory.CreateDirectory(EmptyProjectSaved);
					File.WriteAllText(Path.Combine(EmptyEngineSaved, ".cleanup"), "cleaned");
					File.WriteAllText(Path.Combine(EmptyProjectSaved, ".cleanup"), "cleaned");

					// Copy empty Engine/Saved directory to device
					Log.Info("Cleaning Engine artifacts for {0}", BundleID);
					string CleanEngineCommand = string.Format("device copy to --device {0} --source {1} --destination {2} --domain-type appDataContainer --domain-identifier {3}",
						UUID, EmptyEngineSaved, DevicectlEngineSaved, BundleID);
					IProcessResult EngineResult = ExecuteDevicectlCommand(CleanEngineCommand, UseDeviceID: false, WaitTime: 120);
					EngineResult.WaitForExit();

					if (EngineResult.ExitCode != 0)
					{
						// ContainerLookupErrorDomain error -1 means the app container doesn't exist (app not installed or was uninstalled)
						if (EngineResult.Output.Contains("ContainerLookupErrorDomain error -1"))
						{
							Log.Verbose("App container for {0} not found - no artifacts to clean", BundleID);
							return;
						}
						Log.Warning("Cleaning Engine artifacts for {0} exited with code {1}:\n{2}", BundleID, EngineResult.ExitCode, EngineResult.Output);
					}

					// Copy empty Project/Saved directory to device
					Log.Info("Cleaning Project artifacts for {0}", BundleID);
					string CleanProjectCommand = string.Format("device copy to --device {0} --source {1} --destination {2} --domain-type appDataContainer --domain-identifier {3}",
						UUID, EmptyProjectSaved, DevicectlProjectSaved, BundleID);
					IProcessResult ProjectResult = ExecuteDevicectlCommand(CleanProjectCommand, UseDeviceID: false, WaitTime: 120);
					ProjectResult.WaitForExit();

					if (ProjectResult.ExitCode != 0)
					{
						// ContainerLookupErrorDomain error -1 means the app container doesn't exist (app not installed or was uninstalled)
						if (ProjectResult.Output.Contains("ContainerLookupErrorDomain error -1"))
						{
							Log.Verbose("App container for {0} not found - no artifacts to clean", BundleID);
							return;
						}
						Log.Warning("Cleaning Project artifacts for {0} exited with code {1}:\n{2}", BundleID, ProjectResult.ExitCode, ProjectResult.Output);
					}

					// Verify cleanup by checking directory contents
					Log.Verbose("Verifying artifact cleanup for {0}", BundleID);
					string VerifyPath = Path.Combine(TempCleanupPath, "Verify");
					if (Directory.Exists(VerifyPath))
					{
						Directory.Delete(VerifyPath, true);
					}
					Directory.CreateDirectory(VerifyPath);

					// Download Engine/Saved to verify
					string VerifyEngineCommand = string.Format("device copy from --device {0} --source {1} --destination {2} --domain-type appDataContainer --domain-identifier {3}",
						UUID, DevicectlEngineSaved, VerifyPath, BundleID);
					IProcessResult VerifyEngineResult = ExecuteDevicectlCommand(VerifyEngineCommand, UseDeviceID: false, WaitTime: 60, bQuiet: true);
					VerifyEngineResult.WaitForExit();

					if (VerifyEngineResult.ExitCode == 0 && Directory.Exists(VerifyPath))
					{
						// Count all files recursively from the verify path
						string[] engineFiles = Directory.GetFiles(VerifyPath, "*", SearchOption.AllDirectories);
						Log.Info("Engine/Saved contains {0} file(s) after cleanup", engineFiles.Length);
						if (engineFiles.Length > 0 && Log.IsVeryVerbose)
						{
							foreach (string file in engineFiles)
							{
								string relativePath = file.Replace(VerifyPath, "").TrimStart(Path.DirectorySeparatorChar, Path.AltDirectorySeparatorChar);
								Log.Verbose("  - {0}", relativePath);
							}
						}
					}
					else
					{
						Log.Verbose("Could not verify Engine/Saved cleanup (exit code {0})", VerifyEngineResult.ExitCode);
					}
				}
				finally
				{
					// Clean up temporary directories
					if (Directory.Exists(TempCleanupPath))
					{
						try
						{
							Directory.Delete(TempCleanupPath, true);
						}
						catch (Exception Ex)
						{
							Log.Verbose("Failed to delete temp cleanup directory: {0}", Ex.Message);
						}
					}
				}

				return;
			}

			// Use ios-deploy for cleanup (legacy approach)
			string EngineSavedDirectory = "/Documents/Engine/Saved";
			string ProjectSavedDirectory = string.Format("/Documents/{0}/Saved", ProjectName);

			string ListEngineSaved  = string.Format("--bundle_id {0} --list={1}",   BundleID, EngineSavedDirectory);
			string CleanEngineSaved = string.Format("--bundle_id {0} --rmtree {1}", BundleID, EngineSavedDirectory);
			string MakeEngineSaved  = string.Format("--bundle_id {0} --mkdir {1}",  BundleID, EngineSavedDirectory);

			// The output of the --list command will print all files in that path
			// so if the output contains the path, it exists
			// if no output is provided, the path does not exist

			IProcessResult Result = ExecuteIOSDeployCommand(ListEngineSaved);
			Result.WaitForExit();

			if (Result.ExitCode == 253) // Exit Code 253 - no bundle exists meaning there is nothing to clean
			{
				return;
			}
			else if (Result.ExitCode != 0)
			{
				Log.Warning(String.Format("Listing Engine artifacts for {0} exited with code {1}:\n{2}", BundleID, Result.ExitCode, Result.Output));
			}

			if (Result.Output.Contains(EngineSavedDirectory))
			{
				Result = ExecuteIOSDeployCommand(CleanEngineSaved);
				Result.WaitForExit();
				if (Result.ExitCode != 0)
				{
					Log.Warning(String.Format("Cleaning Engine artifacts for {0} exited with code {1}:\n{2}", BundleID, Result.ExitCode, Result.Output));
				}

				Result = ExecuteIOSDeployCommand(MakeEngineSaved);
				Result.WaitForExit();
				if (Result.ExitCode != 0)
				{
					throw new DeviceException("Making empty Engine artifacts for {0} exited with code {1}:\n{2}", BundleID, Result.ExitCode, Result.Output);
				}
			}

			string ListProjectSaved  = string.Format("--bundle_id {0} --list={1}",   BundleID, ProjectSavedDirectory);
			string CleanProjectSaved = string.Format("--bundle_id {0} --rmtree {1}", BundleID, ProjectSavedDirectory);
			string MakeProjectSaved  = string.Format("--bundle_id {0} --mkdir {1}",  BundleID, ProjectSavedDirectory);

			Result = ExecuteIOSDeployCommand(ListProjectSaved);
			Result.WaitForExit();
			if (Result.ExitCode != 0)
			{
				Log.Warning(String.Format("Listing Project artifacts for {0} exited with code {1}:\n{2}", BundleID, Result.ExitCode, Result.Output));
			}

			// The same applies here for the --list
			if (Result.Output.Contains(ProjectSavedDirectory))
			{
				Result = ExecuteIOSDeployCommand(CleanProjectSaved);
				Result.WaitForExit();
				if (Result.ExitCode != 0)
				{
					Log.Warning(String.Format("Cleaning Project artifacts for {0} exited with code {1}:\n{2}", BundleID, Result.ExitCode, Result.Output));
				}

				Result = ExecuteIOSDeployCommand(MakeProjectSaved);
				Result.WaitForExit();
				if (Result.ExitCode != 0)
				{
					throw new DeviceException("Making empty Project artifacts for {0} exited with code {1}:\n{2}", BundleID, Result.ExitCode, Result.Output);
				}
			}
		}

		public void InstallBuild(UnrealAppConfig AppConfig)
		{
			if (AppConfig.Build is not IOSBuild Build)
			{
				throw new AutomationException("Unsupported build type {0} for iOS!", AppConfig.Build.GetType());
			}

			string BuildPath = Build.SourcePath;
			string AppName = Path.GetFileName(Build.SourcePath);
			if (Globals.IsRunningDev && AppConfig.OverlayExecutable.GetOverlay(BuildPath, out string OverlayExecutable))
			{
				BuildPath = OverlayExecutable;
			}
			else if (Build.IsIPAFile)
			{
				// If the build is an IPA file, unzip it to the cache directory
				string BuildCache = Path.Combine(LocalCachePath, "TempBuildCache");
				string PayloadDirectory = Path.Combine(BuildCache, "Payload");

				if (Directory.Exists(PayloadDirectory))
				{
					Directory.Delete(PayloadDirectory, true);
				}
				Directory.CreateDirectory(BuildCache);

				string IPACommand = string.Format("-x -k {0} {1}", BuildPath, BuildCache);
				if (!AppleBuild.ExecuteIPADittoCommand(IPACommand, out string Output, PayloadDirectory))
				{
					throw new AutomationException("Unable to extract IPA {0}:\n{1}", Build.SourcePath, Output);
				}

				BuildPath = Directory.EnumerateDirectories(PayloadDirectory, "*.app").FirstOrDefault();
				if (BuildPath == null)
				{
					throw new AutomationException("Failed to find app within IPA payload directory!");
				}
			}

			// Uninstall any existing form of the app
			IProcessResult Result;
			if (UseDeviceCtl)
			{
				Result = ExecuteDevicectlCommand(string.Format("device uninstall app {0}", Build.PackageName));
			}
			else
			{
				Result = ExecuteIOSDeployCommand(string.Format("--bundle_id {0} --uninstall_only", Build.PackageName));
			}

			Result.WaitForExit();
			if (Result.ExitCode != 0)
			{
				throw new DeviceException("Uninstall exited with code {0}:\n{1}", Result.ExitCode, Result.Output);
			}

			// Install the app
			// Bad provisions can cause this to spit out errors, ignore these and let higher levels manage the exception's verbosity

			DateTime StartTime = DateTime.Now;
			using (ScopedSuspendECErrorParsing ErrorSuspension = new())
			{
				if (UseDeviceCtl)
				{
					Result = ExecuteDevicectlCommand(string.Format("device install app {0} -v", BuildPath), MaxInstallTime, AdditionalOptions: ERunOptions.NoStdOutRedirect);
				}
				else
				{
					Result = ExecuteIOSDeployCommand(string.Format("-b \"{0}\"", BuildPath), MaxInstallTime, AdditionalOptions: ERunOptions.NoStdOutRedirect);
				}
			}

			if (Result.ExitCode != 0)
			{
				throw new DeviceException("Install exited with code {0}:\n{1}", Result.ExitCode, Result.Output);
			}

			double RemainingTime = MaxInstallTime - (StartTime - DateTime.Now).TotalSeconds;

			if (Build.Flags.HasFlag(BuildFlags.Bulk))
			{
				string AppPath = Build.SourcePath;

				// When targeting a build in the network share, pull the build content down to the host first for a faster install.
				// Note: it's only faster when the device is connected via USB 3.0, so WiFi is actually faster for our device farm because we use 2.0 for host connections
				// When running on builders, we'll just copy the build directly from the share
				string BuildVolume = GetVolumeName(Build.SourcePath);
				string CurrentVolume = GetVolumeName(Environment.CurrentDirectory);
				if (!BuildVolume.Equals(CurrentVolume, StringComparison.OrdinalIgnoreCase) && !IsBuildMachine)
				{
					string TempBuildCache = Path.Combine(Globals.TempDir, "TempIOSBuildCache");
					string SourceBulkPath = Path.GetFullPath(Path.Combine(Build.SourcePath, "..", ".."));
					string SourceBulkContentsPath = Path.GetFullPath(Path.Combine(Build.SourcePath, "..", "..", "..", "BulkContents"));

					if (!Directory.Exists(SourceBulkContentsPath))
					{
						throw new AutomationException("Could not locate a bulk contents directory when attempting to install a bulk build. Is your bulk build configured correctly?");
					}

					string DestinationBulkPath = Path.Combine(TempBuildCache, Path.GetFileName(SourceBulkPath));
					string DestinationBulkContentsPath = Path.Combine(TempBuildCache, Path.GetFileName(SourceBulkContentsPath));

					lock (Mutex)
					{
						if (!HasCopiedNetworkBuild)
						{
							if (Directory.Exists(TempBuildCache))
							{
								Directory.Delete(TempBuildCache, true);
							}
							Directory.CreateDirectory(TempBuildCache);

							Log.Info("Copying {Source} to {Dest}", SourceBulkPath, DestinationBulkPath);
							SystemHelpers.CopyDirectory(SourceBulkPath, DestinationBulkPath, SystemHelpers.CopyOptions.Copy);

							Log.Info("Copying {Source} to {Dest}", SourceBulkContentsPath, DestinationBulkContentsPath);
							SystemHelpers.CopyDirectory(SourceBulkContentsPath, DestinationBulkContentsPath, SystemHelpers.CopyOptions.Copy);

							Log.Info("Copy to host complete. Starting deploy to {DeviceName}", UUID);

							// If a device has an issue and we need to select a new one, don't re-copy the whole build from the network again!
							HasCopiedNetworkBuild = true;
						}

						AppPath = Path.Combine(DestinationBulkPath, AppPath.Replace(SourceBulkPath, string.Empty).Trim('\\').Trim('/'));
					}
				}

				int CurrentFileIndex = 1;
				int BulkContentFileCount = Build.BulkContents.Count;
				Log.Info("Starting installation of Bulk Content ({FileCount} files)...", BulkContentFileCount);

				foreach (KeyValuePair<string, string> ContentFile in Build.BulkContents)
				{
					StartTime = DateTime.Now;
					string FileName = Path.GetFileName(ContentFile.Key);
					Log.Info("({Current}/{Count}) Installing {FileName}...", CurrentFileIndex, BulkContentFileCount, FileName);

					double RemainingTimePerFile = Math.Min(RemainingTime, MaxPerFileInstallTime);
					string QualifiedSourcePath = Path.GetFullPath(Path.Combine(AppPath, ContentFile.Key));
					IProcessResult ContentCopyResult = CopyFileToDevice(QualifiedSourcePath, ContentFile.Value, Build.PackageName, (int)RemainingTimePerFile, true);

					double DeltaTime = (DateTime.Now - StartTime).TotalSeconds;
					RemainingTime -= DeltaTime;

					if (ContentCopyResult.ExitCode != 0)
					{
						if (RemainingTimePerFile <= DeltaTime)
						{
							throw new DeviceException("Install timed out after {0} seconds", RemainingTimePerFile);
						}
						else
						{
							throw new DeviceException("Install of content file {0} failed with exit code {1}:\n{2}", FileName, ContentCopyResult.ExitCode, ContentCopyResult.Output);
						}
					}

					++CurrentFileIndex;
				}
			}
		}

		public IAppInstall CreateAppInstall(UnrealAppConfig AppConfig)
		{
			if (AppConfig.Build is not IOSBuild Build)
			{
				throw new AutomationException("Unsupported build type {0} for iOS!", AppConfig.Build.GetType());
			}

			string ProjectDirectory = string.Format("/Documents/{0}", AppConfig.ProjectName);
			PopulateDirectoryMappings(ProjectDirectory);

			string AppPath = null;
			if (!UseDeviceCtl)
			{
				if (Globals.IsRunningDev)
				{
					AppConfig.OverlayExecutable.GetOverlay(Build.SourcePath, out AppPath);
				}
				else
				{
					AppPath = Build.SourcePath;
				}
			}

			InstallCache = new IOSAppInstall(AppConfig.Name, this, Build.PackageName, AppConfig.CommandLine, AppConfig.ProjectName, AppPath);
			return InstallCache;
		}

		public void CopyAdditionalFiles(IEnumerable<UnrealFileToCopy> FilesToCopy)
		{
			if (InstallCache == null || string.IsNullOrEmpty(InstallCache.PackageName))
			{
				throw new AutomationException("Could not deduce Bundle ID due to invalid installation cache. " +
					"Either specify the Bundle ID using this function's overload OR ensure CreateAppInstall has been called first.");
			}

			CopyAdditionalFiles(FilesToCopy, InstallCache.PackageName);
		}

		public void CopyAdditionalFiles(IEnumerable<UnrealFileToCopy> FilesToCopy, string BundleID)
		{
			if (FilesToCopy == null || !FilesToCopy.Any())
			{
				return;
			}

			if (!LocalDirectoryMappings.Any())
			{
				throw new AutomationException("Attempted to copy additional files before LocalDirectoryMappings were populated." +
					"{0} must call PopulateDirectoryMappings before attempting to call CopyAdditionalFiles", this.GetType());
			}

			foreach (UnrealFileToCopy FileToCopy in FilesToCopy)
			{
				FileInfo SourceFile = new FileInfo(FileToCopy.SourceFileLocation);
				FileInfo DestinationFile = new FileInfo(string.Format("/{0}/{1}", LocalDirectoryMappings[FileToCopy.TargetBaseDirectory], FileToCopy.TargetRelativeLocation));
				if (!SourceFile.Exists)
				{
					Log.Warning(KnownLogEvents.Gauntlet_DeviceEvent, "File to copy {File} not found. Skipping copy.", FileToCopy);
					continue;
				}
				SourceFile.IsReadOnly = false;

				Log.Verbose("Copying {Source} to {Destination}", SourceFile, DestinationFile);

				IProcessResult Result = CopyFileToDevice(SourceFile.FullName, DestinationFile.FullName, BundleID);
				if (Result.ExitCode != 0)
				{
					throw new AutomationException("Copy of {0} to {1} exited with code {2}:\n{3}",
						SourceFile, DestinationFile, Result.ExitCode, Result.Output);
				}
			}
		}

		public IProcessResult CopyFileToDevice(string Source, string Destination, string PackageName, int WaitTime = 60, bool bQuiet = false)
		{
			IProcessResult CopyResult;

			if (!Destination.StartsWith("/Documents", StringComparison.OrdinalIgnoreCase)
			&& !Destination.StartsWith("Documents", StringComparison.OrdinalIgnoreCase))
			{
				Destination = Destination.StartsWith('/')
					? "/Documents" + Destination
					: "/Documents/" + Destination;
			}

			if (UseDeviceCtl && !UseIOSDeployForCopy)
			{
				if (Destination.StartsWith('/'))
				{
					Destination = Destination.Substring(1);
				}

				string CopyCommand = FormatDeviceCopyCommand(UUID, Source, Destination, PackageName);
				CopyResult = ExecuteDevicectlCommand(CopyCommand, UseDeviceID: false, WaitTime: WaitTime, bQuiet: bQuiet);
			}
			else
			{
				string CopyCommand = string.Format("--bundle_id {0} --upload={1} --to {2}",
					PackageName, Source, Destination);
				CopyResult = ExecuteIOSDeployCommand(CopyCommand, WaitTime, bQuiet: bQuiet);
			}

			return CopyResult;
		}

		public void PopulateDirectoryMappings(string ProjectDirectory)
		{
			string SavedDirectory = ProjectDirectory + "/Saved";

			LocalDirectoryMappings.Add(EIntendedBaseCopyDirectory.Build, ProjectDirectory + "/Build");
			LocalDirectoryMappings.Add(EIntendedBaseCopyDirectory.Binaries, ProjectDirectory + "/Binaries");
			LocalDirectoryMappings.Add(EIntendedBaseCopyDirectory.Config, SavedDirectory + "/Config");
			LocalDirectoryMappings.Add(EIntendedBaseCopyDirectory.Content, ProjectDirectory + "/Content");
			LocalDirectoryMappings.Add(EIntendedBaseCopyDirectory.Demos, SavedDirectory + "/Demos");
			LocalDirectoryMappings.Add(EIntendedBaseCopyDirectory.PersistentDownloadDir, SavedDirectory + "/PersistentDownloadDir");
			LocalDirectoryMappings.Add(EIntendedBaseCopyDirectory.Profiling, SavedDirectory + "/Profiling");
			LocalDirectoryMappings.Add(EIntendedBaseCopyDirectory.Saved, SavedDirectory);
		}

		public IAppInstance Run(IAppInstall Install)
		{
			if (Install is not IOSAppInstall IOSInstall)
			{
				throw new AutomationException("AppInstance is of incorrect type!");
			}

			// Copy the commandline into a uecommandline.txt
			string CommandLine = IOSInstall.CommandLine.Trim();
			string TempCommandLine = Path.GetTempFileName();
			File.WriteAllText(TempCommandLine, CommandLine);

			IProcessResult CommandLineCopyResult;
			if (UseDeviceCtl)
			{
				string CopyCommand = FormatDeviceCopyCommand(UUID, TempCommandLine, Path.Combine("Documents", "uecommandline.txt"), IOSInstall.PackageName);
				CommandLineCopyResult = ExecuteDevicectlCommand(CopyCommand, UseDeviceID: false);
			}
			else
			{
				string CopyCommand = string.Format("--bundle_id {0} --upload={1} --to {2}",
					IOSInstall.PackageName, TempCommandLine, Path.Combine("/Documents", "uecommandline.txt"));
				CommandLineCopyResult = ExecuteIOSDeployCommand(CopyCommand);
			}

			if (CommandLineCopyResult.ExitCode != 0)
			{
				throw new DeviceException("Failed to copy uecommandline file to device!");
			}
			
			if (AppiumContainer != null)
			{
				Log.Info("Appium Enabled: {0} on {1}", Install.Name, this);
				AppiumContainer.Start(IOSInstall.PackageName);
			}

			Log.Info("Launching {0} on {1}", Install.Name, this);
			Log.Verbose(IOSInstall.CommandLine);

			ILongProcessResult AppProcess;
			if (UseDeviceCtl)
			{
				// device process launch	base command
				// --terminate-existing		exit any existing forms of the app
				// --console				pipe process stdout to devicectl, also holds onto the process until exit instead of immediate exit
				// --device					device to run on
				string LaunchCommand = $"device process launch --terminate-existing --console --environment-variables {{\"OS_ACTIVITY_DT_MODE\":\"enable\"}} --device {UUID} {IOSInstall.PackageName}";
				AppProcess = ExecuteDevicectlCommandNoWait(LaunchCommand, AdditionalOptions: ERunOptions.SpewIsVerbose, LocalCache: IOSInstall.Device.LocalCachePath);
			}
			else
			{
				if (IOSInstall.AppPath == null)
				{
					// Sanity check, this should never happen, but it would be tricky to debug without an explicit message if it did!
					throw new AutomationException("The provided install to run has a null app path, this likely means the install was designed " +
						"to run with devicectl instead of ios-deploy.");
				}

				// -m		Skip installation
				// -I		Non-interactive mode, just launch and end launch process when app process ends
				// -b		path to bundle to run
				string LaunchCommand = string.Format("-m -I -b \"{0}\"", IOSInstall.AppPath);
				AppProcess = ExecuteIOSDeployCommandNoWait(LaunchCommand, LocalCache: IOSInstall.Device.LocalCachePath);
			}

			if (AppProcess.HasExited)
			{
				throw new DeviceException("Failed to launch on {0}. {1}", Name, AppProcess.Output);
			}

			if (AppiumContainer != null)
			{
				AppiumContainer.StartMonitorThread();
			}

			return new IOSAppInstance(IOSInstall, AppProcess, IOSInstall.CommandLine);
		}

		public static IProcessResult ExecuteIOSDeployCommand(string CommandLine, string Device, int WaitTime = 60, bool WarnOnTimeout = true, ERunOptions AdditionalOptions = ERunOptions.None, bool bQuiet = false)
		{
			string IOSDeploy = Path.Combine(Globals.UnrealRootDir, "Engine/Extras/ThirdPartyNotUE/ios-deploy/bin/ios-deploy");
			if (!string.IsNullOrEmpty(Device))
			{
				CommandLine = string.Format("{0} -i {1}", CommandLine, Device);
			}

			return ExecuteDeploymentCommand(IOSDeploy, CommandLine, WaitTime, WarnOnTimeout, null, AdditionalOptions, bQuiet);
		}

		public static ILongProcessResult ExecuteIOSDeployCommandNoWait(string CommandLine, ERunOptions AdditionalOptions = ERunOptions.None, string LocalCache = null, bool bQuiet = false)
		{
			string IOSDeploy = Path.Combine(Globals.UnrealRootDir, "Engine/Extras/ThirdPartyNotUE/ios-deploy/bin/ios-deploy");
			return ExecuteDeploymentCommandNoWait(IOSDeploy, CommandLine, null, AdditionalOptions, LocalCache, bQuiet);
		}

		public IProcessResult ExecuteIOSDeployCommand(string CommandLine, int WaitTime = 60, bool WarnOnTimeout = true, bool UseDeviceID = true, ERunOptions AdditionalOptions = ERunOptions.None, bool bQuiet = false)
		{
			if (UseDeviceID && !IsDefaultDevice)
			{
				return ExecuteIOSDeployCommand(CommandLine, UUID, WaitTime, WarnOnTimeout, AdditionalOptions, bQuiet);
			}
			else
			{
				return ExecuteIOSDeployCommand(CommandLine, null, WaitTime, WarnOnTimeout, AdditionalOptions, bQuiet);
			}
		}

		public static IProcessResult ExecuteDevicectlCommand(string CommandLine, string Device, int WaitTime = 60, bool WarnOnTimeout = true, ERunOptions AdditionalOptions = ERunOptions.None, bool bQuiet = false)
		{
			CommandLine = string.Format("devicectl {0}", CommandLine);

			string XCRun = "/usr/bin/xcrun";
			if (!string.IsNullOrEmpty(Device))
			{
				CommandLine = string.Format("{0} --device {1}", CommandLine, Device);
			}

			return ExecuteDeploymentCommand(XCRun, CommandLine, WaitTime, WarnOnTimeout, null, AdditionalOptions, bQuiet);
		}

		public static ILongProcessResult ExecuteDevicectlCommandNoWait(string CommandLine, ERunOptions AdditionalOptions = ERunOptions.None, string LocalCache = null, bool bQuiet = false)
		{
			CommandLine = string.Format("devicectl {0}", CommandLine);
			string XCRun = "/usr/bin/xcrun";
			return ExecuteDeploymentCommandNoWait(XCRun, CommandLine, AdditionalOptions: AdditionalOptions, LocalCache: LocalCache, bQuiet: bQuiet);
		}

		public IProcessResult ExecuteDevicectlCommand(string CommandLine, int WaitTime = 60, bool WarnOnTimeout = true, bool UseDeviceID = true, ERunOptions AdditionalOptions = ERunOptions.None, bool bQuiet = false)
		{
			if (UseDeviceID)
			{
				return ExecuteDevicectlCommand(CommandLine, UUID, WaitTime, WarnOnTimeout, AdditionalOptions, bQuiet);
			}
			else
			{
				return ExecuteDevicectlCommand(CommandLine, null, WaitTime, WarnOnTimeout, AdditionalOptions, bQuiet);
			}
		}

		private static string FormatDeviceCopyCommand(string UUID, string Source, string Destination, string PackageName)
		{
			// User specification is very important to get proper permissions set.
			// Otherwise, directory tree would be readable, but not writeable.
			string CopyCommand = string.Format("device copy to --device {0} --source {1} --destination {2} --user mobile --domain-type appDataContainer --domain-identifier {3}",
				UUID, Source, Destination, PackageName);
			return CopyCommand;
		}

		private static IProcessResult ExecuteDeploymentCommand(string Executable, string CommandLine, int WaitTime = 60, bool WarnOnTimeout = true, string WorkingDir = null, ERunOptions AdditionalOptions = ERunOptions.None, bool bQuiet = false)
		{
			if (!AdditionalOptions.HasFlag(ERunOptions.UseShellExecute) && !File.Exists(Executable))
			{
				throw new AutomationException("Unable to find deployment binary at {0}", Executable);
			}

			if (!bQuiet)
			{
				Log.Info("{DeploymentExecutable} executing '{Command}'", Executable, CommandLine);
			}

			ERunOptions RunOptions = ERunOptions.NoWaitForExit | AdditionalOptions;
			RunOptions |= Log.IsVeryVerbose
				? ERunOptions.AllowSpew
				: ERunOptions.NoLoggingOfRunCommand;


			IProcessResult Result = null;
			try
			{
				Result = CommandUtils.Run(Executable, CommandLine, Options: RunOptions, WorkingDir: WorkingDir);

				if (WaitTime > 0)
				{
					DateTime StartTime = DateTime.Now;

					Result.ProcessObject.WaitForExit(WaitTime * 1000);

					if (Result.HasExited == false)
					{
						if ((DateTime.Now - StartTime).TotalSeconds >= WaitTime)
						{
							string Message = string.Format("{0} timeout after {1} secs: {2}, killing process", Executable, WaitTime, CommandLine);

							if (WarnOnTimeout)
							{
								Log.Warning(Message);
							}
							else
							{
								Log.Info(Message);
							}

							Result.ProcessObject?.Kill();
							Result.ProcessObject?.WaitForExit(15000);
						}
					}
				}
			}
			catch (Exception Ex)
			{
				Log.Warning("Encountered an {ExceptionType} while running device command {Exception}", Ex.GetType().Name, Ex.Message);
			}

			return Result;
		}

		public static ILongProcessResult ExecuteDeploymentCommandNoWait(string Executable, string CommandLine, string WorkingDir = null, ERunOptions AdditionalOptions = ERunOptions.None, string LocalCache = null, bool bQuiet = false)
		{
			if (!AdditionalOptions.HasFlag(ERunOptions.UseShellExecute) && !File.Exists(Executable))
			{
				throw new AutomationException("Unable to find deployment binary at {0}", Executable);
			}

			if (!bQuiet)
			{
				Log.Info("{DeploymentExecutable} executing '{Command}'", Executable, CommandLine);
			}

			ERunOptions RunOptions = ERunOptions.NoWaitForExit | AdditionalOptions;
			RunOptions |= Log.IsVeryVerbose
				? ERunOptions.AllowSpew
				: ERunOptions.NoLoggingOfRunCommand;

			ILongProcessResult Result = null;
			try
			{
				Result = new LongProcessResult(Executable, CommandLine, Options: RunOptions, WorkingDir: WorkingDir, LocalCache: LocalCache);
			}
			catch (Exception Ex)
			{
				Log.Warning("Encountered an {ExceptionType} while running device command {Exception}", Ex.GetType().Name, Ex.Message);
			}

			return Result;
		}

		private static void InitializeDevices()
		{
			lock (Mutex)
			{
				if (bStaticInitialized)
				{
					if (!ConnectedDevices.Any())
					{
						throw new AutomationException("No iOS device connections were detected! Verify the host's setup.");
					}

					return;
				}

				bStaticInitialized = true;

				// Determine which version of XCode is installed
				// On XCode version 15 or greater, devicectl will be used to manage deployments
				// On XCode version 14 or lower, ios-deploy will be used to manage deployments
				IProcessResult VersionResult = CommandUtils.Run("xcodebuild", "-version");
				if (VersionResult.ExitCode != 0)
				{
					throw new AutomationException("XCode version query exited with code {0}. Do you have XCode installed?\n{1}", VersionResult.ExitCode, VersionResult.Output);
				}

				Match VersionMatch = Regex.Match(VersionResult.Output, @"Xcode\ (?<Version>\d+)");
				if (!VersionMatch.Success)
				{
					throw new AutomationException("Failed to match version output, has Apple changed output formatting?");
				}

				XCodeMajorVersion = int.Parse(VersionMatch.Groups[1].Value);

				UseDeviceCtl = XCodeMajorVersion >= 15;

				// Clear any zombie processes previous UAT instances may have left running
				AppleBuild.ExecuteCommand("killall", "xcrun").WaitForExit();
				AppleBuild.ExecuteCommand("killall", "ios-deploy").WaitForExit();
				AppleBuild.ExecuteCommand("killall", "lldb").WaitForExit();
				//AppleBuild.ExecuteCommand("killall", "usbmuxd").WaitForExit();
				//AppleBuild.ExecuteCommand("launchctl", "kickstart -k system/com.apple.usbmuxd").WaitForExit();

				// Gather the iOS devices are connected to this UAT host
				ConnectedDevices = GetConnectedDevices();

				if (!ConnectedDevices.Any())
				{
					throw new AutomationException("No iOS device connections were detected! Verify the host's setup.");
				}
			}
		}

		private static Dictionary<string, ConnectedDevice> GetConnectedDevices()
		{
			Dictionary<string, ConnectedDevice> Devices = GetConnectedDevicesInternal();

			if ((Devices == null || !Devices.Any()))
			{
				Devices = GetConnectedDevicesInternal();

				if (Devices == null || !Devices.Any())
				{
					Log.Warning("Still no devices found after retry.");
				}
				else
				{
					Log.Info("Found {0} device(s) after retry.", Devices.Count);
				}
			}

			return Devices;
		}

		private static Dictionary<string, ConnectedDevice> GetConnectedDevicesInternal()
		{
			Dictionary<string, ConnectedDevice> Devices = new();

			if (UseDeviceCtl)
			{
				const int MaxRetries = 3; // 1 original attempt + 2 retries
				int attemptCount = 0;
				bool success = false;

				while (attemptCount < MaxRetries && !success)
				{
					attemptCount++;
					if (attemptCount > 1)
					{
						Log.Warning("Retrying devicectl list devices (attempt {0}/{1})...", attemptCount, MaxRetries);
						Thread.Sleep(3000); // Wait 3 seconds between retries
					}

					string ListFile = Path.GetTempFileName();
					string ListCommand = string.Format("list devices -j {0}", ListFile);

					IProcessResult ListResult = ExecuteDevicectlCommand(ListCommand, null);
					if (ListResult.ExitCode != 0)
					{
						Log.Warning("List devices attempt {0} exited with code {1}:\n{2}", attemptCount, ListResult.ExitCode, ListResult.Output);
						if (attemptCount >= MaxRetries)
						{
							Log.Warning("List devices failed after {0} attempts", MaxRetries);
							return null;
						}
						continue; // Retry
					}

					if (!File.Exists(ListFile))
					{
						Log.Warning("List device output file not found at {0} (attempt {1})", ListFile, attemptCount);
						if (attemptCount >= MaxRetries)
						{
							Log.Warning("Could not find list device output file after {0} attempts", MaxRetries);
							return null;
						}
						continue; // Retry
					}

					JsonObject[] JsonDevices = JsonObject.Parse(File.ReadAllText(ListFile))
						.GetObjectField("result")
						.GetObjectArrayField("devices");

					foreach (JsonObject JsonDevice in JsonDevices)
					{
						ConnectedDevice IOSDevice = new ConnectedDevice(string.Empty, -1, false);

						if (JsonDevice.TryGetObjectField("hardwareProperties", out JsonObject HardwareProperties))
						{
							HardwareProperties.TryGetStringField("udid", out IOSDevice.UUID);
						}

						if (JsonDevice.TryGetObjectField("deviceProperties", out JsonObject DeviceProperties) &&
							DeviceProperties.TryGetStringField("osVersionNumber", out string OSVersionString) &&
							!string.IsNullOrWhiteSpace(OSVersionString))
						{
							IOSDevice.IOSMajorVersion = int.Parse(OSVersionString.Substring(0, OSVersionString.IndexOf('.')));
						}

						if (!string.IsNullOrWhiteSpace(IOSDevice.UUID) && IOSDevice.IOSMajorVersion != -1)
						{
							IOSDevice.bConnected = ValidateDeviceConnection(JsonDevice);
							Devices.Add(IOSDevice.UUID, IOSDevice);
						}
					}

					// Check if we found any devices
					if (Devices.Count == 0)
					{
						Log.Warning("devicectl list devices returned 0 devices (attempt {0}/{1})", attemptCount, MaxRetries);
						if (attemptCount >= MaxRetries)
						{
							Log.Warning("No devices found after {0} attempts", MaxRetries);
							return null;
						}
						Devices.Clear(); // Clear for next retry
						continue; // Retry
					}

					// Success - we have devices
					success = true;

					// Log summary of found devices
					Log.Info("Found {0} device(s) from devicectl (attempt {1}):", Devices.Count, attemptCount);
					foreach (var device in Devices.Values)
					{
						Log.Info("  - {0}: iOS {1}, Connected={2}", device.UUID, device.IOSMajorVersion, device.bConnected);
					}
				}
			}
			else
			{
				IProcessResult DetectResult = ExecuteIOSDeployCommand("--detect", null);
				if (DetectResult.ExitCode != 0)
				{
					Log.Warning("Detect devices exited with code {0}:\n{1}", DetectResult.ExitCode, DetectResult.Output);
					return null;
				}

				IEnumerable<Match> DeviceMatches = Regex
					.Matches(DetectResult.Output, @"(.?)Found\ ([a-z0-9]{40}|[A-Z0-9]{8}-[A-Z0-9]{16})(.*?)(\d+\.)")
					.Where(Match => Match.Success);

				foreach (Match Match in DeviceMatches)
				{
					string UUID = Match.Groups[2].ToString();
					string OSVersionString = Match.Groups[4].ToString();
					int IOSMajorVersion = int.Parse(OSVersionString.Substring(0, OSVersionString.IndexOf('.')));

					ConnectedDevice Device = new ConnectedDevice(UUID, IOSMajorVersion, true);
					Devices.Add(UUID, Device);
				}
			}

			return Devices;
		}

		private static string GetVolumeName(string InPath)
		{
			Match M = Regex.Match(InPath, @"/Volumes/(.+?)/");

			if (M.Success)
			{
				return M.Groups[1].ToString();
			}

			return string.Empty;
		}

		private static bool ValidateDeviceConnection(JsonObject JsonDevice, bool LogFailure = false)
		{
			JsonObject HardwareProperites = JsonDevice.GetObjectField("hardwareProperties");
			bool bParseUUID = HardwareProperites.TryGetStringField("udid", out string UUID);

			JsonObject ConnectionProperties = JsonDevice.GetObjectField("connectionProperties");
			// Two hardware values that indicate a healthy pairing to the host agent.
			bool bParsePairStatus = ConnectionProperties.TryGetStringField("pairingState", out string PairStatus);
			bool bParseTunnelState = ConnectionProperties.TryGetStringField("tunnelState", out string TunnelState);

			bool bParseSuccessful = bParseUUID && bParsePairStatus && bParseTunnelState;
			bool bIsConnected = bParseSuccessful &&
								PairStatus.Equals("paired", StringComparison.OrdinalIgnoreCase) &&
								TunnelState.Equals("connected", StringComparison.OrdinalIgnoreCase);

			if (!bIsConnected && LogFailure)
			{
				Log.Info("{DeviceName} is still not connected to host. Skipping connection. \n Pair Status: {PairStatus} \n Tunnel State: {TunnelState}", UUID, PairStatus, TunnelState);
			}

			return bIsConnected;
		}

		private void WriteLockdownPairRecord()
		{
			try
			{
				if (MegazordService is null)
				{
					Log.Verbose("IMegazordService implementation not available, skipping lockdown plist setup");
					return;
				}

				Task<string> LockdownTask = MegazordService.GetDeviceLockdownKeyAsync(UUID);
				LockdownTask.Wait();
				string LockdownKey = LockdownTask.Result;

				if (string.IsNullOrEmpty(LockdownKey))
				{
					Log.Verbose("No lockdown key available from Megazord for device {0}, skipping lockdown plist setup", UUID);
					return;
				}

				const string LockdownDir = "/var/db/lockdown";
				const string LockdownOwner = "_usbmuxd";
				const string LockdownGroup = "_usbmuxd";

				// Decode base64 lockdown key and write device pair record
				byte[] PlistBytes = Convert.FromBase64String(LockdownKey);
				string TempPlistPath = Path.GetTempFileName();
				string LockdownPlistPath = $"{LockdownDir}/{UUID}.plist";

				try
				{
					File.WriteAllBytes(TempPlistPath, PlistBytes);
					Log.Info("Writing lockdown pair record to {0}...", LockdownPlistPath);

					IProcessResult CopyResult = ExecuteDeploymentCommand("/usr/bin/sudo", $"cp \"{TempPlistPath}\" \"{LockdownPlistPath}\"");
					if (CopyResult.ExitCode != 0)
					{
						Log.Warning("Failed to write lockdown plist (exit code {0}): {1}", CopyResult.ExitCode, CopyResult.Output);
						return;
					}

					ExecuteDeploymentCommand("/usr/bin/sudo", $"chown {LockdownOwner}:{LockdownGroup} \"{LockdownPlistPath}\"");
					ExecuteDeploymentCommand("/usr/bin/sudo", $"chmod 0644 \"{LockdownPlistPath}\"");

					LockdownPlistPaths.Add(LockdownPlistPath);
					Log.Info("Lockdown pair record written for device {0}", UUID);
				}
				finally
				{
					File.Delete(TempPlistPath);
				}

				// Fetch SystemConfiguration plist from Megazord and write it (always, not just when missing)
				Task<string> SystemConfigTask = MegazordService.GetDeviceSystemConfigAsync(UUID);
				SystemConfigTask.Wait();
				string SystemConfigKey = SystemConfigTask.Result;

				if (string.IsNullOrEmpty(SystemConfigKey))
				{
					Log.Verbose("No system_config available from Megazord for device {0}, skipping SystemConfiguration.plist setup", UUID);
				}
				else
				{
					byte[] SystemConfigBytes = Convert.FromBase64String(SystemConfigKey);
					string SystemConfigPath = $"{LockdownDir}/SystemConfiguration.plist";
					string TempSysConfigPath = Path.GetTempFileName();
					try
					{
						File.WriteAllBytes(TempSysConfigPath, SystemConfigBytes);
						Log.Info("Writing SystemConfiguration.plist to {0}...", SystemConfigPath);

						IProcessResult SysConfigCopyResult = ExecuteDeploymentCommand("/usr/bin/sudo", $"cp \"{TempSysConfigPath}\" \"{SystemConfigPath}\"");
						if (SysConfigCopyResult.ExitCode == 0)
						{
							ExecuteDeploymentCommand("/usr/bin/sudo", $"chown {LockdownOwner}:{LockdownGroup} \"{SystemConfigPath}\"");
							ExecuteDeploymentCommand("/usr/bin/sudo", $"chmod 0644 \"{SystemConfigPath}\"");
							LockdownPlistPaths.Add(SystemConfigPath);
							Log.Info("SystemConfiguration.plist written for device {0}", UUID);
						}
						else
						{
							Log.Warning("Failed to write SystemConfiguration.plist (exit code {0}): {1}", SysConfigCopyResult.ExitCode, SysConfigCopyResult.Output);
						}
					}
					finally
					{
						File.Delete(TempSysConfigPath);
					}
				}

				// Restart usbmuxd so it picks up the new pair record
				Log.Info("Restarting usbmuxd to load lockdown pair record for device {0}...", UUID);
				ExecuteDeploymentCommand("/usr/bin/sudo", "launchctl kickstart -k system/com.apple.usbmuxd");
			}
			catch (AggregateException Ex)
			{
				Log.Warning("Failed to write lockdown pair record for {0}: {1}", UUID, Ex.InnerException?.Message ?? Ex.Message);
			}
			catch (Exception Ex)
			{
				Log.Warning("Failed to write lockdown pair record for {0}: {1}", UUID, Ex.Message);
			}
		}

		private void CleanupLockdownPairRecord()
		{
			if (LockdownPlistPaths.Count == 0)
			{
				return;
			}

			Log.Info("Cleaning up lockdown plist file(s) for device {0}...", UUID);

			foreach (string PlistPath in LockdownPlistPaths)
			{
				IProcessResult RemoveResult = ExecuteDeploymentCommand("/usr/bin/sudo", $"rm -f \"{PlistPath}\"");
				if (RemoveResult.ExitCode != 0)
				{
					Log.Verbose("Failed to remove lockdown plist {0}: {1}", PlistPath, RemoveResult.Output);
				}
				else
				{
					Log.Verbose("Removed lockdown plist: {0}", PlistPath);
				}
			}

			LockdownPlistPaths.Clear();
		}

		/// <summary>
		/// Query the IP address of an iOS device from Megazord API using its UUID
		/// </summary>
		/// <param name="DeviceUUID">The device UUID/serial number</param>
		/// <returns>IP address string if found, null otherwise</returns>
		private string QueryDeviceIPAddress(string DeviceUUID)
		{
			try
			{
				if (MegazordService is null)
				{
					Log.Verbose("IMegazordService implementation not available, cannot query device IP address");
					return null;
				}

				Task<string> IpAddressTask = MegazordService.GetDeviceIPAddressAsync(DeviceUUID);
				IpAddressTask.Wait();
				return IpAddressTask.Result;
			}
			catch (AggregateException Ex)
			{
				Log.Verbose("Failed to query Megazord for device IP address: {0}", Ex.InnerException?.Message ?? Ex.Message);
				return null;
			}
			catch (Exception Ex)
			{
				Log.Verbose("Failed to query Megazord for device IP address: {0}", Ex.Message);
				return null;
			}
		}

		// DEPRECATED
		public IAppInstall InstallApplication(UnrealAppConfig AppConfig)
		{
			InstallBuild(AppConfig);
			return CreateAppInstall(AppConfig);
		}
	}

	class IOSAppInstall : IAppInstall
	{
		public string Name { get; protected set; }

		public string AppPath { get; protected set; }

		public string CommandLine { get; protected set; }

		public string PackageName { get; protected set; }

		public string ProjectName { get; protected set; }

		public ITargetDevice Device => IOSDevice;

		public TargetDeviceIOS IOSDevice;

		public IOSAppInstall(string InName, TargetDeviceIOS InDevice, string InPackageName,
			string InCommandLine, string InProjectName, string InAppPath = null)
		{
			Name = InName;
			AppPath = InAppPath;
			IOSDevice = InDevice;
			CommandLine = InCommandLine;
			PackageName = InPackageName;
			ProjectName = InProjectName;
		}

		public IAppInstance Run()
		{
			return Device.Run(this);
		}
	}

	class IOSAppInstance : IAppInstance
	{
		public ITargetDevice Device => Install.Device;
		public TargetDeviceIOS IOSDevice => Device as TargetDeviceIOS;
		public ILongProcessResult LaunchProcess { get; private set; }
		public string CommandLine { get; private set; }
		public bool WasKilled { get; protected set; }
		public int ExitCode => LaunchProcess.ExitCode;
		public bool HasExited => LaunchProcess.HasExited;

		public string ArtifactPath
		{
			get
			{
				if (!bHaveSavedArtifacts)
				{
					if (HasExited)
					{
						SaveArtifacts();
						bHaveSavedArtifacts = true;
					}
				}

				return Path.Combine(IOSDevice.LocalCachePath, "Saved");
			}
		}

		public string StdOut
		{
			get
			{
				CheckGeneratedCrashLog();
				return LaunchProcess.Output;
			}
		}
		private void CheckGeneratedCrashLog()
		{
			// The ios application is being run under lldb by ios-deploy
			// lldb catches crashes and we have it setup to dump thread callstacks
			// parse any crash dumps into Unreal crash format and append to output
			if (HasExited && !bWasCheckedForCrash)
			{
				bWasCheckedForCrash = true;
				string CrashLog = LLDBCrashParser.GenerateCrashLog(LaunchProcess.GetLogReader());
				if (!string.IsNullOrEmpty(CrashLog))
				{
					LaunchProcess.AppendToOutput(CrashLog, false);
				}
			}
		}

		public ILogStreamReader GetLogReader()
		{
			CheckGeneratedCrashLog();
			return LaunchProcess.GetLogReader();
		}

		public ILogStreamReader GetLogBufferReader() => LaunchProcess.GetLogBufferReader();

		public bool WriteOutputToFile(string FilePath) => LaunchProcess.WriteOutputToFile(FilePath) != null;

		protected IOSAppInstall Install;

		protected bool bHaveSavedArtifacts = false;
		protected bool bWasCheckedForCrash = false;

		public IOSAppInstance(IOSAppInstall InInstall, ILongProcessResult InProcess, string InCommandLine)
		{
			Install = InInstall;
			this.CommandLine = InCommandLine;
			this.LaunchProcess = InProcess;
		}

		public int WaitForExit()
		{
			if (!HasExited)
			{
				LaunchProcess.WaitForExit();
			}

			return ExitCode;
		}

		public void Kill(bool bGenerateDump = false)
		{
			if (!HasExited)
			{
				WasKilled = true;
				LaunchProcess.ProcessObject.Kill(true);
			}
		}

		protected void SaveArtifacts()
		{
			IProcessResult DownloadResult = null;
			string SourceArtifactPath = IOSDevice.GetPlatformDirectoryMappings()[EIntendedBaseCopyDirectory.Saved];
			string DestinationArtifactPath = Path.Combine(IOSDevice.LocalCachePath, "Saved");

			if (Directory.Exists(DestinationArtifactPath))
			{
				Directory.Delete(DestinationArtifactPath, true);
			}

			// ios-deploy will copy the entire Documents directory, we just want the contents
			// We'll copy the directory to a temp path before moving the contents into the saved directory
			string TempPath = Path.Combine(IOSDevice.LocalCachePath, "Temp");
			if (Directory.Exists(TempPath))
			{
				Directory.Delete(TempPath, true);
			}
			Directory.CreateDirectory(TempPath);

			if (TargetDeviceIOS.UseDeviceCtl && !IOSDevice.UseIOSDeployForCopy)
			{
				// Use devicectl to copy artifacts from device
				// devicectl requires paths without leading slashes
				string DevicectlSource = SourceArtifactPath.TrimStart('/');

				string CopyCommand = string.Format("device copy from --device {0} --source {1} --destination {2} --domain-type appDataContainer --domain-identifier {3}",
					IOSDevice.UUID, DevicectlSource, TempPath, Install.PackageName);
				// Retry logic for devicectl copy from command due to intermittent connection issues
				const int MaxRetries = 3;
				for (int attempt = 1; attempt <= MaxRetries; attempt++)
				{
					DownloadResult = IOSDevice.ExecuteDevicectlCommand(CopyCommand, UseDeviceID: false, WaitTime: 180);

					if (DownloadResult.ExitCode == 0)
					{
						Log.Verbose("Successfully copied artifacts from device on attempt {0}", attempt);
						break;
					}
					else
					{
						Log.Warning("Copying artifacts from device attempt {0} failed with exit code {1}:\n{2}",
							attempt, DownloadResult.ExitCode, DownloadResult.Output);

						if (attempt < MaxRetries)
						{
							Log.Info("Retrying artifact copy after 5 second wait...");
							Thread.Sleep(5000);
						}
						else
						{
							Log.Warning("Artifact copy failed after {0} attempts", MaxRetries);
						}
					}
				}
			}
			else
			{
				// Fallback to ios-deploy for XCode < 16 or when UseIOSDeployForCopy is true
				string CopyCommand = string.Format("--bundle_id {0} --download={1} --to {2}",
					Install.PackageName, SourceArtifactPath, TempPath);
				DownloadResult = IOSDevice.ExecuteIOSDeployCommand(CopyCommand, 180);
			}

			if (DownloadResult != null && DownloadResult.ExitCode != 0)
			{
				string Error = string.Format("Copying artifacts from device failed with code {0}:\n{1}\nArtifacts will not be saved.",
					DownloadResult.ExitCode, DownloadResult.Output);
				Log.Error(KnownLogEvents.Gauntlet_DeviceEvent, Error);
			}

			try
			{
				// ios-deploy wraps files in a Documents subdirectory, devicectl does not
				string DocumentsPath = Path.Combine(TempPath, "Documents");
				if (Directory.Exists(DocumentsPath))
				{
					// ios-deploy case: move from Documents subdirectory
					Log.Verbose("Moving artifacts from {0} to {1}", DocumentsPath, DestinationArtifactPath);
					Directory.Move(DocumentsPath, DestinationArtifactPath);
				}
				else if (Directory.Exists(TempPath) && Directory.GetFileSystemEntries(TempPath).Length > 0)
				{
					// devicectl case: files are directly in TempPath, move TempPath itself
					Log.Verbose("Moving artifacts from {0} to {1}", TempPath, DestinationArtifactPath);
					Directory.Move(TempPath, DestinationArtifactPath);
				}
				else
				{
					Log.Warning("No artifacts found in {0} to move", TempPath);
				}

				// Clean up temp directory if it still exists
				if (Directory.Exists(TempPath))
				{
					Directory.Delete(TempPath, true);
				}
			}
			catch (Exception Ex)
			{
				Log.Error(KnownLogEvents.Gauntlet_DeviceEvent, "Failed to move artifacts out of temp directory:\n{Exception}", Ex.Message);
			}

			// Copy Appium screenshots to artifact directory
			if (IOSDevice.AppiumContainer != null && AppiumContainer.Enabled)
			{
				try
				{
					string ScreenshotsDir = Path.Combine(DestinationArtifactPath, "Screenshots");
					Log.Info("Copying Appium screenshots to artifact directory: {0}", ScreenshotsDir);
					IOSDevice.AppiumContainer.StopMonitorLoop(ScreenshotsDir);
				}
				catch (Exception Ex)
				{
					Log.Warning("Failed to copy Appium screenshots to artifact directory: {0}", Ex.Message);
				}
			}
		}
	}

	/// <summary>
	/// Helper class to parses LLDB crash threads and generate Unreal compatible log callstack
	/// </summary>
	static class LLDBCrashParser
	{
		// Frame in callstack
		class FrameInfo
		{
			public string Module;
			public string Symbol = String.Empty;
			public string Address;
			public string Offset;
			public string Source;
			public string Line;

			public override string ToString()
			{
				// symbolicated
				if (!string.IsNullOrEmpty(Source))
				{
					return string.Format("Error: [Callstack] 0x{0} {1}!{2} [{3}{4}]", Address, Module, Symbol.Replace(" ", "^"), Source, string.IsNullOrEmpty(Line) ? "" : ":" + Line);
				}

				// unsymbolicated
				return string.Format("Error: [Callstack] 0x{0} {1}!{2} [???]", Address, Module, Symbol.Replace(" ", "^"));
			}

		}

		// Parsed thread callstack
		class ThreadInfo
		{
			public int Num;
			public string Status;
			public bool Current;
			public List<FrameInfo> Frames = new List<FrameInfo>();

			public override string ToString()
			{
				return string.Format("{0}{1}{2}\n{3}", Num, string.IsNullOrEmpty(Status) ? "" : " " + Status + " ", Current ? " (Current)" : "", string.Join("\n", Frames));
			}
		}

		/// <summary>
		/// Parse lldb thread crash dump to Unreal log format
		/// </summary>
		public static string GenerateCrashLog(ILogStreamReader LogReader)
		{
			try
			{
				DateTime TimeStamp;
				int Frame;
				ThreadInfo Thread = ParseCallstack(LogReader, out TimeStamp, out Frame);
				if (Thread == null)
				{
					return null;
				}

				StringBuilder CrashLog = new StringBuilder();
				CrashLog.Append(string.Format("[{0}:000][{1}]LogCore: === Fatal Error: ===\n", TimeStamp.ToString("yyyy.mm.dd - H.mm.ss"), Frame));
				CrashLog.Append(string.Format("Error: Thread #{0} {1}\n", Thread.Num, Thread.Status));
				CrashLog.Append(string.Join("\n", Thread.Frames));

				return CrashLog.ToString();
			}
			catch (Exception Ex)
			{
				Log.Warning(KnownLogEvents.Gauntlet_DeviceEvent, "Exception parsing LLDB callstack {Exception}", Ex.Message);
			}

			return null;

		}

		private static ThreadInfo ParseCallstack(ILogStreamReader LogReader, out DateTime Timestamp, out int FrameNum)
		{
			Timestamp = DateTime.UtcNow;
			FrameNum = 0;

			Regex LogLineRegex = new Regex(@"(?<timestamp>\s\[\d.+\]\[\s*\d+\])(?<log>.*)");
			Regex TimeRegex = new Regex(@"\[(?<year>\d+)\.(?<month>\d+)\.(?<day>\d+)-(?<hour>\d+)\.(?<minute>\d+)\.(?<second>\d+):(?<millisecond>\d+)\]\[(?<frame>\s*\d+)\]", RegexOptions.IgnoreCase);
			Regex ThreadRegex = new Regex(@"(thread\s#)(?<threadnum>\d+),?(?<status>.+)");
			Regex SymbolicatedFrameRegex = new Regex(@"\*?\s#(?<framenum>\d+):\s0x(?<address>[\da-f]+)\s(?<module>.+)\`(?<symbol>.+)(\sat\s)(?<source>.+)\s\[opt\]");
			Regex UnsymbolicatedFrameRegex = new Regex(@"\*?frame\s#(?<framenum>\d+):\s0x(?<address>[\da-f]+)\s(?<module>.+)\`(?<symbol>.+)(\s\+\s(?<offset>\d+))?");

			List<ThreadInfo> Threads = new List<ThreadInfo>();
			ThreadInfo Thread = null;

			foreach (string Line in LogReader.EnumerateNextLines())
			{
				// If Gauntlet marks the test as complete, ignore any thread dumps from forcing process to exit
				if (Line.Contains("**** TEST COMPLETE. EXIT CODE: 0 ****"))
				{
					return null;
				}

				// Parse log timestamps
				if (LogLineRegex.IsMatch(Line))
				{
					GroupCollection LogGroups = LogLineRegex.Match(Line).Groups;
					if (TimeRegex.IsMatch(LogGroups["timestamp"].Value))
					{
						GroupCollection TimeGroups = TimeRegex.Match(LogGroups["timestamp"].Value).Groups;
						int Year = int.Parse(TimeGroups["year"].Value);
						int Month = int.Parse(TimeGroups["month"].Value);
						int Day = int.Parse(TimeGroups["day"].Value);
						int Hour = int.Parse(TimeGroups["hour"].Value);
						int Minute = int.Parse(TimeGroups["minute"].Value);
						int Second = int.Parse(TimeGroups["second"].Value);
						FrameNum = int.Parse(TimeGroups["frame"].Value);
						Timestamp = new DateTime(Year, Month, Day, Hour, Minute, Second);
					}

					continue;
				}

				if (Thread != null)
				{
					FrameInfo Frame = null;
					GroupCollection FrameGroups = null;

					// Parse symbolicated frame
					if (SymbolicatedFrameRegex.IsMatch(Line))
					{
						FrameGroups = SymbolicatedFrameRegex.Match(Line).Groups;

						Frame = new FrameInfo()
						{
							Address = FrameGroups["address"].Value,
							Module = FrameGroups["module"].Value,
							Symbol = FrameGroups["symbol"].Value,
						};

						Frame.Source = FrameGroups["source"].Value;
						if (Frame.Source.Contains(":"))
						{
							Frame.Source = FrameGroups["source"].Value.Split(':')[0];
							Frame.Line = FrameGroups["source"].Value.Split(':')[1];
						}
					}

					// Parse unsymbolicated frame
					if (UnsymbolicatedFrameRegex.IsMatch(Line))
					{
						FrameGroups = UnsymbolicatedFrameRegex.Match(Line).Groups;

						Frame = new FrameInfo()
						{
							Address = FrameGroups["address"].Value,
							Offset = FrameGroups["offset"].Value,
							Module = FrameGroups["module"].Value,
							Symbol = FrameGroups["symbol"].Value
						};
					}

					if (Frame != null)
					{
						Thread.Frames.Add(Frame);
					}
					else
					{
						Thread = null;
					}

				}

				// Parse thread
				if (ThreadRegex.IsMatch(Line))
				{

					GroupCollection ThreadGroups = ThreadRegex.Match(Line).Groups;
					int Num = int.Parse(ThreadGroups["threadnum"].Value);
					string Status = ThreadGroups["status"].Value.Trim();

					Thread = Threads.SingleOrDefault(T => T.Num == Num);

					if (Thread == null)
					{
						Thread = new ThreadInfo()
						{
							Num = Num,
							Status = Status
						};

						if (Line.Trim().StartsWith("*"))
						{
							Thread.Current = true;
						}

						Threads.Add(Thread);

					}
				}
			}

			if (Threads.Count(T => T.Current == true) > 1)
			{
				Log.Warning(KnownLogEvents.Gauntlet_DeviceEvent, "LLDB debug parsed more than one current thread");
			}

			Thread = Threads.FirstOrDefault(T => T.Current == true);

			if (Threads.Count > 0 && Thread == null)
			{
				Log.Warning(KnownLogEvents.Gauntlet_DeviceEvent, "Unable to parse full crash callstack");
			}


			// Do not want to surface crashes which happen as a result of requesting exit
			if (Thread != null && Thread.Frames.FirstOrDefault(F => F.Symbol.Contains("::RequestExit")) != null)
			{
				Thread = null;
			}

			return Thread;
		}
	}

	public class IOSDeviceFactory : IDeviceFactory
	{
		public bool CanSupportPlatform(UnrealTargetPlatform? Platform)
		{
			return Platform == UnrealTargetPlatform.IOS;
		}

		public ITargetDevice CreateDevice(string InRef, string InCachePath, string InParam)
		{
			return new TargetDeviceIOS(InRef, InCachePath);
		}
	}

	public class IOSBuildSupport : BaseBuildSupport
	{
		protected override BuildFlags SupportedBuildTypes => BuildFlags.Packaged | BuildFlags.CanReplaceCommandLine | BuildFlags.CanReplaceExecutable | BuildFlags.Bulk | BuildFlags.NotBulk;
		protected override UnrealTargetPlatform? Platform => UnrealTargetPlatform.IOS;
	}

	class IOSPlatformSupport : TargetPlatformSupportBase
	{
		public override UnrealTargetPlatform? Platform => UnrealTargetPlatform.IOS;
		public override bool IsHostMountingSupported() => false;
	}
}
