// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using UnrealBuildTool;

namespace Gauntlet
{
	public class LinuxBuildSource : StagedBuildSource<StagedBuild>
	{
		public override string BuildName { get { return "LinuxStagedBuild"; } }

		public override UnrealTargetPlatform Platform { get { return UnrealTargetPlatform.Linux; } }

		public override string PlatformFolderPrefix { get { return "Linux"; } }
	}

	public class LinuxArmBuildSource : StagedBuildSource<StagedBuild>
	{
		public override string BuildName { get { return "LinuxArm64StagedBuild"; } }

		public override UnrealTargetPlatform Platform { get { return UnrealTargetPlatform.LinuxArm64; } }

		public override string PlatformFolderPrefix { get { return "LinuxArm64"; } }
	}
}
