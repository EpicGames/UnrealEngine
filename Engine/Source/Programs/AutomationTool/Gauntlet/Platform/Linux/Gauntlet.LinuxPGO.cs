// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

namespace Gauntlet
{
	/// <summary>
	/// Skeleton Linux PGO platform implementation (primarily used to test PGO locally with editor builds)
	/// </summary>
	internal class LinuxPGOPlatform : IPGOPlatform
	{

		public UnrealTargetPlatform GetPlatform()
		{
			return UnrealTargetPlatform.Linux;
		}

		public void GatherResults(string ArtifactPath)
		{

		}

		public void ApplyConfiguration(PGOConfig Config)
		{

		}

		public bool TakeScreenshot(ITargetDevice Device, string ScreenshotDirectory, out string ImageFilename)
		{			
			ImageFilename = string.Empty;
			return false;
		}
	}
}



