// Copyright Epic Games, Inc. All Rights Reserved.

using Gauntlet;
using System;
using System.Collections.Generic;
using System.Diagnostics;
using System.IO;
using System.Linq;
using System.Text;
using System.Threading.Tasks;
using UnrealBuildTool;

namespace Gauntlet.SelfTest
{
	[TestGroup("Devices")]
	class TestTargetDeviceLinux : TestTargetDevice
	{
		public override void TickTest()
		{
			string TempDir = Path.Combine(Environment.CurrentDirectory, "GauntletTemp");

			TargetDeviceLinux Device = new TargetDeviceLinux("Local Linux", Globals.TempDir);

			CheckEssentialFunctions(Device);

			MarkComplete();
		}
	}
}
