// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

namespace Gauntlet.SelfTest
{
	class TestUnrealInstallAndRunLinux : TestUnrealInstallAndRunDesktop
	{
		public TestUnrealInstallAndRunLinux()
		{
			Platform = UnrealTargetPlatform.Linux;
		}
	}
}