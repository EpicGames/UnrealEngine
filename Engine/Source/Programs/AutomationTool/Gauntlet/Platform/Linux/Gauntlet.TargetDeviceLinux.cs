// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.IO;
using AutomationTool;
using Gauntlet.Utils;
using UnrealBuildTool;
using static AutomationTool.ProcessResult;

namespace Gauntlet
{
	/// <summary>
	/// Linux Implementation of the Linux Platform Support
	/// </summary>
	public class LinuxPlatformSupport : TargetPlatformSupportBase
	{
		public override UnrealTargetPlatform? Platform => UnrealTargetPlatform.Linux;
		public override bool IsHostMountingSupported() => true;
		public override string GetHostMountedPath(string Path) => Path;
	}

	/// <summary>
	/// Linux implementation of a device to run applications
	/// </summary>
	public class TargetDeviceLinux : TargetDeviceDesktopCommon
	{
		public bool IsArm64 { get; set; }

		public TargetDeviceLinux(string InName, string InCacheDir)
			: base(InName, InCacheDir)
		{
			Platform = UnrealTargetPlatform.Linux;
			RunOptions = CommandUtils.ERunOptions.NoWaitForExit | CommandUtils.ERunOptions.NoLoggingOfRunCommand;
		}

		public TargetDeviceLinux(string InName, string InCacheDir, bool InIsArm64)
			: base(InName, InCacheDir)
		{
			IsArm64 = InIsArm64;
			Platform = InIsArm64 ? UnrealTargetPlatform.LinuxArm64 : UnrealTargetPlatform.Linux;
			RunOptions = CommandUtils.ERunOptions.NoWaitForExit | CommandUtils.ERunOptions.NoLoggingOfRunCommand;
		}

		public override IAppInstance Run(IAppInstall App)
		{
			LinuxAppInstall LinuxApp = App as LinuxAppInstall;

			if (LinuxApp == null)
			{
				throw new AutomationException("AppInstance is of incorrect type!");
			}

			if (File.Exists(LinuxApp.ExecutablePath) == false)
			{
				throw new AutomationException("Specified path {0} not found!", LinuxApp.ExecutablePath);
			}

			string CmdLine = LinuxApp.CommandArguments;
			Log.Info("Launching {0} on {1}", App.Name, ToString());
			Log.Verbose("\t{0}", CmdLine);

			bool bAppContainerized = LinuxApp is IContainerized;

			// Forward command to Docker container if running containerized
			if (bAppContainerized)
			{
				ContainerInfo Container = ((IContainerized)LinuxApp).ContainerInfo;
				string ContainerApp = Container.WorkingDir + "/" + Path.GetRelativePath(Globals.UnrealRootDir, LinuxApp.ExecutablePath).Replace('\\', '/');
				CmdLine = $"run --name {Container.ContainerName} {Container.ImageName} {Container.RunCommandPrepend} {ContainerApp} {CmdLine}";
			}

			ILongProcessResult Result = new LongProcessResult(
				bAppContainerized ? "docker" : LinuxApp.ExecutablePath,
				CmdLine,
				LinuxApp.RunOptions,
				OutputCallback: LinuxApp.FilterLoggingDelegate,
				WorkingDir: LinuxApp.WorkingDirectory,
				LocalCache: LinuxApp.Device.LocalCachePath
			);

			if (Result.HasExited && Result.ExitCode != 0)
			{
				throw new AutomationException("Failed to launch {0}. Error {1}", LinuxApp.ExecutablePath, Result.ExitCode);
			}


			return new LinuxAppInstance(LinuxApp, Result, LinuxApp.LogFile);
		}

		protected override IAppInstall CreateNativeStagedInstall(UnrealAppConfig AppConfig, NativeStagedBuild Build)
		{
			LinuxAppInstall LinuxApp;
			if (AppConfig.ContainerInfo != null)
			{
				LinuxApp = new LinuxAppContainerInstall(AppConfig.Name, AppConfig.ProjectName, AppConfig.ContainerInfo, this);
			}
			else
			{
				LinuxApp = new LinuxAppInstall(AppConfig.Name, AppConfig.ProjectName, this);
			}

			LinuxApp.ExecutablePath = Path.Combine(Build.BuildPath, Build.ExecutablePath);
			LinuxApp.WorkingDirectory = Build.BuildPath;
			LinuxApp.SetDefaultCommandLineArguments(AppConfig, RunOptions, Build.BuildPath);

			return LinuxApp;
		}

		protected override IAppInstall CreateStagedInstall(UnrealAppConfig AppConfig, StagedBuild Build)
		{
			string BuildDir = GetStagedBuildDirectory(AppConfig, Build);

			PopulateDirectoryMappings(Path.Combine(BuildDir, AppConfig.ProjectName));

			string ExecutablePath = Path.IsPathRooted(Build.ExecutablePath)
					? Build.ExecutablePath
					: Path.Combine(BuildDir, Build.ExecutablePath);

			LinuxAppInstall LinuxApp = new LinuxAppInstall(AppConfig.Name, AppConfig.ProjectName, this)
			{
				ExecutablePath = ExecutablePath,
				WorkingDirectory = Path.GetDirectoryName(ExecutablePath)
			};
			LinuxApp.SetDefaultCommandLineArguments(AppConfig, RunOptions, BuildDir);

			return LinuxApp;
		}

		protected override IAppInstall CreateEditorInstall(UnrealAppConfig AppConfig, EditorBuild Build)
		{
			PopulateDirectoryMappings(AppConfig.ProjectFile.Directory.FullName);

			LinuxAppInstall LinuxApp = new LinuxAppInstall(AppConfig.Name, AppConfig.ProjectName, this)
			{
				ExecutablePath = Build.ExecutablePath,
				WorkingDirectory = Path.GetDirectoryName(Build.ExecutablePath)
			};
			LinuxApp.SetDefaultCommandLineArguments(AppConfig, RunOptions, LinuxApp.WorkingDirectory);

			return LinuxApp;
		}

		#region Legacy Implementations

		protected override IAppInstall InstallNativeStagedBuild(UnrealAppConfig AppConfig, NativeStagedBuild InBuild)
		{
			LinuxAppInstall LinuxApp;
			if (AppConfig.ContainerInfo != null)
			{
				LinuxApp = new LinuxAppContainerInstall(AppConfig.Name, AppConfig.ProjectName, AppConfig.ContainerInfo, this);
			}
			else
			{
				LinuxApp = new LinuxAppInstall(AppConfig.Name, AppConfig.ProjectName, this);
			}
			LinuxApp.SetDefaultCommandLineArguments(AppConfig, RunOptions, InBuild.BuildPath);

			CopyAdditionalFiles(AppConfig.FilesToCopy);

			LinuxApp.ExecutablePath = Path.Combine(InBuild.BuildPath, InBuild.ExecutablePath);
			LinuxApp.WorkingDirectory = InBuild.BuildPath;

			return LinuxApp;
		}

		protected override IAppInstall InstallStagedBuild(UnrealAppConfig AppConfig, StagedBuild InBuild)
		{
			string BuildDir = InBuild.BuildPath;

			if (Utils.SystemHelpers.IsNetworkPath(BuildDir))
			{
				string SubDir = string.IsNullOrEmpty(AppConfig.Sandbox) ? AppConfig.ProjectName : AppConfig.Sandbox;
				string InstallDir = Path.Combine(InstallRoot, SubDir, AppConfig.ProcessType.ToString());

				if (!AppConfig.SkipInstall)
				{
					InstallDir = StagedBuild.InstallBuildParallel(AppConfig, InBuild, BuildDir, InstallDir, ToString());
				}
				else
				{
					Log.Info("Skipping install of {0} (-SkipInstall)", BuildDir);
				}

				BuildDir = InstallDir;
				Utils.SystemHelpers.MarkDirectoryForCleanup(InstallDir);
			}

			LinuxAppInstall LinuxApp = new LinuxAppInstall(AppConfig.Name, AppConfig.ProjectName, this);
			LinuxApp.SetDefaultCommandLineArguments(AppConfig, RunOptions, BuildDir);

			if (LocalDirectoryMappings.Count == 0)
			{
				PopulateDirectoryMappings(Path.Combine(BuildDir, AppConfig.ProjectName));
			}

			CopyAdditionalFiles(AppConfig.FilesToCopy);

			if (Path.IsPathRooted(InBuild.ExecutablePath))
			{
				LinuxApp.ExecutablePath = InBuild.ExecutablePath;
			}
			else
			{
				// TODO - this check should be at a higher level....
				string BinaryPath = Path.Combine(BuildDir, InBuild.ExecutablePath);

				// check for a local newer executable
				if (Globals.Params.ParseParam("dev") && AppConfig.ProcessType.UsesEditor() == false)
				{
					string LocalBinary = Path.Combine(Environment.CurrentDirectory, InBuild.ExecutablePath);

					bool LocalFileExists = File.Exists(LocalBinary);
					bool LocalFileNewer = LocalFileExists && File.GetLastWriteTime(LocalBinary) > File.GetLastWriteTime(BinaryPath);

					Log.Verbose("Checking for newer binary at {0}", LocalBinary);
					Log.Verbose("LocalFile exists: {0}. Newer: {1}", LocalFileExists, LocalFileNewer);

					if (LocalFileExists && LocalFileNewer)
					{
						// need to -basedir to have our exe load content from the path
						LinuxApp.CommandArguments += string.Format(" -basedir={0}", Path.GetDirectoryName(BinaryPath));

						BinaryPath = LocalBinary;
					}
				}

				LinuxApp.ExecutablePath = BinaryPath;
			}

			LinuxApp.WorkingDirectory = Path.GetDirectoryName(LinuxApp.ExecutablePath);

			return LinuxApp;
		}

		protected override IAppInstall InstallEditorBuild(UnrealAppConfig AppConfig, EditorBuild Build)
		{
			LinuxAppInstall LinuxApp = new LinuxAppInstall(AppConfig.Name, AppConfig.ProjectName, this);

			LinuxApp.WorkingDirectory = Path.GetDirectoryName(Build.ExecutablePath);
			LinuxApp.SetDefaultCommandLineArguments(AppConfig, RunOptions, LinuxApp.WorkingDirectory);
			LinuxApp.ExecutablePath = Build.ExecutablePath;

			if (LocalDirectoryMappings.Count == 0)
			{
				PopulateDirectoryMappings(AppConfig.ProjectFile.Directory.FullName);
			}

			CopyAdditionalFiles(AppConfig.FilesToCopy);

			return LinuxApp;
		}
		#endregion
	}

	public class LinuxAppInstall : DesktopCommonAppInstall<TargetDeviceLinux>
	{
		[Obsolete("Will be removed in a future release. Use 'DesktopDevice' instead.")]
		public TargetDeviceLinux LinuxDevice { get; private set; }

		public LinuxAppInstall(string InName, string InProjectName, TargetDeviceLinux InDevice)
			: base(InName, InProjectName, InDevice)
		{ }
	}

	public class LinuxAppContainerInstall : LinuxAppInstall, IContainerized
	{
		public ContainerInfo ContainerInfo { get; set; }

		public LinuxAppContainerInstall(string InName, string InProjectName, ContainerInfo InContainerInfo, TargetDeviceLinux InDevice)
			: base(InName, InProjectName, InDevice)
		{
			ContainerInfo = InContainerInfo;
		}
	}

	public class LinuxAppInstance : DesktopCommonAppInstance<LinuxAppInstall, TargetDeviceLinux>
	{
		public LinuxAppInstance(LinuxAppInstall InInstall, ILongProcessResult InProcess, string InProcessLogFile = null)
			: base(InInstall, InProcess, InProcessLogFile)
		{ }
	}

	public class LinuxDeviceFactory : IDeviceFactory
	{
		public bool CanSupportPlatform(UnrealTargetPlatform? Platform)
		{
			return Platform == UnrealTargetPlatform.Linux;
		}

		public ITargetDevice CreateDevice(string InRef, string InCachePath, string InParam = null)
		{
			return new TargetDeviceLinux(InRef, InCachePath);
		}
	}

	public class LinuxArm64DeviceFactory : IDeviceFactory
	{
		public bool CanSupportPlatform(UnrealTargetPlatform? Platform)
		{
			return Platform == UnrealTargetPlatform.LinuxArm64;
		}

		public ITargetDevice CreateDevice(string InRef, string InCachePath, string InParam = null)
		{
			return new TargetDeviceLinux(InRef, InCachePath, true);
		}
	}
}