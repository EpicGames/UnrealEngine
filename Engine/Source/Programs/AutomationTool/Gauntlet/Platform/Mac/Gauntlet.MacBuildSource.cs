// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.IO;
using AutomationTool;
using UnrealBuildTool;
using System.Threading;
using System.Text.RegularExpressions;
using System.Linq;
using EpicGames.Core;
using System.Security.Cryptography.X509Certificates;

namespace Gauntlet
{
	internal static class MacAppChecker
	{
		public static bool IsAppBundleUsable(string AppPath, string ProjectName)
		{
			string MacOSDir = Path.Combine(AppPath, "Contents", "MacOS");

			// check both binary AND Engine content. Regular Unreal builds won't have the latter
			if (Directory.Exists(MacOSDir)
				&& Directory.Exists(Path.Combine(AppPath, "Contents", "UE")))
			{
				// if we don't have a project passed in, we have checked all we can, so we are valid now
				if (ProjectName == null)
				{
					return true;
				}

				// check there's an executable with the right name 
				string ShortName = Regex.Replace(ProjectName, "Game", "", RegexOptions.IgnoreCase);
				FileInfo Executable = new DirectoryInfo(MacOSDir).GetFiles().Where(Fi => Fi.Name.StartsWith(ShortName, StringComparison.OrdinalIgnoreCase)).FirstOrDefault();

				// make sure it was found
				if (Executable != null)
				{
					return true;
				}
			}

			// if we got here, we are not usable
			return false;
		}
	}

	public class MacStagedBuild : StagedBuild
	{
		public MacStagedBuild(UnrealTargetPlatform InPlatform, UnrealTargetConfiguration InConfig, UnrealTargetRole InRole, string InBuildPath, string InExecutablePath, string InFlavor = "")
		: base(InPlatform, InConfig, InRole, InBuildPath, InExecutablePath, InFlavor)
		{
		}

		public override int PreferenceOrder { get { return 0; } }
	}

	/// <summary>
	/// Represents a loose collection of files. For Mac the executable will point to the binary.app bundle and
	/// not the inner executable file
	/// </summary>
	public class MacStagedBuildSource : StagedBuildSource<MacStagedBuild>
	{
		public override string BuildName { get { return "MacStagedBuildSource"; } }

		public override UnrealTargetPlatform Platform { get { return UnrealTargetPlatform.Mac; } }

		public override string PlatformFolderPrefix { get { return "Mac"; } }

		public override bool ShouldMakeBuildAvailable(string AppPath)
		{
			return MacAppChecker.IsAppBundleUsable(AppPath, null);
		}

	}

	/// <summary>
	/// Represents a packaged mac build. E.g. a Game.app that contains the executable and content
	/// </summary>
	public class MacPackagedBuild : PackagedBuild
	{
		public MacPackagedBuild(UnrealTargetPlatform InPlatform, UnrealTargetConfiguration InConfig, UnrealTargetRole InRole, string InBuildPath, BuildFlags InFlags)
			: base(InPlatform, InConfig, InRole, InBuildPath, InFlags)
		{
		}

		public override int PreferenceOrder { get { return 1; } }
	}

	/// <summary>
	/// Build source that discovers packaged builds for Mac. A packaged build is a bundle.app that contains both an executable and UE content  
	/// </summary>
	public class MacPackagedBuildSource : IFolderBuildSource
	{
		public string BuildName { get { return "MacPackagedBuildSource"; } }

		public string PlatformFolderPrefix { get { return "Mac"; } }

		public bool CanSupportPlatform(UnrealTargetPlatform InPlatform)
		{
			return InPlatform == UnrealTargetPlatform.Mac;
		}

		public List<IBuild> GetBuildsAtPath(string InProjectName, string InPath, int MaxRecursion = 3)
		{
			List<IBuild> Builds = new List<IBuild>();

			// c:\path\to\build
			DirectoryInfo RootDir = new DirectoryInfo(InPath);

			if (RootDir.Exists == false)
			{
				Log.Warning("No such path {0} to search for builds", InPath);
				return Builds;
			}

			List<DirectoryInfo> SearchDirs = new List<DirectoryInfo>();

			// If this folder starts with the platform prefix the path is likely /path/builds/MacClient etc
			if (RootDir.Name.StartsWith(PlatformFolderPrefix))
			{
				SearchDirs.Add(RootDir);
			}
			else
			{
				// assume this is something like /Path/Builds and add any /Path/Builds/MacClient, /Path/Builds/MacNoEditor etc
				SearchDirs = RootDir.GetDirectories().Where(D => D.Name.StartsWith(PlatformFolderPrefix)).ToList();
			}
			
			// recurse and add subfolders.
			IEnumerable<DirectoryInfo> DirsToRecurse = new List<DirectoryInfo>(SearchDirs);

			List<DirectoryInfo> AllPackages = new List<DirectoryInfo>();

			// get subdirs
			while (MaxRecursion-- > 0)
			{
				IEnumerable<DirectoryInfo> DiscoveredDirs = DirsToRecurse.SelectMany(D => D.GetDirectories("*", SearchOption.TopDirectoryOnly));

				// mac packages are folders so we only want things that end with .app
				IEnumerable<DirectoryInfo> Packages = DiscoveredDirs.Where(D => Path.GetExtension(D.Name).Equals(".app", StringComparison.OrdinalIgnoreCase));

				AllPackages.AddRange(Packages);

				// don't recurse into the dirs that end with .app since those are builds we'll look at
				DirsToRecurse = DiscoveredDirs.Except(Packages);
			}

			// Now 
			foreach (DirectoryInfo Di in AllPackages)
			{
				// Mac packages are directories such as GameName.app or GameClient-Mac-Test.app so we can use the Gauntlet helpers
				// to discover them. If config is unknown we don't recognize the format
				UnrealTargetConfiguration Config = UnrealHelpers.GetConfigurationFromExecutableName(InProjectName, Di.Name);

				UnrealTargetRole Role = UnrealHelpers.GetRoleFromExecutableName(InProjectName, Di.Name);

				if (Config != UnrealTargetConfiguration.Unknown)
				{
					if (MacAppChecker.IsAppBundleUsable(Di.FullName, InProjectName))
					{
						// We can use the base packaged build class. it has everything we need.
						MacPackagedBuild Build = new MacPackagedBuild(
							UnrealTargetPlatform.Mac,
							Config,
							Role,
							Di.FullName,
							BuildFlags.Packaged | BuildFlags.CanReplaceCommandLine | BuildFlags.CanReplaceExecutable);

						Builds.Add(Build);
					}
				}
			}

			return Builds;
		}
	}
}
