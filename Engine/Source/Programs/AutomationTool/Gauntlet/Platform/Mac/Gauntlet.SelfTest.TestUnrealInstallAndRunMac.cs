// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

namespace Gauntlet.SelfTest
{
	class TestUnrealInstallAndRunMac : TestUnrealInstallAndRunDesktop
	{
		public TestUnrealInstallAndRunMac()
		{
			Platform = UnrealTargetPlatform.Mac;
		}
	}
}