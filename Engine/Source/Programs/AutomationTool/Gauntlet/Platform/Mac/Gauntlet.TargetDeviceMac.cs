// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.IO;
using AutomationTool;
using UnrealBuildTool;
using System.Text.RegularExpressions;
using EpicGames.Core;
using UnrealBuildBase;
using Gauntlet.Utils;
using static AutomationTool.ProcessResult;

namespace Gauntlet
{
	public class TargetDeviceMac : TargetDeviceDesktopCommon
	{
		public TargetDeviceMac(string InName, string InCachePath)
			: base(InName, InCachePath)
		{
			Platform = UnrealTargetPlatform.Mac;
			RunOptions = CommandUtils.ERunOptions.NoWaitForExit;
		}

		public override void InstallBuild(UnrealAppConfig AppConfig)
		{
			IBuild Build = AppConfig.Build;
			switch (Build)
			{
				case NativeStagedBuild:
				case EditorBuild:
				{
					Log.Info("Skipping installation of {BuildType}", Build.GetType().Name);
					break;
				}

				case StagedBuild:
				case MacPackagedBuild:
				{
					string BuildDir;
					if (Build is StagedBuild)
					{
						BuildDir = (Build as StagedBuild).BuildPath;
					}
					else
					{
						BuildDir = (Build as MacPackagedBuild).BuildPath;
					}

					// If the build already exists on this drive, we can skip installation
					string BuildVolume = GetVolumeName(BuildDir);
					string LocalRoot = GetVolumeName(Environment.CurrentDirectory);
					if (!string.IsNullOrEmpty(BuildVolume) && BuildVolume.Equals(LocalRoot, StringComparison.OrdinalIgnoreCase))
					{
						Log.Info("Build exists on desired volume, skipping installation");
						return;
					}

					// Otherwise, determine the desired installation directory and install the build
					string InstallDir = GetTargetBuildDirectory(BuildDir, AppConfig.Sandbox,
						AppConfig.ProjectName, AppConfig.ProcessType.ToString());

					SystemHelpers.CopyDirectory(BuildDir, InstallDir, SystemHelpers.CopyOptions.Mirror);
					break;
				}

				default:
					throw new AutomationException("{0} is not a valid build type for {1}", Build.GetType().Name, Platform);
			}
		}

		public override IAppInstall CreateAppInstall(UnrealAppConfig AppConfig)
		{
			IAppInstall Install;
			IBuild Build = AppConfig.Build;

			switch (AppConfig.Build)
			{
				case NativeStagedBuild:
					Install = CreateNativeStagedInstall(AppConfig, Build as NativeStagedBuild);
					break;
				case StagedBuild:
					Install = CreateStagedInstall(AppConfig, Build as StagedBuild);
					break;
				case EditorBuild:
					Install = CreateEditorInstall(AppConfig, Build as EditorBuild);
					break;
				case MacPackagedBuild:
					Install = CreatePackagedInstall(AppConfig, Build as MacPackagedBuild);
					break;
				default:
					throw new AutomationException("{0} is an invalid build type for {1}!", Build.GetType().Name, Platform);
			}

			return Install;
		}

		public override IAppInstance Run(IAppInstall App)
		{
			MacAppInstall MacInstall = App as MacAppInstall;

			if (MacInstall == null)
			{
				throw new AutomationException("Invalid install type!");
			}

			Log.Info("Launching {0} on {1}", App.Name, ToString());
			Log.Verbose("\t{0}", MacInstall.CommandArguments);

			ILongProcessResult Result = new LongProcessResult(
				GetExecutableIfBundle(MacInstall.ExecutablePath),
				MacInstall.CommandArguments,
				MacInstall.RunOptions,
				OutputCallback: MacInstall.FilterLoggingDelegate,
				WorkingDir: MacInstall.WorkingDirectory,
				LocalCache: MacInstall.Device.LocalCachePath
			);

			if (Result.HasExited && Result.ExitCode != 0)
			{
				throw new AutomationException("Failed to launch {0}. Error {1}", MacInstall.ExecutablePath, Result.ExitCode);
			}

			return new MacAppInstance(MacInstall, Result, MacInstall.LogFile);
		}

		protected override IAppInstall CreateNativeStagedInstall(UnrealAppConfig AppConfig, NativeStagedBuild Build)
		{
			PopulateDirectoryMappings(Build.BuildPath);

			MacAppInstall MacApp = new MacAppInstall(AppConfig.Name, AppConfig.ProjectName, this)
			{
				ExecutablePath = Path.Combine(Build.BuildPath, Build.ExecutablePath),
				WorkingDirectory = Build.BuildPath
			};
			MacApp.SetDefaultCommandLineArguments(AppConfig, RunOptions, Build.BuildPath);

			return MacApp;
		}

		protected override IAppInstall CreateStagedInstall(UnrealAppConfig AppConfig, StagedBuild Build)
		{
			string BuildPath = GetTargetBuildDirectory(Build.BuildPath, AppConfig.Sandbox,
				AppConfig.ProjectName, AppConfig.ProcessType.ToString());
			string BundlePath = Path.Combine(BuildPath, Build.ExecutablePath);

			return CreateMacInstall(AppConfig, BuildPath, BundlePath);
		}

		protected override IAppInstall CreateEditorInstall(UnrealAppConfig AppConfig, EditorBuild Build)
		{
			PopulateDirectoryMappings(AppConfig.ProjectFile.Directory.FullName);

			MacAppInstall MacApp = new MacAppInstall(AppConfig.Name, AppConfig.ProjectName, this)
			{
				ExecutablePath = GetExecutableIfBundle(Build.ExecutablePath),
				WorkingDirectory = Path.GetFullPath(Build.ExecutablePath)
			};
			MacApp.SetDefaultCommandLineArguments(AppConfig, RunOptions, Build.ExecutablePath);

			return MacApp;
		}

		protected IAppInstall CreatePackagedInstall(UnrealAppConfig AppConfig, MacPackagedBuild Build)
		{
			string BuildDir = GetTargetBuildDirectory(Build.BuildPath, AppConfig.Sandbox,
				AppConfig.ProjectName, AppConfig.ProcessType.ToString());

			// Packaged builds use the BuildDir as the bundle
			return CreateMacInstall(AppConfig, BuildDir, BuildDir);
		}

		// Creates MacAppInstalls for StagedBuilds and MacPackagedBuilds
		protected IAppInstall CreateMacInstall(UnrealAppConfig AppConfig, string BuildPath, string BundlePath)
		{
			PopulateDirectoryMappings(BuildPath);

			MacAppInstall MacApp = new MacAppInstall(AppConfig.Name, AppConfig.ProjectName, this)
			{
				ExecutablePath = GetExecutableIfBundle(BundlePath),
				WorkingDirectory = BuildPath
			};
			MacApp.SetDefaultCommandLineArguments(AppConfig, RunOptions, BuildPath);


			return MacApp;
		}

		protected string GetVolumeName(string InPath)
		{
			Match M = Regex.Match(InPath, @"/Volumes/(.+?)/");

			if (M.Success)
			{
				return M.Groups[1].ToString();
			}

			return "";
		}

		/// <summary>
		/// If the path is to Foo.app this returns the actual executable to use (e.g. Foo.app/Contents/MacOS/Foo).
		/// </summary>
		/// <param name="InBundlePath"></param>
		/// <returns></returns>
		protected string GetExecutableIfBundle(string InBundlePath)
		{
			if (Path.GetExtension(InBundlePath).Equals(".app", StringComparison.OrdinalIgnoreCase))
			{
				// Technically we should look at the plist, but for now...
				string BaseName = Path.GetFileNameWithoutExtension(InBundlePath);
				return Path.Combine(InBundlePath, "Contents", "MacOS", BaseName);
			}

			return InBundlePath;
		}

		/// <summary>
		/// Copies a build folder (either a package.app or a folder with a staged built) to a local path if necessary.
		/// Necessary is defined as not being on locally attached storage
		/// </summary>
		/// <param name="AppConfig"></param>
		/// <param name="InBuildPath"></param>
		/// <returns></returns>
		protected string CopyBuildIfNecessary(UnrealAppConfig AppConfig, string InBuildPath)
		{
			string BuildDir = InBuildPath;

			string BuildVolume = GetVolumeName(BuildDir);
			string LocalRoot = GetVolumeName(Environment.CurrentDirectory);

			// Must be on our volume to run
			if (!string.IsNullOrEmpty(BuildVolume) && BuildVolume.Equals(LocalRoot, StringComparison.OrdinalIgnoreCase) == false)
			{
				string SubDir = string.IsNullOrEmpty(AppConfig.Sandbox) ? AppConfig.ProjectName : AppConfig.Sandbox;
				string InstallDir = Path.Combine(InstallRoot, SubDir, AppConfig.ProcessType.ToString());

				if (!AppConfig.SkipInstall)
				{
					SystemHelpers.CopyDirectory(BuildDir, InstallDir, SystemHelpers.CopyOptions.Mirror);
				}
				else
				{
					Log.Info("Skipping install of {0} (-SkipInstall)", BuildDir);
				}

				BuildDir = InstallDir;
				SystemHelpers.MarkDirectoryForCleanup(InstallDir);
			}

			return BuildDir;
		}

		private string GetTargetBuildDirectory(string BasePath, string Sandbox, string Project, string ProcessType)
		{
			string TargetDir = BasePath;

			string BuildVolume = GetVolumeName(TargetDir);
			string LocalRoot = GetVolumeName(Environment.CurrentDirectory);
			if (!string.IsNullOrEmpty(BuildVolume) && !BuildVolume.Equals(LocalRoot, StringComparison.OrdinalIgnoreCase))
			{
				string SubDir = string.IsNullOrEmpty(Sandbox) ? Project : Sandbox;
				string InstallDir = Path.Combine(InstallRoot, SubDir, ProcessType);
				TargetDir = InstallDir;
			}

			return TargetDir;
		}

		#region Legacy Implementations

		public override IAppInstall InstallApplication(UnrealAppConfig AppConfig)
		{
			switch (AppConfig.Build)
			{
				case NativeStagedBuild:
					return InstallNativeStagedBuild(AppConfig, AppConfig.Build as NativeStagedBuild);

				case StagedBuild:
					return InstallStagedBuild(AppConfig, AppConfig.Build as StagedBuild);

				case MacPackagedBuild:
					return InstallPackagedBuild(AppConfig, AppConfig.Build as MacPackagedBuild);

				case EditorBuild:
					return InstallEditorBuild(AppConfig, AppConfig.Build as EditorBuild);

				default:
					throw new AutomationException("{0} is an invalid build type!", AppConfig.Build.ToString());
			}
		}

		protected override IAppInstall InstallNativeStagedBuild(UnrealAppConfig AppConfig, NativeStagedBuild InBuild)
		{
			MacAppInstall MacApp = new MacAppInstall(AppConfig.Name, AppConfig.ProjectName, this);
			MacApp.SetDefaultCommandLineArguments(AppConfig, RunOptions, InBuild.BuildPath);
			MacApp.WorkingDirectory = InBuild.BuildPath;
			MacApp.ExecutablePath = Path.Combine(InBuild.BuildPath, InBuild.ExecutablePath);

			CopyAdditionalFiles(AppConfig.FilesToCopy);

			return MacApp;
		}

		protected override IAppInstall InstallEditorBuild(UnrealAppConfig AppConfig, EditorBuild Build)
		{
			MacAppInstall MacApp = new MacAppInstall(AppConfig.Name, AppConfig.ProjectName, this);
			MacApp.SetDefaultCommandLineArguments(AppConfig, RunOptions, Build.ExecutablePath);
			MacApp.WorkingDirectory = Path.GetFullPath(Build.ExecutablePath);
			MacApp.ExecutablePath = GetExecutableIfBundle(Build.ExecutablePath);

			if (LocalDirectoryMappings.Count == 0)
			{
				PopulateDirectoryMappings(AppConfig.ProjectFile.Directory.FullName);
			}

			CopyAdditionalFiles(AppConfig.FilesToCopy);

			return MacApp;
		}

		protected override IAppInstall InstallStagedBuild(UnrealAppConfig AppConfig, StagedBuild Build)
		{
			string BuildPath = CopyBuildIfNecessary(AppConfig, Build.BuildPath);
			string BundlePath = Path.Combine(BuildPath, Build.ExecutablePath);
			return InstallBuild(AppConfig, BuildPath, BundlePath);
		}

		protected IAppInstall InstallPackagedBuild(UnrealAppConfig AppConfig, MacPackagedBuild Build)
		{
			string BuildPath = CopyBuildIfNecessary(AppConfig, Build.BuildPath);
			return InstallBuild(AppConfig, BuildPath, BuildPath);
		}

		protected IAppInstall InstallBuild(UnrealAppConfig AppConfig, string BuildPath, string BundlePath)
		{
			MacAppInstall MacApp = new MacAppInstall(AppConfig.Name, AppConfig.ProjectName, this);
			MacApp.SetDefaultCommandLineArguments(AppConfig, RunOptions, BuildPath);
			MacApp.WorkingDirectory = BuildPath;
			MacApp.ExecutablePath = GetExecutableIfBundle(BundlePath);

			PopulateDirectoryMappings(BuildPath);

			CopyAdditionalFiles(AppConfig.FilesToCopy);

			// check for a local newer executable
			if (Globals.Params.ParseParam("dev")
				&& AppConfig.ProcessType.UsesEditor() == false
				&& AppConfig.ProjectFile != null)
			{
				// Get project properties
				ProjectProperties Props = ProjectUtils.GetProjectProperties(AppConfig.ProjectFile,
					new List<UnrealTargetPlatform>(new[] { AppConfig.Platform.Value }),
					new List<UnrealTargetConfiguration>(new[] { AppConfig.Configuration }));

				// Would this executable be built under Engine or a Project?
				DirectoryReference WorkingDir = Props.bIsCodeBasedProject ? AppConfig.ProjectFile.Directory : Unreal.EngineDirectory;

				// The bundlepath may be under Binaries/Mac for a staged build, or it could be in any folder for a packaged build so just use the name and
				// build the path ourselves
				string LocalProjectBundle = FileReference.Combine(WorkingDir, "Binaries", "Mac", Path.GetFileName(BundlePath)).FullName;

				string LocalProjectBinary = GetExecutableIfBundle(LocalProjectBundle);

				bool LocalFileExists = File.Exists(LocalProjectBinary);
				bool LocalFileNewer = LocalFileExists && File.GetLastWriteTime(LocalProjectBinary) > File.GetLastWriteTime(MacApp.ExecutablePath);

				Log.Verbose("Checking for newer binary at {0}", LocalProjectBinary);
				Log.Verbose("LocalFile exists: {0}. Newer: {1}", LocalFileExists, LocalFileNewer);

				if (LocalFileExists && LocalFileNewer)
				{
					// need to -basedir to have our exe load content from the path that the bundle sits in
					MacApp.CommandArguments += string.Format(" -basedir={0}", Path.GetDirectoryName(BundlePath));
					MacApp.ExecutablePath = LocalProjectBinary;
				}
			}

			return MacApp;
		}
		#endregion
	}

	public class MacAppInstall : DesktopCommonAppInstall<TargetDeviceMac>
	{
		[Obsolete("Will be removed in a future release. Use WorkingDirectory instead.")]
		public string LocalPath
		{
			get => WorkingDirectory;
			set { } // intentional nop
		}

		public MacAppInstall(string InName, string InProjectName, TargetDeviceMac InDevice)
			: base(InName, InProjectName, InDevice)
		{ }
	}

	public class MacAppInstance : DesktopCommonAppInstance<MacAppInstall, TargetDeviceMac>
	{
		public MacAppInstance(MacAppInstall InInstall, ILongProcessResult InProcess, string ProcessLogFile = null)
			: base(InInstall, InProcess, ProcessLogFile)
		{ }
	}

	public class MacDeviceFactory : IDeviceFactory
	{
		public bool CanSupportPlatform(UnrealTargetPlatform? Platform)
		{
			return Platform == UnrealTargetPlatform.Mac;
		}

		public ITargetDevice CreateDevice(string InRef, string InCachePath, string InParam = null)
		{
			return new TargetDeviceMac(InRef, InCachePath);
		}
	}

	public class MacPlatformSupport : TargetPlatformSupportBase
	{
		public override UnrealTargetPlatform? Platform => UnrealTargetPlatform.Mac;
		public override bool IsHostMountingSupported() => true;
		public override string GetHostMountedPath(string Path) => Path;
	}
}