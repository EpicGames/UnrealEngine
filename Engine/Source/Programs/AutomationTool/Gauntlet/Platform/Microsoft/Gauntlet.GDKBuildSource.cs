// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.IO;
using System.Linq;
using System.Text.RegularExpressions;
using System.Xml;
using System.Xml.Linq;
using System.Xml.XPath;
using EpicGames.Core;
using Gauntlet;
using UnrealBuildTool;
using UnrealBuildTool.GDK;

namespace Gauntlet
{
	/// <summary>
	/// Represents a packaged GDK build that can be installed
	/// </summary>
	public class GDKPackagedBuild : IBuild
	{
		/// <summary>
		/// Preference Order for this build over other types (loose file)
		/// </summary>
		public int PreferenceOrder { get { return 0; } }

		/// <summary>
		/// Metadata for this package
		/// </summary>
		public UnrealBuildTool.GDK.GDKGameConfigInfo MetaData { get; protected set; }

		/// <summary>
		/// Flags that represent the properties of this build
		/// </summary>
		public BuildFlags Flags { get; protected set; }

		/// <summary>
		/// Flavor of this build.
		/// </summary>
		public string Flavor => "";

		public bool SupportsAdditionalFileCopy => false;

		/// <summary>
		/// Configuration of the executable in this package
		/// </summary>
		public UnrealTargetConfiguration Configuration { get; protected set; }

		/// <summary>
		/// Platform that this build runs on
		/// </summary>
		public UnrealTargetPlatform Platform { get; private set; }

		/// <summary>
		/// Path of this package
		/// </summary>
		public string PackageFilename;

		/// <summary>
		/// /Constructor that takes all relevant info
		/// </summary>
		/// <param name="InPlatform"></param>
		/// <param name="InConfig"></param>
		/// <param name="InFlags"></param>
		/// <param name="InPackageFilename"></param>
		/// <param name="InInfo"></param>
		public GDKPackagedBuild(UnrealTargetPlatform InPlatform, UnrealTargetConfiguration InConfig, BuildFlags InFlags, string InPackageFilename, GDKGameConfigInfo InInfo)
		{
			Platform = InPlatform;
			Configuration = InConfig;
			Flags = InFlags | BuildFlags.Packaged;
			MetaData = InInfo;
			PackageFilename = InPackageFilename;
		}

		/// <summary>
		/// Packaged builds can only support Clients
		/// </summary>
		/// <param name="Role"></param>
		/// <returns></returns>
		public bool CanSupportRole(UnrealTargetRole Role)
		{
			return Role.IsClient();
		}

		/// <summary>
		/// Attempts to find any packaged builds at the provided path.
		/// </summary>
		/// <param name="InPlatform"></param>
		/// <param name="InProjectName"></param>
		/// <param name="InPath"></param>
		/// <param name="InPackageFileRexEx"></param>
		/// <returns></returns>
		public static IEnumerable<GDKPackagedBuild> CreateFromPath(UnrealTargetPlatform InPlatform, string InProjectName, string InPath, string InPackageFileRexEx)
		{
			return CreateFromPath<GDKPackagedBuild>(InPlatform, InProjectName, InPath, InPackageFileRexEx);
		}

		internal static IEnumerable<T> CreateFromPath<T>(UnrealTargetPlatform InPlatform, string InProjectName, string InPath, string InPackageFileRexEx) where T : GDKPackagedBuild
		{
			List<T> DiscoveredBuilds = new List<T>();

			IEnumerable<string> Packages = DirectoryUtils.FindFiles(InPath, new Regex(InPackageFileRexEx));

			foreach (string PackageName in Packages)
			{
				// First get common meta data
				string PackageFilename = Path.GetFileNameWithoutExtension(PackageName);
				GDKGameConfigInfo PackageMetaData = new GDKGameConfigInfo(PackageFilename);

				// now we look at the appxmanifest file to see what binaries were in it
				string AppxManifestFilename = Path.Combine(Path.GetDirectoryName(PackageName), $"{PackageFilename}_appxmanifest.xml");

				if (!File.Exists(AppxManifestFilename))
				{
					Log.Warning(KnownLogEvents.Gauntlet_BuildDropEvent, "No appxmanifest {File} file for GDK package {Package}. Ignoring.", Path.GetFileName(AppxManifestFilename), PackageName);
					continue;
				}

				// Get the package build configuration
				XDocument AppxManifestXML = null;
				try
				{
					AppxManifestXML = XDocument.Load(AppxManifestFilename);
				}
				catch (Exception Ex)
				{
					Log.Warning(KnownLogEvents.Gauntlet_BuildDropEvent, "Error loading appxmanifest xml in {File} : {Exception}, ignoring", AppxManifestFilename, Ex.Message);
					continue;
				}

				XNamespace np = AppxManifestXML.Root.GetDefaultNamespace();
				string[] PackageBinaries = (from Application in AppxManifestXML.Descendants(np+"Application")
											where ((string)Application.Attribute("Executable")).Contains(".exe") && ((string)Application.Attribute("Executable")).Contains(InProjectName)
											select Application.Attribute("Executable").Value).ToArray();
				if (PackageBinaries.Length == 0)
				{
					Log.Warning(KnownLogEvents.Gauntlet_BuildDropEvent, "Unable to parse application binary from appxmanifest {File}, ignoring", AppxManifestFilename);
					return DiscoveredBuilds;
				}
				foreach (string PackageBinary in PackageBinaries)
				{
					string PackageBinaryFileName = Path.GetFileName(PackageBinary);

					UnrealTargetConfiguration UnrealConfig = UnrealHelpers.GetConfigurationFromExecutableName(InProjectName, PackageBinaryFileName);

					// all builds are packaged and can be launched with any command line
					BuildFlags Flags = BuildFlags.Packaged | BuildFlags.CanReplaceCommandLine;

					// flag as content only if we're using UnrealGame.exe
					if (PackageBinaryFileName.Equals("UnrealGame.exe", StringComparison.OrdinalIgnoreCase))
					{
						Flags |= BuildFlags.ContentOnlyProject;
					}

					T DiscoveredBuild = Activator.CreateInstance(typeof(T), InPlatform, UnrealConfig, Flags, PackageName, PackageMetaData) as T;
					DiscoveredBuilds.Add(DiscoveredBuild);
				}
			}

			return DiscoveredBuilds;
		}
	}

	/// <summary>
	/// A build source for GDK that finds and returns packaged builds
	/// </summary>
	public abstract class GDKPackagedBuildSource : IFolderBuildSource
	{
		protected UnrealTargetPlatform Platform { get; set; }

		protected string DirectorySearchPattern { get; set; }

		public string BuildName { get; private set; }

		public string PackageFileExtension { get; protected set; }

		public string PackageSuffix { get; protected set; }

		/// <summary>
		/// Regular expression for standard GDK package file name - [PackageName][Suffix].[Extension]
		/// </summary>
		public string PackageFileRexEx { get { return GDKGameConfigInfo.PackageIdentityRegEx + PackageSuffix + "." + PackageFileExtension + "$"; } }


		/// <summary>
		/// Platforms we can support
		/// </summary>
		/// <param name="InPlatform"></param>
		public bool CanSupportPlatform(UnrealTargetPlatform InPlatform)
		{
			return InPlatform == Platform;
		}

		protected GDKPackagedBuildSource( UnrealTargetPlatform InPlatform, string InPackageFileExtension, string InDirectorySearchPattern, string InBuildName, string InPackageSuffix = "")
		{
			Platform = InPlatform;
			PackageFileExtension = InPackageFileExtension;
			DirectorySearchPattern = InDirectorySearchPattern;
			BuildName = InBuildName;
			PackageSuffix = InPackageSuffix;
		}

		/// <summary>
		/// Find all builds at the specified path, recursing as specified
		/// </summary>
		/// <param name="InProjectName"></param>
		/// <param name="InPath"></param>
		/// <param name="MaxRecursion"></param>
		/// <returns></returns>
		public virtual List<IBuild> GetBuildsAtPath(string InProjectName, string InPath, int MaxRecursion = 3)
		{
			return GetBuildsAtPath<GDKPackagedBuild>(InProjectName, InPath, MaxRecursion);

		}

		internal List<IBuild> GetBuildsAtPath<T>(string InProjectName, string InPath, int MaxRecursion = 3) where T : GDKPackagedBuild
		{
			List<IBuild> Builds = new List<IBuild>();

			// Check if the platform name is in the path. If so we'll search from here
			bool IsInPlatformFolder = DirectoryUtils.PathContainsComponent(InPath, Platform.ToString(), true);

			IEnumerable<DirectoryInfo> RootFolders = null;

			if (IsInPlatformFolder)
			{
				RootFolders = new DirectoryInfo[] { new DirectoryInfo(InPath) };
			}
			else
			{
				// find all the top level subdirectories that match this platform
				RootFolders = DirectoryUtils.FindMatchingDirectories(InPath, DirectorySearchPattern);
			}

			// Find all matching files
			IEnumerable<FileInfo> PackageFiles = RootFolders.SelectMany(D => DirectoryUtils.FindMatchingFiles(D.FullName, PackageFileRexEx, MaxRecursion));

			// Now try to create something out of their locations
			foreach (FileInfo Package in PackageFiles)
			{
				var FoundBuilds = GDKPackagedBuild.CreateFromPath<T>(Platform, InProjectName, Package.DirectoryName, PackageFileRexEx);
				Builds.AddRange(FoundBuilds);
			}			

			return Builds;
		}
	}
}
