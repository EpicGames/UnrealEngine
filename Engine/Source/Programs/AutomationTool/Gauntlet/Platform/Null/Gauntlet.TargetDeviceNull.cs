// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.IO;
using AutomationTool;
using UnrealBuildTool;
using System.Text.RegularExpressions;

namespace Gauntlet
{

	class NullAppInstance : IAppInstance
	{

		public NullAppInstance(ITargetDevice InDevice)
		{
			Device = InDevice;
			HasExited = true;
		}

		public string ArtifactPath
		{
			get
			{
				return Globals.TempDir;
			}
		}

		public string CommandLine { get { return ""; } }

		public ITargetDevice Device { get; protected set; }

		public bool HasExited { get; protected set; }

		public bool WasKilled { get; protected set; }

		public string StdOut { get { return ""; } }

		public ILogStreamReader GetLogReader() { return new DynamicStringReader(() => ""); }

		public ILogStreamReader GetLogBufferReader() { return new DynamicStringReader(() => ""); }

		public bool WriteOutputToFile(string FilePath) { return false; }

		public int ExitCode { get { return 0; }}

		public void Kill(bool GenerateDump)
		{
			if (!HasExited)
			{
				WasKilled = true;
				HasExited = true;
			}
		}

		public int WaitForExit()
		{
			HasExited = true;
			return 0;
		}
	}


	class NullAppInstall : IAppInstall
	{
		public ITargetDevice Device { get; protected set; }

		public string Name { get; protected set; }

		public string CommandLine { get; protected set; }

		public NullAppInstall(string InName, TargetDeviceNull InDevice, string InCommandLine)
		{
			Name = InName;
			Device = InDevice;
			CommandLine = InCommandLine;
		}

		public IAppInstance Run()
		{
			return Device.Run(this);
		}
	}

	public class NullDeviceFactory : IDeviceFactory
	{
		public bool CanSupportPlatform(UnrealTargetPlatform? Platform)
		{
			return Platform == null;
		}

		public ITargetDevice CreateDevice(string InRef, string InCachePath, string InParam = null)
		{
			return new TargetDeviceNull(InRef);
		}
	}

	/// <summary>
	/// Win32/64 implementation of a device to run applications
	/// </summary>
	public class TargetDeviceNull : ITargetDevice
	{
		public string Name { get; protected set; }

		protected Dictionary<EIntendedBaseCopyDirectory, string> LocalDirectoryMappings { get; set; }

		public TargetDeviceNull(string InName)
		{
			Name = InName;
			LocalDirectoryMappings = new Dictionary<EIntendedBaseCopyDirectory, string>();
		}

		#region IDisposable Support
		private bool disposedValue = false; // To detect redundant calls

		protected virtual void Dispose(bool disposing)
		{
			if (!disposedValue)
			{
				if (disposing)
				{
					// TODO: dispose managed state (managed objects).
				}

				// TODO: free unmanaged resources (unmanaged objects) and override a finalizer below.
				// TODO: set large fields to null.

				disposedValue = true;
			}
		}

		// This code added to correctly implement the disposable pattern.
		public void Dispose()
		{
			// Do not change this code. Put cleanup code in Dispose(bool disposing) above.
			Dispose(true);
			// TODO: uncomment the following line if the finalizer is overridden above.
			// GC.SuppressFinalize(this);
		}
		#endregion

		public CommandUtils.ERunOptions RunOptions { get; set; }

		public IAppInstance Run(IAppInstall App)
		{
			NullAppInstall NullApp = App as NullAppInstall;

			if (NullApp == null)
			{
				throw new DeviceException("AppInstance is of incorrect type!");
			}

			Log.Info("Launching {0} on {1}", App.Name, ToString());
			Log.Info("\t{0}", NullApp.CommandLine);

			return new NullAppInstance(this);
		}

		public IAppInstall InstallApplication(UnrealAppConfig AppConfig)
		{
			return new NullAppInstall(AppConfig.Name, this, AppConfig.CommandLine);
		}

		public void FullClean()
		{

		}

		public void CleanArtifacts(UnrealAppConfig AppConfig = null)
		{

		}

		public string LocalCachePath => null;

		public void InstallBuild(UnrealAppConfig AppConfiguration)
		{

		}

		public IAppInstall CreateAppInstall(UnrealAppConfig AppConfig)
		{
			return null;
		}

		public void CopyAdditionalFiles(IEnumerable<UnrealFileToCopy> FilesToCopy)
		{

		}

		public UnrealTargetPlatform? Platform { get { return null; } }

		public bool IsAvailable { get { return true; } }
		public bool IsConnected { get { return true; } }
		public bool IsOn { get { return true; } }
		public bool PowerOn() { return true; }
		public bool PowerOff() { return true; }
		public bool Reboot() { return true; }
		public bool Connect() { return true; }
		public bool Disconnect(bool bForce = false) { return true; }

		public override string ToString()
		{
			return Name;
		}

		public Dictionary<EIntendedBaseCopyDirectory, string> GetPlatformDirectoryMappings()
		{
			if (LocalDirectoryMappings.Count == 0)
			{
				Log.Warning("Platform directory mappings have not been populated for this platform! This should be done within InstallApplication()");
			}
			return LocalDirectoryMappings;
		}
	}
}