// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

namespace Gauntlet.SelfTest
{
	class TestUnrealInstallAndRunWindows : TestUnrealInstallAndRunDesktop
	{
		public TestUnrealInstallAndRunWindows()
		{
			Platform = UnrealTargetPlatform.Win64;
		}
	}
}
