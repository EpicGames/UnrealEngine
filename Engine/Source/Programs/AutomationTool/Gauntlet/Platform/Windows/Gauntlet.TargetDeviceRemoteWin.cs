// Copyright Epic Games, Inc. All Rights Reserved.

using AutomationTool;
using System.Collections.Generic;
using System.IO;
using UnrealBuildTool;

namespace Gauntlet
{
	/*
	 * 
	 * Represents a remote Windows device
	 * 
	 */
	public class TargetDeviceRemoteWin : ITargetDevice
	{
		public TargetDeviceRemoteWin(string InName, string InCacheDir, UnrealTargetPlatform InPlatform)
		{
			Name = InName;
			Platform = InPlatform;
			RunOptions = CommandUtils.ERunOptions.NoWaitForExit | CommandUtils.ERunOptions.NoLoggingOfRunCommand;
			LocalCachePath = InCacheDir ?? Path.Combine(Globals.TempDir, $"Remote{Platform}_{Name}");
		}

		#region IDisposable Support
		private bool disposedValue = false; // To detect redundant calls

		~TargetDeviceRemoteWin()
		{
			Dispose(false);
		}

		// This code added to correctly implement the disposable pattern.
		public void Dispose()
		{
			// Do not change this code. Put cleanup code in Dispose(bool disposing) above.
			Dispose(true);
		}

		protected virtual void Dispose(bool disposing)
		{
			if (!disposedValue)
			{
				if (disposing)
				{
					// TODO: dispose managed state (managed objects).
				}

				if (IsConnected)
				{
					Disconnect();
				}

				disposedValue = true;
			}
		}
		#endregion


		#region ITargetDevice
		public string Name { get; protected set; }
		public UnrealTargetPlatform? Platform { get; protected set; }
		public CommandUtils.ERunOptions RunOptions { get; set; }
		public bool IsAvailable => true;
		public bool IsConnected => true;
		public bool IsOn => true;
		public string LocalCachePath { get; protected set; }
		public void CleanArtifacts(UnrealAppConfig AppConfig) { }
		public bool Connect() => true;
		public void CopyAdditionalFiles(IEnumerable<UnrealFileToCopy> FilesToCopy) { }
		public bool Disconnect(bool bForce = false) => true;
		public void FullClean() { }
		public bool PowerOff() => true;
		public bool PowerOn() => true;
		public bool Reboot() => true;


		public Dictionary<EIntendedBaseCopyDirectory, string> GetPlatformDirectoryMappings() => LocalDirectoryMappings;

		public IAppInstall CreateAppInstall(UnrealAppConfig AppConfig)
		{
			switch (AppConfig.Build)
			{
				case StagedBuild:
					return CreateStagedInstall(AppConfig, AppConfig.Build as StagedBuild);

				default:
					throw new AutomationException("{0} is not supported for remote Windows devices", AppConfig.Build.GetType().Name);
			}
		}

		public IAppInstall InstallApplication(UnrealAppConfig AppConfig)
		{
			InstallBuild(AppConfig);
			return CreateAppInstall(AppConfig);
		}

		public void InstallBuild(UnrealAppConfig AppConfig)
		{
			switch(AppConfig.Build)
			{
				case StagedBuild: 
					InstallStagedBuild(AppConfig, AppConfig.Build as StagedBuild);
					break;

				default:
					throw new AutomationException("{0} is not supported for remote Windows devices", AppConfig.Build.GetType().Name);
			}
		}

		public IAppInstance Run(IAppInstall App)
		{
			RemoteWinAppInstall WinApp = App as RemoteWinAppInstall;
			if (WinApp == null)
			{
				throw new AutomationException("AppInstance is of incorrect type!");
			}

			Log.Info("Launching {0} on {1}", WinApp.Name, ToString());

			string ExtraCmdLine = "";
			RemoteWinSupport.Placeholder_RemoteWinProcessMonitor ProcessMonitor = new(Name, EpicGames.Core.Log.Logger);
			if (ProcessMonitor.IsAvailable)
			{
				ExtraCmdLine += $" -RemoteConsoleHost={ProcessMonitor.ServerAddress}";
			}

			if (!RemoteWinSupport.Run(Name, WinApp.ExecutablePath, WinApp.CommandLine + ExtraCmdLine))
			{
				throw new AutomationException("Failed to launch {0}", WinApp.ExecutablePath);
			}

			return new RemoteWinAppInstance(this, WinApp, ProcessMonitor);
		}
		#endregion

		// @fixme: no way to access remote paths yet, so this has to remain empty
		protected Dictionary<EIntendedBaseCopyDirectory, string> LocalDirectoryMappings { get; set; } = [];

		public override string ToString() => Name;

		protected IAppInstall CreateStagedInstall(UnrealAppConfig AppConfig, StagedBuild Build)
		{
			string RemoteExecutablePath = Build.ExecutablePath;
			if (Path.IsPathRooted(RemoteExecutablePath))
			{
				RemoteExecutablePath = Path.GetRelativePath(Build.BuildPath, RemoteExecutablePath);
			}

			string RemoteRoot = GetRemoteRoot(AppConfig);
			RemoteExecutablePath = Path.Combine(RemoteRoot, RemoteExecutablePath);

			RemoteWinAppInstall AppInstall = new RemoteWinAppInstall(AppConfig.Name, this)
			{
				ExecutablePath = RemoteExecutablePath,
				CommandLine = AppConfig.CommandLine,
			};

			return AppInstall;
		}

		protected IAppInstall InstallStagedBuild(UnrealAppConfig AppConfig, StagedBuild Build)
		{
			string RemoteRoot = GetRemoteRoot(AppConfig);
			if (!RemoteWinSupport.Deploy(Name, Build.BuildPath, RemoteRoot))
			{
				throw new AutomationException("Could not deploy {0} to {1}", Build.BuildPath, Name);
			}

			return CreateStagedInstall(AppConfig, Build);
		}

		protected string GetRemoteRoot(UnrealAppConfig AppConfig)
		{
			return $"Gauntlet_{AppConfig.ProjectName}_{Platform}";
		}
	}

	/*
	 * 
	 * A running application on a remote Windows device
	 * 
	 */
	public class RemoteWinAppInstance : IAppInstance
	{
		private TargetDeviceRemoteWin TargetDevice { get; set; }

		private readonly RemoteWinSupport.Placeholder_RemoteWinProcessMonitor ProcessMonitor;

		private readonly object LoggerSyncObject = null;
		private readonly FileLogger Logger = null;


		public RemoteWinAppInstance(TargetDeviceRemoteWin InTargetDevice, RemoteWinAppInstall WinApp, RemoteWinSupport.Placeholder_RemoteWinProcessMonitor InProcessMonitor)
		{
			CommandLine = WinApp.CommandLine;
			TargetDevice = InTargetDevice;
			ProcessMonitor = InProcessMonitor;
			LoggerSyncObject = new object();
			
			if (ProcessMonitor.IsAvailable)
			{
				string LogFilename = ProcessUtils.GetLogFilePath(WinApp.Name, ProcessUtils.LocalLogsPath);
				if (!string.IsNullOrEmpty(LogFilename))
				{
					Logger = new FileLogger(LogFilename, CommandLine);
				}

				ProcessMonitor.OutputReceived = OnOutputReceived;
				ProcessMonitor.Start();
			}
		}

		private void OnOutputReceived(string Line)
		{
			Logger?.AppendLine(Line);
		}

		public void Kill(bool GenerateDumpOnKill = false)
		{
			if (!HasExited)
			{
				RemoteWinSupport.Kill(TargetDevice.Name);
				ProcessMonitor.Stop();
				WasKilled = true;
			}
		}

		public ITargetDevice Device => TargetDevice;
		public bool HasExited
		{
			get
			{
				bool bHasExited = !ProcessMonitor.IsAvailable;

				if (bHasExited)
				{
					lock(LoggerSyncObject)
					{
						Logger?.CloseStream();
					}
				}

				return bHasExited;
			}
		}
		public bool WasKilled { get; private set; }
		public string CommandLine { get; protected set; }


		public string StdOut
		{
			get
			{
				lock (LoggerSyncObject)
				{
					return Logger?.GetReader().GetContent();
				}
			}
		}

		public int ExitCode
		{
			get
			{
				if (WasKilled)
				{
					return 1; // process was terminated
				}
				else if (ProcessMonitor.DidLaunch)
				{
					return 0; // process connected
				}
				else
				{
					return -1; // connection failure etc.
				}
			}
		}
		public string ArtifactPath => TargetDevice.LocalCachePath;
		public ILogStreamReader GetLogBufferReader() => Logger?.GetReader();
		public ILogStreamReader GetLogReader() => Logger?.GetReader();
		public bool WriteOutputToFile(string FilePath)
		{ 
			Logger?.CopyFile(FilePath);
			return (Logger != null);
		}
		public int WaitForExit()
		{
			if (ProcessMonitor.IsAvailable)
			{
				ProcessMonitor.WaitForExit();
				ProcessMonitor.Stop();
				Logger?.CloseStream();
			}
			return ExitCode;
		}
	};

	/*
	 * 
	 * An installed application on a remote Windows device
	 * 
	 */
	public class RemoteWinAppInstall : IAppInstall
	{
		public RemoteWinAppInstall(string InName, TargetDeviceRemoteWin InTargetDevice)
		{
			Name = InName;
			TargetDevice = InTargetDevice;
		}

		public string Name { get; protected set; }
		public ITargetDevice Device => TargetDevice;
		public string ExecutablePath { get; set; }
		public string CommandLine { get; set; }

		public TargetDeviceRemoteWin TargetDevice { get; protected set; }

		public IAppInstance Run()
		{
			return TargetDevice.Run(this);
		}
	};
}