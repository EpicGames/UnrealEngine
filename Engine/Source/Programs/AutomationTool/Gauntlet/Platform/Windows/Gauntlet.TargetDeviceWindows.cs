// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.IO;
using AutomationTool;
using UnrealBuildTool;
using Gauntlet.Utils;
using static AutomationTool.ProcessResult;
using EpicGames.Core;
using System.ComponentModel;
using System.Runtime.InteropServices;
using System.Text.RegularExpressions;
using System.Collections.Generic;
using System.Linq;
using UnrealBuildBase;

namespace Gauntlet
{
	/// <summary>
	/// Win32/64 implementation of a device to run applications
	/// </summary>
	public class TargetDeviceWindows : TargetDeviceDesktopCommon
	{
		/// <summary>
		/// Optional executable that bootstraps the build
		/// </summary>
		[AutoParam]
		public string Bootstrap { get; protected set; }
		
		public TargetDeviceWindows(string InName, string InCacheDir)
			: base(InName, InCacheDir)
		{
			Platform = UnrealTargetPlatform.Win64;
			RunOptions = CommandUtils.ERunOptions.NoWaitForExit | CommandUtils.ERunOptions.NoLoggingOfRunCommand;
		}

		public override void InstallBuild(UnrealAppConfig AppConfig)
		{
			if (AppConfig.Build is IWindowsSelfInstallingBuild SelfInstallingBuild)
			{
				SelfInstallingBuild.Install(AppConfig);
				return;
			}

			base.InstallBuild(AppConfig);
		}

		public override IAppInstall CreateAppInstall(UnrealAppConfig AppConfig)
		{
			IAppInstall Install;
			IBuild Build = AppConfig.Build;

			switch (AppConfig.Build)
			{
				case NativeStagedBuild:
					Install = CreateNativeStagedInstall(AppConfig, Build as NativeStagedBuild);
					break;
				case StagedBuild:
					Install = CreateStagedInstall(AppConfig, Build as StagedBuild);
					break;
				case EditorBuild:
					Install = CreateEditorInstall(AppConfig, Build as EditorBuild);
					break;
				case IWindowsSelfInstallingBuild:
					Install = CreateSelfInstall(AppConfig, Build as IWindowsSelfInstallingBuild);
					break;
				default:
					throw new AutomationException("{0} is an invalid build type for {1}!", Build.GetType().Name, Platform);
			}

			return Install;
		}

		public override IAppInstance Run(IAppInstall App)
		{
			WindowsAppInstall WinApp = App as WindowsAppInstall;

			if (WinApp == null)
			{
				throw new AutomationException("AppInstance is of incorrect type!");
			}

			if (File.Exists(WinApp.ExecutablePath) == false)
			{
				throw new AutomationException("Specified path {0} not found!", WinApp.ExecutablePath);
			}

			string CmdLine = WinApp.CommandArguments;
			Log.Info("Launching {0} on {1}", App.Name, ToString());
			Log.Verbose("\t{0}", CmdLine);

			ILongProcessResult Result = new LongProcessResult(
				WinApp.ExecutablePath,
				CmdLine,
				WinApp.RunOptions,
				OutputCallback: WinApp.FilterLoggingDelegate,
				WorkingDir: WinApp.WorkingDirectory,
				LocalCache: WinApp.Device.LocalCachePath
			);

			if (Result.HasExited && Result.ExitCode != 0)
			{
				throw new AutomationException("Failed to launch {0}. Error {1}", WinApp.ExecutablePath, Result.ExitCode);
			}

			return new WindowsAppInstance(WinApp, Result, WinApp.LogFile);
		}

		protected override IAppInstall CreateNativeStagedInstall(UnrealAppConfig AppConfig, NativeStagedBuild Build)
		{
			PopulateDirectoryMappings(Path.Combine(Build.BuildPath, AppConfig.ProjectName));

			WindowsAppInstall WinApp = new WindowsAppInstall(AppConfig.Name, AppConfig.ProjectName, this)
			{
				ExecutablePath = Path.Combine(Build.BuildPath, Build.ExecutablePath),
				WorkingDirectory = Build.BuildPath
			};
			WinApp.SetDefaultCommandLineArguments(AppConfig, RunOptions, Build.BuildPath);

			return WinApp;
		}

		protected override IAppInstall CreateStagedInstall(UnrealAppConfig AppConfig, StagedBuild Build)
		{
			string BuildDir = GetStagedBuildDirectory(AppConfig, Build);

			PopulateDirectoryMappings(Path.Combine(BuildDir, AppConfig.ProjectName));

			string ExecutablePath = Path.IsPathRooted(Build.ExecutablePath)
					? Build.ExecutablePath
					: Path.Combine(BuildDir, Build.ExecutablePath);

			WindowsAppInstall WinApp = new WindowsAppInstall(AppConfig.Name, AppConfig.ProjectName, this)
			{
				ExecutablePath = ExecutablePath,
				WorkingDirectory = Path.GetDirectoryName(ExecutablePath)
			};
			WinApp.SetDefaultCommandLineArguments(AppConfig, RunOptions, BuildDir);

			return WinApp;
		}

		protected override IAppInstall CreateEditorInstall(UnrealAppConfig AppConfig, EditorBuild Build)
		{
			PopulateDirectoryMappings(AppConfig.ProjectFile.Directory.FullName);

			WindowsAppInstall WinApp = new WindowsAppInstall(AppConfig.Name, AppConfig.ProjectName, this)
			{
				ExecutablePath = Build.ExecutablePath,
				WorkingDirectory = Path.GetDirectoryName(Build.ExecutablePath)
			};
			WinApp.SetDefaultCommandLineArguments(AppConfig, RunOptions, WinApp.WorkingDirectory);

			return WinApp;
		}

		protected IAppInstall CreateSelfInstall(UnrealAppConfig AppConfig, IWindowsSelfInstallingBuild Build)
		{
			WindowsAppInstall WinApp = Build.CreateAppInstall(this, AppConfig, out string BasePath);
			PopulateDirectoryMappings(Path.Combine(BasePath, AppConfig.ProjectName));
			return WinApp;
		}

		#region Legacy Implementations
		public override IAppInstall InstallApplication(UnrealAppConfig AppConfig)
		{
			switch (AppConfig.Build)
			{
				case NativeStagedBuild:
					return InstallNativeStagedBuild(AppConfig, AppConfig.Build as NativeStagedBuild);

				case StagedBuild:
					return InstallStagedBuild(AppConfig, AppConfig.Build as StagedBuild);

				case EditorBuild:
					return InstallEditorBuild(AppConfig, AppConfig.Build as EditorBuild);

				case IWindowsSelfInstallingBuild:
					return InstallSelfInstallingBuild(AppConfig, AppConfig.Build as IWindowsSelfInstallingBuild);

				default:
					throw new AutomationException("{0} is an invalid build type!", AppConfig.Build.ToString());
			}
		}

		protected void BootstrapInstall(WindowsAppInstall Install)
		{
			if (string.IsNullOrEmpty(Bootstrap))
			{
				return;
			}

			string[] SearchPaths =
			[
				Path.GetDirectoryName(Install.ExecutablePath),
				Path.Combine(Unreal.RootDirectory.FullName, "Engine", "Binaries", Platform.ToString())
			];

			if (SearchPaths.Select(x => Path.Combine(x, Bootstrap)).FirstOrDefault(File.Exists) is not { } BootstrapCandidate)
			{
				Log.Error("Failed to find bootstrapper '{0}'", Bootstrap);
				return;
			}
			
			Install.CommandArguments += " -BootstrapTarget=\"" + Install.ExecutablePath + "\"";
			Install.ExecutablePath = BootstrapCandidate;
		}

		protected override IAppInstall InstallNativeStagedBuild(UnrealAppConfig AppConfig, NativeStagedBuild InBuild)
		{
			WindowsAppInstall WinApp = new WindowsAppInstall(AppConfig.Name, AppConfig.ProjectName, this);
			WinApp.WorkingDirectory = InBuild.BuildPath;
			WinApp.SetDefaultCommandLineArguments(AppConfig, RunOptions, InBuild.BuildPath);
			WinApp.ExecutablePath = Path.Combine(InBuild.BuildPath, InBuild.ExecutablePath);

			BootstrapInstall(WinApp);

			CopyAdditionalFiles(AppConfig.FilesToCopy);

			return WinApp;
		}

		protected override IAppInstall InstallStagedBuild(UnrealAppConfig AppConfig, StagedBuild InBuild)
		{
			string BuildDir = InBuild.BuildPath;

			if (Utils.SystemHelpers.IsNetworkPath(BuildDir))
			{
				string SubDir = string.IsNullOrEmpty(AppConfig.Sandbox) ? AppConfig.ProjectName : AppConfig.Sandbox;
				string InstallDir = Path.Combine(InstallRoot, SubDir, AppConfig.ProcessType.ToString());

				if (!AppConfig.SkipInstall)
				{
					InstallDir = StagedBuild.InstallBuildParallel(AppConfig, InBuild, BuildDir, InstallDir, ToString());
				}
				else
				{
					Log.Info("Skipping install of {0} (-SkipInstall)", BuildDir);
				}

				BuildDir = InstallDir;
				Utils.SystemHelpers.MarkDirectoryForCleanup(InstallDir);
			}

			WindowsAppInstall WinApp = new WindowsAppInstall(AppConfig.Name, AppConfig.ProjectName, this);
			WinApp.SetDefaultCommandLineArguments(AppConfig, RunOptions, BuildDir);

			if (LocalDirectoryMappings.Count == 0)
			{
				PopulateDirectoryMappings(Path.Combine(BuildDir, AppConfig.ProjectName));
			}

			CopyAdditionalFiles(AppConfig.FilesToCopy);

			if (Path.IsPathRooted(InBuild.ExecutablePath))
			{
				WinApp.ExecutablePath = InBuild.ExecutablePath;
			}
			else
			{
				// TODO - this check should be at a higher level....
				string BinaryPath = Path.Combine(BuildDir, InBuild.ExecutablePath);

				// check for a local newer executable
				if (Globals.Params.ParseParam("dev") && AppConfig.ProcessType.UsesEditor() == false)
				{
					string LocalBinary = Path.Combine(Environment.CurrentDirectory, InBuild.ExecutablePath);

					bool LocalFileExists = File.Exists(LocalBinary);
					bool LocalFileNewer = LocalFileExists && File.GetLastWriteTime(LocalBinary) > File.GetLastWriteTime(BinaryPath);

					Log.Verbose("Checking for newer binary at {0}", LocalBinary);
					Log.Verbose("LocalFile exists: {0}. Newer: {1}", LocalFileExists, LocalFileNewer);

					if (LocalFileExists && LocalFileNewer)
					{
						// need to -basedir to have our exe load content from the path
						WinApp.CommandArguments += $" -basedir=\"{Path.GetDirectoryName(BinaryPath)}\"";

						BinaryPath = LocalBinary;
					}
				}

				WinApp.ExecutablePath = BinaryPath;
			}

			WinApp.WorkingDirectory = Path.GetDirectoryName(WinApp.ExecutablePath);

			BootstrapInstall(WinApp);

			return WinApp;
		}

		protected override IAppInstall InstallEditorBuild(UnrealAppConfig AppConfig, EditorBuild Build)
		{
			WindowsAppInstall WinApp = new WindowsAppInstall(AppConfig.Name, AppConfig.ProjectName, this);

			WinApp.WorkingDirectory = Path.GetDirectoryName(Build.ExecutablePath);
			WinApp.SetDefaultCommandLineArguments(AppConfig, RunOptions, WinApp.WorkingDirectory);
			WinApp.ExecutablePath = Build.ExecutablePath;

			if (LocalDirectoryMappings.Count == 0)
			{
				PopulateDirectoryMappings(AppConfig.ProjectFile.Directory.FullName);
			}

			BootstrapInstall(WinApp);

			CopyAdditionalFiles(AppConfig.FilesToCopy);

			return WinApp;
		}

		protected IAppInstall InstallSelfInstallingBuild(UnrealAppConfig AppConfig, IWindowsSelfInstallingBuild Build)
		{
			Build.Install(AppConfig);

			WindowsAppInstall WinApp = Build.CreateAppInstall(this, AppConfig, out string BasePath);

			PopulateDirectoryMappings(Path.Combine(BasePath, AppConfig.ProjectName));

			BootstrapInstall(WinApp);

			CopyAdditionalFiles(AppConfig.FilesToCopy);
			return WinApp;
		}
		#endregion
	}

	public class WindowsAppInstall : DesktopCommonAppInstall<TargetDeviceWindows>, IAppInstall.IDynamicCommandLine
	{
		/// Obsolete! Will be removed in a future release.
		/// Use 'DesktopDevice' instead
		public TargetDeviceWindows WinDevice => DesktopDevice;

		public WindowsAppInstall(string InName, string InProjectName, TargetDeviceWindows InDevice)
			: base(InName, InProjectName, InDevice)
		{ }
	}

	public class WindowsAppInstance : DesktopCommonAppInstance<WindowsAppInstall, TargetDeviceWindows>
	{
		[DllImport("dbghelp", SetLastError = true)]
		private static extern bool MiniDumpWriteDump(SafeHandle hProcess, uint ProcessId, SafeHandle hFile, MINIDUMP_TYPE DumpType, IntPtr ExceptionParam, IntPtr UserStreamParam, IntPtr CallbackParam);

		private enum MINIDUMP_TYPE
		{
			MiniDumpNormal = 0x00000000,
			MiniDumpWithDataSegs = 0x00000001,
			MiniDumpWithFullMemory = 0x00000002,
			MiniDumpWithHandleData = 0x00000004,
			MiniDumpFilterMemory = 0x00000008,
			MiniDumpScanMemory = 0x00000010,
			MiniDumpWithUnloadedModules = 0x00000020,
			MiniDumpWithIndirectlyReferencedMemory = 0x00000040,
			MiniDumpFilterModulePaths = 0x00000080,
			MiniDumpWithProcessThreadData = 0x00000100,
			MiniDumpWithPrivateReadWriteMemory = 0x00000200,
			MiniDumpWithoutOptionalData = 0x00000400,
			MiniDumpWithFullMemoryInfo = 0x00000800,
			MiniDumpWithThreadInfo = 0x00001000,
			MiniDumpWithCodeSegs = 0x00002000,
			MiniDumpWithoutAuxiliaryState = 0x00004000,
			MiniDumpWithFullAuxiliaryState = 0x00008000,
			MiniDumpWithPrivateWriteCopyMemory = 0x00010000,
			MiniDumpIgnoreInaccessibleMemory = 0x00020000,
			MiniDumpWithTokenInformation = 0x00040000,
			MiniDumpWithModuleHeaders = 0x00080000,
			MiniDumpFilterTriage = 0x00100000,
			MiniDumpWithAvxXStateContext = 0x00200000,
			MiniDumpWithIptTrace = 0x00400000,
			MiniDumpScanInaccessiblePartialPages = 0x00800000,
			MiniDumpFilterWriteCombinedMemory,
			MiniDumpValidTypeFlags = 0x01ffffff
		}

		public WindowsAppInstance(WindowsAppInstall InInstall, ILongProcessResult InProcess, string InProcessLogFile = null)
			: base(InInstall, InProcess, InProcessLogFile)
		{ }

		protected override void GenerateDump()
		{
			string CrashDumpPath = Path.Combine(Install.DesktopDevice.UserDir, "Saved", "Crashes");
			bool WroteDump = false;
			if (!Directory.Exists(CrashDumpPath))
			{
				Directory.CreateDirectory(CrashDumpPath);
			}

			string DumpName = Path.Combine(CrashDumpPath, Path.GetFileNameWithoutExtension(ProcessResult.ProcessObject.ProcessName) + ".dmp");
			using (FileStream CrashDumpStream = File.Create(DumpName))
			{
				WroteDump = MiniDumpWriteDump(ProcessResult.ProcessObject.SafeHandle, (uint)ProcessResult.ProcessObject.Id,
					CrashDumpStream.SafeFileHandle, MINIDUMP_TYPE.MiniDumpNormal, IntPtr.Zero, IntPtr.Zero, IntPtr.Zero);
				if (!WroteDump)
				{
					Int32 Error = Marshal.GetLastWin32Error();
					string ErrorMessage = new Win32Exception(Error).Message;
					Log.Error(KnownLogEvents.Gauntlet, "Failed to write minidump. Error: {Error} (GetLastError={LastError})", ErrorMessage, Error);
				}
			}
			if (!WroteDump)
			{
				File.Delete(DumpName);
			}
			else
			{
				Log.Error(KnownLogEvents.Gauntlet_TestEvent, "Wrote minidump to {FileName}", DumpName);

				if (CommandUtils.IsBuildMachine)
				{
					// Produce a stack analysis on buildmachine.
					FileReference CDB = GetCDBExe();
					if (CDB != null)
					{
						string Args = $"-z \"{DumpName}\" -c \"!analyze -v -hang; q\"";
						string Output = UnrealBuildTool.Utils.RunLocalProcessAndReturnStdOut(CDB.FullName, Args);
						List<string> Stack = new List<string>();
						Match M = Regex.Match(Output, @"STACK_TEXT:.*", RegexOptions.IgnoreCase);
						if (M.Success)
						{
							Output = Output.Substring(M.Index + M.Length + 1);
							foreach (string Line in Output.Split('\n').Select(L => L.Trim()))
							{
								if (string.IsNullOrEmpty(Line)) break;
								// A callstack line looks like this from cdb:
								//   000000ef`8877ad20 00007ff8`71c18cde     : 00000523`3b3cd580 00007722`b1765a01 00007722`b1765a01 00000216`6e3bfc00 : UnrealEditor_Core!LowLevelTasks::Private::FWaitingQueue::PrepareWait+0x89a
								// We are removing the first 115 characters from the each line as they are not going to add meaningful information to the report. 
								Stack.Add(Line.Substring(Line.Length > 115 ? 115 : 0));
							}
						}
						if (Stack.Any())
						{
							Log.Error(KnownLogEvents.Horde_BuildHealth_Ignore, "Stack analysis produced with 'cdb {Args}':\n{StackTrace}", Args, string.Join('\n', Stack));
						}
						else
						{
							Log.Warning(KnownLogEvents.Horde_BuildHealth_Ignore, "Could not find stack analysis from cdb command.");
						}
					}
				}
			}
		}

		private static FileReference GetCDBExe()
		{
			// Trying to look for auto sdk latest WindowsKits debugger tools
			DirectoryReference HostAutoSdkDir = null;
			if (UEBuildPlatformSDK.TryGetHostPlatformAutoSDKDir(out HostAutoSdkDir))
			{
				DirectoryReference WindowsKitsDebuggersDirAutoSdk = DirectoryReference.Combine(HostAutoSdkDir, "Win64", "Windows Kits", "Debuggers");
				if (DirectoryReference.Exists(WindowsKitsDebuggersDirAutoSdk))
				{
					FileReference CDBExe64 = FileReference.Combine(WindowsKitsDebuggersDirAutoSdk, "x64", "cdb.exe");
					if (FileReference.Exists(CDBExe64))
					{
						return CDBExe64;
					}
				}
			}

			return null;
		}
	}

	public class Win64DeviceFactory : IDeviceFactory
	{
		public bool CanSupportPlatform(UnrealTargetPlatform? Platform)
		{
			return Platform == UnrealTargetPlatform.Win64;
		}

		public ITargetDevice CreateDevice(string InRef, string InCachePath, string InParam = null)
		{
			if (RemoteWinSupport.IsRemoteDevice(InRef, UnrealTargetPlatform.Win64))
			{
				return new TargetDeviceRemoteWin(InRef, InCachePath, UnrealTargetPlatform.Win64);
			}

			return new TargetDeviceWindows(InRef, InCachePath);
		}
	}

	public class WindowsPlatformSupport : TargetPlatformSupportBase
	{
		public override UnrealTargetPlatform? Platform => UnrealTargetPlatform.Win64;
		public override bool IsHostMountingSupported() => true;
		public override string GetHostMountedPath(string Path) => Path;
	}
}