// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.IO;
using System.Linq;
using System.Collections.Generic;
using UnrealBuildTool;

namespace Gauntlet
{
	public class Win64BuildSource : StagedBuildSource<StagedBuild>
	{
		public override string BuildName { get { return "Win64StagedBuild"; } }

		public override UnrealTargetPlatform Platform { get { return UnrealTargetPlatform.Win64; } }

		public override string PlatformFolderPrefix { get { return "Windows"; } }
	}

	public interface IWindowsSelfInstallingBuild
	{
		void Install(UnrealAppConfig AppConfiguration);
		WindowsAppInstall CreateAppInstall(TargetDeviceWindows TargetDevice, UnrealAppConfig AppConfig, out string BasePath);
	}

	public class WindowsStagedBuildFilter : IStagedBuildFilter
	{
		public UnrealTargetPlatform Platform => UnrealTargetPlatform.Win64;

		public List<FileInfo> GetFilteredFiles(UnrealAppConfig AppConfig, StagedBuild Build)
		{
			List<FileInfo> FilteredFiles = new List<FileInfo>();


			string[] FilteredExtensions = [".exe", ".pdb", ".dll", ".lib", ".objpaths", ".map", ".exp"];


			DirectoryInfo BuildDir = new DirectoryInfo(Build.BuildPath);
			foreach (FileInfo File in BuildDir.GetFiles("*", SearchOption.AllDirectories))
			{
				bool bFilter = false;
				bool bFileHasFilterableExtension = FilteredExtensions.Contains(File.Extension, StringComparer.OrdinalIgnoreCase);

				if (bFileHasFilterableExtension)
				{
					// Determine if this is a project related file by resolving the config
					UnrealHelpers.ConfigInfo FileConfig = UnrealHelpers.GetUnrealConfigFromFileName(AppConfig.ProjectName, File.Name);
					if (FileConfig.Configuration != UnrealTargetConfiguration.Unknown)
					{
						// We found a config, that means it is relevant file
						if (ShouldFilterFile(FileConfig, AppConfig, Build))
						{
							bFilter = true;
							Log.Verbose("\tSkipping copy of {FileName}", File.FullName);
						}
					}
				}

				if (!bFilter)
				{
					FilteredFiles.Add(File);
				}
			}

			return FilteredFiles;
		}

		protected virtual bool ShouldFilterFile(UnrealHelpers.ConfigInfo FileConfig, UnrealAppConfig AppConfig, StagedBuild Build)
		{
			// Filter if not matches the config and flavor
			return FileConfig.Configuration != AppConfig.Configuration || !FileConfig.Flavor.Equals(Build.Flavor, StringComparison.OrdinalIgnoreCase);
		}
	}
}
