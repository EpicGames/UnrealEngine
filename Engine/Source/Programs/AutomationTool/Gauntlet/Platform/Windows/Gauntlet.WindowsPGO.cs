// Copyright Epic Games, Inc. All Rights Reserved.

using AutomationTool;
using EpicGames.Core;
using System;
using System.IO;
using System.Text.RegularExpressions;
using UnrealBuildTool;
using Log = Gauntlet.Log;

namespace Gauntlet
{
	/// <summary>
	/// Skeleton Windows PGO platform implementation (primarily used to test PGO locally with editor builds)
	/// </summary>
	internal abstract class WinBasePGOPlatform : IPGOPlatform
	{
		string LocalOutputDirectory;
		string PgcFilenamePrefix;
		string ExpectedProfrawFileName;

		UnrealTargetPlatform Platform;

		protected WinBasePGOPlatform(UnrealTargetPlatform InPlatform)
		{
			Platform = InPlatform;
		}

		public UnrealTargetPlatform GetPlatform()
		{
			return Platform;
		}

		private void GatherResultsMSVC(string PathToPGOArtifacts)
		{
			// Find PGC files
			string PGODir = PathToPGOArtifacts;
			string[] PGCFiles = Directory.GetFiles(PGODir, "*.pgc");
			if (PGCFiles.Length == 0)
			{
				Log.Error(KnownLogEvents.Gauntlet, "No .pgc files were generated. Make sure that you are using a build with PGO instrumentation.");
				throw new AutomationException("no .pgc files were found in {0}", PGODir );
			}
			Log.Info("Found {0} .pgc files in \"{1}\"", PGCFiles.Length, PGODir);

			// Copy the PGC files to the output directory
			foreach (string SrcFilePath in PGCFiles)
			{
				string DstFileName = Path.GetFileName(SrcFilePath);

				// Optionally override the pgc filename...
				if (!string.IsNullOrWhiteSpace(PgcFilenamePrefix))
				{
					int FileIndex;
					Match ParsedName = Regex.Match(DstFileName, @".*!([0-9]+)\.pgc");
					if (ParsedName.Success && int.TryParse(ParsedName.Groups[1].Value, out FileIndex))
					{
						string NewDstFileName = string.Format("{0}!{1}.pgc", PgcFilenamePrefix, FileIndex);
						if (NewDstFileName != DstFileName)
						{
							Log.Info("Overriding .pgc filename \"{0}\" to \"{1}\"", DstFileName, NewDstFileName);
							DstFileName = NewDstFileName;
						}
					}
				}

				string DstFilePath = Path.Combine(LocalOutputDirectory, DstFileName);
				if (File.Exists(DstFilePath))
				{
					InternalUtils.SafeDeleteFile(DstFilePath);
				}
				File.Copy(SrcFilePath, DstFilePath);
			}
		}

		private void GatherResultsClang()
		{
			// We've already verified the file was written to the desired path.
			Log.Info("Found expected .profraw file: {0}", ExpectedProfrawFileName);
		}

		public void GatherResults(string ArtifactPath)
		{
			string PGODir = Path.Combine(ArtifactPath, "PGO");

			if (File.Exists(ExpectedProfrawFileName))
			{
				FileInfo Info = new FileInfo(ExpectedProfrawFileName);
				if (Info.Length == 0)
				{
					throw new AutomationException($"file has zero length, probably crashed - {ExpectedProfrawFileName}");
				}

				Log.Info("Clang format profiling files detected");
				GatherResultsClang();
			}
			else
			{
				Log.Info("MSVC format profiling files detected");
				GatherResultsMSVC(PGODir);
			}
		}

		public void ApplyConfiguration(PGOConfig Config)
		{
			PgcFilenamePrefix = Config.PgcFilenamePrefix;
			LocalOutputDirectory = Path.GetFullPath(Config.ProfileOutputDirectory);
			ExpectedProfrawFileName = Environment.GetEnvironmentVariable("LLVM_PROFILE_FILE");
			if (ExpectedProfrawFileName == null && !string.IsNullOrWhiteSpace(PgcFilenamePrefix))
			{
				ExpectedProfrawFileName = Path.Combine(LocalOutputDirectory, PgcFilenamePrefix + ".profraw");
				Environment.SetEnvironmentVariable("LLVM_PROFILE_FILE", ExpectedProfrawFileName);
			}

			UnrealTestRole ClientRole = Config.RequireRole(UnrealTargetRole.Client);
			ClientRole.CommandLine += $" -PGOSweepToSaveDir"; // this means the PGC files will be created in the Gauntlet artifact folder
			ClientRole.ConfigureDevice = ConfigureDevice;
		}

		protected abstract string GetLocalCachePath(ITargetDevice Device);

		protected void ConfigureDevice(ITargetDevice Device)
		{
			// make sure the PGO directory exists, and is empty
			string PGODir = Path.Combine(GetLocalCachePath(Device), "UserDir", "Saved", "PGO");
			if (Directory.Exists(PGODir))
			{
				Directory.Delete(PGODir, true);
			}
			Directory.CreateDirectory(PGODir);
		}

		public bool TakeScreenshot(ITargetDevice Device, string ScreenshotDirectory, out string ImageFilename)
		{			
			ImageFilename = string.Empty;
			return false;
		}
	}

	internal class WindowsPGOPlatform : WinBasePGOPlatform
	{
		public WindowsPGOPlatform() : base(UnrealTargetPlatform.Win64)
		{
		}

		protected override string GetLocalCachePath(ITargetDevice Device)
		{
			using (TargetDeviceWindows WinDevice = Device as TargetDeviceWindows)
			{
				return WinDevice.LocalCachePath;
			}

			throw new AutomationException($"{Device} is a {Device.GetType()} - expecting TargetDeviceWindows");
		}
	}
}



