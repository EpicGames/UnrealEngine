// Copyright Epic Games, Inc. All Rights Reserved.

using AutomationTool;
using EpicGames.Core;
using System;
using System.Collections.Generic;
using System.IO;
using System.Runtime.Versioning;
using UnrealBuildTool;
using UnrealBuildTool.GDK;

namespace Gauntlet
{
	[SupportedOSPlatform("windows")]
	public class MSGameStorePackagedBuild : GDKPackagedBuild, IWindowsSelfInstallingBuild
	{
		public MSGameStorePackagedBuild(UnrealTargetPlatform InPlatform, UnrealTargetConfiguration InConfig, BuildFlags InFlags, string InPackageFilename, GDKGameConfigInfo InInfo)
			: base(InPlatform, InConfig, InFlags, InPackageFilename, InInfo)
		{
		}

		public WindowsAppInstall CreateAppInstall(TargetDeviceWindows TargetDevice, UnrealAppConfig AppConfig, out string BasePath)
		{
			CheckValid(AppConfig.Build);

			MSGameStorePackagedBuild Build = AppConfig.Build as MSGameStorePackagedBuild;

			// find the newly-installed package's install path
			GDKInstalledApp InstalledApp = MSGameStoreUtils.GetInstallApplicationDetail(Build.MetaData.PackageFullName);
			if (InstalledApp == null || InstalledApp.Type != GDKInstalledApp.AppType.Packaged)
			{
				throw new AutomationException($"Could not find installed package {Build.MetaData.PackageFullName} from {Build.PackageFilename} or it does not have an install path");
			}
			string InstallPath = InstalledApp.DeployPath;

			// read the manifest & find the executable to run (ideally the real executable not the bootstrap)
			GDKGameConfigInfo MSGameConfig = new GDKGameConfigInfo(FileReference.Combine(DirectoryReference.FromString(InstallPath), "MicrosoftGame.config"));
			string BaseExecutable = MSGameConfig.GetExecutableForConfiguration(AppConfig.Configuration);
			string ExecutablePath = Path.Combine(InstallPath, BaseExecutable);
			if (string.IsNullOrEmpty(Path.GetDirectoryName(BaseExecutable)))
			{
				// bootstap executable may not have the -Win64-Test etc suffix, so add that to find the correct executable
				string Suffix = $"-{AppConfig.Platform}-{AppConfig.Configuration}.exe";
				if (AppConfig.Configuration != UnrealTargetConfiguration.Development && !BaseExecutable.EndsWith(Suffix, StringComparison.OrdinalIgnoreCase))
				{
					BaseExecutable = Path.GetFileNameWithoutExtension(BaseExecutable) + Suffix;
				}

				// see if the real executable is where we're expecting
				string AltExecutablePath = Path.Combine(InstallPath, AppConfig.ProjectName, "Binaries", AppConfig.Platform.ToString(), BaseExecutable);
				if (File.Exists(AltExecutablePath))
				{
					ExecutablePath = AltExecutablePath;
				}
			}

			// just create a normal WindowsAppInstall to launch the game executable directly
			// (no longer necessary to launch via AUMID in newer GDKs)
			WindowsAppInstall WinApp = new WindowsAppInstall(AppConfig.Name, AppConfig.ProjectName, TargetDevice)
			{
				ExecutablePath = ExecutablePath,
				WorkingDirectory = InstallPath
			};
			WinApp.SetDefaultCommandLineArguments(AppConfig, TargetDevice.RunOptions, InstallPath);

			BasePath = InstallPath;
			return WinApp;
		}

		public void Install(UnrealAppConfig AppConfig)
		{
			CheckValid(AppConfig.Build);

			MSGameStorePackagedBuild Build = AppConfig.Build as MSGameStorePackagedBuild;

			// remove old package
			MSGameStoreUtils.UninstallPreviousPackageIfNewer(Build.MetaData.PackageFamilyName, Build.MetaData.Version, bRemoveAllPackages: true, bRemoveAllStaged: true);

			// install new package
			if (!MSGameStoreUtils.InstallPackage(Build.PackageFilename))
			{
				throw new AutomationException($"Unable to install package {Build.PackageFilename}");
			}
			Log.Verbose("Installed package from {0}", Build.PackageFilename);
		}

		private void CheckValid(IBuild Build)
		{
			if (!(Build is MSGameStorePackagedBuild))
			{
				throw new AutomationException("The given build is not an MSGameStore package");
			}

			if (!MSGamingExports.IsWindowsVersionSupported())
			{
				throw new AutomationException("PC GDK requires at least Windows 10 May 2019 Update (version 1903)");
			}

			if (!MSGamingExports.IsWindowsDeveloperMode())
			{
				throw new AutomationException("Windows is not in developer mode.");
			}

			if (!MSGamingExports.IsMSGamingRuntimeInstalled())
			{
				throw new AutomationException("Microsoft.GamingServices is not installed.");
			}
		}
	}


	/// <summary>
	/// A build source for MSGameStore that finds and returns packaged builds
	/// </summary>
	[SupportedOSPlatform("windows")]
	public class MSGameStorePackagedBuildSource : GDKPackagedBuildSource
	{
		public MSGameStorePackagedBuildSource()
			: base( UnrealTargetPlatform.Win64, "msixvc", "Windows*", "Win64PackagedBuildGDK")
		{
		}


		/// <summary>
		/// Find all builds at the specified path, recursing as specified
		/// </summary>
		/// <param name="InProjectName"></param>
		/// <param name="InPath"></param>
		/// <param name="MaxRecursion"></param>
		/// <returns></returns>
		public override List<IBuild> GetBuildsAtPath(string InProjectName, string InPath, int MaxRecursion = 3)
		{
			return GetBuildsAtPath<MSGameStorePackagedBuild>(InProjectName, InPath, MaxRecursion);
		}
	}
}
