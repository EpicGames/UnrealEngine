// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.IO;
using AutomationTool;
using UnrealBuildTool;
using UnrealBuildTool.GDK;
using System.Linq;
using Microsoft.Win32;
using System.Runtime.Versioning;
using System.Text.Json;

namespace Gauntlet
{
	/// <summary>
	/// Helper functions for using Gauntlet with MSGameStore
	/// </summary>
	public class MSGameStoreUtils
	{
		/// <summary>
		/// Installs the provided package
		/// </summary>
		/// <param name="PackagePath"></param>
		/// <returns></returns>
		public static bool InstallPackage(string PackagePath)
		{
			Log.Info("Installing {Package}...",  new FileInfo(PackagePath).Name);
			IProcessResult Result = RunGDKCommand("wdapp", string.Format("install \"{0}\"", PackagePath));
			return Result.ExitCode == 0;
		}

		/// <summary>
		/// Uninstall the provided package
		/// </summary>
		/// <param name="PackageName"></param>
		/// <returns></returns>
		public static bool UninstallPackage(string PackageName)
		{
			Log.Verbose("Uninstalling existing packaged build: {0}", PackageName);
			IProcessResult Result = RunGDKCommand("wdapp", string.Format("uninstall {0}", PackageName));
			return Result.ExitCode == 0;
		}

		/// <summary>
		/// Given a package family name (ElementalDemo_z844xmwvfmpty) returns all AUMIDs
		/// </summary>
		/// <param name="PackageFamilyName"></param>
		/// <returns></returns>
		public static IEnumerable<string> GetAUMIDsForInstalledPackage(string PackageFamilyName)
		{
			IEnumerable<GDKInstalledApp> Apps = GetInstalledApplicationsDetail()
				.Where( App => App.GameConfig.PackageFamilyName == PackageFamilyName );

			foreach( GDKInstalledApp App in Apps.Where( App => App.GameConfig.AUMIDs != null) )
			{
				foreach (string AUMID in App.GameConfig.AUMIDs)
				{
					yield return AUMID;
				}
			}
		}

		/// <summary>
		/// Returns a collection of installed applications
		/// </summary>
		public static IEnumerable<GDKInstalledApp> GetInstalledApplicationsDetail()
		{
			string Output = RunGDKCommandAndGetOutput("wdapp", "list /d /json");
			Output = Output.Substring(Output.IndexOf('{'));
			IEnumerable<GDKInstalledApp> Apps = GDKInstalledApp.ParseAppList( JsonDocument.Parse(Output) );
			foreach( GDKInstalledApp App in Apps )
			{
				yield return App;
			}
		}

		/// <summary>
		/// Returns the properties for the given package
		/// </summary>
		public static GDKInstalledApp GetInstallApplicationDetail(string PackageFullName)
		{
			IEnumerable<GDKInstalledApp> Matches = GetInstalledApplicationsDetail()
				.Where( App => App.GameConfig.PackageFullName == PackageFullName );

			if (!Matches.Any())
			{
				return null;
			}
			else if (Matches.Count() > 1)
			{
				throw new AutomationException($"there is more than one installed package named {PackageFullName}");
			}

			return Matches.First();
		}



		/// <summary>
		/// Uninstalls all previous builds from the given package family if it is newer than the given version, or optionally all matching builds
		/// </summary>
		/// <param name="InPackageFamilyName"></param>
		/// <param name="InPackageVersion"></param>
		/// <param name="bRemoveAllPackages"></param>
		/// <param name="bRemoveAllStaged"></param>
		public static void UninstallPreviousPackageIfNewer(string InPackageFamilyName, string InPackageVersion, bool bRemoveAllPackages = false, bool bRemoveAllStaged = false)
		{
			Version PackageVersion;
			if (!Version.TryParse(InPackageVersion, out PackageVersion))
			{
				return;
			}

			// get all currently installed versions
			var InstalledApplications = GetInstalledApplicationsDetail()
				.Where( App => App.GameConfig.PackageFamilyName == InPackageFamilyName );

			foreach (var InstalledApplication in InstalledApplications)
			{
				GDKGameConfigInfo PackageData = InstalledApplication.GameConfig;

				// should only force remove newer packages by default
				Version InstalledPackageVersion;
				bool bShouldRemove = (Version.TryParse(PackageData.Version, out InstalledPackageVersion) && InstalledPackageVersion > PackageVersion);

				// determine package type and uninstall it
				if (InstalledApplication.Type == GDKInstalledApp.AppType.Staged)
				{
					if (bShouldRemove || bRemoveAllStaged)
					{
						Log.Info($"uninstalling previous staged build {PackageData.PackageFullName}");
						RunGDKCommand("wdapp", $"unregister {PackageData.PackageFullName}" );
					}
				}
				else if (InstalledApplication.Type == GDKInstalledApp.AppType.Packaged)
				{
					if (bShouldRemove || bRemoveAllPackages)
					{
						Log.Info($"uninstalling previous package {PackageData.PackageFullName}");
						UninstallPackage(PackageData.PackageFullName);
					}
				}
				else
				{
					Log.Warning($"encountered a package that Gauntlet does not know how to remove: {PackageData.PackageFullName}");
				}
			}
		}


		/// <summary>
		/// Given an IdentityName (ElementalDemo_z844xmwvfmpty) and target configuration, returns the best matched AUMID (ElementalDemo_z844xmwvfmpty!AppElementalDemoTest)
		/// </summary>
		/// <param name="PackageFamilyName"></param>
		/// <param name="Configuration"></param>
		/// <returns></returns>
		public static string GetAUMIDForInstalledPackage( string PackageFamilyName, UnrealTargetConfiguration Configuration )
		{
			// Get the AUMID from the Identity names and registered app on the PC
			IEnumerable<string> AUMIDs = GetAUMIDsForInstalledPackage(PackageFamilyName);
			if (!AUMIDs.Any())
			{
				throw new AutomationException($"Could not find any AUMIDs in package {PackageFamilyName}");
			}

			// Packages can contain multiple binaries, so find one that matches our config
			return GDKGameConfigInfo.GetAUMIDForConfiguration(AUMIDs, Configuration);
		}

		/// <summary>
		/// Run the given GDK command
		/// </summary>
		public static IProcessResult RunGDKCommand( string Command, string Args = "", bool bWait = true)
		{
			CommandUtils.ERunOptions RunOptions = CommandUtils.ERunOptions.AppMustExist;

			if (Log.IsVeryVerbose)
			{
				RunOptions |= CommandUtils.ERunOptions.AllowSpew;
			}
			else
			{
				RunOptions |= CommandUtils.ERunOptions.NoLoggingOfRunCommand;
			}


			if (bWait == false)
			{
				RunOptions |= CommandUtils.ERunOptions.NoWaitForExit;
			}

			string GDKDir = Environment.GetEnvironmentVariable("GameDK");
			if (string.IsNullOrEmpty(GDKDir))
			{
				throw new AutomationException("GameDK not set. Cannot find GDK tools");
			}
			string ToolPath = Path.Combine(GDKDir,"bin");

			string Cmd = Path.Combine(ToolPath, Command);
			if (Path.GetExtension(Cmd).Length == 0)
			{
				Cmd = Path.ChangeExtension(Cmd, "exe");
			}

			IProcessResult Result = CommandUtils.Run(Cmd, Args, Options: RunOptions);

			if (bWait && Result.ProcessObject != null)
			{
				Result.ProcessObject.WaitForExit(60 * 1000);

				if (Result.HasExited == false)
				{
					throw new AutomationException("Command {0} {1} timed out", Cmd, Args);
				}
			}

			return Result;
		}

		/// <summary>
		/// Run the given GDK command and capture the output
		/// </summary>
		public static string RunGDKCommandAndGetOutput( string Command, string Args = "")
		{
			return RunGDKCommand( Command, Args ).Output;
		}
	}
}