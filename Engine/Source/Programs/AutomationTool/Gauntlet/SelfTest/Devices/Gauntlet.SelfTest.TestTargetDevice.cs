// Copyright Epic Games, Inc. All Rights Reserved.

using AutomationTool.DeviceReservation;
using AutomationTool;
using System;
using System.Collections.Generic;
using System.Linq;
using UnrealBuildTool;

namespace Gauntlet.SelfTest
{
	abstract class TestTargetDevice : BaseTestNode
	{
		protected bool CheckEssentialFunctions(ITargetDevice TestDevice)
		{
			// Device should power on (or ignore this if it's already on);
			CheckResult(TestDevice.PowerOn() && TestDevice.IsOn, "Failed to power on device");

			// Device should reboot (or pretend it did)
			CheckResult(TestDevice.Reboot(), "Failed to reboot device");

			// Device should connect
			CheckResult(TestDevice.Connect() && TestDevice.IsConnected, "Failed to connect to device");

			return TestFailures.Count == 0;
		}
	}

	[TestGroup("DeviceReservation")]
	public class TestDeviceReservation : BaseTestNode
	{
		private enum TestConstraintType
		{
			Name,
			Tag,
			Platform
		}

		/// <summary>
		/// Comma separated list of platforms to make a reservation for
		/// </summary>
		private List<string> Platforms;

		/// <summary>
		/// Base device reservation URI
		/// </summary>
		private Uri DeviceServiceURL;

		/// <summary>
		/// Device pool to make reservations from
		/// </summary>
		private string PoolID = "RESERVATION-TEST";

		/// <summary>
		/// Whether or not devices have been marked as a problem
		/// </summary>
		private bool HasMarkedProblems;

		/// <summary>
		/// Time test will be marked as complete.
		/// Used to stall the test so users can view the problem device
		/// </summary>
		private DateTime TargetTime;

		/// <summary>
		/// Internal list of constraints. Used for making reservations
		/// </summary>
		private List<UnrealDeviceTargetConstraint> Constraints;

		/// <summary>
		/// Internal list of active reservations
		/// </summary>
		private List<DeviceReservationAutoRenew> Reservations;

		/// <summary>
		/// Internal list of constraints and their intended test purpose
		/// </summary>
		private Dictionary<TestConstraintType, List<UnrealDeviceTargetConstraint>> ConstraintMap;

		/// <summary>
		/// Internal list of constraints mapped to their associated reservation
		/// </summary>
		private Dictionary<UnrealDeviceTargetConstraint, DeviceReservationAutoRenew> ReservationMap;

		public TestDeviceReservation()
		{
			Constraints = new List<UnrealDeviceTargetConstraint>();
			Reservations = new List<DeviceReservationAutoRenew>();
			ConstraintMap = new Dictionary<TestConstraintType, List<UnrealDeviceTargetConstraint>>()
			{
				{TestConstraintType.Name,  new List<UnrealDeviceTargetConstraint>() },
				{TestConstraintType.Tag,  new List<UnrealDeviceTargetConstraint>() },
				{TestConstraintType.Platform,  new List<UnrealDeviceTargetConstraint>() },
			};
			ReservationMap = new Dictionary<UnrealDeviceTargetConstraint, DeviceReservationAutoRenew>();

			Platforms = Gauntlet.Globals.Params.ParseValues("Platforms", true);
			if (Platforms.Count == 0)
			{
				throw new AutomationException("One or more values for -Platforms= needs to be specified");
			}

			if (!Uri.TryCreate(Gauntlet.Globals.Params.ParseValue("DeviceServiceURL", string.Empty), UriKind.Absolute, out DeviceServiceURL))
			{
				throw new AutomationException("-DeviceServiceURL= needs to be specified and valid");
			}

			PoolID = Gauntlet.Globals.Params.ParseValue("PoolID", PoolID);
		}

		public override bool StartTest(int Pass, int NumPasses)
		{
			// For each platform requested, define 3 constraints
			foreach (string PlatformString in Platforms)
			{
				UnrealTargetPlatform Platform = UnrealTargetPlatform.Parse(PlatformString);

				// First constraint, reserve a device that has a name matching "<Platform>-TestDevice-Name"
				string Name = PlatformString + "-TestDevice-Name";
				UnrealDeviceTargetConstraint NameConstraint = new UnrealDeviceTargetConstraint(Platform, DeviceName: Name);

				// Second constraint, reserve a device with "Required Tag" metadata
				Reservation.Tags Tags = new Reservation.Tags();
				Tags.AddTag("Required Tag", Reservation.Tags.Type.Required);

				UnrealDeviceTargetConstraint TagConstraint = new UnrealDeviceTargetConstraint(Platform, Tags: Tags);

				// Third constraint, just a default platform constraint. We'll also test failing this device
				UnrealDeviceTargetConstraint PlatformConstraint = new UnrealDeviceTargetConstraint(Platform);

				ConstraintMap[TestConstraintType.Name].Add(NameConstraint);
				ConstraintMap[TestConstraintType.Tag].Add(TagConstraint);
				ConstraintMap[TestConstraintType.Platform].Add(PlatformConstraint);
				Constraints.AddRange([NameConstraint, TagConstraint, PlatformConstraint]);
			}

			// Make all the reservations
			foreach (UnrealDeviceTargetConstraint Constraint in Constraints)
			{
				string[] FormattedConstraint = [Constraint.FormatWithIdentifier()];

				DeviceReservationAutoRenew NewReservation = new DeviceReservationAutoRenew(
					DeviceServiceURL.ToString(), 
					Reservation.Create(DeviceServiceURL, FormattedConstraint, TimeSpan.FromMinutes(5), 5, PoolID, Constraint.DeviceName, Constraint.Tags));

				Reservations.Add(NewReservation);
				ReservationMap.Add(Constraint, NewReservation);
			}

			// Ensure each reservation succeeded and returned the expected device type
			Dictionary<UnrealDeviceTargetConstraint, string> FailedNames = AssertReservedDeviceStatus(TestConstraintType.Name);
			Dictionary<UnrealDeviceTargetConstraint, string> FailedTags = AssertReservedDeviceStatus(TestConstraintType.Tag);
			Dictionary<UnrealDeviceTargetConstraint, string> FailedPlatforms = AssertReservedDeviceStatus(TestConstraintType.Platform);

			bool bAnyFailures = FailedNames.Any() || FailedTags.Any() || FailedPlatforms.Any();
			if (bAnyFailures)
			{
				LogFailures(TestConstraintType.Name, FailedNames);
				LogFailures(TestConstraintType.Tag, FailedTags);
				LogFailures(TestConstraintType.Platform, FailedPlatforms);
			}

			TargetTime = DateTime.Now + TimeSpan.FromMinutes(1);

			return !bAnyFailures && base.StartTest(Pass, NumPasses);
		}

		public override void TickTest()
		{
			if (!HasMarkedProblems)
			{
				HasMarkedProblems = true;

				// Report an erorr for each "Platform" device
				foreach (UnrealDeviceTargetConstraint PlatformConstraint in ConstraintMap[TestConstraintType.Platform])
				{
					Device Device = ReservationMap[PlatformConstraint].Devices[0];
					Reservation.ReportDeviceError(DeviceServiceURL.ToString(), Device.Name, "Marking device as problem for self test");
					ReservationMap[PlatformConstraint].Dispose();
				}
			}

			if (DateTime.Now > TargetTime)
			{
				MarkComplete();
			}
		}

		public override void StopTest(StopReason InReason)
		{
			foreach (DeviceReservationAutoRenew Entry in Reservations)
			{
				Entry.Dispose();
			}

			Reservations.Clear();
			ReservationMap.Clear();

			base.StopTest(InReason);
		}

		private Dictionary<UnrealDeviceTargetConstraint, string> AssertReservedDeviceStatus(TestConstraintType Type)
		{
			Dictionary<UnrealDeviceTargetConstraint, string> FailedReservations = new Dictionary<UnrealDeviceTargetConstraint, string>();

			foreach (UnrealDeviceTargetConstraint Constraint in ConstraintMap[Type])
			{
				DeviceReservationAutoRenew Reservation = ReservationMap[Constraint];

				if (Reservation.Devices.Count == 0)
				{
					FailedReservations.Add(Constraint, "No devices were reserved");
					continue;
				}

				if (Reservation.Devices.Count > 1)
				{
					FailedReservations.Add(Constraint, "More than one device was reserved");
					continue;
				}

				Device Device = Reservation.Devices[0];
				DeviceDefinition DeviceDefinition = new DeviceDefinition()
				{
					Address = Device.IPOrHostName,
					Name = Device.Name,
					Platform = UnrealTargetPlatform.Parse(UnrealTargetPlatform.GetValidPlatformNames().FirstOrDefault(Entry => Entry == Device.Type.Replace("-DevKit", "", StringComparison.OrdinalIgnoreCase))),
					DeviceData = Device.DeviceData,
					Model = Device.Model,
					Tags = Device.Tags
				};

				if (!Constraint.Check(DeviceDefinition))
				{
					Log.Error("{Constraint} failed check", Constraint.ToString());
				}

				switch (Type)
				{
					case TestConstraintType.Name:
						string ExpectedDeviceName = Constraint.Platform.ToString() + "-TestDevice-Name";
						string ActualDeviceName = DeviceDefinition.Name;
						if (!ActualDeviceName.Equals(ExpectedDeviceName, StringComparison.OrdinalIgnoreCase))
						{
							FailedReservations.Add(Constraint, $"Device name mismatch. Expected : {ExpectedDeviceName} | Actual: {ActualDeviceName}");
							continue;
						}
						break;

					case TestConstraintType.Tag:
						string ExpectedDeviceTags = "Required Tag";
						if (DeviceDefinition.Tags == null || DeviceDefinition.Tags.Count == 0)
						{
							string ActualTags = DeviceDefinition.Tags == null ? "<Null>" : "<Empty>";
							FailedReservations.Add(Constraint, $"Device missing tags. Expected : {ExpectedDeviceTags} | Actual: {ActualTags}");
							continue;
						}

						if (DeviceDefinition.Tags.Count > 1)
						{
							FailedReservations.Add(Constraint, $"Device unexpected tags. Expected : {ExpectedDeviceTags} | Actual: {string.Join(", ", DeviceDefinition.Tags)}");
							continue;
						}

						string ActualDeviceTags = DeviceDefinition.Tags[0];
						if (!ActualDeviceTags.Equals(ExpectedDeviceTags, StringComparison.OrdinalIgnoreCase))
						{
							FailedReservations.Add(Constraint, $"Device tag mismatch. Expected : {ExpectedDeviceTags} | Actual {ActualDeviceTags}");
							continue;
						}
						break;

					case TestConstraintType.Platform:
						UnrealTargetPlatform ExpectedPlatform = Constraint.Platform!.Value;
						UnrealTargetPlatform ActualPlatform = DeviceDefinition.Platform!.Value;
						if (ExpectedPlatform != ActualPlatform)
						{
							FailedReservations.Add(Constraint, $"Platform mismatch. Expected : {ExpectedPlatform} | Actual {ActualPlatform}");
							continue;
						}
						break;
				}
			}

			return FailedReservations;
		}

		private void LogFailures(TestConstraintType Type, Dictionary<UnrealDeviceTargetConstraint, string> Failures)
		{
			if (!Failures.Any())
			{
				return;
			}

			string Error = $"{Type} Failures ({Failures.Count})";
			foreach (var Failure in Failures)
			{
				Error += $"\n\t{Failure.Key.ToString()} | {Failure.Value}";
			}
			Log.Error(Error);
		}
	}
}
