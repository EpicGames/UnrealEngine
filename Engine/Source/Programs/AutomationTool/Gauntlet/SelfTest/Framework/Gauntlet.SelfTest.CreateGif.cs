// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.IO;
using System.Linq;
using System.Text;
using System.Threading.Tasks;

namespace Gauntlet.SelfTest
{
	[TestGroup("Framework")]
	class CreateGif : BaseTestNode
	{
		public override void TickTest()
		{
			string ImagePath = Path.Combine(Environment.CurrentDirectory, @"Engine\Source\Programs\AutomationTool\Gauntlet\SelfTest\TestData\GifTest");

			string OutPath = Path.Combine(Path.GetTempPath(), "Test.gif");

			bool Success = Utils.Image.SaveImagesAsGif(ImagePath, OutPath);

			if (Success)
			{
				File.Delete(OutPath);
			}

			MarkComplete(Success ? TestResult.Passed : TestResult.Failed);		
		}
	}
}
