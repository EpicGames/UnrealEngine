// Copyright Epic Games, Inc. All Rights Reserved.

using AutomationTool;
using System;
using System.Collections.Generic;
using System.Linq;
using System.Text;
using System.Threading.Tasks;

namespace Gauntlet.SelfTest
{
	enum TestStages
	{
		Init,
		//SubTests,
		IsTestReady,
		StartTest,
		TickTest,
		Status,
		StopTest,
		GetResult,
		SetFinalResult,
		CleanupTest,
		Summary
	};

	/// <summary>
	/// Tests that all functions are called in the expected order
	/// </summary>
	/// 
	[TestGroup("Framework")]
	class OrderOfOpsTest : ITestNode
	{
		TestStages CurrentStage = TestStages.Init;
		bool Completed;

		public float MaxDuration
		{
			get { return 0;	}
		}

		/// <summary>
		/// What the test result should be treated as if we reach max duration.
		/// </summary>
		public virtual EMaxDurationReachedResult MaxDurationReachedResult
		{
			get
			{
				return EMaxDurationReachedResult.Failure;
			}
		}

		public TestPriority Priority { get { return TestPriority.Normal; } }

		public string Name
		{
			get	{ return ToString();}
		}

		public bool HasWarnings { get { return false; } }

		public virtual bool LogWarningsAndErrorsAfterSummary { get { return true; } }

		protected void SetNewStage(TestStages NewStage)
		{
			Log.Info(" >> Executing {Stage}", NewStage.ToString());

			int CurrentValue = (int)CurrentStage;
			int NewValue = (int)NewStage;

			if (NewValue <= CurrentValue)
			{
				throw new TestException("New state {0} is <= current state {1}", NewStage, CurrentStage);
			}

			int Delta = NewValue - CurrentValue;
			if (Delta > 1)
			{
				throw new TestException("New state {0} is {1} steps more than the current state {2}", NewStage, Delta, CurrentStage);
			}

			CurrentStage = NewStage;
		}

		public TestResult GetTestResult()
		{
			// this is a stage that can hit multiple times.
			// we can just return Passed because SetNewStage will 
			// throw an exception if something is wrong here
			if (CurrentStage != TestStages.GetResult)
			{
				SetNewStage(TestStages.GetResult);
			}
			return TestResult.Passed;
		}

		/// <summary>
		/// Manually set the value of the test result
		/// </summary>
		public void SetTestResult(TestResult testResult)
		{
			SetNewStage(TestStages.SetFinalResult);
		}

		public string GetTestSummary()
		{
			SetNewStage(TestStages.Summary);
			return "Summary";
		}

		public virtual void AddTestEvent(UnrealTestEvent InEvent)
		{

		}

		public TestStatus GetTestStatus()
		{
			SetNewStage(TestStages.Status);
			return Completed ? TestStatus.Complete : TestStatus.InProgress;
		}

		/*public ITestNode[] GetSubTests()
		{
			SetNewStage(TestStages.SubTests);
			return new ITestNode[0];
		}*/

		/// <summary>
		/// Return list of warnings. Empty by default
		/// </summary>
		/// <returns></returns>
		public virtual IEnumerable<string> GetWarnings()
		{
			return new string[0];
		}

		/// <summary>
		/// Return list of errors. Empty by default
		/// </summary>
		/// <returns></returns>
		public virtual IEnumerable<string> GetErrors()
		{
			return new string[0];
		}

		public bool IsReadyToStart()
		{
			SetNewStage(TestStages.IsTestReady);
			return true;
		}

		public bool StartTest(int Pass, int NumPasses)
		{
			SetNewStage(TestStages.StartTest);
			return true;
		}

		public virtual void SetCancellationReason(string Reason)
		{
		}

		public void StopTest(StopReason InReason)
		{
			SetNewStage(TestStages.StopTest);
		}

		public void TickTest()
		{
			SetNewStage(TestStages.TickTest);
			Completed = true;
		}

		public bool RestartTest()
		{
			throw new NotImplementedException();
		}

		public void SetContext(ITestContext InContext)
		{
		}

		public virtual string GetRunLocalCommand(string LaunchingBuildCommand)
		{
			string CommandToRunLocally =
				string.Format("RunUAT {0} -Test={1} ", LaunchingBuildCommand, GetType());
			return CommandToRunLocally;
		}

		public void CleanupTest()
		{
			SetNewStage(TestStages.CleanupTest);
		}

		/// <summary>
		/// Output all defined commandline information for this test to the gauntlet window and exit test early.
		/// </summary>
		public virtual void DisplayCommandlineHelp()
		{

		}
	}
}
