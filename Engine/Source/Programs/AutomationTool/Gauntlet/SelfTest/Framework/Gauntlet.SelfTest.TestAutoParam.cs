// Copyright Epic Games, Inc. All Rights Reserved.

using Gauntlet;
using System;
using System.Collections.Generic;
using System.Diagnostics;
using System.IO;
using System.Linq;
using System.Text;
using System.Threading.Tasks;

namespace Gauntlet.SelfTest
{
	public enum TestAutoParamEnum
	{
		FirstType,
		SecondType
	};

	
	public class TestAutoParamOptions
	{
		public class SubParamOptions
		{
			[AutoParam(1)]
			public int SubInt;
		}

		[AutoParam("StringDefault")]
		public string StringParam;

		[AutoParamWithNames("StringDefault", OptionNames: "StringParam")]
		public string StringProperty { get; set; }

		[AutoParamWithNames("InternalDefault", OptionNames: ["ExternalName", "OtherExternalName"])]
		public string InternalName;

		[AutoParam(42)]
		public int IntParam;

		[AutoParam(1.0)]
		public float FloatParam;

		[AutoParam(1.0f)]
		public float UnspecifiedFloatParam;

		[AutoParam(false)]
		public bool BoolParam;

		[AutoParam(TestAutoParamEnum.FirstType)]
		public TestAutoParamEnum EnumParam;

		[AutoParam("")]
		public SubParamOptions SubParams;
	};

	[TestGroup("Framework")]
	class TestAutoParam : BaseTestNode
	{
		public override void TickTest()
		{
			TestAutoParamOptions Options = new TestAutoParamOptions();

			// Test that default assignments work
			AutoParam.ApplyDefaults(Options);

			CheckResult(Options.StringParam == "StringDefault", "Parsing StringParam failed");
			CheckResult(Options.StringProperty == "StringDefault", "Parsing StringProperty failed");
			CheckResult(Options.InternalName == "InternalDefault", "Parsing InternalName failed");
			CheckResult(Options.IntParam == 42, "Parsing IntParam failed");
			CheckResult(Options.FloatParam == 1.0f, "Parsing FloatParam failed");
			CheckResult(Options.BoolParam == false, "Parsing BoolParam failed");
			CheckResult(Options.BoolParam == false, "Parsing BoolParam failed");
			CheckResult(Options.SubParams.SubInt == 1, "Parsing SubParams failed");
			CheckResult(Options.EnumParam == TestAutoParamEnum.FirstType, "Parsing EnumParam failed");

			// Test that params are applied

			// set this and check it does not return to the default
			Options.UnspecifiedFloatParam = 99;

			string Params = "-stringparam=NewString -externalname=ExternalValue -intparam=84 -floatparam=2.0 -boolparam -subint=2 -enumparam=SecondType";
			AutoParam.ApplyParams(Options, Params.Split('-').Select(S => S.Trim()).ToArray());

			CheckResult(Options.StringParam == "NewString", "Parsing StringParam failed");
			CheckResult(Options.StringProperty == "NewString", "Parsing StringProperty failed");
			CheckResult(Options.InternalName == "ExternalValue", "Parsing InternalName failed");
			CheckResult(Options.IntParam == 84, "Parsing IntParam failed");
			CheckResult(Options.FloatParam == 2.0f, "Parsing FloatParam failed");
			CheckResult(Options.BoolParam == true, "Parsing BoolParam failed");
			CheckResult(Options.SubParams.SubInt == 2, "Parsing SubParams failed");
			CheckResult(Options.UnspecifiedFloatParam == 99, "Parsing UnspecifiedFloatParam failed");
			CheckResult(Options.EnumParam == TestAutoParamEnum.SecondType, "Parsing EnumParam failed");

			// Test that multiple param names are supported
			Params = "-OtherExternalName=SecondName";
			AutoParam.ApplyParams(Options, new string[] { "OtherExternalName=SecondName" });

			CheckResult(Options.InternalName == "SecondName", "Parsing InternalName failed");

			MarkComplete();
		}
	}
}
