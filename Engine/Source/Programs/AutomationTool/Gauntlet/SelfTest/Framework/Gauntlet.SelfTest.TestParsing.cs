// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Linq;
using System.Text;
using System.Threading.Tasks;

namespace Gauntlet.SelfTest
{
	[TestGroup("Framework")]
	class TestParamParsing : BaseTestNode
	{
	
		public override void TickTest()
		{
			// check repeated args
			Params MultiArgs1 = new Params(new[] { "foo=one", "foo=two" });
			List<string> MultiArgs1Result = MultiArgs1.ParseValues("foo");
			CheckResult(MultiArgs1Result.Count == 2, "Incorrect result count");
			CheckResult(MultiArgs1Result[0] == "one" && MultiArgs1Result[1] == "two", "Incorrect result count");

			// check comma-separated
			Params MultiArgs2 = new Params(new[] { "foo=one,two" });
			List<string> MultiArgs2Result = MultiArgs2.ParseValues("foo", true);
			CheckResult(MultiArgs2Result.Count == 2, "Incorrect result count");
			CheckResult(MultiArgs2Result[0] == "one" && MultiArgs2Result[1] == "two", "Incorrect result count");

			// check comma-separated with spaces (on cmdline these would be quote-wrapped)
			Params MultiArgs3 = new Params(new[] { "foo=one, two" });		
			List<string> MultiArgs3Result = MultiArgs3.ParseValues("foo",true);
			CheckResult(MultiArgs3Result.Count == 2, "Incorrect result count");
			CheckResult(MultiArgs3Result[0] == "one" && MultiArgs3Result[1] == "two", "Incorrect result count");

			// check default value
			Params MultiArgs4 = new Params(new[] { "foo=one", "foo=two", "foo=three" });
			List<string> MultiDefaultResult4 = ["four", "five"];
			List<string> MultiArgs4Result = MultiArgs4.ParseValues("dummy", Default: MultiDefaultResult4);
			CheckResult(MultiArgs4Result.Count == 2, "Incorrect result count");
			CheckResult(MultiArgs4Result[0] == "four" && MultiArgs4Result[1] == "five", "Incorrect result count");

			// negative check
			Params MultiArgs5 = new Params(new[] { "foo=one" });
			List<string> MultiArgs5Result = MultiArgs5.ParseValues("dummy");
			CheckResult(MultiArgs5Result != null, "Result is NULL");
			CheckResult(MultiArgs5Result.Count == 0, "Incorrect result count");


			// simple comma-separated list with no subparams
			string TestCase1 = "Test1, Test2";
			var Results1 = ArgumentWithParams.CreateFromString(TestCase1);

			CheckResult(Results1.Count == 2, "Incorrect result count");
			CheckResult(Results1[0].Argument == "Test1", "Incorrect Test Result");
			CheckResult(Results1[0].AllArguments.Length == 0, "Incorrect Test Result");
			CheckResult(Results1[1].Argument == "Test2", "Incorrect Test Result");
			CheckResult(Results1[1].AllArguments.Length == 0, "Incorrect Test Result");


			// comma-separated list with params
			string TestCase2 = "Test1,Test2(p1,p2=foo)";

			var Results2 = ArgumentWithParams.CreateFromString(TestCase2);

			CheckResult(Results2.Count == 2, "Incorrect result count");
			CheckResult(Results2[0].Argument == "Test1", "Incorrect Test Result");
			CheckResult(Results2[0].AllArguments.Length == 0, "Incorrect Test Result");

			CheckResult(Results2[1].Argument == "Test2", "Incorrect Test Result");
			CheckResult(Results2[1].AllArguments.Length==2, "Failed to parse correct param count");
			CheckResult(Results2[1].ParseParam("p1"), "Failed to parse p1 from params");
			CheckResult(Results2[1].ParseValue("p2", null)=="foo", "Failed to parse p2 from params");


			// comma-separated list with params that are what we'd get if the user encloses them in quotes
			string TestCase3 = "Test3(p1,p2=foo bar,p3)";

			var Results3 = ArgumentWithParams.CreateFromString(TestCase3);

			CheckResult(Results3.Count == 1, "Incorrect result count");
			CheckResult(Results3[0].Argument == "Test3", "Incorrect Test Result");

			CheckResult(Results3[0].AllArguments.Length == 3, "Failed to parse correct param count");
			CheckResult(Results3[0].ParseParam("p1"), "Failed to parse p1 from params");
			CheckResult(Results3[0].ParseValue("p2", null) == "foo bar", "Failed to parse p2 from params");
			CheckResult(Results3[0].ParseParam("p3"), "Failed to parse p3 from params");

			// One day...
			//string TestCase4 = "Test4(p1,p2(foo,bar),p3)";
			//var Results4 = SubParams.CreateFromString(TestCase4);
			//CheckResult(Results4.Count == 1, "Incorrect result count");

			MarkComplete();
		}
	}
}
