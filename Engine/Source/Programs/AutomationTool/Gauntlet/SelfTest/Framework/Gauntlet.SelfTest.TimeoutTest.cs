// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Linq;
using System.Text;
using System.Threading.Tasks;

namespace Gauntlet.SelfTest
{

	// Disabled as there's no longer an easy way for tests to overrule timeout results

	/*
	 * [TestGroup("Framework")]
	 * class TimeoutTest : BaseTestNode
	{
		DateTime StartTime;

		public override float MaxDuration
		{
			get		{	return 4;	}
		}

		public override bool StartTest(int Pass, int NumPasses)
		{
			StartTime = DateTime.Now;
			return base.StartTest(Pass, NumPasses);
		}

		public override void TickTest()
		{
			// Fail us if we run too long
			if ((DateTime.Now - StartTime).TotalSeconds > MaxDuration * 2)
			{
				MarkComplete(TestResult.Failed);
			}
		}

		public override void StopTest(bool WasCancelled)
		{
			// we pass, if we're canceled :)
			if (WasCancelled)
			{
				MarkComplete(TestResult.Passed);
			}
			else
			{
				MarkComplete(TestResult.Failed);
			}
		}
	}
	*/
}
