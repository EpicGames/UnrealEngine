// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Diagnostics;
using System.IO;
using System.Linq;
using System.Text;
using System.Threading.Tasks;

namespace Gauntlet.SelfTest
{
	public abstract class BaseTestNode : ITestNode
	{
		public virtual float MaxDuration
		{
			get
			{
				return 300;
			}
		}


		/// <summary>
		/// What the test result should be treated as if we reach max duration.
		/// </summary>
		public virtual EMaxDurationReachedResult MaxDurationReachedResult
		{
			get
			{
				return EMaxDurationReachedResult.Failure;
			}
		}


		public TestPriority Priority { get { return TestPriority.Normal; } }

		public string Name
		{
			get
			{
				return ToString();
			}
		}

		public bool HasWarnings { get { return false; } }

		public virtual bool LogWarningsAndErrorsAfterSummary { get { return true; } }

		public TestResult GetTestResult()
		{
			return InnerResult;
		}

		/// <summary>
		/// Manually set the value of the test result
		/// </summary>
		public void SetTestResult(TestResult testResult)
		{
		}

		public TestStatus GetTestStatus()
		{
			return InnerStatus;
		}

		public string GetTestSummary()
		{
			return string.Format("{0} {1}", Name, GetTestResult());
		}

		/// <summary>
		/// Return list of warnings. Empty by default
		/// </summary>
		/// <returns></returns>
		public virtual IEnumerable<string> GetWarnings()
		{
			return new string[0];
		}

		/// <summary>
		/// Return list of errors. Empty by default
		/// </summary>
		/// <returns></returns>
		public virtual IEnumerable<string> GetErrors()
		{
			return new string[0];
		}

		public virtual string GetRunLocalCommand(string LaunchingBuildCommand)
		{
			string CommandToRunLocally =
				string.Format("RunUAT {0} -Test={1} ", LaunchingBuildCommand, GetType());
			return CommandToRunLocally;
		}

		public virtual ITestNode[] GetSubTests()
		{
			return new ITestNode[0];
		}
		
		public bool RestartTest()
		{
			return false;
		}

		public void SetContext(ITestContext InContext)
		{
		}

		protected TestResult InnerResult;
		protected TestStatus InnerStatus;

		public virtual bool IsReadyToStart()
		{
			return true;
		}
	
		public virtual bool StartTest(int Pass, int NumPasses)
		{
			InnerStatus = TestStatus.InProgress;
			return true;
		}
		public virtual void CleanupTest()
		{
		}

		public virtual void SetCancellationReason(string Reason)
		{
		}

		public virtual void StopTest(StopReason InReason)
		{
		}

		public virtual void AddTestEvent(UnrealTestEvent InEvent)
		{ 
		
		}

		public BaseTestNode()
		{
			TestFailures = new List<string>();
		}

		public bool CheckResult(bool Result, string Format, params object[] Args)
		{

			if (Result == false)
			{
				StackTrace Stack = new StackTrace();
				StackFrame StackFrame = Stack.GetFrame(1);
				string TestClass = GetType().Name;

				string UserMessage = string.Format(Format, Args);

				string Msg = string.Format("Test Failure: {0}::{1} {2}", TestClass, StackFrame.GetMethod().Name, UserMessage);
				TestFailures.Add(Msg);
				Log.Error(Msg);
			}
			Trace.Assert(Result);

			return Result;
		}

		protected void MarkComplete(TestResult InResult)
		{
			InnerResult = InResult;
			InnerStatus = TestStatus.Complete;
		}

		protected void MarkComplete()
		{
			MarkComplete(TestFailures.Count == 0 ? TestResult.Passed : TestResult.Failed);
		}

		/// <summary>
		/// Output all defined commandline information for this test to the gauntlet window and exit test early.
		/// </summary>
		public virtual void DisplayCommandlineHelp()
		{

		}

		// Self test interface
		public abstract void TickTest();

		protected List<string> TestFailures;

	}
}
