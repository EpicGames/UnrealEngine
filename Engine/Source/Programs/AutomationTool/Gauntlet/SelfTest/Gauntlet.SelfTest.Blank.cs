// Copyright Epic Games, Inc. All Rights Reserved.

using Gauntlet;
using System;
using System.Collections.Generic;
using System.Diagnostics;
using System.IO;
using System.Linq;
using System.Text;
using System.Threading.Tasks;
using UnrealBuildTool;

namespace Gauntlet.SelfTest
{
	class Blank : BaseTestNode
	{
		public override void TickTest()
		{
		
			MarkComplete(TestResult.Passed);
		}
	}
}
