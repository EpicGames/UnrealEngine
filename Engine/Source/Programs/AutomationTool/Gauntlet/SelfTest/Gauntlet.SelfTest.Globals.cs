// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.IO;

namespace Gauntlet.SelfTest
{
	public class Globals
	{
		public string DeviceDevkitName { get; protected set; }

		private static Globals _Instance;

		protected Globals()
		{
			DeviceDevkitName = "AndrewGNeo";
		}

		public static Globals Instance
		{
			get
			{
				if (_Instance == null)
				{
					_Instance = new Globals();
				}
				return _Instance;
			}
		}

		public static string TempDir 
		{
			get { return Path.Combine(Gauntlet.Globals.TempDir, "SelfTest"); }
		}
	}

}