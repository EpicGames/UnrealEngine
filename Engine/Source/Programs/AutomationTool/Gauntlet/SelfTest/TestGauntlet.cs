// Copyright Epic Games, Inc. All Rights Reserved.

using AutomationTool;
using System;
using System.Collections.Generic;
using System.Linq;
using System.Reflection;
using System.Text;
using System.Threading.Tasks;
using UnrealBuildBase;

namespace Gauntlet
{
	public class TestGauntlet : BuildCommand
	{

		[AutoParamWithNames("Test", "Tests")]
		public string Tests = "";

		[AutoParamWithNames("Group")]
		public string Group = "";

		[AutoParam(false)]
		public bool Verbose;

		public override ExitCode Execute()
		{
			AutoParam.ApplyParamsAndDefaults(this, Environment.GetCommandLineArgs());

			if (Verbose)
			{
				Gauntlet.Log.Level = Gauntlet.LogLevel.Verbose;
			}

			IEnumerable<string> TestList = new string[0];

			// If a group was specified...
			if (Group.Length > 0)
			{
				// if a group was specified, find those tests
				TestList = Utils.TestConstructor.GetTestNamesByGroup<ITestNode>(Group, new[] { "Gauntlet.SelfTest" });
			}	
			else if (Tests.Length > 0)
			{
				// list of tests?
				TestList = Tests.Split(',').Select(S => S.Trim());	
			}		
			else
			{
				// Ok, run everything!
				TestList = Utils.TestConstructor.GetTestNamesByGroup<ITestNode>(null, new[] { "Gauntlet.SelfTest" });
			}

			// Create the list of tests
			IEnumerable<ITestNode> Nodes = Utils.TestConstructor.ConstructTests<ITestNode, string[]>(TestList, null, new[] { "Gauntlet.SelfTest" });

			// Create the test executor
			var Executor = new TestExecutor(ToString());

			TestExecutorOptions Options = new TestExecutorOptions();
			AutoParam.ApplyParamsAndDefaults(Options, this.Params);

			bool AllTestsPassed = Executor.ExecuteTests(Options, Nodes);

			return AllTestsPassed ? ExitCode.Success : ExitCode.Error_TestFailure;
		}
	}
}
