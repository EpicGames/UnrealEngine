// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.IO;
using System.Linq;
using System.Text;
using System.Threading.Tasks;
using Gauntlet;

namespace Gauntlet.SelfTest
{
	
	/// <summary>
	/// Base class for log parser tests
	/// </summary>
	abstract class TestUnrealLogParserBase : BaseTestNode
	{
		protected string BaseDataPath = Path.Combine(Environment.CurrentDirectory, @"Engine\Source\Programs\AutomationTool\Gauntlet\SelfTest\TestData\LogParser");

		protected string GetFileContents(string FileName)
		{
			string FilePath = Path.Combine(BaseDataPath, FileName);

			if (File.Exists(FilePath) == false)
			{
				throw new TestException("Missing data file {0}", FilePath);
			}

			return File.ReadAllText(FilePath);
		}

		public override bool StartTest(int Pass, int NumPasses)
		{
			return true;
		}

		public override void TickTest()
		{
			MarkComplete(TestResult.Passed);
		}
	}

	
	[TestGroup("LogParser")]
	class LogParserTestGauntletExitSuccess : TestUnrealLogParserBase
	{
		public override void TickTest()
		{
			HashSet<string> Platforms = new HashSet<string>();
			Platforms.Add("Win64");
			Platforms.Add(Gauntlet.Globals.Params.ParseValue("Platform", "Win64"));
			foreach (var Platform in Platforms)
			{
				UnrealLogParser Parser = new UnrealLogParser(GetFileContents($"{Platform}NormalExit.txt"));

				int ExitCode = 2;
				Parser.GetTestExitCode(out ExitCode);

				if (ExitCode != 0)
				{
					throw new TestException("LogParser did not find succesful exit for {0}", Platform);
				}
			}

			MarkComplete(TestResult.Passed);
		}
	}


	[TestGroup("LogParser")]
	class LogParserTestEnsure : TestUnrealLogParserBase
	{
		public override void TickTest()
		{
			HashSet<string> Platforms = new HashSet<string>();
			Platforms.Add("Win64");
			Platforms.Add(Gauntlet.Globals.Params.ParseValue("Platform", "Win64"));
			foreach (var Platform in Platforms)
			{
				UnrealLogParser Parser = new UnrealLogParser(GetFileContents($"{Platform}NormalExit.txt"));

				var Ensures = Parser.GetEnsures();

				if (!Ensures.Any())
				{
					throw new TestException("LogParser failed to find ensure for {0}", Platform);
				}

				var Ensure = Ensures.First();

				if (string.IsNullOrEmpty(Ensure.Message) || Ensure.Callstack.Length < 8)
				{
					throw new TestException("LogParser failed to find ensure details for {0}", Platform);
				}
			}

			MarkComplete(TestResult.Passed);
		}
	}

	[TestGroup("LogParser")]
	class LogParserTestAssert : TestUnrealLogParserBase
	{
		public override void TickTest()
		{
			HashSet<string> Platforms = new HashSet<string>();
			Platforms.Add("Win64");
			Platforms.Add("Linux");
			Platforms.Add(Gauntlet.Globals.Params.ParseValue("Platform", "Win64"));
			foreach (var Platform in Platforms)
			{
				UnrealLogParser Parser = new UnrealLogParser(GetFileContents($"{Platform}Assert.txt"));

				UnrealLog.CallstackMessage FatalError = Parser.GetFatalError();

				if (FatalError == null
					|| FatalError.Callstack.Length < 8
					|| string.IsNullOrEmpty(FatalError.Message)
					|| !FatalError.Message.Contains("Assert", StringComparison.OrdinalIgnoreCase))
				{
					throw new TestException("LogParser returned incorrect assert info for {0}", Platform);
				}
			}

			MarkComplete(TestResult.Passed);
		}
	}

	[TestGroup("LogParser")]
	class LogParserTestAssertWithCircularBuffer : TestUnrealLogParserBase
	{
		public override void TickTest()
		{
			HashSet<string> Platforms = new HashSet<string>();
			Platforms.Add("Win64");
			Platforms.Add("Linux");
			Platforms.Add(Gauntlet.Globals.Params.ParseValue("Platform", "Win64"));
			foreach (var Platform in Platforms)
			{
				CircularLogBuffer Buffer = ProcessUtils.CreateLogBuffer().Feed(GetFileContents($"{Platform}Assert.txt"));
				UnrealLogParser Parser = new UnrealLogParser(Buffer.GetReader());

				UnrealLog.CallstackMessage FatalError = Parser.GetFatalError();

				if (FatalError == null
					|| FatalError.Callstack.Length < 8
					|| string.IsNullOrEmpty(FatalError.Message)
					|| !FatalError.Message.Contains("Assert", StringComparison.OrdinalIgnoreCase))
				{
					throw new TestException("LogParser returned incorrect assert info for {0}", Platform);
				}
			}

			MarkComplete(TestResult.Passed);
		}
	}

	/// <summary>
	/// Tests that the log parser correctly finds a fatal error in a logfile
	/// </summary>
	[TestGroup("LogParser")]
	class LogParserTestFatalError : TestUnrealLogParserBase
	{
		private int IncompleteLineLength = "0x0000000000236999  [Unknown File]".Length;
		public override void TickTest()
		{
			HashSet<string> Platforms = new HashSet<string>();
			Platforms.Add("Win64");
			Platforms.Add("Linux");
			Platforms.Add(Gauntlet.Globals.Params.ParseValue("Platform", "Win64"));
			foreach (var Platform in Platforms)
			{
				Log.Info("Processing log: {Path}", Platform + "FatalError" + ".txt");
				UnrealLogParser Parser = new UnrealLogParser(GetFileContents(Platform + "FatalError" + ".txt"));

				UnrealLog.CallstackMessage FatalError = Parser.GetFatalError();

				if (FatalError == null || FatalError.Callstack.Length == 0 || string.IsNullOrEmpty(FatalError.Message))
				{
					throw new TestException("LogParser returned incorrect assert info for {0}", Platform);
				}
				else
				{
					int IncompleteCallstackLineCount = FatalError.Callstack.Where(L => L.Length <= IncompleteLineLength).Count();
					if(IncompleteCallstackLineCount > 0)
					{
						string Lines = string.Join("\n", FatalError.Callstack.Where(L => L.Length <= IncompleteLineLength));
						throw new TestException("LogParser returned some incomplete callstack lines for {0}:\n{1}", Platform, Lines);
					}
				}
			}			

			MarkComplete(TestResult.Passed);
		}
	}

	/// <summary>
	/// Tests that the logfile correctly finds a RequestExit line in a log file
	/// </summary>
	[TestGroup("LogParser")]
	class LogParserTestRequestExit : TestUnrealLogParserBase
	{
		public override void TickTest()
		{
			string Platform = Gauntlet.Globals.Params.ParseValue("Platform", "Win64");
			UnrealLogParser Parser = new UnrealLogParser(GetFileContents($"{Platform}NormalExit.txt"));

			// Get warnings
			bool HadExit = Parser.HasRequestExit();

			if (HadExit == false)
			{
				throw new TestException("LogParser returned incorrect RequestExit");
			}

			MarkComplete(TestResult.Passed);
		}
	}

	/// <summary>
	/// Tests that the log parser correctly extracts channel-lines from a logfile
	/// </summary>
	[TestGroup("LogParser")]
	class LogParserTestChannels: TestUnrealLogParserBase
	{
		public override void TickTest()
		{
			string Platform = Gauntlet.Globals.Params.ParseValue("Platform", "Win64");
			UnrealLogParser Parser = new UnrealLogParser(GetFileContents($"{Platform}NormalExit.txt"));

			// Get warnings
			IEnumerable<string> Lines = Parser.GetLogChannel("AutomationController");

			if (!Lines.Any())
			{
				throw new TestException("LogParser did not find any AutomationController channel message");
			}

			MarkComplete(TestResult.Passed);
		}
	}

	/// <summary>
	/// Tests that the log parser correctly pulls warnings from a log file
	/// </summary>
	[TestGroup("LogParser")]
	class LogParserTestWarnings : TestUnrealLogParserBase
	{
		public override void TickTest()
		{
			string Platform = Gauntlet.Globals.Params.ParseValue("Platform", "Win64");
			UnrealLogParser Parser = new UnrealLogParser(GetFileContents($"{Platform}NormalExit.txt"));

			// Get warnings
			IEnumerable<string> WarningLines = Parser.GetWarnings();

			if (!WarningLines.Any())
			{
				throw new TestException("LogParser did not find any warning");
			}

			MarkComplete(TestResult.Passed);
		}
	}

	/// </summary>
	[TestGroup("LogParser")]
	class LogParserTestErrors : TestUnrealLogParserBase
	{
		public override void TickTest()
		{
			string Platform = Gauntlet.Globals.Params.ParseValue("Platform", "Win64");
			UnrealLogParser Parser = new UnrealLogParser(GetFileContents($"{Platform}NormalExit.txt"));

			// Get warnings
			IEnumerable<string> ErrorLines = Parser.GetErrors();

			if (!ErrorLines.Any())
			{
				throw new TestException("LogParser did not find any error");
			}

			MarkComplete(TestResult.Passed);
		}
	}
}

