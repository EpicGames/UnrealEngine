// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.IO;
using System.Linq;
using EpicGames.Core;
using UnrealBuildTool;
using UnrealBuildBase;
using AutomationTool;

namespace Gauntlet.SelfTest
{
	/// <summary>
	/// Base class for Unreal tests. Mostly provides Build / Environment info
	/// </summary>
	abstract class TestUnrealBase : BaseTestNode
	{
		/// <summary>
		/// Name of the project we're testing
		/// </summary>
		public string ProjectName { get; protected set; }

		/// <summary>
		/// Name of the project we're testing
		/// </summary>
		public FileReference ProjectFile { get; protected set; }

		/// <summary>
		/// Path to Unreal
		/// </summary>
		public DirectoryReference UnrealPath { get; protected set; }

		/// <summary>
		/// True if this project uses the Game/NoEditor shared build type instead of separate client/server ones
		/// </summary>
		public bool UsesSharedBuildType { get; protected set; }

		/// <summary>
		/// Name of devkit to use for tests
		/// </summary>
		public string DevkitName { get; protected set; }

		/// <summary>
		/// Path of the build to test against
		/// </summary>
		public string BuildPath { get; protected set; }

		/// <summary>
		/// Client platforms we support
		/// </summary>
		public UnrealTargetPlatform[] SupportedClientPlatforms { get; protected set; }

		/// <summary>
		/// Server platforms we support
		/// </summary>
		public UnrealTargetPlatform[] SupportedServerPlatforms { get; protected set; }

		/// <summary>
		/// Configurations we support
		/// </summary>
		public UnrealTargetConfiguration[] SupportedConfigurations { get; protected set; }

		/// <summary>
		/// Target Configuration
		/// </summary>
		public UnrealTargetConfiguration Configuration { get; protected set; }

		public TestUnrealBase()
		{
			ProjectName = Gauntlet.Globals.Params.ParseValue("Project", "EngineTest");
			string ConfigurationString = Gauntlet.Globals.Params.ParseValue("Configuration", "Development"); 
			BuildPath = Gauntlet.Globals.Params.ParseValue("Build", null);
			DevkitName = Gauntlet.Globals.Params.ParseValue("Device", "Default");
			UnrealPath = Unreal.RootDirectory;
			UsesSharedBuildType = false;

			if (File.Exists(ProjectName))
			{
				ProjectFile = new FileReference(ProjectName);
				ProjectName = ProjectFile.GetFileNameWithoutExtension();
			}
			else
			{
				if (!string.IsNullOrEmpty(ProjectName))
				{
					ProjectFile = ProjectUtils.FindProjectFileFromName(ProjectName);

					if (ProjectFile == null)
					{
						throw new AutomationException("Could not find project file for {0}", ProjectName);
					}
					ProjectName = ProjectFile.GetFileNameWithoutExtension();
				}
			}

			if (BuildHostPlatform.Current.Platform == UnrealTargetPlatform.Win64)
			{
				IEnumerable<UnrealTargetPlatform> SupportedDevices = Gauntlet.Utils.InterfaceHelpers.FindImplementations<IDeviceBuildSupport>().Select(D => D.GetPlatform() ?? UnrealTargetPlatform.Win64);
				SupportedClientPlatforms = SupportedDevices.Append(UnrealTargetPlatform.Win64).Distinct().ToArray();
				SupportedServerPlatforms = new[] { UnrealTargetPlatform.Win64, UnrealTargetPlatform.Linux };
			}
			else
			{
				SupportedClientPlatforms = new[] { UnrealTargetPlatform.Mac };
				SupportedServerPlatforms = new[] { UnrealTargetPlatform.Mac };
			}

			SupportedConfigurations = new[] { UnrealTargetConfiguration.Development, UnrealTargetConfiguration.Test };

			object RequestedConfiguration;
			if (!Enum.TryParse(typeof(UnrealTargetConfiguration), ConfigurationString, true, out RequestedConfiguration))
			{
				string AllKeys = string.Join(", ", SupportedConfigurations);
				throw new AutomationException(string.Format("Unknown Configuration '{0}', it must be one of the values: {1}.", ConfigurationString, AllKeys));
			}
			Configuration = (UnrealTargetConfiguration)RequestedConfiguration;
		}
	}
}