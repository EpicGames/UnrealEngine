// Copyright Epic Games, Inc. All Rights Reserved.

using Gauntlet;
using System;
using System.Collections.Generic;
using System.Diagnostics;
using System.IO;
using System.Linq;
using System.Text;
using System.Threading.Tasks;
using UnrealBuildTool;

namespace Gauntlet.SelfTest
{
	/// <summary>
	/// This test validates that an UnrealBuildSource can create an app configuration for a number of different 
	/// roles using different platforms and configurations.
	/// 
	/// The test uses the base node to provide a build to test and a list of platforms and configurations that
	/// should be supported.
	/// 
	/// Marked as P3 because this should ideally run before any tests that actually try to run things
	/// </summary>
	[TestGroup("Unreal", 4)]
	class TestUnrealBuildSource : TestUnrealBase
	{
		const string extra_arguments = "-somearg -some=\"arg\" -some[arg]:with/path -some-arg";
		/// <summary>
		/// Test entry point
		/// </summary>
		public override void TickTest()
		{
			// create the build source
			UnrealBuildSource BuildSource = new UnrealBuildSource(this.ProjectName, ProjectFile, this.UnrealPath, UsesSharedBuildType, BuildPath);

			// check if editor build can be found
			CheckResult(BuildSource.GetBuildCount(BuildHostPlatform.Current.Platform, BuildFlags.None) > 0, "No Editor build found.");

			// simple check with an editor role
			UnrealSessionRole EditorRole = new UnrealSessionRole(UnrealTargetRole.Editor, BuildHostPlatform.Current.Platform, Configuration);
			// set extra arguments
			EditorRole.CommandLineParams.AddRawCommandline(extra_arguments);

			List<string> Reasons = new List<string>();

			// Check the build source can support this role
			bool ContainsEditor = BuildSource.CanSupportRole(EditorRole, ref Reasons);
			CheckResult(ContainsEditor, "{0", string.Join(", ", Reasons));

			// now actually try to create it
			UnrealAppConfig Config = BuildSource.CreateConfiguration(EditorRole, new UnrealSessionRole[] { });
			CheckResult(Config != null, "Build source did not return a config for {0}", EditorRole.ToString());

			ValidateEditorConfig(Config, BuildSource);
			ValidateEditorExtraArguments(Config, BuildSource);

			// Check all editor types (game, server, etc)
			TestBuildSourceForEditorTypes(BuildSource);

			// Test all monolithics that our base test says we support
			TestBuildSourceForMonolithics(BuildSource);
	
			MarkComplete();
		}

		/// <summary>
		/// Tests that this BuildSource is capable of providing a config for all supported editor-based roles
		/// </summary>
		/// <param name="BuildSource"></param>
		void TestBuildSourceForEditorTypes(UnrealBuildSource BuildSource)
		{
			// create a config for all editor based types
			foreach (var E in Enum.GetValues(typeof(UnrealTargetRole)).Cast<UnrealTargetRole>())
			{
				if (E.UsesEditor())
				{
					UnrealAppConfig Config = BuildSource.CreateConfiguration(new UnrealSessionRole(E, BuildHostPlatform.Current.Platform, UnrealTargetConfiguration.Development));

					CheckResult(Config != null, "Editor config for {0} returned null!", E);
					CheckResult(string.IsNullOrEmpty(Config.Name) == false, "No config name!");

					ValidateEditorConfig(Config, BuildSource);
				}
			}
		}

		/// <summary>
		/// Tests that this BuildSource is capable of returning configs for all the roles,
		/// platforms, and configurations that our base class says it should support
		/// </summary>
		/// <param name="BuildSource"></param>
		/// <returns></returns>
		void TestBuildSourceForMonolithics(UnrealBuildSource BuildSource)
		{
			List<UnrealSessionRole> AllRoles = new List<UnrealSessionRole>();

			// Add a role for all supported clients on all supported configurations
			foreach (var Platform in SupportedClientPlatforms)
			{
				foreach (var Config in SupportedConfigurations)
				{
					AllRoles.Add(new UnrealSessionRole(UnrealTargetRole.Client, Platform, Config));
				}
			}

			// Add a role for all supported servers on all supported configurations
			foreach (var Platform in SupportedServerPlatforms)
			{
				foreach (var Config in SupportedConfigurations)
				{
					AllRoles.Add(new UnrealSessionRole(UnrealTargetRole.Server, Platform, Config));
				}
			}

			// Now check the build source can create all of these
			foreach (var Role in AllRoles)
			{
				List<string> Issues = new List<string>();
				bool Result = BuildSource.CanSupportRole(Role, ref Issues);
				Issues.ForEach(S => Log.Error(S));
				CheckResult(Result, "Failed to get artifacts for {0}", Role);

				// now actually try to create it
				UnrealAppConfig Config = BuildSource.CreateConfiguration(Role);

				CheckResult(Config != null, "Build source did not return a config for {0}", Role.ToString());
			}
		}

		/// <summary>
		/// Simple validation of a config that uses
		/// </summary>
		/// <param name="Config"></param>
		/// <param name="BuildSource"></param>
		void ValidateEditorConfig(UnrealAppConfig Config, UnrealBuildSource BuildSource)
		{
			string Args = Config.CommandLine.Trim().ToLower();

			CheckResult(Config.ProcessType.UsesEditor(), "Config does not use editor!");

			// Check the project name was the first arg
			CheckResult(Args.Contains(BuildSource.ProjectName.ToLower()), "Editor-based config for {0} needs to include project name as first argument", Config.ProcessType);

			// for clients, check for -game
			if (Config.ProcessType.IsClient())
			{
				CheckResult(Args.Contains("-game"), "Editor-based game needs to include -game");
			}

			// for servers, check for -server
			if (Config.ProcessType.IsServer())
			{
				CheckResult(Args.Contains("-server"), "Editor-based game needs to include -server");
			}
		}

		/// <summary>
		/// Simple validation for config extra arguments passthrough
		/// </summary>
		/// <param name="Config"></param>
		/// <param name="BuildSource"></param>
		void ValidateEditorExtraArguments(UnrealAppConfig Config, UnrealBuildSource BuildSource)
		{
			string Args = Config.CommandLine.Trim().ToLower();
			foreach (var expected in extra_arguments.Split(" "))
			{
				CheckResult(Args.Contains(expected), $"Argument {expected} not passed to Editor command line.");
			}
		}
	}
}
