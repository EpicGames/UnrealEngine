// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Data;
using System.IO;
using System.Linq;
using System.Threading;
using AutomationTool;
using UnrealBuildTool;

namespace Gauntlet.SelfTest
{
	/// <summary>
	/// Base self test class for installing and running processes for the request role(s)
	/// </summary>
	abstract class TestUnrealInstallAndRunBase<TTargetDevice> : TestUnrealBase
		where TTargetDevice : ITargetDevice
	{
		protected struct TestCase
		{
			public UnrealTargetRole Role;
			public IAppInstall Install;
			public IAppInstance Instance;
			public bool bHasStarted;

			public TestCase(UnrealTargetRole Role)
			{
				this.Role = Role;
				this.Install = null;
				this.Instance = null;
				this.bHasStarted = false;
			}
		}

		[AutoParamWithNames(UnrealTargetConfiguration.Development, "TargetConfiguration", "Configuration", "Config")]
		public UnrealTargetConfiguration TargetConfiguration { get; set; }

		/// <summary>
		/// List of roles to use as test cases to override the default, separated by comma
		/// </summary>
		[AutoParamWithNames(Default: "", "TestCases", "Roles")]
		public string TestCasesOverride { get; set; }

		protected Queue<TestCase> TestCases;
		protected TTargetDevice TargetDevice;
		protected UnrealTargetPlatform Platform;
		protected UnrealTestConfiguration Options;
		protected UnrealAppConfig AppConfig;

		private UnrealBuildSource Build;
		private DateTime AppStartTime;
		private TestCase CurrentTestCase;

		public TestUnrealInstallAndRunBase()
		{
			AutoParam.ApplyParamsAndDefaults(this, Gauntlet.Globals.Params.AllArguments);
			Options = new UnrealTestConfiguration();
			TestCases = new Queue<TestCase>();
		}

		public override bool StartTest(int Pass, int NumPasses)
		{
			// Create the target device used for the test
			IDeviceFactory Factory = Utils.InterfaceHelpers.FindImplementations<IDeviceFactory>()
					.Where(F => F.CanSupportPlatform(Platform))
					.FirstOrDefault();
			TargetDevice = (TTargetDevice)Factory.CreateDevice(DevkitName, GetCleanCacheDirectory());

			// Construct each test case
			if(!string.IsNullOrEmpty(TestCasesOverride))
			{
				TestCases.Clear();
				string[] Roles = TestCasesOverride.Split(',');

				foreach(string Role in Roles)
				{
					if(Enum.TryParse(Role, out UnrealTargetRole TargetRole))
					{
						TestCases.Enqueue(new TestCase(TargetRole));
					}
					else
					{
						throw new AutomationException("Requested role override \"{0}\" could not be parsed into a TargetRole", Role);
					}
				}
			}
			else if (!TestCases.Any())
			{
				throw new AutomationException("No test cases have been defined for this platform. Has the test been implemented?");
			}

			CurrentTestCase = TestCases.Dequeue();

			return base.StartTest(Pass, NumPasses);
		}

		public override void TickTest()
		{
			if(!CurrentTestCase.bHasStarted)
			{
				if(!SetupTestCase())
				{
					Log.Error("Failed test case for Role {TargetRole} during Setup stage. Starting next test case.", CurrentTestCase.Role);
					StartNextTestCase();
				}

				// Run the app and wait for either a timeout or it to exit
				CurrentTestCase.Instance = CurrentTestCase.Install.Run();
				CurrentTestCase.bHasStarted = true;
				AppStartTime = DateTime.Now;
				return;
			}

			// Run for 60 seconds
			if ((DateTime.Now - AppStartTime).TotalSeconds < 60)
			{
				return;
			}

			// Check the app didn't exit unexpectedly
			if(CurrentTestCase.Instance.HasExited)
			{
				CheckResult(false, "Failed test case for {0} during Run stage. Starting next test case.", CurrentTestCase.Role);
			}

			// Now kill it
			CurrentTestCase.Instance.Kill();

			// Wait 10 seconds for any remaining process handles to shut down
			Thread.Sleep(10 * 1000);

			// Check that it left behind some artifacts (minimum should be a log)
			bool bAnyArtifacts = new DirectoryInfo(CurrentTestCase.Instance.ArtifactPath).GetFiles("*", SearchOption.AllDirectories).Any();
			CheckResult(bAnyArtifacts, "No artifacts on device!");

			StartNextTestCase();
		}

		// Platforms should completely wipe the kit with this function to ensure a clean environment.
		protected virtual bool PerformFullDeviceClean() { return true; }

		// Should create a file, store the file onto the kit, and clear the device cache.
		protected abstract bool TestClearSavedDirectory();

		// Should install the app and return an IAppInstall
		protected abstract bool TestInstallApplication(out IAppInstall Install);

		// Should add a file to the AppConfig, copy it over, and then verify it exists in the target diretory
		protected abstract bool TestCopyAppConfigurationFiles();

		protected bool SetupTestCase()
		{
			// Create a build source
			string Path = CurrentTestCase.Role == UnrealTargetRole.Editor ? "Editor" : BuildPath;
			Build = new UnrealBuildSource(ProjectName, ProjectFile, UnrealPath, UsesSharedBuildType, Path);
			if (!CheckResult(Build.GetBuildCount(Platform) > 0, "Selected build was invalid"))
			{
				return false;
			}

			// Create Role
			UnrealSessionRole Role = new UnrealSessionRole(CurrentTestCase.Role, Platform, TargetConfiguration, Options);
			Log.Info("Running test for {Role}", Role);

			// Create App Config
			AppConfig = Build.CreateConfiguration(Role);
			if (!CheckResult(AppConfig != null, "Could not create config for {0} {1} with platform {2} from build.", TargetConfiguration, Role, TargetDevice.Platform))
			{
				return false;
			}

			// Clean the device
			if (!CheckResult(AppConfig.SkipInstall || PerformFullDeviceClean(), "Could not fully clean device {0}", TargetDevice))
			{
				return false;
			}

			// Clean any artifacts in the event skip clean was requested
			/* TODO enable after TargetDevices implement this
			if (!CheckResult(TestClearSavedDirectory(), "Failed to clean device cache for device {Device}", TargetDevice))
			{
				return false;
			}
			*/

			IAppInstall Install;
			if (!CheckResult(TestInstallApplication(out Install), "Could not create AppInstall for {0} {1} with platform {2} from build.", TargetConfiguration, Role, TargetDevice.Platform))
			{
				return false;
			}

			CurrentTestCase.Install = Install;

			// InstallApplication populates the directory mappings, the ensure copy additional files has proper handling for empty mappings, clear them out now
			/* TODO enable after TargetDevices implement this
			TargetDevice.GetPlatformDirectoryMappings().Clear();
			if (!CheckResult(TestCopyAppConfigurationFiles(), "Could not copy {0} additional files to device {1}", AppConfig.FilesToCopy.Count, TargetDevice))
			{
				return false;
			}
			*/

			return true;
		}

		protected FileInfo CreateDummyFile()
		{
			FileInfo DummyFile = new FileInfo(Path.GetTempFileName());
			File.WriteAllText(DummyFile.FullName, "Foo");
			return DummyFile;
		}

		protected UnrealFileToCopy CreateDummyUnrealFileToCopy()
		{
			FileInfo DummyFile = CreateDummyFile();
			UnrealFileToCopy FileToCopy = new UnrealFileToCopy(DummyFile.FullName, EIntendedBaseCopyDirectory.Saved, DummyFile.Name);
			return FileToCopy;
		}

		private string GetCleanCacheDirectory()
		{
			DirectoryInfo TempDir = new DirectoryInfo(Gauntlet.Globals.TempDir);
			if (TempDir.Exists)
			{
				TempDir.Delete(true);
			}
			TempDir.Create();

			DirectoryInfo CacheDirectory = new DirectoryInfo(Path.Combine(TempDir.FullName, "DeviceCache", Platform.ToString(), "TestInstallRun"));
			if (CacheDirectory.Exists)
			{
				CacheDirectory.Delete(true);
			}
			CacheDirectory.Create();

			return CacheDirectory.FullName;
		}

		private void StartNextTestCase()
		{
			if (TestCases.Any())
			{
				CurrentTestCase = TestCases.Dequeue();
			}
			else
			{
				// All tests complete.
				Log.Info("All test cases completed");
				MarkComplete();
			}
		}
	}
}