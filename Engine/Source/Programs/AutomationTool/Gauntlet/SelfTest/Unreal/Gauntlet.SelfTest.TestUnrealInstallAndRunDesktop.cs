// Copyright Epic Games, Inc. All Rights Reserved.

using System.IO;
using UnrealBuildTool;

namespace Gauntlet.SelfTest
{
	abstract class TestUnrealInstallAndRunDesktop : TestUnrealInstallAndRunBase<TargetDeviceDesktopCommon>
	{
		public TestUnrealInstallAndRunDesktop()
		{
			TestCases.Enqueue(new TestCase(UnrealTargetRole.Editor));
			TestCases.Enqueue(new TestCase(UnrealTargetRole.Client));
			TestCases.Enqueue(new TestCase(UnrealTargetRole.Server));
		}

		protected override bool TestClearSavedDirectory()
		{
			// Staged builds have their saved within the build folder while native/editor builds reside in the project dir
			string PlatformName = Platform == UnrealTargetPlatform.Win64 ? "Windows" : Platform.ToString();
			string SavedPath = AppConfig.Build is StagedBuild
				? Path.Combine(BuildPath, PlatformName + "Client", ProjectName, "Saved")
				: Path.Combine(ProjectFile.Directory.FullName, "Saved");

			DirectoryInfo SavedUserDirectory = new DirectoryInfo(Path.Combine(TargetDevice.LocalCachePath, "UserDir", "Saved"));
			DirectoryInfo SavedDirectory = new DirectoryInfo(SavedPath);

			// Create a couple of dummy files to copy over, we'll test
			FileInfo SourceFile = CreateDummyFile();
			FileInfo SavedFile = new FileInfo(Path.Combine(SavedUserDirectory.FullName, SourceFile.Name));
			FileInfo UserSavedFile = new FileInfo(Path.Combine(SavedDirectory.FullName, SourceFile.Name));

			SavedFile.Directory.Create();
			UserSavedFile.Directory.Create();

			SourceFile.CopyTo(SavedFile.FullName);
			SourceFile.CopyTo(UserSavedFile.FullName);

			// Verify it exists at the target location
			if (!SavedFile.Exists || !UserSavedFile.Exists)
			{
				Log.Error("Failed to copy over dummy file");
				return false;
			}

			// Now delete the file using clear cache
			TargetDevice.CleanArtifacts();

			// We should see these files and their parent directories are now gone.
			bool bSavedWasDeleted = !SavedFile.Directory.Exists;
			bool bUserSavedWasDeleted = !UserSavedFile.Directory.Exists;

			return bSavedWasDeleted && bUserSavedWasDeleted;
		}

		protected override bool TestInstallApplication(out IAppInstall Install)
		{
			// Clear the platform directory mappings because we populated them inside TestClearDeviceCache, install should populate them.
			TargetDevice.GetPlatformDirectoryMappings().Clear();

			Install = TargetDevice.InstallApplication(AppConfig);
			return Install != null;
		}

		protected override bool TestCopyAppConfigurationFiles()
		{
			UnrealFileToCopy File = CreateDummyUnrealFileToCopy();
			AppConfig.FilesToCopy.Add(File);

			TargetDevice.CopyAdditionalFiles(AppConfig.FilesToCopy);

			string CopyDirectory = Path.Combine(TargetDevice.GetPlatformDirectoryMappings()[EIntendedBaseCopyDirectory.Saved], File.TargetRelativeLocation);
			FileInfo CopiedFile = new FileInfo(CopyDirectory);

			return CopiedFile.Exists;
		}
	}
}