// Copyright Epic Games, Inc. All Rights Reserved.

using Gauntlet;
using System;
using System.Collections.Generic;
using System.Diagnostics;
using System.IO;
using System.Linq;
using System.Text;
using System.Threading.Tasks;
using UnrealBuildTool;

namespace Gauntlet.SelfTest
{
	/// <summary>
	/// This test validates that our basic Unreal options are correctly applied to client and server
	/// configurations
	/// </summary>
	[TestGroup("Unreal")]
	class TestUnrealOptions: TestUnrealBase
	{
		public override void TickTest()
		{
			// Grab the most recent Release build of Orion
			UnrealBuildSource Build = new UnrealBuildSource(this.ProjectName, this.ProjectFile, this.UnrealPath, this.UsesSharedBuildType, this.BuildPath);

			// create client and server riles
			UnrealSessionRole ClientRole = new UnrealSessionRole(UnrealTargetRole.Client, UnrealTargetPlatform.Win64, UnrealTargetConfiguration.Development);
			UnrealSessionRole ServerRole = new UnrealSessionRole(UnrealTargetRole.Server, UnrealTargetPlatform.Win64, UnrealTargetConfiguration.Development);

			// create configurations from the build
			UnrealAppConfig ClientConfig = Build.CreateConfiguration(ClientRole);
			UnrealAppConfig ServerConfig = Build.CreateConfiguration(ServerRole);

			UnrealOptions Options = new UnrealOptions();

			// create some params
			string[] Params = new string[] { "nullrhi", "ResX=800", "ResY=600", "Map=FooMap", "epicapp=Prod", "buildidoverride=1111", "commonargs=-somethingcommon", "-clientargs=-somethingclient", "-serverargs=-somethingserver" };

			// apply them to options
			AutoParam.ApplyParams(Options, Params);

			Options.ApplyToConfig(ClientConfig);
			Options.ApplyToConfig(ServerConfig);

			CheckResult(ClientConfig.CommandLine.Contains("-nullrhi"), "Client Arg not applied!");
			CheckResult(ClientConfig.CommandLine.Contains("-ResX=800"), "Client Arg not applied!");
			CheckResult(ClientConfig.CommandLine.Contains("-ResY=600"), "Client Arg not applied!");
			CheckResult(ClientConfig.CommandLine.Contains("-somethingclient"), "Client Arg not applied!");

			CheckResult(ServerConfig.CommandLine.Contains("-Map=FooMap") == false, "Server Arg incorrectly applied!");
			CheckResult(ServerConfig.CommandLine.StartsWith("FooMap"), "Server Args not start with map!");
			CheckResult(ServerConfig.CommandLine.Contains("-somethingserver"), "Server Arg not applied!");

			CheckResult(ClientConfig.CommandLine.Contains("-buildidoverride=1111"), "Common Arg not applied!");
			CheckResult(ClientConfig.CommandLine.Contains("-somethingcommon"), "Common Arg not applied!");
			CheckResult(ServerConfig.CommandLine.Contains("-buildidoverride=1111"), "Common Arg not applied!");
			CheckResult(ServerConfig.CommandLine.Contains("-somethingcommon"), "Common Arg not applied!");

			MarkComplete();
		}
	}
}
