// Copyright Epic Games, Inc. All Rights Reserved.

using AutomationTool;
using System;
using System.Collections.Generic;
using System.Data;
using System.IO;
using System.Linq;
using System.Net;
using System.Threading;
using System.Drawing;
using UnrealBuildTool;
using System.Drawing.Imaging;
using ImageMagick;

namespace Gauntlet.SelfTest
{
	/// <summary>
	/// This test validates that the UnrealSession helper class brings everything together correctly
	/// </summary>
	[TestGroup("Unreal", 7)]
	class TestUnrealSession : TestUnrealBase
	{
		public override void TickTest()
		{
			AccountPool.Initialize();

			AccountPool.Instance.RegisterAccount(new EpicAccount("Foo", "Bar"));

			// Add three devices to the pool
			DevicePool.Instance.RegisterDevices(new ITargetDevice[] {
				new TargetDeviceWindows("Local PC1", Globals.TempDir)
				, new TargetDeviceWindows("Local PC2", Globals.TempDir)
			});

			// Create a new build (params come from our base class will be similar to "OrionGame" and "p:\builds\orion\branch-cl")
			UnrealBuildSource Build = new UnrealBuildSource(this.ProjectName, this.ProjectFile, this.UnrealPath, this.UsesSharedBuildType, this.BuildPath, new string[] { "" });

			// create a new options structure
			UnrealOptions Options = new UnrealOptions();

			// set the mapname, this will be applied automatically to the server
			Options.Map = "OrionEntry";
			Options.Log = true;

			// add some common options.
			string ServerArgs = " -nomcp -log";

			// We want the client to connect to the server, so get the IP address of this PC and add it to the client args as an ExecCmd
			string LocalIP = Dns.GetHostEntry(Dns.GetHostName()).AddressList.Where(o => o.AddressFamily == System.Net.Sockets.AddressFamily.InterNetwork).First().ToString();
			string ClientArgs = string.Format(" -ExecCmds=\"open {0}\"", LocalIP);

			// create a new session with client & server roles
			string PlatformString = Gauntlet.Globals.Params.ParseValue("Platform", "Win64");
			UnrealTargetPlatform Platform = UnrealTargetPlatform.Parse(PlatformString);
			UnrealSession Session = new UnrealSession(Build, new[] {
				new UnrealSessionRole(UnrealTargetRole.Client, Platform, Configuration, ClientArgs, Options)
				, new UnrealSessionRole(UnrealTargetRole.Server, UnrealTargetPlatform.Win64, Configuration, ServerArgs, Options)            
			});

			// launch an instance of this session
			UnrealSessionInstance SessionInstance = Session.LaunchSession();

			// wait for two minutes - long enough for anything to go wrong :)
			DateTime StartTime = DateTime.Now;

			while (SessionInstance.ClientsRunning && SessionInstance.ServerRunning)
			{
				if ((DateTime.Now - StartTime).TotalSeconds > 120)
				{
					break;
				}

				Thread.Sleep(1000);
			}

			// check these are both still running
			CheckResult(SessionInstance.ClientsRunning, "Clients exited during test");
			CheckResult(SessionInstance.ServerRunning, "Server exited during test");

			// shutdown the session
			SessionInstance.Shutdown();

			// shutdown the pools
			AccountPool.Shutdown();
			DevicePool.Shutdown();

			MarkComplete(TestResult.Passed);
		}
	}

	class TestUnrealSessionSaveRoleArtifacts : TestUnrealBase
	{
		// Cached references to mocked dependencies
		private UnrealSession Session;
		private UnrealSessionRole Client;
		private UnrealSessionRole Server;
		private UnrealTestContext Context;
		private UnrealSessionInstance Instance;

		// Maps a role type to the SessionRole's artifact (saved) directory
		private Dictionary<UnrealTargetRole, DirectoryInfo> SourceArtifactDirectories;

		// Cached references to files on the
		private string ClientCrashReporterFileName;
		private string ClientDummyFileName;
		private string ClientBuildFileName;
		private DirectoryInfo ClientBuildDirectory;

		private string ServerCrashReporterFileName;
		private string ServerDummyFileName;
		private string ServerBuildFileName;
		private DirectoryInfo ServerBuildDirectory;

		public TestUnrealSessionSaveRoleArtifacts()
		{
			if(Utils.SystemHelpers.IsNetworkPath(BuildPath))
			{
				throw new AutomationException("{0} expects to write temp files to the build directory. You to provide a local build path!", this);
			}
			SourceArtifactDirectories = new();
			if (Directory.Exists(Gauntlet.Globals.LogDir))
			{
				Directory.Delete(Gauntlet.Globals.LogDir, true);
			}
		}

		public override void TickTest()
		{
			MarkComplete();
		}

		public override bool StartTest(int Pass, int NumPasses)
		{
			MockDependencies();

			// To test this, we place a few dummy files in the source artifact directory
			// We then call SaveRoleArtifacts and check the destination artifact directory
			// to verify files appeared as we would expect.
			// We also verify the files in the source directory are no longer there
			// because we want them to move intead of being copied.

			try
			{
				// Generate some files in the source artifact directory
				List<FileInfo> SourceArtifacts = CreateSourceArtifacts();

				// Call SaveRoleArtifacts
				List<UnrealRoleArtifacts> Artifacts = Session.SaveRoleArtifacts(Context, Instance, Gauntlet.Globals.LogDir).ToList();

				// Ensure the source artifacts are deleted
				if (!VerifySourceFilesAreDeleted(SourceArtifacts))
				{
					Log.Error("Failed to verify SaveRoleArtifacts deleted source artifacts. See above for details.");
					return false;
				}

				// Verify files exist in the destination location
				if (!VerifyArtifactsAreInDestinationDirectory(Artifacts))
				{
					Log.Error("Failed to verify SaveRoleArtifacts moved over all expected artifact files. See above for details.");
					return false;
				}

			}
			finally
			{
				// Avoid dirtying the build folder
				FileInfo ClientBuildFile = new(Path.Combine(ClientBuildDirectory.FullName, ClientBuildFileName));
				if (ClientBuildFile.Exists)
				{
					ClientBuildFile.Delete();
				}

				FileInfo ServerBuildFile = new(Path.Combine(ServerBuildDirectory.FullName, ServerBuildFileName));
				if (ServerBuildFile.Exists)
				{
					ServerBuildFile.Delete();
				}
			}

			InnerStatus = TestStatus.Complete;
			return true;
		}

		private void MockDependencies()
		{
			UnrealTargetPlatform Platform = UnrealTargetPlatform.Win64;
			UnrealBuildSource Build = new(ProjectName, ProjectFile, UnrealPath, UsesSharedBuildType, BuildPath);
			DevicePool.Instance.SetLocalOptions(Gauntlet.Globals.TempDir, false, null);

			// Set up roles
			UnrealDeviceTargetConstraint DefaultConstraint = new UnrealDeviceTargetConstraint(UnrealTargetPlatform.Win64);
			List<EIntendedBaseCopyDirectory> DefaultAdditionalDirectories = new();
			Client = new UnrealSessionRole(UnrealTargetRole.Client, Platform, Configuration, null)
			{
				Constraint = DefaultConstraint,
				AdditionalArtifactDirectories = DefaultAdditionalDirectories
			};
			Server = new UnrealSessionRole(UnrealTargetRole.Server, Platform, Configuration, null)
			{
				Constraint = DefaultConstraint,
				AdditionalArtifactDirectories = DefaultAdditionalDirectories
			};
			List<UnrealSessionRole> Roles = new() { Client, Server };

			// TestContext
			UnrealTestRoleContext ClientContext = new();
			ClientContext.Type = Client.RoleType;
			ClientContext.Platform = Platform;
			ClientContext.Configuration = Client.Configuration;

			UnrealTestRoleContext ServerContext = new();
			ServerContext.Type = Server.RoleType;
			ServerContext.Platform = Platform;
			ServerContext.Configuration = Server.Configuration;

			Dictionary<UnrealTargetRole, UnrealTestRoleContext> RoleContexts = new()
			{
				{ Client.RoleType, ClientContext },
				{ Server.RoleType, ServerContext }
			};

			Session = new UnrealSession(Build, Roles);
			Session.TryReserveDevices();

			TargetDeviceWindows ClientDevice = (TargetDeviceWindows)Session.UnrealDeviceReservation.ReservedDevices[0];
			TargetDeviceWindows ServerDevice = (TargetDeviceWindows)Session.UnrealDeviceReservation.ReservedDevices[1];
			ClientDevice.PopulateDirectoryMappings(Path.Combine(BuildPath, "WindowsClient", ProjectName));
			ServerDevice.PopulateDirectoryMappings(Path.Combine(BuildPath, "WindowsServer", ProjectName));

			// SessionInstance
			UnrealSessionInstance.RoleInstance ClientRoleInstance = new(Client, GetAppInstanceForDevice(ClientDevice, Build.BuildName));
			UnrealSessionInstance.RoleInstance ServerRoleInstance = new(Server, GetAppInstanceForDevice(ServerDevice, Build.BuildName));
			UnrealSessionInstance.RoleInstance[] RoleInstances = { ClientRoleInstance, ServerRoleInstance };

			ClientBuildDirectory = new (ClientRoleInstance.AppInstance.Device.GetPlatformDirectoryMappings()[EIntendedBaseCopyDirectory.Build]);
			ServerBuildDirectory = new(ServerRoleInstance.AppInstance.Device.GetPlatformDirectoryMappings()[EIntendedBaseCopyDirectory.Build]);

			SourceArtifactDirectories.Add(Client.RoleType, new DirectoryInfo(ClientRoleInstance.AppInstance.ArtifactPath));
			SourceArtifactDirectories.Add(Server.RoleType, new DirectoryInfo(ServerRoleInstance.AppInstance.ArtifactPath));

			Context = new UnrealTestContext(Build, RoleContexts, new UnrealTestOptions());
			Instance = new UnrealSessionInstance(RoleInstances);
		}

		private List<FileInfo> CreateSourceArtifacts()
		{
			DirectoryInfo ClientArtifactSource = SourceArtifactDirectories[UnrealTargetRole.Client];
			DirectoryInfo ServerArtifactSource = SourceArtifactDirectories[UnrealTargetRole.Server];
			string CrashReporterPath = Path.Combine("Config", "CrashReportClient", "UECC-Windows-ABCabc123");

			// Client
			DirectoryInfo ClientCrashReporterDirectory = new(Path.Combine(ClientArtifactSource.FullName, CrashReporterPath));
			FileInfo ClientFileWithinLongCrashReporterDirectory = CreateDummyFileInDirectory(ClientCrashReporterDirectory);
			FileInfo ClientBasicFile = CreateDummyFileInDirectory(ClientArtifactSource);
			FileInfo ClientBuildFile = CreateDummyFileInDirectory(ClientBuildDirectory);
			ClientCrashReporterFileName = ClientFileWithinLongCrashReporterDirectory.Name;
			ClientDummyFileName = ClientBasicFile.Name;
			ClientBuildFileName = ClientBuildFile.Name;

			Client.AdditionalArtifactDirectories.Add(EIntendedBaseCopyDirectory.Build);

			// Create a couple images for the client - these should get converted to a .gif
			DirectoryInfo ScreenshotDirectory = new(Path.Combine(ClientArtifactSource.FullName, "Screenshots", "Windows"));
			FileInfo RedImage = CreateDummyPNGInDirectory("Red", ScreenshotDirectory);
			FileInfo BlueImage = CreateDummyPNGInDirectory("Blue", ScreenshotDirectory);

			// Server
			DirectoryInfo ServerCrashReporterDirectory = new(Path.Combine(ServerArtifactSource.FullName, CrashReporterPath));
			FileInfo ServerFileWithinLongCrashReporterDirectory = CreateDummyFileInDirectory(ServerCrashReporterDirectory);
			FileInfo ServerBasicFile = CreateDummyFileInDirectory(ServerArtifactSource);
			FileInfo ServerBuildFile = CreateDummyFileInDirectory(ServerBuildDirectory);
			ServerCrashReporterFileName = ServerFileWithinLongCrashReporterDirectory.Name;
			ServerDummyFileName = ServerBasicFile.Name;
			ServerBuildFileName = ServerBuildFile.Name;

			Server.AdditionalArtifactDirectories.Add(EIntendedBaseCopyDirectory.Build);

			return new List<FileInfo>()
			{
				// Build files created to test AdditionalArtifactDirectories should be copied and not moved, so don't include them here
				ClientFileWithinLongCrashReporterDirectory, ClientBasicFile,
				ServerFileWithinLongCrashReporterDirectory, ServerBasicFile,
				RedImage, BlueImage
			};
		}

		private bool VerifyArtifactsAreInDestinationDirectory(List<UnrealRoleArtifacts> Artifacts)
		{
			UnrealRoleArtifacts ClientArtifacts = Artifacts.Where(Role => Role.SessionRole.RoleType == UnrealTargetRole.Client).First();
			UnrealRoleArtifacts ServerArtifacts = Artifacts.Where(Role => Role.SessionRole.RoleType == UnrealTargetRole.Server).First();

			DirectoryInfo ClientDirectory = new(ClientArtifacts.ArtifactPath);
			DirectoryInfo ServerDirectory = new(ServerArtifacts.ArtifactPath);
			string TruncatedCrashReporterPath = Path.Combine("Config", "CrashReportClient", "UECC-Windows-00");

			FileInfo ClientCrashReporterFile = new(Path.Combine(ClientDirectory.FullName, TruncatedCrashReporterPath, ClientCrashReporterFileName));
			FileInfo ClientBasicFile = new(Path.Combine(ClientDirectory.FullName, ClientDummyFileName));
			FileInfo ClientBuildFile = new(Path.Combine(ClientDirectory.FullName, "Build", ClientBuildFileName));

			// We should see the .pngs were converted to .jpgs and a .gif file in the root directory
			DirectoryInfo ScreenshotDirectory = new(Path.Combine(ClientDirectory.FullName, "Screenshots", "Windows"));
			FileInfo RedImage = new(Path.Combine(ScreenshotDirectory.FullName, "Red.jpg"));
			FileInfo BlueImage = new(Path.Combine(ScreenshotDirectory.FullName, "Blue.jpg"));
			FileInfo TestGif = new(Path.Combine(ClientDirectory.FullName, "ClientTest.gif"));

			FileInfo ServerCrashReporterFile = new(Path.Combine(ServerDirectory.FullName, TruncatedCrashReporterPath, ServerCrashReporterFileName));
			FileInfo ServerBasicFile = new(Path.Combine(ServerDirectory.FullName, ServerDummyFileName));
			FileInfo ServerBuildFile = new(Path.Combine(ServerDirectory.FullName, "Build", ServerBuildFileName));

			// A log for each process should also be generated
			FileInfo ClientLog = new(Path.Combine(ClientDirectory.FullName, "ClientOutput.log"));
			FileInfo ServerLog = new(Path.Combine(ServerDirectory.FullName, "ServerOutput.log"));

			List<FileInfo> FilesToVerifyExist = new()
			{
				ClientCrashReporterFile, ClientBasicFile, ClientBuildFile, ClientLog,
				ServerCrashReporterFile, ServerBasicFile, ServerBuildFile, ServerLog,
				RedImage, BlueImage, TestGif,
			};

			bool bSucceeded = true;
			foreach (FileInfo File in FilesToVerifyExist)
			{
				if (!File.Exists)
				{
					bSucceeded = false;
					Log.Error("Unable locate file {File}", File);
				}
			}

			return bSucceeded;
		}

		private WindowsAppInstance GetAppInstanceForDevice(TargetDeviceWindows Device, string BuildName)
		{
			string ArtifactPath = Path.Combine(Device.LocalCachePath, "UserDir", "Saved");

			// This lets us mock the call to StdOut
			FileInfo Log = new(Path.Combine(ArtifactPath, "Logs", "Output.log"));
			Log.Directory.Create();
			File.WriteAllText(Log.FullName, "Foo Log");

			WindowsAppInstall Install = new WindowsAppInstall(BuildName, ProjectName, Device)
			{
				ArtifactPath = ArtifactPath
			};

			LongProcessResult DummyProcess = new("cmd", "/C echo Dummy", CommandUtils.ERunOptions.NoStdOutRedirect);
			WindowsAppInstance AppInstance = new(Install, DummyProcess, Log.FullName);

			// This is a pretty dumb workaround, but the log file reader runs on a separate thread.
			// We need to wait a few seconds to avoid accidentally moving the file in SaveRoleArtifacts
			// before the thread has had time to map the contents to the stdout
			Thread.Sleep(5000);
			return AppInstance;
		}

		private FileInfo CreateDummyFileInDirectory(DirectoryInfo Directory)
		{
			if(!Directory.Exists)
			{
				Directory.Create();
			}

			FileInfo TempFile = new FileInfo(Path.GetTempFileName());
			FileInfo DummyFile = new FileInfo(Path.Combine(Directory.FullName, TempFile.Name));
			File.WriteAllText(DummyFile.FullName, "Foo");

			return DummyFile;
		}

		private FileInfo CreateDummyPNGInDirectory(string BrushColor, DirectoryInfo Directory)
		{
			if (!Directory.Exists)
			{
				Directory.Create();
			}

			string ImagePath = Path.Combine(Directory.FullName, BrushColor + ".png");
			Color NamedColor = Color.FromName(BrushColor);
			using (MagickImage Image = new MagickImage(MagickColor.FromRgb(NamedColor.R, NamedColor.G, NamedColor.B), 16, 16))
			{
				Image.Format = MagickFormat.Png;
				Image.Write(ImagePath);
			}

			return new FileInfo(ImagePath);
		}

		bool VerifySourceFilesAreDeleted(IEnumerable<FileInfo> SourceFiles)
		{
			bool bSucceeded = true;
			foreach(FileInfo File in SourceFiles)
			{
				File.Refresh();
				if(File.Exists)
				{
					Log.Error("Source artifact file {File} should no longer exist, but was found!", File);
					bSucceeded = false;
				}
			}

			return bSucceeded;
		}
	}
}
