// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Linq;

namespace Gauntlet
{
	/// <summary>
	/// Object that allow to execute several callbacks until they return true.
	/// </summary>
	public class Checker
	{
		public Dictionary<string, Validation> Validations { get; }

		public Checker()
		{
			Validations = new();
		}

		/// <summary>
		/// Register a callback to validate
		/// </summary>
		/// <param name="ActionKey"></param>
		/// <param name="Action"></param>
		public void AddValidation(string ActionKey, Func<bool> Action)
		{
			Validations.Add(ActionKey, new Validation(ActionKey, Action));
		}

		/// <summary>
		/// Check all callbacks, return true if all passed.
		/// </summary>
		/// <returns></returns>
		public bool PerformValidations()
		{
			if (!Validations.Any())
			{
				return false; 
			}

			IEnumerable<Validation> PassedValidations = Validations.Values.Where(I => !I.IsValidated && I.Validate());

			foreach (var Item in PassedValidations)
			{
				Item.IsValidated = true;
				Log.Info($"Validated: {Item.ActionKey}");
			}

			return Validations.Values.All(I => I.IsValidated);
		}

		/// <summary>
		/// Check the specific callback, return true if it passed and false otherwise.
		/// </summary>
		/// <returns></returns>
		public bool HasValidated(string ActionKey)
		{
			return Validations.TryGetValue(ActionKey, out var Validation) && Validation.IsValidated;
		}
	}

	/// <summary>
	/// The class combining a key, a validation callback and its result
	/// </summary>
	public class Validation
	{
		public string ActionKey { get; }
		public Func<bool> Validate { get; }
		public bool IsValidated { get; set; }

		public Validation(string InActionKey, Func<bool> InValidate)
		{
			ActionKey = InActionKey;
			Validate = InValidate;
		}
	}
}
