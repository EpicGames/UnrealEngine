// Copyright Epic Games, Inc. All Rights Reserved.

using System.Collections.Generic;
using System.Text;

namespace Gauntlet
{
	/// <summary>
	/// Keeps a folder and file structure of generated content starting from the given path and 
	/// is used for comparison using <see cref="ContentFolderEqualityComparer"/>
	/// </summary>
	public class ContentFolder
	{
		public string Name { get; }
		public IList<ContentFolder> SubFolders { get; set; }
		public IList<string> Files { get; set; }

		public ContentFolder(string InName)
		{
			Name = InName;
			SubFolders = new List<ContentFolder>();
			Files = new List<string>();
		}

		/// <summary>
		/// The recursive method visualizes a folder/file tree as a string using indentation
		/// </summary>
		/// <returns></returns>
		public override string ToString()
		{
			var Sb = new StringBuilder();
			BuildFolderString(Sb, 0);
			return Sb.ToString();
		}

		private void BuildFolderString(StringBuilder Sb, int IndentLevel)
		{
			Sb.AppendLine($"{new string(' ', IndentLevel * 2)}{Name}");

			foreach (string File in Files)
			{
				Sb.AppendLine($"{new string(' ', (IndentLevel + 1) * 2)}- {File}");
			}

			foreach (ContentFolder SubFolder in SubFolders)
			{
				SubFolder.BuildFolderString(Sb, IndentLevel + 1);
			}
		}
	}
}
