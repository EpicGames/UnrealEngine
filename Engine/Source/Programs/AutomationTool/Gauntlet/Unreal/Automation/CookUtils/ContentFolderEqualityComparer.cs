// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Linq;

namespace Gauntlet
{
	/// <summary>
	/// The class defines custom equality comparison logic for <see cref="ContentFolder"/> objects.
	/// Equality refers to the occurrence of a set of expected files/folders in a set of actual files/folders.
	/// </summary>
	public sealed class ContentFolderEqualityComparer : IEqualityComparer<ContentFolder>
	{
		public bool Equals(ContentFolder x, ContentFolder y)
		{
			if (ReferenceEquals(x, y))
			{
				return true;
			}

			if (ReferenceEquals(x, null) || ReferenceEquals(y, null))
			{
				return false;
			}

			if (x.GetType() != y.GetType())
			{
				return false;
			}

			return x.Name.Equals(y.Name, StringComparison.InvariantCultureIgnoreCase) &&
			       !x.Files.Except(y.Files, StringComparer.InvariantCultureIgnoreCase).Any() &&
			       x.SubFolders.All(s => y.SubFolders.Any(subY => Equals(s, subY)));
		}

		public int GetHashCode(ContentFolder obj)
		{
			return 0; // The same hash for two objects indicates a hash collision, the Equals will be called 
		}
	}
}
