// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.IO;
using System.Linq;

namespace Gauntlet
{
	/// <summary>
	/// The class for reading folder and file structure
	/// </summary>
	public static class ContentFolderReader
	{
		/// <summary>
		/// Reads folder and file structure from a specific directory and represents it using <see cref="ContentFolder"/> class.
		/// Reading is recursive with read depth limit.
		/// </summary>
		/// <param name="Dir"></param>
		/// <param name="Depth"></param>
		/// <returns></returns>
		/// <exception cref="ArgumentException"></exception>
		public static ContentFolder ReadFolderStructure(string Dir, int Depth)
		{
			if (!Directory.Exists(Dir))
			{
				throw new ArgumentException($"The cooked content path does not exist {Dir}");
			}

			ContentFolder RootFolder = new ContentFolder(Path.GetFileName(Dir));
			PopulateFolderDetails(RootFolder, Dir, Depth);
			return RootFolder;
		}

		private static void PopulateFolderDetails(ContentFolder Folder, string Dir, int Depth)
		{
			if (Depth < 0)
			{
				return;
			}

			try
			{
				Folder.Files = Directory.GetFiles(Dir).Select(Path.GetFileName).ToList();
				Folder.SubFolders = Directory.GetDirectories(Dir).Select(D =>
				{
					ContentFolder SubFolder = new ContentFolder(Path.GetFileName(D));
					PopulateFolderDetails(SubFolder, D, Depth - 1);
					return SubFolder;
				}).ToList();
			}
			catch (Exception E)
			{
				Log.Warning(E.Message);
			}
		}
	}
}
