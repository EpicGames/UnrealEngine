// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using UnrealBuildTool;

namespace Gauntlet
{
	internal interface ICookedContentProvider
	{
		ContentFolder GetCookedContent();
	}

	internal class ContentProviderConfig
	{
		public UnrealTargetPlatform TargetPlatform { get; set; }
		public string TargetPlatformName { get; set; }
		public string ProjectName { get; set; }
		public bool IsZenStoreUsed { get; set; }
	}

	internal class BaseCookedContentProvider(ContentProviderConfig Config) : ICookedContentProvider
	{
		protected ContentProviderConfig Config = Config;

		public virtual ContentFolder GetCookedContent()
		{
			ContentFolder CookedContent = new ContentFolder(Config.TargetPlatformName)
			{
				SubFolders =
				[
					new ContentFolder("Engine")
					{
						SubFolders = new List<ContentFolder>
						{
							new("Content")
						}
					},
					new ContentFolder(Config.ProjectName)
					{
						SubFolders = new List<ContentFolder>
						{
							new("Content"),
							new("Metadata")
						},
						Files = new List<string> { "AssetRegistry.bin" }
					}
				]
			};

			if (Config.IsZenStoreUsed)
			{
				CookedContent.Files.Add("ue.projectstore");
			}
			else
			{
				foreach (ContentFolder Folder in CookedContent.SubFolders)
				{
					Folder.SubFolders.Add(new("Plugins"));
				}
			}

			return CookedContent;
		}
	}

	internal class WindowsCookedContentProvider(ContentProviderConfig Config) : BaseCookedContentProvider(Config)
	{
		public override ContentFolder GetCookedContent()
		{
			ContentFolder CookedContent = base.GetCookedContent();
			CookedContent.SubFolders[0].Files = ["GlobalShaderCache-PCD3D_SM5.bin", "GlobalShaderCache-VULKAN_SM6.bin", "GlobalShaderCache-PCD3D_SM6.bin"];

			return CookedContent;
		}
	}

	internal class LinuxCookedContentProvider(ContentProviderConfig Config) : BaseCookedContentProvider(Config)
	{
		public override ContentFolder GetCookedContent()
		{
			ContentFolder CookedContent = base.GetCookedContent();
			CookedContent.SubFolders[0].Files = ["GlobalShaderCache-VULKAN_SM6.bin"];

			return CookedContent;
		}
	}

	internal class MacCookedContentProvider(ContentProviderConfig Config) : BaseCookedContentProvider(Config)
	{
		public override ContentFolder GetCookedContent()
		{
			ContentFolder CookedContent = base.GetCookedContent();
			CookedContent.SubFolders[0].Files = ["GlobalShaderCache-METAL_SM5.bin"];

			return CookedContent;
		}
	}

	internal class CookedContentProviderFactory(ContentProviderConfig Config)
	{
		private readonly UnrealTargetPlatform TargetPlatform = Config.TargetPlatform;
		private readonly Dictionary<UnrealTargetPlatform, ICookedContentProvider> ProvidersByPlatform = new()
		{
			{ UnrealTargetPlatform.Win64, new WindowsCookedContentProvider(Config) },
			{ UnrealTargetPlatform.Linux, new LinuxCookedContentProvider(Config) },
			{ UnrealTargetPlatform.Mac, new MacCookedContentProvider(Config) }
		};

		public ICookedContentProvider GetProvider()
		{
			if (!ProvidersByPlatform.TryGetValue(TargetPlatform, out ICookedContentProvider Provider))
			{
				throw new NotSupportedException($"Unsupported platform: {TargetPlatform}");
			}

			return Provider;
		}
	}
}
