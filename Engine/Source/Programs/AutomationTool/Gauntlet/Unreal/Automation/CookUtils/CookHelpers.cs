// Copyright Epic Games, Inc. All Rights Reserved.

using System.Collections.Generic;
using System.Linq;
using static Gauntlet.UnrealSessionInstance;

namespace Gauntlet
{
	internal static class CookHelpers
	{
		internal static bool TryLaunchDeferredRole(UnrealSessionInstance TestInstance, UnrealTargetRole RoleType)
		{
			RoleInstance DeferredRole = TestInstance
				?.DeferredRoles
				?.FirstOrDefault(R => R.Role.RoleType == RoleType);

			if (DeferredRole is null || !TestInstance.LaunchDeferredRole(DeferredRole.Role))
			{
				return false;
			}

			return true;
		}

		internal static IAppInstance[] GetRunningInstances(UnrealSession UnrealApp)
		{
			return GetRunningRoles(UnrealApp).Select(R => R.AppInstance).ToArray();
		}

		internal static IAppInstance GetRunningInstance(UnrealSession UnrealApp, UnrealTargetRole RoleType)
		{
			return GetRunningRoles(UnrealApp).FirstOrDefault(R => R.Role.RoleType == RoleType)?.AppInstance;
		}

		internal static bool HaveAllInstancesExited(UnrealSession UnrealApp)
		{
			return GetAllRoles(UnrealApp).All(R => R.AppInstance.HasExited);
		}

		// Adds one role of the specified type on each call, unlike UnrealTestConfiguration.RequireRole,
		// which can return an existing role instead of adding a new one.
		internal static UnrealTestRole AddRequiredRole(UnrealTestConfiguration Config, UnrealTargetRole InRole)
		{
			if (!Config.RequiredRoles.ContainsKey(InRole))
			{
				Config.RequiredRoles[InRole] = [];
			}

			UnrealTestRole NewRole = new UnrealTestRole(InRole, null);
			Config.RequiredRoles[InRole].Add(NewRole);

			return NewRole;
		}

		private static IEnumerable<RoleInstance> GetRunningRoles(UnrealSession UnrealApp)
		{
			return GetAllRoles(UnrealApp).Where(R => !R.AppInstance.WasKilled && !R.AppInstance.HasExited);
		}

		private static IEnumerable<RoleInstance> GetAllRoles(UnrealSession UnrealApp) 
		{
			return UnrealApp.SessionInstance?.RunningRoles ?? [];
		}
	}
}
