// Copyright Epic Games, Inc. All Rights Reserved.

using AutomationTool;
using Gauntlet;
using System;
using System.Collections.Generic;
using System.IO;
using System.Linq;
using UnrealBuildTool;
using EpicGames.Core;
using Log = Gauntlet.Log;
using Gauntlet.Report;

namespace UE
{
	/// <summary>
	/// Define a Config class for UE.EditorAutomation and UE.TargetAutomation with the
	/// options that can be used
	/// </summary>
	public class AutomationTestConfig : UnrealTestConfiguration
	{
		/// <summary>
		/// Run with specific RHI
		/// </summary>
		[AutoParam]
		public string RHI = "";
		/// <summary>
		/// Valid RHI names based on Platform
		/// </summary>
		private static class ValidRHI
		{
			public enum Win64
			{
				d3d11,
				d3d12,
				vulkan
			}
			public enum Linux
			{
				vulkan
			}
			public enum Mac
			{
				metal
			}
			public enum ShaderModel
			{
				sm5,
				sm6
			}
		}

		/// <summary>
		/// Enable the RHI validation layer.
		/// </summary>
		[AutoParam]
		public virtual bool RHIValidation { get; set; } = false;

		/// <summary>
		/// Run with Nvidia cards for raytracing support
		/// </summary>
		[AutoParam]
		public bool PreferNvidia = false;

		/// <summary>
		/// Run forcing raytracing 
		/// </summary>
		[AutoParam]
		public bool RayTracing = false;

		/// <summary>
		/// Run forcing raytracing 
		/// </summary>
		[AutoParam]
		public bool StompMalloc = false;

		/// <summary>
		/// Run with d3ddebug
		/// </summary>
		[AutoParam]
		public bool D3DDebug = false;

		/// <summary>
		/// Attach to RenderDoc to capture frame traces for image based tests
		/// </summary>
		[AutoParam]
		public bool AttachRenderDoc = false;

		/// <summary>
		/// Filter or groups of tests to apply
		/// </summary>
		[AutoParam]
		public string RunTest = "";

		/// <summary>
		/// Absolute or project relative path to write an automation report to.
		/// </summary>
		[AutoParam]
		public string ReportExportPath = "";

		/// <summary>
		/// Path the report can be found at
		/// </summary>
		[AutoParam]
		public string ReportURL = "";

		/// <summary>
		/// Absolute or project relative directory path to write automation telemetry outputs.
		/// </summary>
		[AutoParam]
		public string TelemetryDirectory = "";

		/// <summary>
		/// Use Simple Horde Report instead of Unreal Automated Tests
		/// </summary>
		[AutoParam]
		public virtual bool SimpleHordeReport { get; set; } = false;

		/// <summary>
		/// Resume test run on critical failure through pass retry
		/// </summary>
		[AutoParam]
		public bool ResumeOnCriticalFailure = false;

		/// <summary>
		/// Validate DDC during tests
		/// </summary>
		[AutoParam]
		public bool VerifyDDC = false;

		/// <summary>
		/// Validate DDC during tests
		/// </summary>
		[AutoParam]
		public string DDC = "";

		/// <summary>
		/// Log Idle timeout in second
		/// </summary>
		[AutoParam]
		public virtual int LogIdleTimeout { get; set; } = 30 * 60;

		/// <summary>
		/// Enable stereo variants for image based tests
		/// </summary>
		[AutoParam]
		public bool EnableStereoTestVariants = false;

		/// <summary>
		/// Used for having the editor and any client communicate
		/// </summary>
		public string SessionID = Guid.NewGuid().ToString();
		public void GenerateSessionID() { SessionID = Guid.NewGuid().ToString(); }


		/// <summary>
		/// Implement how the settings above are applied to different roles in the session. This is the main avenue for converting options
		/// into command line parameters for the different roles
		/// </summary>
		/// <param name="AppConfig"></param>
		/// <param name="ConfigRole"></param>
		/// <param name="OtherRoles"></param>
		public override void ApplyToConfig(UnrealAppConfig AppConfig, UnrealSessionRole ConfigRole, IEnumerable<UnrealSessionRole> OtherRoles)
		{

			base.ApplyToConfig(AppConfig, ConfigRole, OtherRoles);

			// The "RunTest" argument is required since it is what is passed to the editor to specify which tests to execute
			if (string.IsNullOrEmpty(RunTest))
			{
				throw new AutomationException("No AutomationTest argument specified. Use -RunTest=\"Group:AI\", -RunTest=\"Project\", -RunTest=\"Navigation.Landscape Ramp Test\" etc.");
			}

			if (ConfigRole.RoleType.IsEditor())
			{
				if (ResumeOnCriticalFailure)
				{
					AppConfig.CommandLine += " -ResumeRunTest";
				}
				// Are we writing out info for Horde?
				if (WriteTestResultsForHorde)
				{
					if (string.IsNullOrEmpty(ReportExportPath))
					{
						ReportExportPath = Path.Combine(Globals.TempDir, "TestReport");
					}
					if (string.IsNullOrEmpty(HordeTestDataPath))
					{
						HordeTestDataPath = HordeReport.DefaultTestDataDir;
					}
					if (string.IsNullOrEmpty(HordeArtifactPath))
					{
						HordeArtifactPath = HordeReport.DefaultArtifactsDir;
					}
				}
				// Arguments for writing out the report and providing a URL where it can be viewed
				string ReportArgs = "";
				if (!string.IsNullOrEmpty(ReportExportPath))
				{
					ReportArgs = string.Format(" -ReportExportPath=\"{0}\"", ReportExportPath);
				}
				if (!string.IsNullOrEmpty(ReportURL))
				{
					ReportArgs += string.Format(" -ReportURL=\"{0}\"", ReportURL);
				}
				AppConfig.CommandLine += ReportArgs;
			}

			// Setup commandline for telemetry outputs
			if (!string.IsNullOrEmpty(PublishTelemetryTo) || !string.IsNullOrEmpty(TelemetryDirectory))
			{
				if (string.IsNullOrEmpty(TelemetryDirectory))
				{
					TelemetryDirectory = Path.Combine(Globals.TempDir, "Telemetry");
				}
				if (Directory.Exists(TelemetryDirectory))
				{
					// clean any previous data
					Directory.Delete(TelemetryDirectory, true);
				}
				AppConfig.CommandLine += string.Format("-TelemetryDirectory=\"{0}\"", TelemetryDirectory);
			}

			string AutomationTestArgument = string.Format("RunTest {0};", RunTest);

			if (!string.IsNullOrEmpty(TagFilter))
			{
				AutomationTestArgument = $"ApplyTagFilter {TagFilter};{AutomationTestArgument}";
			}

			bool bAllowZeroTestResults = Globals.Params.ParseParam("AllowZeroTestResults");
			if (bAllowZeroTestResults)
			{
				AutomationTestArgument = $"AllowZeroTestResults;{AutomationTestArgument}";
			}

			// if this is not attended then we'll quit the editor after the tests and disable any user interactions
			if (this.Attended == false)
			{
				AutomationTestArgument += "Quit;";
				AppConfig.CommandLine += " -unattended";
			}

			// Enable stereo variants for image based tests if requested
			if (EnableStereoTestVariants)
			{
				AutomationTestArgument += "EnableStereoTests;";
			}

			bool HasNoOtherRole = !OtherRoles.Any();
			// If there's only one role and it's the editor then tests are running under the editor with no target
			if (ConfigRole.RoleType.IsEditor() && HasNoOtherRole)
			{ 
				AppConfig.CommandLine += string.Format(" -ExecCmds=\"Automation {0}\"", AutomationTestArgument);		
			}
			else
			{
				// If the test isnt using the editor for both roles then pass the IP of the editor (us) to the client
				string HostIP = UnrealHelpers.GetHostIpAddress();

				if (ConfigRole.RoleType.IsClient())
				{
					// Have the client list the tests it knows about. Useful for troubleshooting discrepancies
					string ClientAutomationTestArgument = "List;";

					// Make sure the stereo test setting propagates to the client
					if (EnableStereoTestVariants)
					{
						ClientAutomationTestArgument += "EnableStereoTests;";
					}

					AppConfig.CommandLine += string.Format(" -sessionid={0} -messaging -TcpMessagingConnect={1}:6666 -ExecCmds=\"Automation {2}\"", SessionID, HostIP, ClientAutomationTestArgument);
				}
				else if (ConfigRole.RoleType.IsEditor())
				{
					AppConfig.CommandLine += string.Format(" -ExecCmds=\"Automation StartRemoteSession {0}; {1}\" -TcpMessagingListen={2}:6666 -multihome={3}", SessionID, AutomationTestArgument, HostIP, HostIP);
				}
			}

			if ((ConfigRole.RoleType.IsEditor() && HasNoOtherRole) || ConfigRole.RoleType.IsClient())
			{
				// These are flags that are required only on the main role that is going to execute the tests. ie raytracing is required only on the client or if it is an editor test.
				if (RayTracing)
				{
					AppConfig.CommandLine += " -dpcvars=r.RayTracing=1,r.Lumen.HardwareRayTracing=1";
				}
				else
				{
					AppConfig.CommandLine += " -noraytracing";
				}
				
				// Options specific to Desktop platforms
				if (ConfigRole.Platform != null && ((UnrealTargetPlatform)ConfigRole.Platform).IsInGroup(UnrealPlatformGroup.Desktop))
				{
					Type RHIType = null;
					UnrealTargetPlatform TargetPlatform = (UnrealTargetPlatform)ConfigRole.Platform;

					// Set our options by platform
					if (TargetPlatform.IsInGroup(UnrealPlatformGroup.Windows))
					{
						RHIType = typeof(ValidRHI.Win64);
						if (AttachRenderDoc && !RayTracing)
						{
							AppConfig.CommandLine += " -attachRenderDoc";
						}

						if (PreferNvidia)
						{
							AppConfig.CommandLine += " -preferNvidia";
						}

						if (D3DDebug)
						{
							AppConfig.CommandLine += " -d3ddebug";
						}

						if (StompMalloc)
						{
							AppConfig.CommandLine += " -stompmalloc";
						}
					}
					else if (TargetPlatform.IsInGroup(UnrealPlatformGroup.Apple))
					{
						RHIType = typeof(ValidRHI.Mac);
					}
					else if (TargetPlatform.IsInGroup(UnrealPlatformGroup.Linux))
					{
						RHIType = typeof(ValidRHI.Linux);
					}

					// RHI can specify both RHI and ShaderModel (SM) for Windows, Mac, and Linux platforms
					if (!string.IsNullOrEmpty(RHI))
					{
						if (RHIType == null)
						{
							throw new AutomationException(string.Format("Unknown target platform '{0}' for RHI '{1}'", TargetPlatform, RHI));
						}

						// The RHI can include the Shader Model version to go along with it and should be split out e.g d3d12-sm6
						string[] RHIArguments = RHI.Split('-');
						foreach (string Argument in RHIArguments)
						{
							if (Enum.IsDefined(RHIType, Argument.ToLower()) || Enum.IsDefined(typeof(ValidRHI.ShaderModel), Argument.ToLower()))
							{
								AppConfig.CommandLine += string.Format(" -{0}", Argument);
							}
							else
							{
								throw new AutomationException(string.Format("Unknown RHI target or Shader Model '{0}' for {1}", Argument, RHIType.Name));
							}
						}
					}
				}
			}

			if (RHIValidation)
			{
				AppConfig.CommandLine += " -rhivalidation";
			}

			// Options specific to roles running under the editor
			if (ConfigRole.RoleType.UsesEditor())
			{
				if (VerifyDDC)
				{
					AppConfig.CommandLine += " -VerifyDDC";
				}

				if (!string.IsNullOrEmpty(DDC))
				{
					AppConfig.CommandLine += string.Format(" -ddc={0}", DDC);
				}
			}

			if (!string.IsNullOrEmpty(VerboseLogCategories))
			{
				string LogCmdsString = string.Join(',',
					VerboseLogCategories
					.Split(',')
					.Select(LogName => LogName + " Verbose"));
				AppConfig.CommandLine += string.Format(" -LogCmds=\"{0}\"", LogCmdsString);
			}
		}
	}

	/// <summary>
	/// Implements a node that runs Unreal automation tests using the editor. The primary argument is "RunTest". E.g
	/// RunUnreal -test=UE.EditorAutomation -RunTest="Group:Animation"
	/// </summary>
	public class EditorAutomation : AutomationNodeBase<AutomationTestConfig>
	{
		public EditorAutomation(Gauntlet.UnrealTestContext InContext)
			: base(InContext)
		{

		}

		/// <summary>
		/// Define the configuration for this test. Most options are applied by the test config above
		/// </summary>
		/// <returns></returns>
		public override AutomationTestConfig GetConfiguration()
		{
			if (CachedConfig != null)
			{
				return CachedConfig;
			}

			AutomationTestConfig Config = base.GetConfiguration();

			// Tests in the editor only require a single role
			UnrealTestRole EditorRole = Config.RequireRole(Config.CookedEditor ? UnrealTargetRole.CookedEditor : UnrealTargetRole.Editor);
			EditorRole.CommandLineParams.AddRawCommandline("-NoWatchdog -stdout -FORCELOGFLUSH -CrashForUAT");

			return Config;
		}
	}

	/// <summary>
	/// Implements a node that runs Unreal automation tests on a target, monitored by an editor. The primary argument is "RunTest". E.g
	/// RunUnreal -test=UE.EditorAutomation -RunTest="Group:Animation"
	/// </summary>
	public class TargetAutomation : AutomationNodeBase<AutomationTestConfig>
	{
		public TargetAutomation(Gauntlet.UnrealTestContext InContext)
			: base(InContext)
		{
		}

		/// <summary>
		/// Define the configuration for this test. Most options are applied by the test config above
		/// </summary>
		/// <returns></returns>
		public override AutomationTestConfig GetConfiguration()
		{
			if (CachedConfig != null)
			{
				return CachedConfig;
			}

			AutomationTestConfig Config = base.GetConfiguration();

			// Target tests require an editor which hosts the process
			UnrealTestRole EditorRole = Config.RequireRole(UnrealTargetRole.Editor);
			EditorRole.CommandLineParams.AddRawCommandline("-NoWatchdog -stdout -FORCELOGFLUSH -CrashForUAT");

			if (Config.Attended == false)
			{
				// if this is unattended we don't need the UI ( this also alows a wider range of machines to run the test under CIS)
				EditorRole.CommandLineParams.Add("nullrhi");
			}

			// target tests also require a client
			Config.RequireRole(UnrealTargetRole.Client);
			return Config;
		}
	}

	/// <summary>
	/// Base class for automation tests. Most of the magic is in here with the Editor/Target forms simply defining the roles
	/// </summary>
	public abstract class AutomationNodeBase<TConfigClass> : UnrealTestNode<TConfigClass>
		where TConfigClass : UnrealTestConfiguration, new()
	{
		// used to track stdout from the processes 
		private UnrealLogStreamParser LogReader = null;

		private UnrealAutomatedTestPassResults TestPassResults = null;

		private DateTime LastAutomationEntryTime = DateTime.MinValue;

		protected float IdleTimeoutSec = 30 * 60;

		/// Maximum of events display per test
		protected virtual int MaxEventsDisplayPerTest { get; set; } = 10;

		public AutomationNodeBase(Gauntlet.UnrealTestContext InContext)
			: base(InContext)
		{
		}

	
		/// <summary>
		/// Override our name to include the filter we're testing
		/// </summary>
		public override string Name
		{
			get
			{
				string BaseName = base.Name;

				var Config = GetCachedConfiguration();
				if (Config is AutomationTestConfig)
				{
					var AutomationConfig = Config as AutomationTestConfig;
					if (!string.IsNullOrEmpty(AutomationConfig.RunTest))
					{
						BaseName += string.Format("(RunTest={0})", AutomationConfig.RunTest);
					}
				}

				return BaseName;
			}
		}

		/// <summary>
		/// Override the Suite by GroupName if RunTest is a Group
		/// </summary>
		public override string Suite
		{
			get
			{
				if (GroupName != null)
				{
					return GroupName;
				}

				if (GetCachedConfiguration() is AutomationTestConfig AutomationConfig)
				{
					if (!string.IsNullOrEmpty(AutomationConfig.RunTest))
					{
						// try to get a usefull identification token
						int StartPos = AutomationConfig.RunTest.LastIndexOf('.') + 1;
						string TestName = AutomationConfig.RunTest.Substring(StartPos);
						if (!string.IsNullOrEmpty(TestName))
						{
							return TestName;
						}
						// get the token before if the first one is empty
						else if (StartPos > 1)
						{
							return AutomationConfig.RunTest.Substring(0, StartPos - 1).Substring(AutomationConfig.RunTest.LastIndexOf('.', StartPos - 2) + 1);
						}
					}
				}

				return base.Suite;
			}
		}

		/// <summary>
		/// The test group name if any from the command line arguments
		/// </summary>
		protected virtual string GroupName
		{
			get
			{
				// try to get the group name from the test config by parsing the Runtest command line argument.
				if (GetCachedConfiguration() is AutomationTestConfig AutomationConfig)
				{
					if (!string.IsNullOrEmpty(AutomationConfig.RunTest) && AutomationConfig.RunTest.StartsWith("group:", StringComparison.OrdinalIgnoreCase))
					{
						int StartPos = ("group:").Length;
						return AutomationConfig.RunTest.Substring(StartPos);
					}
				}

				return null;
			}
		}

		/// <summary>
		/// Override the HordeReportTestName in case a simple report is used
		/// </summary>
		protected override string HordeReportTestName
		{
			get
			{
				return Suite;
			}
		}

		/// <summary>
		/// Override the Type
		/// </summary>
		public override string Type
		{
			get
			{
				string Base = "UE.Automation";
				var Config = GetCachedConfiguration();
				if (Config is AutomationTestConfig)
				{
					var AutomationConfig = Config as AutomationTestConfig;
					if (!string.IsNullOrEmpty(AutomationConfig.RunTest))
					{
						return string.Format("{0}({1}) {2}", Base, AutomationConfig.RunTest, Context.BuildInfo.ProjectName);
					}
				}

				return string.Format("{0}({1})", Base, Suite);
			}
		}

		/* Provide test metadata context for test reporting */
		/// <inheritdoc/>
		protected override TestHarness TestHarness => TestHarness.UEFunctionalTest;
		/// <inheritdoc/>
		protected override TestTeam TestTeam => GroupName != null ? new TestTeam(GroupName) : null;
		/// <inheritdoc/>
		protected override TestProfile TestProfile => TestProfile.Functional;
		/// <inheritdoc/>
		protected override string TestIntent => GroupName != null ? $"Run functional tests from {Context.BuildInfo.ProjectName} {GroupName} suite" : null;

		/// <summary>
		/// Called when a test is starting
		/// </summary>
		/// <param name="Pass"></param>
		/// <param name="InNumPasses"></param>
		/// <returns></returns>
		public override bool StartTest(int Pass, int InNumPasses)
		{
			LastAutomationEntryTime = DateTime.MinValue;
			LogReader = null;
			TestPassResults = null;

			if (GetCachedConfiguration() is AutomationTestConfig Config)
			{
				if ((Config.WriteTestResultsForHorde || Config.ResumeOnCriticalFailure) && string.IsNullOrEmpty(Config.ReportExportPath))
				{
					Config.ReportExportPath = Path.Combine(Globals.TempDir, "TestReport");
				}
				string ReportExportPath = Config.ReportExportPath;
				if (!string.IsNullOrEmpty(ReportExportPath))
				{
					// clean up previous run if any on StartTest (not RestartTest)
					if (Directory.Exists(ReportExportPath))
					{
						DirectoryInfo ReportDirInfo = new DirectoryInfo(ReportExportPath);
						ReportDirInfo.Delete(true);
					}
				}
				if (Config.LogIdleTimeout > 0)
				{
					IdleTimeoutSec = Config.LogIdleTimeout;
				}
			}

			return base.StartTest(Pass, InNumPasses);
		}

		public override bool RestartTest()
		{
			LastAutomationEntryTime = DateTime.MinValue;
			LogReader = null;
			TestPassResults = null;

			if (GetCachedConfiguration() is AutomationTestConfig Config)
			{
				Config.GenerateSessionID();
			}

			return base.RestartTest();
		}

		/// <summary>
		/// Override TickTest to log interesting things and make sure nothing has stalled
		/// </summary>
		public override void TickTest()
		{
			IAppInstance AppInstance = null;
			
			// If we have only one Client running without Editor
			if (TestInstance.EditorApp == null && TestInstance.ClientApps.Length == 1)
			{
				AppInstance = TestInstance.ClientApps[0];
			}
			else
			{
				// We are primarily interested in what the editor is doing
				AppInstance = TestInstance.EditorApp;
			}

			if (LogReader == null)
			{
				LogReader = new UnrealLogStreamParser(AppInstance.GetLogBufferReader());
			}
			LogReader.ReadStream();

			IEnumerable<string> ChannelEntries = LogReader.GetLogFromEditorBusyChannels();
			// Any new entries?
			if (ChannelEntries.Any())
			{
				// log new entries so people have something to look at
				ChannelEntries.ToList().ForEach(S => Log.Info(S));
				LastAutomationEntryTime = DateTime.Now;
			}
			else
			{
				// Check for timeouts
				if (LastAutomationEntryTime == DateTime.MinValue)
				{
					LastAutomationEntryTime = DateTime.Now;
				}

				double ElapsedTime = (DateTime.Now - LastAutomationEntryTime).TotalSeconds;

				// Check for timeout
				if (ElapsedTime > IdleTimeoutSec)
				{
					Log.Warning(KnownLogEvents.Gauntlet_TestEvent, "No activity observed in last {Time:0.00} minutes. Aborting test", IdleTimeoutSec / 60);
					MarkTestComplete();
					SetUnrealTestResult(TestResult.TimedOut);
				}
			}

			base.TickTest();
		}

		private UnrealAutomatedTestPassResults GetTestPassResults(UnrealLog InLog)
		{
			if (TestPassResults == null)
			{
				string AutomationReportPath = string.Empty;
				if (GetCachedConfiguration() is AutomationTestConfig Config)
				{
					AutomationReportPath = Config.ReportExportPath;
					if (!string.IsNullOrEmpty(AutomationReportPath))
					{
						string JsonReportFile = Path.Combine(AutomationReportPath, "index.json");
						if (File.Exists(JsonReportFile))
						{
							try
							{
								TestPassResults = UnrealAutomatedTestPassResults.LoadFromJson(JsonReportFile);
							}
							catch (Exception Ex)
							{
								Log.Warning(KnownLogEvents.Gauntlet_TestEvent, "Failed to load Json report. {Exception}", Ex);
							}
						}
					}
				}
				
				if(TestPassResults == null && InLog != null)
				{
					// Parse automaton info from the log then
					TestPassResults = new UnrealAutomatedTestPassResults();
					AutomationLogParser LogParser = new AutomationLogParser(InLog);
					IEnumerable<UnrealAutomatedTestResult> LogTestResults = LogParser.GetResults();
					if (LogTestResults.Any())
					{
						foreach (UnrealAutomatedTestResult LogTestResult in LogTestResults)
						{
							TestPassResults.AddTest(LogTestResult);
						}
					}
					else
					{
						foreach (UnrealLog.LogEntry Entry in LogParser.AutomationWarningsAndErrors)
						{
							switch (Entry.Level)
							{
								case UnrealLog.LogLevel.Error:
									AddTestEvent(new UnrealTestEvent(EventSeverity.Error, Entry.Message, Enumerable.Empty<string>()));
									break;
								case UnrealLog.LogLevel.Warning:
									AddTestEvent(new UnrealTestEvent(EventSeverity.Warning, Entry.Message, Enumerable.Empty<string>()));
									break;
							}
						}
					}
				}
			}

			return TestPassResults;
		}

		/// <summary>
		/// Look for critical failure during the test session and update the test states
		/// </summary>
		/// <param name="JsonTestPassResults"></param>
		private void UpdateTestStateOnCriticalFailure(UnrealAutomatedTestPassResults JsonTestPassResults)
		{
			bool HasSessionTimeout = RoleResults != null && RoleResults.Where(R => R.ProcessResult == UnrealProcessResult.TimeOut).Any();
			string HordeArtifactPath = GetCachedConfiguration().HordeArtifactPath;
			var MainRole = GetCachedConfiguration().GetMainRequiredRole();
			// Get any critical error and push it to json report and resave it.
			UnrealLog.CallstackMessage FatalError = null;
			UnrealRoleResult FatalErrorRoleResult = null;
			if (RoleResults != null)
			{
				foreach (var Result in RoleResults)
				{
					if (Result.LogSummary.FatalError != null)
					{
						FatalError = Result.LogSummary.FatalError;
						FatalErrorRoleResult = Result;
						break;
					}
				}
			}
			UnrealAutomatedTestResult LastTestInProgress = null;
			if (JsonTestPassResults.InProcess > 0)
			{
				// The test pass did not run completely
				Log.Verbose("Found in-process tests: {Count}", JsonTestPassResults.InProcess);
				LastTestInProgress = JsonTestPassResults.Tests.FirstOrDefault((T => T.State == TestStateType.InProcess));
				if (!String.IsNullOrEmpty(LastTestInProgress.TestDisplayName))
				{
					string ErrorMessage = null;
					if (HasSessionTimeout)
					{
						ErrorMessage = String.Format("Session reached timeout after {0:0.00} minutes.", MaxDuration / 60);
					}
					else if (GetTestResult() == TestResult.TimedOut)
					{
						// No log acitivity observer, but we might have still unrun tests.
						ErrorMessage = String.Format("No activity observed in last {0:0.00} minutes. Test aborted.", IdleTimeoutSec / 60);
					}
					else
					{
						if (FatalError != null)
						{
							string StartedTestFullName = GetLastStartedTestFullNameFromRoleResult(FatalErrorRoleResult);
							if (!string.IsNullOrEmpty(StartedTestFullName))
							{
								if (LastTestInProgress.FullTestPath != StartedTestFullName)
								{
									// We have to find the test that has last started. Find that test result in the json report.
									UnrealAutomatedTestResult LastStartedTestFromLog = JsonTestPassResults.Tests.FirstOrDefault((T => T.FullTestPath == StartedTestFullName));
									if (LastStartedTestFromLog == null)
									{
										Log.Warning("Failed to find the test from the test state log entry '{0}'", StartedTestFullName);
										// We revert to the standard approached as the json report seems inconsistent with the log.
										// The test in-progress from the json report will get the callstack attached.
									}
									else
									{
										// The last known running test needs to be rescheduled
										LastTestInProgress.State = TestStateType.NotRun;
										JsonTestPassResults.NotRun++;
										JsonTestPassResults.InProcess--;
										// Then the last started test according to the log gets flagged as failed
										LastTestInProgress = LastStartedTestFromLog;

										if (LastTestInProgress.State != TestStateType.Fail)
										{
											switch (LastTestInProgress.State)
											{
												case TestStateType.InProcess:
													JsonTestPassResults.InProcess--;
													break;

												case TestStateType.NotRun:
													Log.Warning("The current state from the json report for the test '{0}' is NotRun (it will be changed to Fail). Log entry says it was started. That state is inconsistent.", StartedTestFullName);
													JsonTestPassResults.NotRun--;
													break;

												case TestStateType.Success:
													JsonTestPassResults.Succeeded--;
													break;

												default:
													break;
											};
											LastTestInProgress.State = TestStateType.Fail;
											JsonTestPassResults.Failed++;
										}
									}
								}
							}
							ErrorMessage = FatalError.FormatForLog();
						}
						else
						{
							ErrorMessage = "Engine crashed. No callstack could be found in the log.";
						}
					}
					if (LastTestInProgress != null)
					{
						if (!String.IsNullOrEmpty(ErrorMessage))
						{
							LastTestInProgress.AddError(ErrorMessage, !HasSessionTimeout && FatalError != null);
						}
						if (!CanRetry() || JsonTestPassResults.NotRun == 0)
						{
							// Setting the test as fail because no retry will be done anymore.
							// The InProcess state won't be used for pass resume
							LastTestInProgress.State = TestStateType.Fail;
							if (!CanRetry())
							{
								LastTestInProgress.AddWarning(string.Format("Session reached maximum of retries({0}) to resume on critical failure!", Retries));
							}
						}
					}
					JsonTestPassResults.WriteToJson();
				}
			}
			else if (FatalError != null)
			{
				string ErrorMessage = FatalError.FormatForLog();
				string StartedTestFullName = GetLastStartedTestFullNameFromRoleResult(FatalErrorRoleResult);
				if (!string.IsNullOrEmpty(StartedTestFullName))
				{
					// We have to find the test that has last started. Find that test result in the json report.
					UnrealAutomatedTestResult LastStartedTestFromLog = JsonTestPassResults.Tests.FirstOrDefault((T => T.FullTestPath == StartedTestFullName));
					if (LastStartedTestFromLog == null)
					{
						Log.Warning("Failed to find the test from the test state log entry '{0}'", StartedTestFullName);
					}
					else
					{
						LastStartedTestFromLog.AddError(ErrorMessage, true);
						if(LastStartedTestFromLog.State == TestStateType.Success)
						{
							LastStartedTestFromLog.State = TestStateType.Fail;
							JsonTestPassResults.Succeeded--;
							JsonTestPassResults.Failed++;
						}
						if (!CanRetry())
						{
							LastStartedTestFromLog.AddWarning(string.Format("Session reached maximum of retries({0}) to resume on critical failure!", Retries));
						}
						JsonTestPassResults.WriteToJson();
					}
				}
			}
			if (JsonTestPassResults.NotRun > 0)
			{
				// The test pass did not run at all
				Log.Verbose("Found not-run tests: {Count}", JsonTestPassResults.NotRun);
				if ((GetCachedConfiguration() is AutomationTestConfig Config) && Config.ResumeOnCriticalFailure && !HasSessionTimeout)
				{
					// Reschedule test to resume from last 'in-process' test.
					if (SetToRetryIfPossible())
					{
						// Attach current artifacts to Horde output
						if (SessionArtifacts != null && !string.IsNullOrEmpty(HordeArtifactPath))
						{
							HordeReport.SimpleTestReport TempReport = new HordeReport.SimpleTestReport();
							TempReport.SetOutputArtifactPath(HordeArtifactPath);
							foreach (UnrealRoleArtifacts Artifact in SessionArtifacts)
							{
								if (string.IsNullOrEmpty(Artifact.LogPath)) continue;
								string LogName = Path.GetRelativePath(Path.GetFullPath(Context.Options.LogDir), Path.GetFullPath(Artifact.LogPath));
								if (TempReport.AttachArtifact(Artifact.LogPath, LogName) && Artifact.SessionRole.RoleType == MainRole.Type)
								{
									// Reference last run instance log
									JsonTestPassResults.Devices.Last().AppInstanceLog = Gauntlet.FileUtils.ConvertPathToUnix(Path.GetRelativePath(Globals.UnrealRootDir, Path.Combine(HordeArtifactPath, LogName)));
								}
							}
							JsonTestPassResults.WriteToJson();
						}
						// Discard the report as we are going to do another pass.
					}
					else
					{
						Log.Warning(KnownLogEvents.Gauntlet_TestEvent, "Reach maximum of retries({Count}) to resume on critical failure!", Retries);
						// Adding a note to the report about why the not-run are not going to be run
						string Message = string.Format("Session reached maximum of retries({0}) to resume on critical failure!", Retries);
						if (LastTestInProgress != null && !string.IsNullOrEmpty(LastTestInProgress.FullTestPath))
						{
							Message += string.Format(" \nLast critical failure was caught on {0}", LastTestInProgress.FullTestPath);
						}
						var NotRunTests = JsonTestPassResults.Tests.Where((T => T.State == TestStateType.NotRun));
						foreach (var Test in NotRunTests)
						{
							Test.AddInfo(Message);
						}
						JsonTestPassResults.WriteToJson();
					}
				}
				else if (HasSessionTimeout)
				{
					// Adding a note to the report about why the not-run are not going to be run
					string Message = string.Format("Session reached timeout after {0} seconds.", MaxDuration);
					var NotRunTests = JsonTestPassResults.Tests.Where((T => T.State == TestStateType.NotRun));
					foreach (var Test in NotRunTests)
					{
						Test.AddInfo(Message);
					}
					JsonTestPassResults.WriteToJson();
				}
			}
		}

		/// <summary>
		/// Get last started test full name from Unreal Role result
		/// </summary>
		/// <param name="RoleResult"></param>
		/// <returns></returns>
		private string GetLastStartedTestFullNameFromRoleResult(UnrealRoleResult RoleResult)
		{
			var LogEntry = RoleResult == null ? null : RoleResult.LogSummary.LogEntries.LastOrDefault(Entry => Entry.Category == "AutomationTestStateTrace", null);

			if (LogEntry != null)
			{
				string LogEntryPrefix = "Test is about to start. Name={";
				string LogEntrySuffix = "}";
				if (LogEntry.Message.StartsWith(LogEntryPrefix) && LogEntry.Message.EndsWith(LogEntrySuffix))
				{
					string StartedTestFullName = LogEntry.Message.Substring(
						LogEntryPrefix.Length,
						LogEntry.Message.Length - LogEntryPrefix.Length - LogEntrySuffix.Length);

					return StartedTestFullName;
				}
			}

			return null;
		}

		/// <summary>
		/// Override GetExitCodeAndReason to provide additional checking of success / failure based on what occurred
		/// </summary>
		/// <param name="InReason"></param>
		/// <param name="InLog"></param>
		/// <param name="InArtifacts"></param>
		/// <param name="ExitReason"></param>
		/// <param name="ExitCode"></param>
		/// <returns></returns>
		protected override UnrealProcessResult GetExitCodeAndReason(StopReason InReason, UnrealLog InLog, UnrealRoleArtifacts InArtifacts, out string ExitReason, out int ExitCode)
		{
			UnrealProcessResult UnrealResult = base.GetExitCodeAndReason(InReason, InLog, InArtifacts, out ExitReason, out ExitCode);

			// The editor is an additional arbiter of success
			if (InArtifacts.SessionRole.RoleType.IsEditor()
				&& InLog.HasAbnormalExit == false)
			{
				// if no fatal errors, check test results
				if (InLog.FatalError == null)
				{
					var TestResults = GetTestPassResults(InLog);

					// Tests failed so list that as our primary cause of failure
					if (TestResults != null && TestResults.Failed > 0)
					{
						ExitReason = string.Format("{0} of {1} test(s) failed", TestResults.Failed, TestResults.Tests.Count);
						ExitCode = -1;
						return UnrealProcessResult.EngineTestError;
					}

					// If no tests were run then that's a failure (possibly a bad RunTest argument?)
					if (TestResults == null || TestResults.Tests.Count == 0)
					{
						bool bAllowZeroTestResults = Globals.Params.ParseParam("AllowZeroTestResults");
						if (bAllowZeroTestResults)
						{
							ExitReason = "No tests were executed, but AllowZeroTestResults is set, so treating this as a success.";
							ExitCode = 0;
							return UnrealProcessResult.ExitOk;
						}
						else
						{
							ExitReason = "No tests were executed!";
							ExitCode = -1;
							return UnrealProcessResult.InitializationFailure;
						}
					}
				}
			}

			return UnrealResult;
		}

		public override IEnumerable<UnrealRoleArtifacts> SaveRoleArtifacts(string OutputPath)
		{
			// Clean Automation/Tmp and Screenshots folders before copy to avoid duplication of screenshot comparison files
			foreach (UnrealSessionInstance.RoleInstance RoleInstance in TestInstance.RunningRoles)
			{
				DirectoryInfo SrcDirectory = new DirectoryInfo(RoleInstance.AppInstance.ArtifactPath);
				if (SrcDirectory.Exists)
				{
					foreach (DirectoryInfo Dir in SrcDirectory.GetDirectories())
					{
						if (Dir.Name.Equals("Screenshots", StringComparison.OrdinalIgnoreCase))
						{
							try { Dir.Delete(true); } catch { Log.Verbose("Could not delete directory: {Dir}", Dir.FullName); }
						}
						else if (Dir.Name.Equals("Automation", StringComparison.OrdinalIgnoreCase))
						{
							Dir.GetDirectories("Tmp", SearchOption.TopDirectoryOnly).ToList().ForEach(T => { try { T.Delete(true); } catch { Log.Verbose("Could not delete directory: {Dir}", T.FullName); } });
						}
					}
				}
			}

			return base.SaveRoleArtifacts(OutputPath);
		}

		/// <summary>
		/// Optional function that is called on test completion and gives an opportunity to create a report
		/// </summary>
		/// <param name="Result"></param>
		public override ITestReport CreateReport(TestResult Result)
		{
			ITestReport Report = null;
			if (GetCachedConfiguration() is AutomationTestConfig Config)
			{
				// Parse the json test pass results and look for critical failure to add to report
				var TestPassResults = GetTestPassResults(null);
				if (TestPassResults != null)
				{
					UpdateTestStateOnCriticalFailure(TestPassResults);
				}
				// Save test result data for Horde build system
				bool WriteTestResultsForHorde = Config.WriteTestResultsForHorde;
				if (WriteTestResultsForHorde)
				{
					if (Config.SimpleHordeReport)
					{
						Report = base.CreateReport(Result);
					}
					else
					{
						string ReportPath = Config.ReportExportPath;
						if (!string.IsNullOrEmpty(ReportPath))
						{
							Report = CreateUnrealEngineTestPassReport(ReportPath, Config.ReportURL);
							if (Report != null)
							{
								var MainRolePlatform = Context.GetRoleContext(Config.GetMainRequiredRole().Type).Platform;
								Report.SetMetadata("RHI", string.IsNullOrEmpty(Config.RHI) || !MainRolePlatform.IsInGroup(UnrealPlatformGroup.Desktop) ? "default" : Config.RHI.ToLower());
							}
						}
					}
				}

				string TelemetryDirectory = Config.TelemetryDirectory;
				if (Report != null && !string.IsNullOrEmpty(TelemetryDirectory) && Directory.Exists(TelemetryDirectory))
				{
					if (Report is ITelemetryReport Telemetry)
					{
						UnrealAutomationTelemetry.LoadOutputsIntoReport(TelemetryDirectory, Telemetry);
					}
					else
					{
						Log.Warning("Publishing Telemetry is requested but '{Config}' does not support telemetry input.", Report.GetType().FullName);
					}
				}
			}

			return Report;
		}

		/// <summary>
		/// Override the summary report so we can create a custom summary with info about our tests and
		/// a link to the reports
		/// </summary>
		/// <returns></returns>
		protected override void LogTestSummaryHeader()
		{
			base.LogTestSummaryHeader();

			bool bAllowZeroTestResults = Globals.Params.ParseParam("AllowZeroTestResults");

			// Everything we need is in the editor artifacts
			var EditorRole = RoleResults.Where(R => R.Artifacts.SessionRole.RoleType.IsEditor()).FirstOrDefault();

			if (EditorRole != null)
			{
				UnrealAutomatedTestPassResults JsonTestPassResults = GetTestPassResults(EditorRole.LogSummary);

				// Filter our tests into categories
				List<UnrealAutomatedTestResult> AllTests = JsonTestPassResults.Tests.Where(T => !T.WasSkipped).ToList();
				List<UnrealAutomatedTestResult> IncompleteTests = AllTests.Where(T => !T.IsComplete).ToList();
				List<UnrealAutomatedTestResult> FailedTests = AllTests.Where(T => T.IsComplete && T.HasFailed).ToList();
				List<UnrealAutomatedTestResult> TestsWithWarnings = AllTests.Where(T => T.HasSucceeded && T.HasWarnings).ToList();

				Action<IEnumerable<object>> NotifyMoreIfNeeded = E =>
				{
					int Count = E.Count();
					if (Count > MaxEventsDisplayPerTest)
					{
						Log.Info(" (and {Count} more)", Count - MaxEventsDisplayPerTest);
					}
				};
				// If there were abnormal exits then look only at the failed and incomplete tests only to avoid confusing things.
				if (GetRolesThatExitedAbnormally().Any())
				{
					if (AllTests.Count == 0)
					{
						if (bAllowZeroTestResults)
						{
							Log.Warning(KnownLogEvents.Gauntlet_UnrealEngineTestEvent, " * No tests were executed, check the test filter.");
							Log.Info("");
						}
						else
						{
							Log.Error(KnownLogEvents.Gauntlet_UnrealEngineTestEvent, " * No tests were executed.");
							Log.Info("");
						}
					}
					else if (FailedTests.Count > 0 || IncompleteTests.Count > 0)
					{
						var Failures = FailedTests.Concat(IncompleteTests.Where(T => T.State == TestStateType.InProcess));
						if (Failures.Any())
						{
							Log.Info(" ### The following Engine test(s) were incomplete or failed:");
							foreach (UnrealAutomatedTestResult Result in Failures)
							{
								string Message = !Result.IsComplete ? " * Test '{Name}' did not complete." : " * Test '{Name}' failed.";
								Log.Error(KnownLogEvents.Gauntlet_UnrealEngineTestEvent, Message, Result.FullTestPath);
								var Errors = Result.GetErrorEntries().Distinct().Select(E => E.AsLogEvent());
								foreach (LogEvent Error in Errors.Take(MaxEventsDisplayPerTest))
								{
									Log.Error(Error.Id, "    " + (Error.Format ?? Error.Message), Args: Error.Properties?.Select(P => P.Value).ToArray());
								}
								NotifyMoreIfNeeded(Errors);
								var Warnings = Result.WarningEvents.Distinct();
								foreach (var Warning in Warnings)
								{
									Log.Warning(KnownLogEvents.Gauntlet_UnrealEngineTestEvent, "    " + Warning.FormatToString());
								}
								NotifyMoreIfNeeded(Result.WarningEvents);
								Log.Info("");
							}
						}
						List<UnrealAutomatedTestResult> NotRun = IncompleteTests.Where(T => T.State == TestStateType.NotRun).ToList();
						if (NotRun.Any())
						{
							int TotalNotRun = NotRun.Count;
							Log.Info(" ### The following Engine test(s) were not run:");
							Log.Info(string.Join("\n", NotRun.Take(10).Select(T => " * "+T.FullTestPath)));
							if (TotalNotRun > 10)
							{
								Log.Info(" (and {Count} more)", TotalNotRun - 10);
							}
							Log.Info("");
						}
					}
				}
				else
				{
					if (AllTests.Count == 0)
					{
						if (bAllowZeroTestResults)
						{
							Log.Warning(KnownLogEvents.Gauntlet_UnrealEngineTestEvent, " * No tests were executed - confirm with test filter.");
							Gauntlet.Horde.UpdateStepMessage(Gauntlet.Horde.GauntletHordeStepMessage.ZeroTestsRun);
						}
						else
						{
							Log.Error(KnownLogEvents.Gauntlet_UnrealEngineTestEvent, " * No tests were executed.");
						}

						IEnumerable<UnrealTestEvent> Errors = TestNodeEvents.Where(E => E.IsError).Distinct();
						IEnumerable<UnrealTestEvent> Warnings = TestNodeEvents.Where(E => E.IsWarning).Distinct();
						if (Errors.Any() || Warnings.Any())
						{
							Log.Info("   See log above for details.");
							foreach (var Error in Errors.Take(MaxEventsDisplayPerTest))
							{
								if (bAllowZeroTestResults)
								{
									Log.Warning(KnownLogEvents.Gauntlet_UnrealEngineTestEvent, "    " + Error.Summary);
								}
								else
								{
									Log.Error(KnownLogEvents.Gauntlet_UnrealEngineTestEvent, "    " + Error.Summary);
								}
							}
							NotifyMoreIfNeeded(Errors);
							foreach (var Warning in Warnings.Take(MaxEventsDisplayPerTest))
							{
								Log.Warning(KnownLogEvents.Gauntlet_UnrealEngineTestEvent, "    " + Warning.Summary);
							}
							NotifyMoreIfNeeded(Warnings);
							Log.Info("");
						}
						else
						{
							Log.Info("   Unknown failure.\n");
						}
					}
					else
					{
						// Now list the tests that failed
						if (FailedTests.Count > 0)
						{
							Log.Info(" ### The following Engine test(s) failed:");

							foreach (UnrealAutomatedTestResult Result in FailedTests)
							{
								Log.Error(KnownLogEvents.Gauntlet_UnrealEngineTestEvent, " * Test '{Name}' failed.", Result.FullTestPath);
								var Events = Result.GetErrorEntries().Distinct().Select(E => E.AsLogEvent());
								foreach (LogEvent Error in Events.Take(MaxEventsDisplayPerTest))
								{
									Log.Error(Error.Id, "    " + (Error.Format ?? Error.Message), Args: Error.Properties?.Select(P => P.Value).ToArray());
								}
								NotifyMoreIfNeeded(Events);
								Log.Info("");
							}
						}

						if (TestsWithWarnings.Count > 0)
						{
							Log.Info(" ### The following Engine test(s) completed with warnings:");

							foreach (UnrealAutomatedTestResult Result in TestsWithWarnings)
							{
								Log.Warning(KnownLogEvents.Gauntlet_UnrealEngineTestEvent, " * Test '{Name}' completed with warnings.", Result.FullTestPath);
								// only show the first N items
								IEnumerable<UnrealAutomationEvent> WarningEvents = Result.WarningEvents.Distinct();
								foreach (var Event in WarningEvents.Take(MaxEventsDisplayPerTest))
								{
									Log.Warning(KnownLogEvents.Gauntlet_UnrealEngineTestEvent, "    " + Event.FormatToString());
								}
								NotifyMoreIfNeeded(WarningEvents);
								Log.Info("");
							}
						}

						if (IncompleteTests.Count > 0)
						{
							Log.Info(" ### The following Engine test(s) timed out or did not run:");

							foreach (UnrealAutomatedTestResult Result in IncompleteTests)
							{
								Log.Error(KnownLogEvents.Gauntlet_UnrealEngineTestEvent, " * Test '{Name}' did not run or complete.", Result.FullTestPath);
								// only show the first N items
								IEnumerable<UnrealAutomationEvent> ErrorAndWarningEvents = Result.WarningAndErrorEvents.Distinct();
								foreach (var Event in ErrorAndWarningEvents.Take(MaxEventsDisplayPerTest))
								{
									Log.Error(KnownLogEvents.Gauntlet_UnrealEngineTestEvent, "    " + Event.FormatToString());
								}
								NotifyMoreIfNeeded(ErrorAndWarningEvents);
							}
							Log.Info("");
						}

						// show a brief summary at the end where it's most visible
						List<string> TestSummary = new List<string>();

						int PassedTests = AllTests.Count - (FailedTests.Count + IncompleteTests.Count);
						int TestsPassedWithoutWarnings = PassedTests - TestsWithWarnings.Count;

						TestSummary.Add(string.Format(" * {0} Test(s) Requested", AllTests.Count));

						// Print out a summary of each category of result
						if (TestsPassedWithoutWarnings > 0)
						{
							TestSummary.Add(string.Format(" * {0} Test(s) Passed", TestsPassedWithoutWarnings));
						}

						if (TestsWithWarnings.Count > 0)
						{
							TestSummary.Add(string.Format(" * {0} Test(s) Passed with warnings", TestsWithWarnings.Count));
						}

						if (FailedTests.Count > 0)
						{
							TestSummary.Add(string.Format(" * {0} Test(s) Failed", FailedTests.Count));
						}

						if (IncompleteTests.Count > 0)
						{
							TestSummary.Add(string.Format(" * {0} Test(s) didn't complete", IncompleteTests.Count));
						}

						Log.Info(" ### Summary");
						Log.Info(string.Join("\n", TestSummary));
						Log.Info("");
					}
				}

				if (EditorRole.LogSummary.EngineInitialized)
				{
					string AutomationReportPath = string.Empty;
					string AutomationReportURL = string.Empty;
					if (GetCachedConfiguration() is AutomationTestConfig Config)
					{
						AutomationReportPath = Config.ReportExportPath;
						AutomationReportURL = Config.ReportURL;
					}
					if (!string.IsNullOrEmpty(AutomationReportPath) || !string.IsNullOrEmpty(AutomationReportURL))
					{
						Log.Info(" ### Links");

						if (string.IsNullOrEmpty(AutomationReportURL) == false)
						{
							Log.Info("  View results here: {URL}", AutomationReportURL);
						}

						if (string.IsNullOrEmpty(AutomationReportPath) == false)
						{
							Log.Info("  Open results in UnrealEd from {Path}", AutomationReportPath);
						}
						Log.Info("");
					}
				}
			}
		}

		/// <summary>
		/// Returns Errors found during tests. We call the base version to get standard errors then
		/// Add on any errors seen in tests
		/// </summary>
		public override IEnumerable<string> GetErrors()
		{
			List<string> AllErrors = new List<string>(base.GetErrors());

			foreach (var Role in GetRolesThatFailed())
			{
				if (Role.Artifacts.SessionRole.RoleType.IsEditor())
				{
					AutomationLogParser Parser = new AutomationLogParser(Role.LogSummary);
					AllErrors.AddRange(
						Parser.GetResults().Where(R => R.HasFailed)
							.SelectMany(R => R.Entries
								.Where(E => E.Event.Type == EventType.Error)
								.Distinct().Select(E => string.Format("[test={0}] {1}", R.TestDisplayName, E))
							)
						);
				}
			}

			return AllErrors;
		}

		/// <summary>
		/// Returns warnings found during tests. We call the base version to get standard warnings then
		/// Add on any errors seen in tests
		/// </summary>
		public override IEnumerable<string> GetWarnings()
		{
			List<string> AllWarnings = new List<string>(base.GetWarnings());

			if (SessionArtifacts == null)
			{
				return AllWarnings;
			}

			foreach (var Role in RoleResults)
			{
				if (Role.Artifacts.SessionRole.RoleType == UnrealTargetRole.Editor)
				{
					AutomationLogParser Parser = new AutomationLogParser(Role.LogSummary);
					AllWarnings.AddRange(
						Parser.GetResults()
							.SelectMany(R => R.Entries
								.Where(E => E.Event.Type == EventType.Warning)
								.Distinct().Select(E => string.Format("[test={0}] {1}", R.TestDisplayName, E))
							)
						);
				}
			}

			return AllWarnings;
		}
	}

}
