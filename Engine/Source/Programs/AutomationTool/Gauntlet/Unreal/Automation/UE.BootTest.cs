// Copyright Epic Games, Inc. All Rights Reserved.

using AutomationTool;
using EpicGame;
using Gauntlet;
using System;
using System.Collections.Generic;
using System.IO;
using System.Linq;
using System.Text.RegularExpressions;

namespace UE
{
	/// <summary>
	/// Test that waits for the client and server to get to the front-end then quits
	/// </summary>
	public class BootTest : UnrealTestNode<UnrealTestConfiguration>
	{
		/// <summary>
		/// Used to track progress via logging
		/// </summary>
		UnrealLogStreamParser LogReader = null;

		/// <summary>
		/// Time we last saw a change in logging
		/// </summary>
		DateTime LastLogTime = DateTime.Now;

		/// <summary>
		/// Set to true once we detect the game has launched correctly
		/// </summary>
		bool DidDetectLaunch = false;

		/// <summary>
		/// Log idle timeout in seconds
		/// </summary>
		private float kTimeOutDuration = 10 * 60;

		/// <summary>
		/// Default constructor
		/// </summary>
		/// <param name="InContext"></param>
		public BootTest(Gauntlet.UnrealTestContext InContext)
			: base(InContext)
		{
		}

		protected override string HordeReportTestKey => $"UE.BootTest({Suite})";
		protected override string HordeReportTestName => "BootTest";

		/// <summary>
		/// Returns the configuration description for this test
		/// </summary>
		/// <returns></returns>
		public override UnrealTestConfiguration GetConfiguration()
		{
			UnrealTestConfiguration Config = base.GetConfiguration();

			UnrealTestRole Client = Config.RequireRole(UnrealTargetRole.Client);
			if (string.IsNullOrEmpty(GetCompletionString()))
			{
				Client.CommandLineParams.Add("ExecCmds", "Automation SoftQuit");
			}

			float logIdleTimeout = Globals.Params.ParseValue("LogIdleTimeout", kTimeOutDuration);
			if (logIdleTimeout > 0)
			{
				kTimeOutDuration = logIdleTimeout;
			}
			return Config;
		}

		/// <summary>
		/// Called to begin the test.
		/// </summary>
		/// <param name="Pass"></param>
		/// <param name="InNumPasses"></param>
		/// <returns></returns>
		public override bool StartTest(int Pass, int InNumPasses)
		{
			// Call the base class to actually start the test running
			if (!base.StartTest(Pass, InNumPasses))
			{
				return false;
			}

			// track our starting condition
			LastLogTime = DateTime.Now;
			LogReader = null;
			DidDetectLaunch = false;

			return true;
		}

		/// <summary>
		/// String that we search for to be considered "Booted"
		/// </summary>
		/// <returns></returns>
		protected virtual string GetCompletionString()
		{
			// Intentionally setup with no completion string as UnrealTestNode verified initialization string already
			return null;
		}

		/// <summary>
		/// Called periodically while the test is running to allow code to monitor health.
		/// </summary>
		public override void TickTest()
		{
			// run the base class tick;
			base.TickTest();

			// Get the log of the first client app
			IAppInstance RunningInstance = this.TestInstance.RunningRoles.First().AppInstance;

			if (LogReader == null)
			{
				LogReader = new UnrealLogStreamParser(RunningInstance.GetLogBufferReader());
			}
			LogReader.ReadStream();

			IEnumerable<string> BusyLogLines = LogReader.GetLogFromEditorBusyChannels();
			if (BusyLogLines.Any())
			{
				LastLogTime = DateTime.Now;
				// log new entries so people have something to look at
				BusyLogLines.ToList().ForEach(S => Log.Info("{0}", S));
			}

			// Gauntlet will timeout tests based on the -timeout argument, but we have greater insight here so can bail earlier to save
			// tests from idling on the farm needlessly.
			if ((DateTime.Now - LastLogTime).TotalSeconds > kTimeOutDuration)
			{
				ReportError("No logfile activity observed in last {Time:0.00} minutes. Ending test", kTimeOutDuration / 60);
				MarkTestComplete();
				SetUnrealTestResult(TestResult.TimedOut);
			}

			string CompletionString = GetCompletionString();
			if (!string.IsNullOrEmpty(CompletionString))
			{
				if (LogReader.GetLogLinesContaining(CompletionString).Any())
				{
					Log.Info("Found '{0}'. Ending Test", GetCompletionString());
					MarkTestComplete();
					DidDetectLaunch = true;
					SetUnrealTestResult(TestResult.Passed);
				}
			}
		}

		/// <summary>
		/// Called after a test finishes to create an overall summary based on looking at the artifacts
		/// </summary>
		/// <param name="Result"></param>
		/// <param name="Context"></param>
		/// <returns>ITestReport</returns>
		/// <param name="Build"></param>
		/// <param name="InResults"></param>
		/// <param name="InArtifactPath"></param>
		public override ITestReport CreateReport(TestResult Result, UnrealTestContext Context, UnrealBuildSource Build, IEnumerable<UnrealRoleResult> InResults, string InArtifactPath)
		{
			if (Result == TestResult.Passed)
			{
				if (!string.IsNullOrEmpty(GetCompletionString()) && !DidDetectLaunch)
				{
					Gauntlet.Horde.UpdateStepMessage(Gauntlet.Horde.GauntletHordeStepMessage.FailedToStart);
					ReportError("Failed to detect completion of launch");
				}
				else
				{
					// find a logfile or something that indicates the process ran successsfully
					bool MissingLogs = false;

					foreach (var RoleResult in InResults)
					{
						if (!File.Exists(RoleResult.Artifacts.LogPath))
						{
							MissingLogs = true;
							Gauntlet.Horde.UpdateStepMessage(Gauntlet.Horde.GauntletHordeStepMessage.MissingArtifacts);
							ReportError("No log files found for {0}. Were they not retrieved from the device?", RoleResult.Artifacts.SessionRole);
						}
					}

					if (!MissingLogs)
					{
						Log.Info("Found valid log artifacts for test");
					}
				}
			}

			return base.CreateReport(GetTestResult());
		}
	}

	/// <summary>
	/// Test that verifies the editor boots
	/// </summary>
	public class EditorBootTest : BootTest
	{
		public EditorBootTest(Gauntlet.UnrealTestContext InContext)
			: base(InContext)
		{
		}

		/// <summary>
		/// Returns the configuration description for this test
		/// </summary>
		/// <returns></returns>
		public override UnrealTestConfiguration GetConfiguration()
		{
			UnrealTestConfiguration Config = base.GetConfiguration();
			// currently needed as BootTest isn't an abstract class. Can be changed for 4.27
			Config.ClearRoles();
			UnrealTestRole EditorRole = Config.RequireRole(Config.CookedEditor ? UnrealTargetRole.CookedEditor : UnrealTargetRole.Editor);
			EditorRole.CommandLineParams.Add("execcmds", "QUIT_EDITOR");
			return Config;
		}

		protected override string GetCompletionString()
		{
			return null;
		}
	}

	/// <summary>
	/// Test that verifies a target boots
	/// </summary>
	public class TargetBootTest : BootTest
	{
		public TargetBootTest(Gauntlet.UnrealTestContext InContext)
			: base(InContext)
		{
		}

		/// <summary>
		/// Returns the configuration description for this test
		/// </summary>
		/// <returns></returns>
		public override UnrealTestConfiguration GetConfiguration()
		{
			UnrealTestConfiguration Config = base.GetConfiguration();
			Config.RequireRole(UnrealTargetRole.Client);
			return Config;
		}
	}
}

// Provided for backwards compatibility with scripts
namespace Gauntlet.UnrealTest
{
	class BootTest : UE.BootTest
	{
		/// <summary>
		/// Default constructor
		/// </summary>
		/// <param name="InContext"></param>
		public BootTest(Gauntlet.UnrealTestContext InContext)
			: base(InContext)
		{
			Log.Warning("Gauntlet.UnrealTest.BootTest is deprecated and will be removed in 4.27. Use UE.BootTest. All arguments are the same");
		}
	}
}
