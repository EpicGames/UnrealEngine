// Copyright Epic Games, Inc. All Rights Reserved.

using System.IO;
using Gauntlet;

namespace UE
{
	public class ColdCookByTheBook : CookByTheBook
	{
		private readonly string DataCacheFolderPath;

		public ColdCookByTheBook(UnrealTestContext InContext) : base(InContext)
		{
			DataCacheFolderPath = Path.Combine(Context.Options.TempDir, "LocalDataCache");
			RecreateDataCacheFolder();

			BaseEditorCommandLine += @$" -SharedDataCachePath=None -LocalDataCachePath=""{DataCacheFolderPath}"" -ddc=cold";
		}

		public override UnrealTestConfiguration GetConfiguration()
		{
			UnrealTestConfiguration Config = base.GetConfiguration();
			Config.MaxDuration = 60 * 60;

			return Config;
		}

		public override void StopTest(StopReason InReason)
		{
			DeleteDataCacheFolder();
			base.StopTest(InReason);
		}

		private void RecreateDataCacheFolder()
		{
			if (Directory.Exists(DataCacheFolderPath))
			{
				DeleteDataCacheFolder();
			}

			Directory.CreateDirectory(DataCacheFolderPath);
		}

		private void DeleteDataCacheFolder()
		{
			if (!Directory.Exists(DataCacheFolderPath))
			{
				return;
			}

			Directory.Delete(DataCacheFolderPath, true);
		}
	}
}
