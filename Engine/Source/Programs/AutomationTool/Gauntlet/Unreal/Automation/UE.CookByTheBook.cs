// Copyright Epic Games, Inc. All Rights Reserved.

using Gauntlet;
using System.Linq;

namespace UE
{
	/// <summary>
	/// Implements the functionality of testing the cooking process with the launch of a client
	/// </summary>
	public class CookByTheBook : CookByTheBookEditor
	{
		private const string ClientStartedPattern = "Starting Game";

		private UnrealLogStreamParser ClientLogParser;

		public CookByTheBook(UnrealTestContext InContext) : base(InContext)
		{
			// Do nothing
		}

		protected override void InitTest()
		{
			base.InitTest();

			ClientLogParser = new UnrealLogStreamParser();

			Checker.AddValidation("The client is started", IsClientStarted);
		}

		public override UnrealTestConfiguration GetConfiguration()
		{
			UnrealTestConfiguration Config = base.GetConfiguration();

			SetClientRole(Config);

			return Config;
		}

		public override void TickTest()
		{
			ReadClientLogStream();

			base.TickTest();
		}

		protected override UnrealProcessResult GetExitCodeAndReason(StopReason InReason, UnrealLog InLog, UnrealRoleArtifacts InArtifacts,
			out string ExitReason, out int ExitCode)
		{
			if (InArtifacts.SessionRole.RoleType == UnrealTargetRole.Client)
			{
				InLog.EngineInitializedPattern = ClientStartedPattern;
			}

			return base.GetExitCodeAndReason(InReason, InLog, InArtifacts, out ExitReason, out ExitCode);
		}

		private static void SetClientRole(UnrealTestConfiguration Config)
		{
			UnrealTestRole ClientRole = Config.RequireRole(UnrealTargetRole.Client);
			ClientRole.DeferredLaunch = true;

			ClientRole.CommandLine += @" -log -ExecCmds=""Automation SoftQuit""";
		}

		private void ReadClientLogStream()
		{
			IAppInstance ClientApp = CookHelpers.GetRunningInstance(UnrealApp, UnrealTargetRole.Client);

			if (ClientApp is null)
			{
				return;
			}

			ClientLogParser.ReadStream(ClientApp.GetLogBufferReader());
		}

		protected override bool IsCookingComplete()
		{
			bool IsCookingComplete = base.IsCookingComplete();

			if (IsCookingComplete)
			{
				if (!CookHelpers.TryLaunchDeferredRole(UnrealApp.SessionInstance, UnrealTargetRole.Client))
				{
					ReportError($"Couldn't launch the deferred role {UnrealTargetRole.Client}");
					CompleteTest(TestResult.Failed);
				}
			}

			return IsCookingComplete;
		}

		private bool IsClientStarted()
		{
			return ClientLogParser.GetLogLinesContaining(ClientStartedPattern).Any();
		}
	}
}