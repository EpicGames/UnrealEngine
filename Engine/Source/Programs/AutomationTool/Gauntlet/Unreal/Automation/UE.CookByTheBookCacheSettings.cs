// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.IO;
using Gauntlet;

namespace UE
{
	public class CookByTheBookCacheSettings : CookByTheBook
	{
		private const string SettingsFileName = "DefaultEditorPerProjectUserSettings.ini";
		private string SettingsFileContent;
		private string SettingsFilePath;
		private string Setting;

		public CookByTheBookCacheSettings(UnrealTestContext InContext) : base(InContext)
		{
			SetupDataCacheSetting();
		}

		public override void StopTest(StopReason InReason)
		{
			ReverseDataCacheSetting();
			base.StopTest(InReason);
		}

		private void SetupDataCacheSetting()
		{
			string ProjectPath = GetProjectPath();

			Setting = $"[CookPlatformDataCacheSettings]{Environment.NewLine}Texture2D = 1";
			SettingsFilePath = Path.Combine(ProjectPath, "Config", SettingsFileName);
			SettingsFileContent = File.ReadAllText(SettingsFilePath);

			if (DoesOriginalFileContainSetting())
			{
				return;
			}

			RemoveReadOnlyFileAttribute();
			File.AppendAllText(SettingsFilePath, $"{Environment.NewLine}{Setting}");
		}

		private void ReverseDataCacheSetting()
		{
			if (DoesOriginalFileContainSetting())
			{
				return;
			}

			File.WriteAllText(SettingsFilePath, SettingsFileContent);
			SetReadOnlyFileAttribute();
		}

		private void RemoveReadOnlyFileAttribute()
		{
			FileAttributes EditedAttributes = File.GetAttributes(SettingsFilePath) & ~FileAttributes.ReadOnly;
			File.SetAttributes(SettingsFilePath, EditedAttributes);
		}

		private void SetReadOnlyFileAttribute()
		{
			FileAttributes EditedAttributes = File.GetAttributes(SettingsFilePath) | FileAttributes.ReadOnly;
			File.SetAttributes(SettingsFilePath, EditedAttributes);
		}

		private bool DoesOriginalFileContainSetting()
		{
			return SettingsFileContent.Contains(Setting, StringComparison.InvariantCultureIgnoreCase);
		}
	}
}
