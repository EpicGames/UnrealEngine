// Copyright Epic Games, Inc. All Rights Reserved.

using Gauntlet;
using System.IO;
using System.Linq;
using System.Text;
using UnrealBuildTool;
using Log = Gauntlet.Log;

namespace UE
{
	/// <summary>
	/// Implements the functionality of testing the cooking process in the editor
	/// </summary>
	public class CookByTheBookEditor : UnrealTestNode<UnrealTestConfiguration>
	{
		private const string CookingIsStartedKey = "The cooking process is started";		
		private const string CookingStartedPattern = "CookSettings for Memory";
		private const string CookingInProgressPattern = @"Cooked\s+packages\s+\d+\s+Packages\s+Remain\s+\d+\s+Total\s+\d+";
		private const string CookingCompletePattern = "Cook by the book total time in tick";
		private string[] CookErrors;
		private ICookedContentProvider ContentProvider;

		protected const string CookingCompleteKey = "The cooking process is complete";
		protected const string CookingInProgressKey = "The cooking process is in progress";
		protected const string CookingContentIsPlacedCorrectlyKey = "The cooking content is placed correctly";
		protected string BaseEditorCommandLine = " -run=cook -log";
		protected UnrealLogStreamParser EditorLogParser;
		protected Checker Checker;
		protected bool IsZenStoreUsed;

		public CookByTheBookEditor(UnrealTestContext InContext) : base(InContext)
		{
			CleanContentDir();
		}

		public override bool StartTest(int Pass, int InNumPasses)
		{
			bool IsStarted = base.StartTest(Pass, InNumPasses);

			InitTest();

			return IsStarted;
		}

		public override UnrealTestConfiguration GetConfiguration()
		{
			UnrealTestConfiguration Config = base.GetConfiguration();

			Config.AllRolesExit = true;

			SetEditorRole(Config, false);

			return Config;
		}

		public override void TickTest()
		{
			EditorLogParser.ReadStream();

			CookErrors = FindCookErrors();

			if (CookErrors.Any())
			{
				ReportError("There are cook errors. Ending test");
				CompleteTest(TestResult.Failed);
			}

			if (Checker.PerformValidations() && CookHelpers.HaveAllInstancesExited(UnrealApp))
			{
				ReportInfo("The cooking process was successful, the cooking content is correct. Ending test");
				CompleteTest(TestResult.Passed);
			}

			base.TickTest();
		}

		protected virtual void InitTest()
		{
			CookErrors = [];

			Checker = new Checker();
			Checker.AddValidation(CookingIsStartedKey, IsCookingStarted);
			Checker.AddValidation(CookingInProgressKey, IsCookingInProgress);
			Checker.AddValidation(CookingCompleteKey, IsCookingComplete);
			Checker.AddValidation(CookingContentIsPlacedCorrectlyKey, IsCookedContentPlacedCorrectly);

			InitEditorLogStreamParser();
			SetupContentProvider();
		}

		protected override UnrealProcessResult GetExitCodeAndReason(StopReason InReason, UnrealLog InLog, UnrealRoleArtifacts InArtifacts,
			out string ExitReason, out int ExitCode)
		{
			if (InArtifacts.SessionRole.RoleType == UnrealTargetRole.Editor)
			{
				InLog.EngineInitializedPattern = CookingStartedPattern;
			}

			string ErrorDescription = GetExitReason();

			if (!string.IsNullOrEmpty(ErrorDescription))
			{
				ExitCode = -1;
				ExitReason = ErrorDescription;
				return UnrealProcessResult.TestFailure;
			}

			return base.GetExitCodeAndReason(InReason, InLog, InArtifacts, out ExitReason, out ExitCode);
		}

		protected void SetEditorRole(UnrealTestConfiguration Config, bool IsDeferred)
		{
			string TargetPlatformName = GetTargetPlatformName();
			string ZenStoreArg = IsZenStoreUsed ? "-zenstore" : "-skipzenstore";
			UnrealTestRole EditorRole = CookHelpers.AddRequiredRole(Config, UnrealTargetRole.Editor);
			EditorRole.CommandLine += $@" {BaseEditorCommandLine} -targetplatform={TargetPlatformName} {ZenStoreArg}";
			EditorRole.DeferredLaunch = IsDeferred;
		}

		private void SetupContentProvider()
		{
			ContentProviderConfig Config = new ContentProviderConfig
			{
				TargetPlatform = GetTargetPlatform(),
				TargetPlatformName = GetTargetPlatformName(),
				ProjectName = Context.Options.Project,
				IsZenStoreUsed = IsZenStoreUsed
			};

			CookedContentProviderFactory ContentProviderFactory = new CookedContentProviderFactory(Config);
			ContentProvider = ContentProviderFactory.GetProvider();
		}

		private void InitEditorLogStreamParser()
		{
			if (CookHelpers.GetRunningInstance(UnrealApp, UnrealTargetRole.Editor) is not { } EditorApp)
			{
				return;
			}

			ILogStreamReader LogStreamReader = EditorApp.GetLogBufferReader();
			EditorLogParser = new UnrealLogStreamParser(LogStreamReader);
		}

		private bool IsCookingStarted()
		{
			return EditorLogParser.GetLogLinesContaining(CookingStartedPattern).Any();
		}

		private bool IsCookingInProgress()
		{
			return EditorLogParser.GetLogLinesMatchingPattern(CookingInProgressPattern).Any();
		}

		protected virtual bool IsCookingComplete()
		{
			return EditorLogParser.GetLogLinesContaining(CookingCompletePattern).Any();
		}

		protected virtual bool IsCookedContentPlacedCorrectly()
		{
			if (!Checker.HasValidated(CookingCompleteKey))
			{
				return false;
			}

			ContentFolder ActualCookedContent = GetActualCookedContent();
			ContentFolder ExpectedCookedContent = GetExpectedCookedContent();
			ContentFolderEqualityComparer Comparer = new ContentFolderEqualityComparer();
			bool IsCookedContentPlacedCorrectly = Comparer.Equals(ExpectedCookedContent, ActualCookedContent);

			ReportInfo(ActualCookedContent.ToString());

			if (!IsCookedContentPlacedCorrectly)
			{
				ReportError("The cooked content is not valid");
				CompleteTest(TestResult.Failed);
			}

			return IsCookedContentPlacedCorrectly;
		}

		protected ContentFolder GetActualCookedContent()
		{
			string SavedCookedPlatformPath = GetSavedCookedPlatformPath();
			ContentFolder CookedContent = ContentFolderReader.ReadFolderStructure(SavedCookedPlatformPath, 1);

			return CookedContent;
		}

		protected virtual ContentFolder GetExpectedCookedContent()
		{
			ContentFolder CookedContent = ContentProvider.GetCookedContent();

			return CookedContent;
		}

		private string[] FindCookErrors()
		{
			string[] CookErrors = EditorLogParser
				.GetEventsFromChannels(["LogCook"])
				.Where(E => E.Level == UnrealLog.LogLevel.Error)
				.Select(E => E.Message)
				.ToArray();

			return CookErrors;
		}

		protected void CleanContentDir()
		{
			string SavedCookedPath = GetSavedCookedPath();

			if (!Directory.Exists(SavedCookedPath))
			{
				return;
			}

			Directory.Delete(SavedCookedPath, true);
		}

		private string GetExitReason()
		{
			StringBuilder ErrorBuilder = new StringBuilder();

			if (CookErrors.Any())
			{
				ErrorBuilder.Append($"There are cook errors: {string.Join("\n", CookErrors)}");
			}

			string[] FailedEvents = GetFailedValidations();

			if (FailedEvents.Any())
			{
				ErrorBuilder.Append($"The expected events in the logs have not occurred: {string.Join("\n", FailedEvents)}");
			}

			return ErrorBuilder.ToString();
		}

		private string[] GetFailedValidations()
		{
			return Checker.Validations.Values
				.Where(I => !I.IsValidated)
				.Select(I => I.ActionKey)
				.ToArray();
		}

		protected string GetSavedPath()
		{
			string ProjectPath = GetProjectPath();
			string SavedPath = Path.Combine(ProjectPath, "Saved");

			return SavedPath;
		}

		protected string GetProjectPath()
		{
			string NormalizedProjectFilePath = Context.Options.ProjectPath.ToNormalizedPath();
			string ProjectPath = Path.GetDirectoryName(NormalizedProjectFilePath) ?? string.Empty;

			return ProjectPath;
		}

		protected string GetSavedCookedPlatformPath()
		{
			string SavedCookedPath = GetSavedCookedPath();
			string TargetPlatformName = GetTargetPlatformName();
			string SavedCookedPlatformPath = Path.Combine(SavedCookedPath, TargetPlatformName);

			return SavedCookedPlatformPath;
		}

		private string GetSavedCookedPath()
		{
			string SavedPath = GetSavedPath();
			string SavedCookedPath = Path.Combine(SavedPath, "Cooked");

			return SavedCookedPath;
		}

		private UnrealTargetPlatform GetTargetPlatform()
		{
			UnrealTargetPlatform TargetPlatform = Context.GetRoleContext(UnrealTargetRole.Editor).Platform;

			return TargetPlatform;
		}

		protected string GetTargetPlatformName()
		{
			UnrealTargetPlatform TargetPlatform = GetTargetPlatform();
			string TargetPlatformName = UnrealHelpers.GetPlatformName(TargetPlatform, UnrealTargetRole.Editor, false);

			return TargetPlatformName;
		}

		protected void CompleteTest(TestResult TestResult)
		{
			MarkTestComplete();
			SetUnrealTestResult(TestResult);
		}
	}
}
