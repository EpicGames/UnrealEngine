// Copyright Epic Games, Inc. All Rights Reserved.

using System.Linq;
using Gauntlet;

namespace UE
{
	/// <summary>
	/// Implements the functionality of testing the cooking process with the possibility of restarting the cooking in a new editor
	/// </summary>
	public class CookByTheBookEditors : CookByTheBookEditor
	{
		protected bool IsEditorRestarted;

		public CookByTheBookEditors(UnrealTestContext InContext) : base(InContext)
		{
			// Do nothing
		}

		public override void CleanupTest()
		{
			IsEditorRestarted = false;
		}

		public override UnrealTestConfiguration GetConfiguration()
		{
			UnrealTestConfiguration Config = base.GetConfiguration();

			SetEditorRole(Config, true);

			return Config;
		}

		protected void RestartEditorRole()
		{
			ReportInfo("Restart the editor");

			StopRunningRoles();
			MarkTestStarted(); // to prevent the test from being marked as completed and continue to run

			if (!CookHelpers.TryLaunchDeferredRole(UnrealApp.SessionInstance, UnrealTargetRole.Editor))
			{
				ReportError($"Couldn't launch the deferred role {UnrealTargetRole.Editor}");
				CompleteTest(TestResult.Failed);
				return;
			}

			IsEditorRestarted = true;
			InitTest();
		}

		private void StopRunningRoles()
		{
			IAppInstance[] RunningInstances = CookHelpers.GetRunningInstances(UnrealApp);

			if (!RunningInstances.Any())
			{
				ReportError("Couldn't stop the running roles");
				CompleteTest(TestResult.Failed);
				return;
			}

			ReportInfo("Shutting down the running roles");

			foreach (IAppInstance RunningInstance in RunningInstances)
			{
				RunningInstance.Kill();
			}
		}
	}
}
