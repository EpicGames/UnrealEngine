// Copyright Epic Games, Inc. All Rights Reserved.

using System.Collections.Generic;
using System.Linq;
using Gauntlet;
using System.IO;
using System;
using AutomationTool;
using UnrealBuildTool;

namespace UE
{
	/// <summary>
	/// Runs cook on the fly 2 test
	/// </summary>
	public class CookOnTheFly : UnrealTestNode<UnrealTestConfiguration>
	{
		string CookedProjectDirectoryPath;
		string SavedCookedPath;
		string CookedSettingsFilePath;
		DateTime StartTime = DateTime.Now;
		bool DeferredClientStarted = false;
		bool ServerStarted = false;
		bool ClientConnected = false;
		bool CookingRequest = false;
		bool CookOnTheFlyModeChange = false;
		bool CookingProcessStarted = false;
		IEnumerable<string> LogCategories = new string[] { "CookOnTheFly", "Cook" };
		public override IEnumerable<string> GetHeartbeatLogCategories() => LogCategories;
		UnrealLogStreamParser EditorLogParser = null;
		UnrealLogStreamParser ClientLogParser = null;
		Checker ClientCookOnTheFlyCheckers = null;

		public CookOnTheFly(Gauntlet.UnrealTestContext InContext)
			: base(InContext)
		{
		}

		protected virtual string GetStartedCookServerString()
		{
			return "Unreal Network File Server is ready";
		}

		protected virtual string GetClientConnectedString()
		{
			return "Client connected";
		}

		protected virtual string GetCookingRequestString()
		{
			return "Received cook request";
		}

		protected virtual string GetCookingProcessString()
		{
			return "Cooked packages";
		}

		protected virtual string GetGameStartString()
		{
			return "Starting Game";
		}

		protected virtual string GetCookModeString()
		{
			return "CookMode=CookOnTheFly";
		}

		protected virtual string GetEngineInitializedString()
		{
			return "Engine is initialized";
		}

		protected virtual string GetReceivedPackagesCookedString()
		{
			return "Received 'PackagesCooked' message";
		}

		public override UnrealTestConfiguration GetConfiguration()
		{
			UnrealTestConfiguration Config = base.GetConfiguration();
			UnrealTestRole EditorRole = Config.RequireRole(UnrealTargetRole.Editor);
			EditorRole.CommandLine += "-run=cook -cookonthefly -zenstore -log";

			UnrealTargetPlatform TargetPlatform = Context.GetRoleContext(Config.GetMainRequiredRole().Type).Platform;
			CookedProjectDirectoryPath = Path.GetDirectoryName(Context.Options.ProjectPath.ToNormalizedPath());
			string ProjectName = Context.Options.Project;
			string TargetPlatformName = UnrealHelpers.GetPlatformName(TargetPlatform, UnrealTargetRole.Editor, false);
			SavedCookedPath = Path.Combine(CookedProjectDirectoryPath, "Saved", "Cooked");
			CookedSettingsFilePath = Path.Combine(SavedCookedPath, TargetPlatformName, ProjectName, "Metadata", "CookedSettings.txt");

			UnrealTestRole ClientRole = Config.RequireRole(UnrealTargetRole.Client);
			string HostIP = UnrealHelpers.GetHostIpAddress();
			if (HostIP == null)
			{
				throw new AutomationException("Could not find local IP address");
			}

			ClientRole.DeferredLaunch = true;
			ClientRole.CommandLine += string.Format("-cookonthefly -filehostip=\"{0}\" -log -LogCmds=\"LogCookOnTheFly Verbose\"", HostIP);

			return Config;
		}

		public override bool StartTest(int Pass, int InNumPasses)
		{
			if (!base.StartTest(Pass, InNumPasses))
			{
				return false;
			}

			StartTime = DateTime.Now;

			return true;
		}

		private bool AttachToEditorLog()
		{
			if (TestInstance.EditorApp != null)
			{
				EditorLogParser = new(TestInstance.EditorApp.GetLogBufferReader());
				return true;
			}

			return false;
		}

		private bool AttachToClientLog()
		{
			if (TestInstance.ClientApps.Any())
			{
				ClientLogParser = new(TestInstance.ClientApps.First().GetLogBufferReader());
				ClientCookOnTheFlyCheckers = new();
				ClientCookOnTheFlyCheckers.AddValidation(
					"Client Engine initialization", new(() => ClientLogParser.GetLogLinesContaining(GetEngineInitializedString()).Any())
				);
				ClientCookOnTheFlyCheckers.AddValidation(
					"Client Game started", new(() => ClientLogParser.GetLogLinesContaining(GetGameStartString()).Any())
				);
				ClientCookOnTheFlyCheckers.AddValidation(
					"Client Received cooked packages", new(() => ClientLogParser.GetLogLinesContaining(GetReceivedPackagesCookedString()).Any())
				);

				return true;
			}

			// no client started yet.
			return false;
		}

		public override void TickTest()
		{
			base.TickTest();

			if (EditorLogParser == null && !AttachToEditorLog())
			{
				Phases.Start("Start Cook Server");
				return;
			}

			const int TimeoutDuration = 10; // TODO Reduce the timeout to 5 minutes after finding out the reasons for the long run time.
			if ((DateTime.Now - StartTime).TotalMinutes > TimeoutDuration)
			{
				ReportError("No logfile activity observed in last {0} minutes. Ending test", TimeoutDuration);
				MarkTestComplete();
				SetUnrealTestResult(Gauntlet.TestResult.TimedOut);
			}

			IAppInstance EditorInstance = TestInstance.EditorApp;
			if (EditorInstance == null)
			{
				ReportError("Editor instance shouldn't be null");
				MarkTestComplete();
				SetUnrealTestResult(Gauntlet.TestResult.Failed);
			}

			EditorLogParser.ReadStream();
			if (!ServerStarted)
			{
				string CompletionString = GetStartedCookServerString();
				if (EditorLogParser.GetLogLinesContaining(CompletionString).Any())
				{
					ReportInfo("Found '{0}'. Cook Server Started", CompletionString);
					ServerStarted = true;
					Phases.Get("Start Cook Server")?.End();
					Phases.Start("Start Client");
				}
			}

			if (ServerStarted && !DeferredClientStarted)
			{
				if (!CookHelpers.TryLaunchDeferredRole(UnrealApp.SessionInstance, UnrealTargetRole.Client))
				{
					ReportError($"Couldn't launch the deferred role {UnrealTargetRole.Client}");
					MarkTestComplete();
					SetUnrealTestResult(TestResult.Failed);
					return;
				}

				DeferredClientStarted = true;
				Phases.Get("Start Client")?.End();
				Phases.Start("Connect to Client");
			}

			if (!ClientConnected)
			{
				string ClientConnectedString = GetClientConnectedString();

				if (EditorLogParser.GetLogLinesContaining(ClientConnectedString).Any())
				{
					ReportInfo("Found '{0}'. Client connected", ClientConnectedString);
					ClientConnected = true;
					Phases.Get("Connect to Client")?.End();
					Phases.Start("Change CookOnTheFly Mode");
				}
			}

			if (!CookingRequest)
			{
				string CookingRequestString = GetCookingRequestString();

				if (EditorLogParser.GetLogLinesContaining(CookingRequestString).Any())
				{
					ReportInfo("Found '{0}'. Cooking request exists", CookingRequestString);
					CookingRequest = true;
					Phases.Get("Get Cooking Request")?.End();
					Phases.Start("Check for active CookOnTheFly Channel");
				}
			}

			if (!CookingProcessStarted)
			{
				CookingProcessStarted = EditorLogParser.GetLogLinesContaining(GetCookingProcessString()).Any();
			}

			if (CookingProcessStarted && !CookOnTheFlyModeChange && File.Exists(CookedSettingsFilePath))
			{
				string CookedSettingsFileText = File.ReadAllText(CookedSettingsFilePath);
				string CookModeString = GetCookModeString();
				DateTime CookedDirectoryLastModifiedTime = File.GetLastWriteTime(SavedCookedPath);
				if (CookedSettingsFileText.Contains(CookModeString) && (CookedDirectoryLastModifiedTime>StartTime))
				{
					ReportInfo("Found '{0}'. The cook mode changed for the project", CookModeString);
					CookOnTheFlyModeChange = true;
					Phases.Get("Change CookOnTheFly Mode")?.End();
					Phases.Start("Get Cooking Request");
				}
			}

			if (CookOnTheFlyModeChange)
			{
				IAppInstance[] ClientInstances = TestInstance.ClientApps;
				IAppInstance ClientInstance = null;
				if (ClientInstances.Count().Equals(1))
				{
					ClientInstance = ClientInstances.First();
				}
				else
				{
					ReportError("There should only be one client instance");
					MarkTestComplete();
					SetUnrealTestResult(Gauntlet.TestResult.Failed);
				}

				if (ClientInstance == null)
				{
					ReportError("Client instance shouldn't be null");
					MarkTestComplete();
					SetUnrealTestResult(Gauntlet.TestResult.Failed);
				}

				if (ClientLogParser == null && !AttachToClientLog())
				{
					return;
				}

				ClientLogParser.ReadStream();
				if (ClientCookOnTheFlyCheckers.PerformValidations() && CookingProcessStarted)
				{
					ReportInfo("Found '{0}', '{1}', '{2}'. The CookOnTheFly log channel is active. The cooking process is taking place.", GetGameStartString(), GetCookingProcessString(), GetReceivedPackagesCookedString());
					Phases.Get("Check for active CookOnTheFly Channel")?.End();
					MarkTestComplete();
					SetUnrealTestResult(Gauntlet.TestResult.Passed);
				}
			}
		}

		protected override UnrealProcessResult GetExitCodeAndReason(StopReason InReason, UnrealLog InLog, UnrealRoleArtifacts InArtifacts, out string ExitReason, out int ExitCode)
		{
			if (InArtifacts.SessionRole.RoleType == UnrealTargetRole.Editor)
			{
				InLog.EngineInitializedPattern = GetStartedCookServerString();
			}

			return base.GetExitCodeAndReason(InReason, InLog, InArtifacts, out ExitReason, out ExitCode);
		}
	}
}