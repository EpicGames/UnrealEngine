// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.IO;
using System.Linq;
using Gauntlet;

namespace UE
{
	public class CookSinglePackageByTheBook : CookByTheBookEditor
	{
		private const string GeneratedFolder = "_Generated_";
		private readonly string Map;
		private readonly bool IsWorldPartitionMap;

		public CookSinglePackageByTheBook(UnrealTestContext InContext) : base(InContext)
		{
			Map = Context.TestParams.ParseValue("map", string.Empty);
			IsWorldPartitionMap = Context.TestParams.ParseParam("worldpartitionmap");
			BaseEditorCommandLine += $" -map={Map} -cooksinglepackagenorefs";
		}

		public override bool StartTest(int Pass, int InNumPasses)
		{
			bool IsStarted = base.StartTest(Pass, InNumPasses);

			if (string.IsNullOrEmpty(Map))
			{
				ReportError("A map is not set");
				CompleteTest(TestResult.Failed);
			}

			return IsStarted;
		}

		protected override bool IsCookedContentPlacedCorrectly()
		{
			if (!Checker.HasValidated(CookingCompleteKey))
			{
				return false;
			}

			string SavedCookedPlatformPath = GetSavedCookedPlatformPath();

			bool bIsSinglePackageCorrect = IsSinglePackageCorrect(SavedCookedPlatformPath);

			return IsWorldPartitionMap
				? bIsSinglePackageCorrect && AreStreamingPartsCorrect(SavedCookedPlatformPath)
				: bIsSinglePackageCorrect;
		}

		private bool IsSinglePackageCorrect(string SavedCookedPlatformPath)
		{
			FileInfo[] MapFiles = DirectoryUtils
				.FindMatchingFiles(SavedCookedPlatformPath, @$"^{Map}\.(uexp|umap)$", -1)
				.ToArray(); // <Map>.uexp or <Map>.umap
			FileInfo[] AssetFiles = DirectoryUtils
				.FindMatchingFiles(SavedCookedPlatformPath, @"^.*\.uasset$", -1)
				.ToArray(); // any uasset files

			bool IsSinglePackageCorrect = MapFiles.Length == 2 && !AssetFiles.Any();

			if (IsSinglePackageCorrect)
			{
				ReportInfo($"The target package is cooked and correct: {GetFileNames(MapFiles)}");
			}

			return IsSinglePackageCorrect;
		}

		private bool AreStreamingPartsCorrect(string SavedCookedPlatformPath)
		{
			FileInfo[] StreamingParts = DirectoryUtils
				.FindMatchingFiles(SavedCookedPlatformPath, @$"^(?!{Map}\.(uexp|umap)$).*\.(uexp|umap)$", -1)
				.ToArray(); // any uexp or umap files except the <Map>

			bool AreStreamingPartsCorrect = StreamingParts.Any() &&
			                                StreamingParts.All(F =>
				                                GeneratedFolder.Equals(F.Directory?.Name, StringComparison.InvariantCultureIgnoreCase));

			if (AreStreamingPartsCorrect)
			{
				ReportInfo($"The streaming parts of the map are in the _Generated_ folder: {GetFileNames(StreamingParts)}");
			}

			return AreStreamingPartsCorrect;
		}

		private static string GetFileNames(FileInfo[] Files)
		{
			return $"\n{string.Join("\n", Files.Select(F => F.Name))}";
		}
	}
}
