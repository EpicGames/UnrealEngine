// Copyright Epic Games, Inc. All Rights Reserved.

using Gauntlet;
using System;
using System.Collections.Generic;
using System.Linq;
using EpicGame;

namespace UE
{
	/// <summary>
	/// Tests that ensures are emitted and parsed correctly
	/// </summary>
	public class TestEnsuresAreCaught : ErrorTestBase
	{
		public TestEnsuresAreCaught(UnrealTestContext InContext)
			: base(InContext)
		{
			ErrorType = ErrorTypes.Ensure;
		}
	}

	/// <summary>
	/// Tests that checks are emitted and parsed correctly
	/// </summary>
	public class TestChecksAreCaught : ErrorTestBase
	{
		public TestChecksAreCaught(UnrealTestContext InContext)
			: base(InContext)
		{
			ErrorType = ErrorTypes.Check;
		}
	}

	/// <summary>
	/// Tests that fatal logging is emitted and parsed correctly
	/// </summary>
	public class TestFatalErrorsAreCaught : ErrorTestBase
	{
		public TestFatalErrorsAreCaught(UnrealTestContext InContext)
			: base(InContext)
		{
			ErrorType = ErrorTypes.Fatal;
		}
	}

	/// <summary>
	/// Tests that exceptions are emitted and parsed correctly
	/// </summary>
	public class TestCrashesAreCaught : ErrorTestBase
	{
		public TestCrashesAreCaught(UnrealTestContext InContext)
			: base(InContext)
		{
			ErrorType = ErrorTypes.GPF;
		}
	}

	/// <summary>
	/// Base class for error tests. Contains all the logic but the smaller classes should be used.
	/// E.g. -test=ErrorTestEnsure -server
	/// </summary>
	public class ErrorTestBase : UnrealGame.DefaultTest
	{
		protected enum ErrorTypes
		{
			Ensure,
			Check,
			Fatal,
			GPF
		}

		protected ErrorTypes ErrorType;
		protected int ErrorDelay;
		protected bool Server;

		public ErrorTestBase(UnrealTestContext InContext)
			: base(InContext)
		{
			ErrorType = ErrorTypes.Check;
			ErrorDelay = 5;
			Server = false;
		}

		public override UnrealGame.UnrealTestConfig GetConfiguration()
		{
			UnrealGame.UnrealTestConfig Config = base.GetConfiguration();		
			
			string ErrorParam = Context.TestParams.ParseValue("ErrorType", null);

			if (string.IsNullOrEmpty(ErrorParam) == false)
			{
				ErrorType = (ErrorTypes)Enum.Parse(typeof(ErrorTypes), ErrorParam);
			}

			ErrorDelay = Context.TestParams.ParseValue("ErrorDelay", ErrorDelay);
			Server = Context.TestParams.ParseParam("Server");

			string Args = string.Format(" -errortest.type={0} -errortest.delay={1}", ErrorType, ErrorDelay);
			
			if (Server)
			{
				var ServerRole = Config.RequireRole(UnrealTargetRole.Server);
				ServerRole.Controllers.Add("ErrorTest");
				ServerRole.CommandLine += Args;
			}
			else
			{
				var ClientRole = Config.RequireRole(UnrealTargetRole.Client);
				ClientRole.Controllers.Add("ErrorTest");
				ClientRole.CommandLine += Args;
			}

			return Config;
		}

		protected override UnrealProcessResult GetExitCodeAndReason(StopReason InReason, UnrealLog InLog, UnrealRoleArtifacts InArtifacts, out string ExitReason, out int ExitCode)
		{
			string LocalReason = "";

			TestResult FinalResult = TestResult.Invalid;
			if (ErrorType == ErrorTypes.Ensure)
			{
				// for an ensure we should have an entry and a callstack
				int EnsureCount = InLog.Ensures.Count();
				int CallstackLength = EnsureCount > 0 ? InLog.Ensures.First().Callstack.Length : 0;

				if (EnsureCount == 0)
				{
					FinalResult = TestResult.Failed;
					LocalReason = string.Format("No ensure error found for failure of type {0}", ErrorType);
				}
				else if (EnsureCount != 1)
				{
					FinalResult = TestResult.Failed;
					LocalReason = string.Format("Incorrect ensure count found for failure of type {0}", ErrorType);
				}
				else if (CallstackLength == 0)
				{
					FinalResult = TestResult.Failed;
					LocalReason = string.Format("No callstack error found for failure of type {0}", ErrorType);
				}
				else
				{
					FinalResult = TestResult.Passed;
					LocalReason = string.Format("Found {0} ensures, test result = {1}", EnsureCount, FinalResult);
				}
			}
			else
			{
				if (InLog.FatalError == null)
				{
					FinalResult = TestResult.Failed;
					Log.Info("No fatal error found for failure of type {0}", ErrorType);
				}
				else if (InLog.FatalError.Callstack.Length == 0)
				{
					FinalResult = TestResult.Failed;
					Log.Info("No callstack found for failure of type {0}", ErrorType);
				}
				else
				{
					// all of these should contain a message and a result
					if (ErrorType == ErrorTypes.Check)
					{
						if (!InLog.FatalError.Message.ToLower().Contains("assertion failed"))
						{
							FinalResult = TestResult.Failed;
							LocalReason = string.Format("Unexpected assertion message");
						}
						else
						{
							FinalResult = TestResult.Passed;
						}
						Log.Info("Assertion message was {0}", InLog.FatalError.Message);
					}
					else if (ErrorType == ErrorTypes.Fatal)
					{
						if (!InLog.FatalError.Message.ToLower().Contains("fatal erro"))
						{
							FinalResult = TestResult.Failed;
							LocalReason = string.Format("Unexpected Fatal Error message");
						}
						else
						{
							FinalResult = TestResult.Passed;
						}

						Log.Info("Fatal Error message was {0}", InLog.FatalError.Message);
					}
					else if (ErrorType == ErrorTypes.GPF)
					{
						if (!InLog.FatalError.Message.ToLower().Contains("exception"))
						{
							FinalResult = TestResult.Failed;
							LocalReason = string.Format("Unexpected exception message");
						}
						else
						{
							FinalResult = TestResult.Passed;
						}

						Log.Info("Exception message was {0}", InLog.FatalError.Message);
					}
				}
			}

			if (FinalResult != TestResult.Invalid)
			{
				ExitCode = (FinalResult == TestResult.Passed) ? 0 : 6;
				ExitReason = LocalReason;
				return FinalResult == TestResult.Passed ? UnrealProcessResult.ExitOk : UnrealProcessResult.TestFailure;
			}
			
			return base.GetExitCodeAndReason(InReason, InLog, InArtifacts, out ExitReason, out ExitCode);
		}
	}
}
