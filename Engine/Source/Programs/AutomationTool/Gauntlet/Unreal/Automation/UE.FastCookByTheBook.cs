// Copyright Epic Games, Inc. All Rights Reserved.

using Gauntlet;

namespace UE
{
	public class FastCookByTheBook : CookByTheBook
	{
		public FastCookByTheBook(UnrealTestContext InContext) : base(InContext)
		{
			BaseEditorCommandLine += " -fastcook";
		}

		protected override ContentFolder GetExpectedCookedContent()
		{
			ContentFolder CookedContent = base.GetExpectedCookedContent();
			CookedContent.Files.Add("Fastcook.txt");

			return CookedContent;
		}
	}
}
