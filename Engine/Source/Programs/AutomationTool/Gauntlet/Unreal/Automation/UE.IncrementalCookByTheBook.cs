// Copyright Epic Games, Inc. All Rights Reserved.

using System.IO;
using System.Linq;
using Gauntlet;

namespace UE
{
	/// <summary>
	/// Runs an incremental cook and a game twice
	/// </summary>
	public class IncrementalCookByTheBook : CookByTheBookEditors
	{
		private const string ClientStartedKey = "The client is started";
		private UnrealLogStreamParser ClientLogParser;

		public IncrementalCookByTheBook(UnrealTestContext InContext) : base(InContext)
		{
			IsZenStoreUsed = true;
			BaseEditorCommandLine += $" -cookincremental -userdir={Context.Options.ProjectPath.Directory.FullName} -CookIncrementalAllowAllClasses -CookIncrementallyModifiedDiagnostics";
		}

		public override UnrealTestConfiguration GetConfiguration()
		{
			UnrealTestConfiguration Config = base.GetConfiguration();

			SetClientRoles(Config);

			return Config;
		}

		public override void TickTest()
		{
			ReadClientLogStream();

			base.TickTest();

			if (!IsEditorRestarted && Checker.HasValidated(ClientStartedKey))
			{
				RestartEditorRole();
			}
		}

		protected override void InitTest()
		{
			base.InitTest();

			ClientLogParser = new UnrealLogStreamParser();
			Checker.AddValidation(ClientStartedKey, IsClientStarted);

			if (!IsEditorRestarted)
			{
				Checker.AddValidation("Full cook is detected", IsFullCook);
				Checker.AddValidation("Cooked Packages number greater then 0", AreCookedPackages);
			}
			else
			{
				Checker.AddValidation("Incremental cook is detected", IsIncrementalCook);
				Checker.AddValidation("Cooked Packages number is equal to 0", NoCookedPackages);
			}
		}

		protected override UnrealProcessResult GetExitCodeAndReason(StopReason InReason, UnrealLog InLog, UnrealRoleArtifacts InArtifacts,
			out string ExitReason, out int ExitCode)
		{
			if (InArtifacts.SessionRole.RoleType == UnrealTargetRole.Client)
			{
				InLog.EngineInitializedPattern = "Starting Game";
			}

			return base.GetExitCodeAndReason(InReason, InLog, InArtifacts, out ExitReason, out ExitCode);
		}

		protected override bool IsCookingComplete()
		{
			bool IsCookingComplete = base.IsCookingComplete();

			if (IsCookingComplete)
			{
				if (!CookHelpers.TryLaunchDeferredRole(UnrealApp.SessionInstance, UnrealTargetRole.Client))
				{
					ReportError($"Couldn't launch the deferred role {UnrealTargetRole.Client}");
					CompleteTest(TestResult.Failed);
				}
			}

			return IsCookingComplete;
		}

		private static void SetClientRoles(UnrealTestConfiguration Config)
		{
			UnrealTestRole[] ClientRoles = Config.RequireRoles(UnrealTargetRole.Client, 2).ToArray();

			foreach (UnrealTestRole ClientRole in ClientRoles)
			{
				ClientRole.DeferredLaunch = true;
				ClientRole.CommandLine += @" -log -ExecCmds=""Automation SoftQuit""";
			}
		}

		private void ReadClientLogStream()
		{
			IAppInstance ClientApp = CookHelpers.GetRunningInstance(UnrealApp, UnrealTargetRole.Client);

			if (ClientApp is null)
			{
				return;
			}

			ClientLogParser.ReadStream(ClientApp.GetLogBufferReader());
		}

		private bool IsClientStarted() => ClientLogParser.GetLogLinesContaining("Starting Game").Any();
		private bool AreCookedPackages() => EditorLogParser.GetLogLinesMatchingPattern(@"Packages Cooked:\s*[1-9]\d*").Any(); // greater than 0
		private bool NoCookedPackages() => EditorLogParser.GetLogLinesContaining("Packages Cooked: 0").Any();
		private bool IsIncrementalCook() => EditorLogParser.GetLogLinesContaining("INCREMENTAL COOK DEPENDENCIES: Enabled").Any();
		private bool IsFullCook() => EditorLogParser.GetLogLinesContaining("FULL COOK: -cookincremental was specified").Any();
	}
}
 