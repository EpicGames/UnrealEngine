// Copyright Epic Games, Inc. All Rights Reserved.

using Gauntlet;

namespace UE
{
	public class InstallOnlyConfiguration : UnrealTestConfiguration
	{
		[AutoParam]
		public int ClientCount = 1;

		[AutoParam]
		public int ServerCount = 0;
	}

	/// <summary>
	/// No-op test that only installs a build
	/// </summary>
	public class InstallOnly : UnrealTestNode<InstallOnlyConfiguration>
	{
		/// <param name="InContext"></param>
		public InstallOnly(Gauntlet.UnrealTestContext InContext)
			: base(InContext)
		{
		}

		/// <summary>
		/// Returns the configuration description for this test
		/// </summary>
		/// <returns></returns>
		public override InstallOnlyConfiguration GetConfiguration()
		{
			InstallOnlyConfiguration Config = base.GetConfiguration();
			for (int i = 0; i < Config.ClientCount; i++)
			{
				Config.RequireRole(UnrealTargetRole.Client).InstallOnly = true;
			}
			for (int i = 0; i < Config.ServerCount; i++)
			{
				Config.RequireRole(UnrealTargetRole.Server).InstallOnly = true;
			}

			// Force disable test results for Horde as there is not actual test running. It will always pass.
			Config.WriteTestResultsForHorde = false;

			return Config;
		}

		/// <summary>
		/// Completes the test as soon as it starts because this test is a no-op
		/// </summary>
		public override void TickTest()
		{
			MarkTestComplete();
			SetTestResult(TestResult.Passed);
		}
	}
}

