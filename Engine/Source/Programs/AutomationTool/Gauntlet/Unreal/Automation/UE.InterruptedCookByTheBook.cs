// Copyright Epic Games, Inc. All Rights Reserved.

using Gauntlet;

namespace UE
{
	public class InterruptedCookByTheBook : CookByTheBookEditors
	{
		public InterruptedCookByTheBook(UnrealTestContext InContext) : base(InContext)
		{
			// Do nothing
		}

		public override void TickTest()
		{
			base.TickTest();

			if (!IsEditorRestarted && Checker.HasValidated(CookingInProgressKey))
			{
				RestartEditorRole();
			}
		}
	}
}
