// Copyright Epic Games, Inc. All Rights Reserved.

using System.Collections.Generic;
using System.Linq;
using System.Text.RegularExpressions;
using Gauntlet;

namespace UE
{
	public class IterativeCookByTheBook : CookByTheBookEditors
	{
		private const string IterativeCookKey = "Iterative cook enabled";
		private const string StatisticsShownKey = "Statistics of cooked packets are shown";
		private const string PacketsSkippedKey =
			"The number of cooked packets at the first pass is equal to the number of iteratively skipped packets at the second pass";
		private const string DiffTotalKey = "The number of different packages is zero";
		private const string PacketsStatisticsPattern = "Total Packages";
		private const string DetailedPacketsStatisticsPattern = @"Packages Cooked: (\d+), Packages Incrementally Skipped: (\d+)";
		private const string IterativePattern = "IsIterativeCook=true";
		private const string DiffTotalPattern = "NumberOfDifferentPackages=0";

		private List<(string Cooked, string Skipped)> PacketsStatistics;

		public IterativeCookByTheBook(UnrealTestContext InContext) : base(InContext)
		{
			PacketsStatistics = [];
			BaseEditorCommandLine += " -cookcultures=en -IgnoreIniSettingsOutOfDate -legacyiterative";
		}

		protected override void InitTest()
		{
			base.InitTest();

			Checker.AddValidation(IterativeCookKey, IsIterativeCook);
			Checker.AddValidation(StatisticsShownKey, ArePacketStatisticsShown);
			Checker.AddValidation(DiffTotalKey, IsNumberOfDifferentPackagesZero);
		}

		public override void TickTest()
		{
			base.TickTest();

			if (!IsEditorRestarted && Checker.HasValidated(StatisticsShownKey))
			{
				RestartEditorRole();
				Checker.AddValidation(PacketsSkippedKey, AreCookedPacketsSkipped);
			}
		}

		public override UnrealTestConfiguration GetConfiguration()
		{
			UnrealTestConfiguration Config = base.GetConfiguration();
			Config.MaxDuration = 60 * 20;

			return Config;
		}

		public override void CleanupTest()
		{
			PacketsStatistics = [];
			base.CleanupTest();
		}

		private bool ArePacketStatisticsShown() 
		{
			string MatchedLine = EditorLogParser.GetLogLinesContaining(PacketsStatisticsPattern).FirstOrDefault();

			if (MatchedLine == null)
			{
				return false;
			}

			Match Match = Regex.Match(MatchedLine, DetailedPacketsStatisticsPattern);

			if (!Match.Success)
			{
				return false; 
			}

			PacketsStatistics.Add((Match.Groups[1].Value, Match.Groups[2].Value));
			return true;
		}

		private bool AreCookedPacketsSkipped()
		{
			return PacketsStatistics.Count == 2 && PacketsStatistics.First().Cooked == PacketsStatistics.Last().Skipped;
		}

		private bool IsIterativeCook()
		{
			return EditorLogParser.GetLogLinesContaining(IterativePattern).Any();
		}

		private bool IsNumberOfDifferentPackagesZero()
		{
			return EditorLogParser.GetLogLinesContaining(DiffTotalPattern).Any();
		}
	}
}
