// Copyright Epic Games, Inc. All Rights Reserved.
using System;
using System.Collections.Generic;
using System.Linq;
using Gauntlet;
using EpicGame;
using AutomationTool;
using UnrealBuildTool;
using EpicGames.Core;
using Log = Gauntlet.Log;


namespace UE
{
	namespace NetworkAutomation
	{
		public class DedicatedServer : NetworkingNodeBase<EpicGameTestConfig>
		{
			public DedicatedServer(UnrealTestContext InContext) : base(InContext)
			{
			}

			public override EpicGameTestConfig GetConfiguration()
			{
				EpicGameTestConfig Config = base.GetConfiguration();
				Config.PreAssignAccount = false;
				Config.NoMCP = true;

				IEnumerable<UnrealTestRole> Clients = Config.RequireRoles(UnrealTargetRole.Client, 2);
				UnrealTestRole Server = Config.RequireRole(UnrealTargetRole.Server);

				Server.Controllers.Add("NetTestGauntletServerController");
				Server.CommandLine += " -log";
				Clients.ElementAt(0).CommandLine += " -log";
				Clients.ElementAt(0).Controllers.Add("NetTestGauntletClientController");
				Clients.ElementAt(1).CommandLine += " -log";
				Clients.ElementAt(1).Controllers.Add("NetTestGauntletClientController");

				return Config;
			}
		}

		public class ListenServerConfig : UnrealGame.UnrealTestConfig
		{
			string ListenServerIP = "";

			public override void ApplyToConfig(UnrealAppConfig AppConfig, UnrealSessionRole ConfigRole, IEnumerable<UnrealSessionRole> OtherRoles)
			{
				base.ApplyToConfig(AppConfig, ConfigRole, OtherRoles);

				if (ConfigRole.RoleType.IsClient())
				{
					if (ListenServerIP.Length == 0)
					{
						if (AppConfig.Platform == UnrealTargetPlatform.Win64)
						{
							var LocalAddress = UnrealHelpers.GetHostIpAddress();

							if (LocalAddress == null)
							{
								throw new AutomationException("Could not find local IP address");
							}

							string RequestedClientIP = Globals.Params.ParseValue("clientip", "");
							string ClientIP = string.IsNullOrEmpty(RequestedClientIP) ? LocalAddress.ToString() : RequestedClientIP;

							// client too, but only desktop platforms
							if (!string.IsNullOrEmpty(RequestedClientIP))
							{
								AppConfig.CommandLine += string.Format(" -multihome={0}", ClientIP);
							}

							ListenServerIP = ClientIP;
						}
					}
					else
					{
						AppConfig.CommandLine += string.Format(" -ExecCmds=\"open {0}\" -log", ListenServerIP);
					}
				}
			}
		}

		public class ListenServer : NetworkingNodeBase<ListenServerConfig>
		{
			public ListenServer(UnrealTestContext InContext) : base(InContext)
			{
			}

			public override ListenServerConfig GetConfiguration()
			{
				ListenServerConfig Config = base.GetConfiguration();

				IEnumerable<UnrealTestRole> Clients = Config.RequireRoles(UnrealTargetRole.Client, 3);

				Clients.ElementAt(0).Controllers.Add("NetTestGauntletServerController");
				Clients.ElementAt(0).CommandLine += string.Format(" -ExecCmds=\"open {0}?Listen\" -log", Config.Map);
				Clients.ElementAt(1).Controllers.Add("NetTestGauntletClientController");
				Clients.ElementAt(2).Controllers.Add("NetTestGauntletClientController");

				return Config;
			}
		}
	}

	/// <summary>
	/// Runs automated tests on a platform
	/// </summary>
	public abstract class NetworkingNodeBase<TConfigClass> : UnrealTestNode<TConfigClass>
	where TConfigClass : UnrealGame.UnrealTestConfig, new()
	{
		private DateTime LastAutomationEntryTime = DateTime.MinValue;

		public NetworkingNodeBase(Gauntlet.UnrealTestContext InContext)
			: base(InContext)
		{
			LogWarningsAndErrorsAfterSummary = false;
		}

		/// <summary>
		/// Override the summary report
		/// </summary>
		/// <returns></returns>
		protected override void LogTestSummaryHeader()
		{
			base.LogTestSummaryHeader();

			var FailingRoles = RoleResults.Where(R => R.ExitCode != 0);

			if (FailingRoles.Any())
			{
				foreach (var Role in FailingRoles)
				{
					var LogErrors = Role.LogSummary.Errors.Where(E => E.Level == UnrealLog.LogLevel.Error);

					if (LogErrors.Any())
					{
						Log.Info(" * The following errors did appear in {RoleContext} log:", Role.Artifacts.SessionRole.ToString());

						foreach (var LogError in LogErrors.Distinct())
						{
							Log.Error(KnownLogEvents.Gauntlet_UnrealEngineTestEvent, "    - " + LogError.Message);
						}
					}
				}
			}
		}

		public override ITestReport CreateReport(TestResult Result)
		{
			ITestReport Report = base.CreateReport(Result);

			if (Report != null)
			{
				foreach (var Role in RoleResults.Where(R => R.ExitCode != 0))
				{
					var LogErrors = Role.LogSummary.Errors.Where(E => E.Level == UnrealLog.LogLevel.Error);
					foreach (var LogError in LogErrors.Distinct())
					{
						Report.AddEvent(EventType.Error, LogError.Message);
					}
				}

				TConfigClass Config = GetCachedConfiguration();
				if (!string.IsNullOrEmpty(Config.Map)) Report.SetMetadata("Map", Config.Map);
			}

			return Report;
		}
	}
}
