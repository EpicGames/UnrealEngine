// Copyright Epic Games, Inc. All Rights Reserved.

using AutomationTool;
using Gauntlet;
using System;
using System.Collections.Generic;
using System.Linq;

namespace UE
{
	/// <summary>
	/// Configuration parameters for the Process Lifetime Management (PLM) tests
	/// </summary>
	public class PLMTestConfiguration : UnrealTestConfiguration
	{
		/// How long to wait before starting after the engine has been initialized
		[AutoParam]
		public int InitialWait = 0;

		/// How long to wait between loops
		[AutoParam]
		public int LoopWait = 5;

		/// How many times to loop
		[AutoParam]
		public int Loops = 5;

		/// Whether to test suspend/resume
		[AutoParam]
		public bool TestSuspend = true;

		/// Whether to test constrain/unconstrain
		[AutoParam]
		public bool TestConstrain = true;

		/// Whether to pass or fail if the target device does not support the requested action
		[AutoParam]
		public bool PassTestIfUnsupported = true;

		/// <summary>
		/// overrides all the other parameters
		/// S - suspend
		/// R - resume
		/// C - constrain
		/// U - unconstrain
		/// W# - wait # seconds
		/// example: W10SR - wait 10 seconds, suspend the app, resume the app.
		/// </summary>
		[AutoParam]
		public string CustomSequence = "";
	}

	/// <summary>
	/// Test that stresses the game's Process Lifetime Management (PLM) functionality
	/// </summary>
	public class PLMTest : UnrealTestNode<PLMTestConfiguration>
	{
		#region Actions
		abstract class Action
		{
			abstract public bool Run(IAppInstance AppInstance, PLMTest Owner);

			public override string ToString()
			{
				return GetType().Name.Replace("Action", "");
			}
		}

		class ActionSuspend : Action
		{
			public override bool Run(IAppInstance AppInstance, PLMTest Owner)
			{
				return (AppInstance is IWithPLMSuspend) ? (AppInstance as IWithPLMSuspend).Suspend() : Owner.CachedConfig.PassTestIfUnsupported;
			}
		}

		class ActionResume : Action
		{
			public override bool Run(IAppInstance AppInstance, PLMTest Owner)
			{
				return (AppInstance is IWithPLMSuspend) ? (AppInstance as IWithPLMSuspend).Resume() : Owner.CachedConfig.PassTestIfUnsupported;
			}
		}

		class ActionConstrain : Action
		{
			public override bool Run(IAppInstance AppInstance, PLMTest Owner)
			{
				return (AppInstance is IWithPLMConstrain) ? (AppInstance as IWithPLMConstrain).Constrain() : Owner.CachedConfig.PassTestIfUnsupported;
			}
		}

		class ActionUnconstrain : Action
		{
			public override bool Run(IAppInstance AppInstance, PLMTest Owner)
			{
				return (AppInstance is IWithPLMConstrain) ? (AppInstance as IWithPLMConstrain).Unconstrain() : Owner.CachedConfig.PassTestIfUnsupported;
			}
		}

		class ActionWait : Action
		{
			readonly int WaitTimeSeconds;
			public ActionWait( int InWaitTimeSeconds = 5 )
			{
				WaitTimeSeconds = InWaitTimeSeconds;
			}

			public override bool Run(IAppInstance AppInstance, PLMTest Owner)
			{
				System.Threading.Thread.Sleep(WaitTimeSeconds * 1000);
				return true;
			}

			public override string ToString()
			{
				return $"Wait {WaitTimeSeconds}s";
			}
		}


		/// S - suspend
		/// R - resume
		/// C - constrain
		/// U - unconstrain
		/// V - quick save
		/// Q - quick resume
		/// W# - wait # seconds
		private Action[] ParseCustomSequence(string Sequence)
		{
			List<Action> Result = new List<Action>();

			// only need one instance of these as they have no parameters
			Action Suspend = new ActionSuspend();
			Action Resume = new ActionResume();
			Action Constrain = new ActionConstrain();
			Action Unconstrain = new ActionUnconstrain();

			// parse the string and create the sequence
			int Pos = 0;
			while (Pos < Sequence.Length)
			{
				switch (Char.ToUpper(Sequence[Pos]))
				{
					case 'S': Result.Add(Suspend); break;
					case 'R': Result.Add(Resume); break;
					case 'C': Result.Add(Constrain); break;
					case 'U': Result.Add(Unconstrain); break;
					case 'W':
						{
							Pos++;
							int NumStart = Pos;
							while (Pos < Sequence.Length && Char.IsNumber(Sequence[Pos]))
							{
								Pos++;
							}
							if (NumStart == Pos)
							{
								throw new AutomationException($"no number specified for W in custom sequence {Sequence}[{Pos}]");
							}

							string NumString = Sequence.Substring(NumStart, Pos - NumStart);
							int Value = int.Parse(NumString);
							Result.Add(new ActionWait(Value));
							Pos--;
						}
						break;
				}

				Pos++;
			}

			return Result.ToArray();
		}

		Action[] GetDefaultSequence(PLMTestConfiguration Config)
		{
			List<Action> Result = new List<Action>();

			// only need one instance of these as they have no parameters
			Action Suspend = new ActionSuspend();
			Action Resume = new ActionResume();
			Action Constrain = new ActionConstrain();
			Action Unconstrain = new ActionUnconstrain();

			// build the sequence
			for (int Loop = 0; Loop < Config.Loops; Loop++)
			{
				if (Loop == 0)
				{
					Result.Add(new ActionWait(Config.InitialWait));
				}
				else
				{
					Result.Add(new ActionWait(Config.LoopWait));
				}

				if (Config.TestSuspend)
				{
					Result.Add(Suspend);
					Result.Add(Resume);
				}

				if (Config.TestConstrain)
				{
					Result.Add(Constrain);
					Result.Add(Unconstrain);
				}
			}

			return Result.ToArray();
		}

		#endregion

		Action[] Actions;
		int CurrentActionIndex;
		bool bHasEngineInit;

		/// <summary>
		/// Default constructor
		/// </summary>
		/// <param name="InContext"></param>
		public PLMTest(Gauntlet.UnrealTestContext InContext)
			: base(InContext)
		{
			PLMTestConfiguration Config = GetConfiguration();

			CurrentActionIndex = 0;
			if (string.IsNullOrEmpty(Config.CustomSequence))
			{
				Actions = GetDefaultSequence(Config);
			}
			else
			{
				Actions = ParseCustomSequence(Config.CustomSequence);
			}
		}

		/// <summary>
		/// Returns the configuration description for this test
		/// </summary>
		/// <returns></returns>
		public override PLMTestConfiguration GetConfiguration()
		{
			PLMTestConfiguration Config = base.GetConfiguration();

			UnrealTestRole Client = Config.RequireRole(UnrealTargetRole.Client);

			return Config;
		}

		/// <summary>
		/// Called to begin the test.
		/// </summary>
		/// <param name="Pass"></param>
		/// <param name="InNumPasses"></param>
		/// <returns></returns>
		public override bool StartTest(int Pass, int InNumPasses)
		{
			// Call the base class to actually start the test running
			if (!base.StartTest(Pass, InNumPasses))
			{
				return false;
			}

			CurrentActionIndex = 0;
			bHasEngineInit = false;
			return true;
		}

		/// <summary>
		/// String that we search for to be considered "Booted"
		/// </summary>
		/// <returns></returns>
		protected virtual string GetCompletionString()
		{
			return "Engine is initialized. Leaving FEngineLoop::Init()";
		}

		/// <summary>
		/// Called periodically while the test is running to allow code to monitor health.
		/// </summary>
		public override void TickTest()
		{
			// run the base class tick;
			base.TickTest();

			if (!bHasEngineInit)
			{
				// see if the engine has been initialized yet
				IAppInstance RunningInstance = this.TestInstance.RunningRoles.First().AppInstance;
				UnrealLogParser LogParser = new UnrealLogParser(RunningInstance.GetLogBufferReader());
				if (LogParser.GetAllContainingLines(GetCompletionString(), StringComparison.OrdinalIgnoreCase).Any())
				{
					Log.Info("Found '{0}'. Starting PLM tests...", GetCompletionString() );
					bHasEngineInit = true;
				}
			}
			else if (CurrentActionIndex >= Actions.Length)
			{
				Log.Info("All actions completed. Ending test");
				MarkTestComplete();
				SetUnrealTestResult(TestResult.Passed);
			}
			else
			{
				IAppInstance RunningInstance = this.TestInstance.RunningRoles.First().AppInstance;
				Action CurrentAction = Actions[CurrentActionIndex];
				Log.Info($"{CurrentAction}...");

				// run the action and wait for it to finish
				bool bSuccess = CurrentAction.Run( RunningInstance, this );
				if (!bSuccess)
				{
					ReportError($"{CurrentAction} failed. Ending test");
					MarkTestComplete();
					SetUnrealTestResult(TestResult.Failed);
				}
				else
				{
					// do the next action next tick
					CurrentActionIndex++;
				}
			}
		}
	}
}

