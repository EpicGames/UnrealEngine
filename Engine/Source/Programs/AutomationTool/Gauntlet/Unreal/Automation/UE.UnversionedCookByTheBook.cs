// Copyright Epic Games, Inc. All Rights Reserved.

using System.Linq;
using Gauntlet;

namespace UE
{
	public class UnversionedCookByTheBook : CookByTheBook
	{
		private const string UnversionedPattern = "IsUnversioned=true";

		public UnversionedCookByTheBook(UnrealTestContext InContext) : base(InContext)
		{
			BaseEditorCommandLine += " -cookcultures=en -unversioned";
		}

		protected override void InitTest()
		{
			base.InitTest();

			Checker.AddValidation("Unversioned cook enabled", IsUnversionedCook);
		}

		private bool IsUnversionedCook()
		{
			return EditorLogParser.GetLogLinesContaining(UnversionedPattern).Any();
		}
	}
}
