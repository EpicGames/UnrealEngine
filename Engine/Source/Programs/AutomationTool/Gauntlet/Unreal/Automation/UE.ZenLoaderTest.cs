// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.IO;
using System.Linq;
using Gauntlet;
using UnrealBuildTool;

namespace UE
{
	/// <summary>
	/// Extends the <see cref="UnrealTestConfiguration"/> with the packaging parameters
	/// </summary>
	public class ZenLoaderTestConfiguration : UnrealTestConfiguration
	{
		[AutoParam]
		public string ProjectPackaging { get; set; } = string.Empty;
		[AutoParam]
		public string EditorPackaging { get; set; } = string.Empty;
	}

	/// <summary>
	/// Launches a client from a stage with different packaging parameters
	/// </summary>
	public class ZenLoaderTest : UnrealTestNode<ZenLoaderTestConfiguration>
	{
		private StagingOutputCondition[] ActualStagingOutputConditions;
		private StagingOutputCondition[] ExpectedStagingOutputConditions;
		private EditorPackOption InputEditorPackOption;
		private ProjectPackOption InputProjectPackOption;
		private readonly string BuildPath;
		private readonly string ContentPath;

		public ZenLoaderTest(UnrealTestContext InContext) : base(InContext)
		{
			BuildPath = Path.Combine(Context.Options.Build, GetTargetPlatformName());
			ContentPath = Path.Combine(BuildPath, Context.Options.Project, "Content");

			if (!Directory.Exists(ContentPath))
			{
				Directory.CreateDirectory(ContentPath); // TODO remove after the bug UE-254908 is fixed
			}
		}

		public override ZenLoaderTestConfiguration GetConfiguration()
		{
			ZenLoaderTestConfiguration Config = base.GetConfiguration();
			ParsePackagingParams(Config);

			UnrealTestRole ClientRole = Config.RequireRole(UnrealTargetRole.Client);
			ClientRole.CommandLine += @" -log -ExecCmds=""Automation SoftQuit""";

			return Config;
		}

		protected override UnrealProcessResult GetExitCodeAndReason(StopReason InReason, UnrealLog InLog, UnrealRoleArtifacts InArtifacts,
			out string ExitReason, out int ExitCode)
		{
			InitStagingOutputConditions();
			InLog.EngineInitializedPattern = "Starting Game";

			if (!IsStagingOutputValid())
			{
				ExitCode = -1;
				ExitReason = "The staging output doesn't match the packaging parameters";
				return UnrealProcessResult.TestFailure;
			}

			return base.GetExitCodeAndReason(InReason, InLog, InArtifacts, out ExitReason, out ExitCode);
		}

		private void InitStagingOutputConditions()
		{
			ActualStagingOutputConditions =
			[
				new(StagingOutput.Utoc, () =>
					DirectoryUtils.FindMatchingFiles(ContentPath, @"^.*\.(ucas|utoc|uondemandtoc)$", -1).Any()),
				new(StagingOutput.LegacyPak, () =>
					DirectoryUtils.FindMatchingFiles(ContentPath, @"^.*\.pak$", -1).Any()),
				new(StagingOutput.Streaming, () =>
					DirectoryUtils.FindMatchingFiles(BuildPath, @"ue\.projectstore").Any())
			];

			ExpectedStagingOutputConditions =
			[
				new(StagingOutput.Utoc, () =>
					InputProjectPackOption.HasFlag(ProjectPackOption.UseIoStore) &&
					InputEditorPackOption != EditorPackOption.UseLooseFiles),
				new(StagingOutput.LegacyPak, () =>
					InputProjectPackOption == ProjectPackOption.UsePak &&
					InputEditorPackOption != EditorPackOption.UseLooseFiles),
				new(StagingOutput.Streaming, () =>
					InputProjectPackOption.HasFlag(ProjectPackOption.UseIoStore) &&
					InputProjectPackOption.HasFlag(ProjectPackOption.UseZen) &&
					InputEditorPackOption == EditorPackOption.UseLooseFiles)
			];
		}

		private bool IsStagingOutputValid()
		{
			StagingOutput ActualStagingOutput = DefineStagingOutput(ActualStagingOutputConditions);
			StagingOutput ExpectedStagingOutput = DefineStagingOutput(ExpectedStagingOutputConditions);

			Log.Info("{TestName}: Packaging params: {ProjectPackaging}; {EditorPackaging}.\nActual staging output: {Actual}; Expected staging output: {Expected}",
				nameof(ZenLoaderTest), CachedConfig.ProjectPackaging, CachedConfig.EditorPackaging, ActualStagingOutput, ExpectedStagingOutput);

			return ActualStagingOutput == ExpectedStagingOutput;
		}

		private static StagingOutput DefineStagingOutput(StagingOutputCondition[] Conditions) =>
			Conditions.FirstOrDefault(F => F.Exists())?.StagingOutput ?? StagingOutput.LooseFiles;

		private string GetTargetPlatformName()
		{
			UnrealTargetPlatform? TargetPlatform = Context.Constraint.Platform ?? UnrealTargetPlatform.Win64;
			string TargetPlatformName = TargetPlatform == UnrealTargetPlatform.Win64 ? "Windows" : TargetPlatform.ToString();

			return TargetPlatformName;
		}

		private void ParsePackagingParams(ZenLoaderTestConfiguration Config)
		{
			string EditedProjectPackaging = Config.ProjectPackaging.Replace("+", ",");

			InputProjectPackOption = Enum.TryParse(EditedProjectPackaging, true, out ProjectPackOption parsedProjectOptions)
				? parsedProjectOptions
				: ProjectPackOption.None;

			InputEditorPackOption = Enum.TryParse(Config.EditorPackaging, true, out EditorPackOption parsedEditorOptions)
				? parsedEditorOptions
				: EditorPackOption.None;
		}

		/// <summary>
		/// Packaging options
		/// </summary>
		[Flags]
		private enum ProjectPackOption
		{
			None = 0,
			UseIoStore = 1,
			UsePak = 2,
			UseZen = 4
		}

		/// <summary>
		/// Type of pack files for launch on a device 
		/// </summary>
		private enum EditorPackOption
		{
			None,
			UseLooseFiles,
			UseCompressedPakFiles,
			UsePakFilesWithoutCompression
		}

		/// <summary>
		/// Indicates what kind of files were used for staging
		/// </summary>
		private enum StagingOutput
		{
			LooseFiles,
			Utoc,
			LegacyPak,
			Streaming
		}

		/// <summary>
		/// Stores a condition that determines <see cref="StagingOutput"/>
		/// </summary>
		private class StagingOutputCondition
		{
			public StagingOutput StagingOutput { get; }
			public Func<bool> Exists { get; }

			public StagingOutputCondition(StagingOutput InStagingOutput, Func<bool> InExists)
			{
				StagingOutput = InStagingOutput;
				Exists = InExists;
			}
		}
	}
}
