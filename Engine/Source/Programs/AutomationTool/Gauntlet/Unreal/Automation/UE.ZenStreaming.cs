// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.IO;
using Gauntlet;
using System.Linq;
using Microsoft.Extensions.Logging;
using AutomationTool;

namespace UE
{
	public class ZenStreaming: UnrealTestNode<UnrealTestConfiguration>
	{
		private const string StreamingStatsFile = "ZenStreamingClient.log";
		private UnrealLogStreamParser ClientLogParser;
		private readonly string ReplayFile;
		private string ZenArgs;

		public ZenStreaming(UnrealTestContext InContext) : base(InContext)
		{
			File.WriteAllText(StreamingStatsFile, string.Empty);
			ClientLogParser = new UnrealLogStreamParser();

			ZenArgs = Globals.Params.ParseValue("zenargs", string.Empty);
			ReplayFile = Globals.Params.ParseValue("replayfile", string.Empty);
		}

		public override UnrealTestConfiguration GetConfiguration()
		{
			UnrealTestConfiguration Config = base.GetConfiguration();
			UnrealTestRole ClientRole = Config.RequireRole(UnrealTargetRole.Client);

			string command = @$" {ZenArgs} -execcmds=""DemoPlay {ReplayFile}"" -exitafterreplay";
			ClientRole.CommandLineParams.Add(command);
			
			return Config;
		}

		public override void StopTest(StopReason InReason)
		{
			ReadClientLogStream();
			ParseStreamingStats();

			base.StopTest(InReason);
		}

		private void ReadClientLogStream()
		{
			IAppInstance ClientApp = TestInstance.ClientApps.FirstOrDefault();

			if (ClientApp is null)
			{
				return;
			}

			ClientLogParser.ReadStream(ClientApp.StdOut);
		}

		private void ParseStreamingStats()
		{
			string[] LogZenLines = ClientLogParser.GetLogFromChannel("LogStreaming").ToArray();

			if (!LogZenLines.Any())
			{
				return;
			}

			string LogLinesToSave = string.Join(Environment.NewLine, LogZenLines);
			File.AppendAllText(StreamingStatsFile, LogLinesToSave);
		}
	}
}