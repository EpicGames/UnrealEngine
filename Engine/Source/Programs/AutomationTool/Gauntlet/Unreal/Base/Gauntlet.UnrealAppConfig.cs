// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using AutomationTool;
using EpicGames.Core;
using UnrealBuildTool;

namespace Gauntlet
{ 
	/// <summary>
	/// Represents the configuration needed to run an instance of an Unreal app
	/// </summary>
	public class UnrealAppConfig : IAppConfig
    {
		/// <summary>
		/// Reference name
		/// </summary>
		public string Name { get; set; }

		/// <summary>
		/// Name of this unreal project
		/// </summary>
		public string ProjectName { get; set; }

		// <summary>
		/// Path to the file. Can be null if the project isn't on disk
		/// </summary>
		public FileReference ProjectFile { get; set; }

		/// <summary>
		/// Type of role this instance performs
		/// </summary>
		public UnrealTargetRole ProcessType { get; set; }

		/// <summary>
		/// Platform this role runs on
		/// </summary>
		public UnrealTargetPlatform? Platform { get; set; }

		/// <summary>
		/// Configuration for this role
		/// </summary>
		public UnrealTargetConfiguration Configuration { get; set; }

        /// <summary>
        /// Files to copy over to the device, plus what type of files they are.
        /// </summary>
        public List<UnrealFileToCopy> FilesToCopy { get; set; }

		/// <summary>
		/// Some IAppInstall instances can alter command line after it has been copied from AppConfig.CommandLine
		/// Use this to restrict this behavior if necessary.
		/// </summary>
		public bool CanAlterCommandArgs {
			get { return CanAlterCommandArgsPrivate; }
			set { CanAlterCommandArgsPrivate = value; }
		}
		private bool CanAlterCommandArgsPrivate = true;

		/// <summary>
		/// Set this property when the application is executed through a Docker container.
		/// </summary>
		public ContainerInfo ContainerInfo { get; set; }

		/// <summary>
		/// Delegate to filter logging output
		/// </summary>
		public LongProcessResult.OutputFilterCallbackType FilterLoggingDelegate { get; set; }

		/// <summary>
		/// Arguments for this instance
		/// </summary>
		public string CommandLine
		{
			get
			{
				if (CommandLineParams == null)
				{
					CommandLineParams = new GauntletCommandLine();
				}
				return CommandLineParams.GenerateFullCommandLine(CanAlterCommandArgs);
			}
			set
			{
				if (CommandLineParams == null)
				{
					CommandLineParams = new GauntletCommandLine();
				}

				CommandLineParams.ClearCommandLine();

				CommandLineParams.AddRawCommandline(value, false);
			}
		}
		/// <summary>
		/// Dictionary of commandline arguments that are turned into a commandline at the end.
		/// For flags, leave the value set to null.
		/// </summary>
		public GauntletCommandLine CommandLineParams;

		/// <summary>
		/// Sandbox that we'd like to install this instance in
		/// </summary>
		public string Sandbox { get; set; }

		public IBuild Build { get; set; }

		public OverlayExecutable OverlayExecutable{ get; set; }

		// Prevents installing a build on device
		public bool SkipInstall => DevicePool.SkipInstall;

		// Performs a full clean on the device before installing
		public bool FullClean => DevicePool.FullClean;

		/// <summary>
		/// Use additional debug memory on target device if available. 
		/// </summary>
		public bool UsePlatformExtraDebugMemory { get; set; }
		
		/// <summary>
		/// Defines the expected duration of the app process. May not be set depending on the context of this object's creation
		/// </summary>
		public int MaxDuration { get; set; }

		/// <summary>
		/// Disable stdout redirection and write the output of the app process to a log file
		/// </summary>
		public bool DisableStdoutProcessing { get; set; }

		/// <summary>
		/// Constructor that sets some required values to defaults
		/// </summary>
		public UnrealAppConfig()
		{
			Name = "UnrealApp";
			ProjectName = "UnknownProject";
			CommandLine = "";
			Configuration = UnrealTargetConfiguration.Development;
			Sandbox = "Gauntlet";
			FilterLoggingDelegate = null;
		}
	}
}