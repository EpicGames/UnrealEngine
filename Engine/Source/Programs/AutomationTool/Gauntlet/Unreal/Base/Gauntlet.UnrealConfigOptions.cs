// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using AutomationTool;
using UnrealBuildTool;



namespace Gauntlet
{
	/// <summary>
	/// Defines a basic set of options that can be applied to client & server instances of
	/// Unreal
	/// </summary>
	public class UnrealOptions : IConfigOption<UnrealAppConfig>, ICloneable
	{
		[AutoParam(false)]
		public bool NullRHI;

		[AutoParam(false)]
		public bool Debug;

		[AutoParam(true)]
		public bool Windowed;

		[AutoParam(1920)]
		public int ResX;

		[AutoParam(1080)]
		public int ResY;

		[AutoParam(false)]
		public bool Unattended;

		[AutoParam(false)]
		public bool Log;

		[AutoParam("")]
		public string Map;

		[AutoParam("")]
		public string CommonArgs;

		[AutoParam("")]
		public string ClientArgs;

		[AutoParam("")]
		public string ServerArgs;

		public UnrealOptions()
		{

			AutoParam.ApplyDefaults(this);
		}

		public virtual void ApplyToConfig(UnrealAppConfig AppConfig)
		{
			string CmdArgs = CommonArgs;

			// Common args
			if (Unattended)
			{
				CmdArgs += " -unattended";
			}

			if (Log)
			{
				CmdArgs += " -log";
			}

			if (Debug)
			{
				CmdArgs += " -debug";
			}

			// Client-only args
			if (AppConfig.ProcessType.IsClient())
			{
				if (NullRHI)
				{
					CmdArgs += " -nullrhi";
				}

				if (Windowed)
				{
					CmdArgs += " -windowed";

					if (ResX > 0 && ResY > 0)
					{
						CmdArgs += string.Format(" -ResX={0} -ResY={1}", ResX, ResY);
					}
				}
			}

			AppConfig.CommandLine += " " + CmdArgs;

			// map is special and must be first arg (needed for more than server?)
			if (AppConfig.ProcessType.IsServer() && string.IsNullOrEmpty(Map) == false)
			{
				AppConfig.CommandLine = string.Format("{0} {1}", Map, AppConfig.CommandLine);
			}
		}
		public object Clone()
		{
			return this.MemberwiseClone();
		}
	};
}