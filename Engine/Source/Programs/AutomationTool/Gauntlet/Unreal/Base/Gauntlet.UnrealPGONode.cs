// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.IO;
using System.Collections.Generic;
using AutomationTool;
using UnrealBuildTool;
using System.Diagnostics;
using System.Diagnostics.Eventing.Reader;
using Gauntlet.Utils;

namespace Gauntlet
{
	public class PGOConfig : UnrealTestConfiguration, IAutoParamNotifiable
	{
		/// <summary>
		/// Output directory to write the resulting profile data to.
		/// </summary>
		[AutoParam("")]
		public string ProfileOutputDirectory;

		/// <summary>
		/// Directory to save periodic screenshots to whilst the PGO run is in progress.
		/// </summary>
		[AutoParam("")]
		public string ScreenshotDirectory;

		[AutoParam("")]
		public string PGOAccountSandbox;

		[AutoParam("")]
		public string PgcFilenamePrefix;		

		public virtual void ParametersWereApplied(string[] Params)
		{
			if (String.IsNullOrEmpty(ProfileOutputDirectory))
			{
				throw new AutomationException("ProfileOutputDirectory option must be specified for profiling data");
			}
		}
	}

	public class PGONode<TConfigClass> : UnrealTestNode<TConfigClass> where TConfigClass : PGOConfig, new()
	{
		protected string LocalOutputDirectory;
		internal IPGOPlatform PGOPlatform;
		bool ProcessPGODataFailed = false;

		public PGONode(UnrealTestContext InContext) : base(InContext)
		{
			
		}

		public override TConfigClass GetConfiguration()
		{
			if (CachedConfig != null)
			{
				return CachedConfig as TConfigClass;
			}

			var Config = CachedConfig = base.GetConfiguration();

			// Set max duration to 1 hour
			Config.MaxDuration = 60 * 60;

			// Get output filenames
			LocalOutputDirectory = Path.GetFullPath(Config.ProfileOutputDirectory);

			// Create the local profiling data directory if needed
			if (!Directory.Exists(LocalOutputDirectory))
			{
				Directory.CreateDirectory(LocalOutputDirectory);
			}

			ScreenshotDirectory = Config.ScreenshotDirectory;

			if (!String.IsNullOrEmpty(ScreenshotDirectory))
			{
				if (Directory.Exists(ScreenshotDirectory))
				{
					Directory.Delete(ScreenshotDirectory, true);
				}

				Directory.CreateDirectory(ScreenshotDirectory);
			}

			PGOPlatform = PGOPlatformManager.GetPGOPlatform(Context.GetRoleContext(UnrealTargetRole.Client).Platform);
			PGOPlatform.ApplyConfiguration(Config);

			return Config as TConfigClass;

		}

		private DateTime ScreenshotStartTime = DateTime.UtcNow;
		private DateTime ScreenshotTime = DateTime.MinValue;
		private TimeSpan ScreenshotInterval = TimeSpan.FromSeconds(30);
		private const float ScreenshotScale = 1.0f / 3.0f;
		private const int ScreenshotQuality = 30;
		protected string ScreenshotDirectory;
		protected bool ResultsGathered = false;

		protected override TestResult GetUnrealTestResult()
		{
			if (ProcessPGODataFailed)
			{
				return TestResult.Failed;
			}

			return base.GetUnrealTestResult();
		}

		public override void TickTest()
		{
			base.TickTest();

			if (GetTestStatus() == TestStatus.Complete || ResultsGathered)
			{
				if (!ResultsGathered)
				{
					ResultsGathered = true;

					try
					{
						// Gather results and merge PGO data
						Log.Info("Gathering profiling results to {0}", TestInstance.ClientApps[0].ArtifactPath);
						PGOPlatform.GatherResults(TestInstance.ClientApps[0].ArtifactPath);
					}
					catch (Exception Ex)
					{
						ProcessPGODataFailed = true;
						Log.Error("Error getting PGO results: {0}", Ex);
					}

				}

				return;
			}

			// Handle device screenshot update
			TimeSpan Delta = DateTime.Now - ScreenshotTime;
			ITargetDevice Device = TestInstance.ClientApps[0].Device;

			string ImageFilename;
			if (!String.IsNullOrEmpty(ScreenshotDirectory) && Delta >= ScreenshotInterval && Device != null && PGOPlatform.TakeScreenshot(Device, ScreenshotDirectory, out ImageFilename))
			{
				ScreenshotTime = DateTime.Now;

				if (!File.Exists(Path.Combine(ScreenshotDirectory, ImageFilename)))
				{
					Log.Info("PGOPlatform.TakeScreenshot returned true, but output image {0} does not exist! skipping", ImageFilename);
				}
				else if(new FileInfo(Path.Combine(ScreenshotDirectory, ImageFilename)).Length <= 0)
				{
					Log.Info("PGOPlatform.TakeScreenshot returned true, but output image {0} is size 0! skipping", ImageFilename);
				}
				else
				{
					try
					{
						TimeSpan ImageTimestamp = DateTime.UtcNow - ScreenshotStartTime;
						string ImageOutputPath = Path.Combine(ScreenshotDirectory, ImageTimestamp.ToString().Replace(':', '-') + ".jpg");
						ImageUtils.ResaveImageAsJpgWithScaleAndQuality(Path.Combine(ScreenshotDirectory, ImageFilename), ImageOutputPath, ScreenshotScale, ScreenshotQuality);

						// Delete the temporary image file
						try { File.Delete(Path.Combine(ScreenshotDirectory, ImageFilename)); }
						catch (Exception e)
						{
							Log.Warning("Got Exception Deleting temp iamge: {0}", e.ToString());
						}
					}
					catch (Exception e)
					{
						Log.Info("Got Exception Renaming PGO image {0}: {1}", ImageFilename, e.ToString());

						TimeSpan ImageTimestamp = DateTime.UtcNow - ScreenshotStartTime;
						string CopyFileName = Path.Combine(ScreenshotDirectory, ImageTimestamp.ToString().Replace(':', '-') + ".bmp");
						Log.Info("Copying unconverted image {0} to {1}", ImageFilename, CopyFileName);
						try
						{
							File.Copy(Path.Combine(ScreenshotDirectory, ImageFilename), CopyFileName);
						}
						catch (Exception e2)
						{
							Log.Warning("Got Exception copying un-converted screenshot image: {0}", e2.ToString());
						}
					}
				}
			}

		}
	}


	public interface IPGOCustomReplayPlatform
	{
		void ConfigureClientForReplay( PGOConfig Config, UnrealTestRole ClientRole, string LocalReplayFileName );
	}


	public class BasicReplayPGOConfig : PGOConfig
	{
		[AutoParam("")]
		public string LocalReplay;

		[AutoParam("")]
		public string AdditionalCommandLine;

		public override void ParametersWereApplied(string[] Params)
		{
			base.ParametersWereApplied(Params);
			
			// Check that replay file exists if specified
			if (string.IsNullOrEmpty(LocalReplay) || !File.Exists(LocalReplay))
			{
				throw new AutomationException("LocalReplay option must be specified with absolute path to existing replay file");
			}			
		}
	}

	/// <summary>
	/// PGO Profiling Node that runs a replay & gathers the results. Requires an existing build compiled with  -PGOProfile -Configuration=Shipping
	/// e.g. RunUnreal -Test=BasicReplayPGONode -project=MyProject -platform=Win64 -configuration=Shipping -build=MyProject\Saved\StagedBuilds\Windows -LocalReplay=MyProject\Replays\PGO.replay -ProfileOutputDirectory=MyProject\Platforms\Windows\Build\PGO -MaxDuration=7200
	/// </summary>
	public class BasicReplayPGONode : PGONode<BasicReplayPGOConfig>
	{
		public BasicReplayPGONode(UnrealTestContext InContext) : base(InContext)
		{
		}

		public override BasicReplayPGOConfig GetConfiguration()
		{
			if (CachedConfig != null)
			{
				return CachedConfig;
			}
			var Config = base.GetConfiguration();

			// create a client that runs the given replay & then exits when it's finished
			UnrealTestRole ClientRole = Config.RequireRole(UnrealTargetRole.Client);
			ClientRole.CommandLine += $" -Deterministic -ExitAfterReplay -NoCheckpointHangDetector";
			ClientRole.CommandLine += $" {Config.AdditionalCommandLine}";

			if (PGOPlatform is IPGOCustomReplayPlatform PGOCustomReplayPlatform)
			{
				// the platform needs bespoke replay handling
				PGOCustomReplayPlatform.ConfigureClientForReplay(Config, ClientRole, Config.LocalReplay);
			}
			else
			{
				// normal replay handling - copy to the demos folder and then launch from command line
				string ReplayName = Path.GetFileNameWithoutExtension(Config.LocalReplay);
				ClientRole.FilesToCopy.Add(new UnrealFileToCopy(Config.LocalReplay, EIntendedBaseCopyDirectory.Demos, Path.GetFileName(Config.LocalReplay)));
				ClientRole.CommandLine += $" -Replay=\"{ReplayName}\" ";
			}

			return Config;
		}
	}



	internal interface IPGOPlatform
	{
		void ApplyConfiguration(PGOConfig Config);

		void GatherResults(string ArtifactPath);

		bool TakeScreenshot(ITargetDevice Device, string ScreenshotDirectory, out string ImageFilename);

		UnrealTargetPlatform GetPlatform();
	}

	/// <summary>
	/// PGO platform manager
	/// </summary>
	internal abstract class PGOPlatformManager
	{
		public static IPGOPlatform GetPGOPlatform(UnrealTargetPlatform Platform)
		{
			Type PGOPlatformType;
			if (!PGOPlatforms.TryGetValue(Platform, out PGOPlatformType))
			{
				throw new AutomationException("Invalid PGO Platform: {0}", Platform);
			}

			return Activator.CreateInstance(PGOPlatformType) as IPGOPlatform;
		}

		protected static void RegisterPGOPlatform(UnrealTargetPlatform Platform, Type PGOPlatformType)
		{
			PGOPlatforms[Platform] = PGOPlatformType;
		}

		static Dictionary<UnrealTargetPlatform, Type> PGOPlatforms = new Dictionary<UnrealTargetPlatform, Type>();

		static PGOPlatformManager()
		{
			IEnumerable<IPGOPlatform> DiscoveredPGOPlatforms = Utils.InterfaceHelpers.FindImplementations<IPGOPlatform>();

			foreach (IPGOPlatform PGOPlatform in DiscoveredPGOPlatforms)
			{
				PGOPlatforms[PGOPlatform.GetPlatform()] = PGOPlatform.GetType();
			}

		}

	}

}
