// Copyright Epic Games, Inc. All Rights Reserved.

namespace Gauntlet
{
	/// <summary>
	/// Describes reasons an Unreal process may have exited
	/// </summary>
	public enum UnrealProcessResult
	{
		ExitOk,                         // No known issues
		InitializationFailure,          // Process failed to initialize (e.g. the editor or game failed to load)
		EncounteredFatalError,          // A fatal error occurred
		EncounteredEnsure,              // An ensure occurred (will only be returned if the test considers ensures as fatal)
		LoginFailed,					// Client never successfully logged in.
		TimeOut,                        // A timeout occurred
		TestFailure,                    // A test is known to have failed
		EngineTestError,                // Unreal exited with an error code related to Engine Tests
		Unknown,                        // Something not in the above
	}
}
