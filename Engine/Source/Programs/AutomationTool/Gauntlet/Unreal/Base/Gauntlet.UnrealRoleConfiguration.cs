// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using AutomationTool;

namespace Gauntlet
{
	/// <summary>
	/// UnrealRoleConfigurations can be added to an UnrealTestRole to modularize the creation of commandline arguments.
	/// A common use case is generating args for a the CSV Profiler. You would normally need to manually create the arguments for every role,
	/// but with an UnrealRoleConfiguration you can set a couple properties and the commandline will be generated and applied automatically.
	///
	/// Experimental feature! Subject to rapid change.
	/// </summary>
	public interface IUnrealRoleConfiguration
	{
		void ApplyConfigToRole(UnrealTestRole Role);
		void VerifyRoleConfig(UnrealTestRole Role);
	}
}