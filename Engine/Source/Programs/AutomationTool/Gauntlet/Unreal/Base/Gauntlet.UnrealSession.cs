// Copyright Epic Games, Inc. All Rights Reserved.

using AutomationTool;
using EpicGames.Core;
using System;
using System.Collections.Generic;
using System.IO;
using System.Linq;
using System.Text.RegularExpressions;
using System.Threading;
using UnrealBuildTool;
using Gauntlet.Utils;
using System.Net;
using Microsoft.Extensions.Logging;

namespace Gauntlet
{
	public enum ERoleModifier
	{
		None,
		Dummy,
		Null
	};
	
	/// <summary>
	/// Represents a role that will be performed in an Unreal Session
	/// </summary>
	public class UnrealSessionRole
	{

		/// <summary>
		/// Type of role
		/// </summary>
		public UnrealTargetRole RoleType;

		/// <summary>
		/// Platform this role uses
		/// </summary>
		public UnrealTargetPlatform? Platform;

		/// <summary>
		/// Configuration this role runs in
		/// </summary>
		public UnrealTargetConfiguration Configuration;

		/// <summary>
		/// Constraints this role runs under
		/// </summary>
		public UnrealDeviceTargetConstraint Constraint;

		/// <summary>
		/// Options that this role needs
		/// </summary>
		public IConfigOption<UnrealAppConfig> Options;


		/// <summary>
		/// Unique tag for logging purposes
		/// </summary>
		public string Tag;

		/// <summary>
		/// Command line that this role will use
		/// </summary>
		public string CommandLine
		{
			get
			{
				if (CommandLineParams == null)
				{
					CommandLineParams = new GauntletCommandLine();
				}
				return CommandLineParams.GenerateFullCommandLine();
			}
			set
			{
				if (CommandLineParams == null)
				{
					CommandLineParams = new GauntletCommandLine();
				}

				CommandLineParams.ClearCommandLine();

				CommandLineParams.AddRawCommandline(value, false);
			}
		}
		/// <summary>
		/// Dictionary of commandline arguments that are turned into a commandline at the end.
		/// For flags, leave the value set to null. Generated from the Test Role's commandline object
		/// and modified as needed. Then passed through to the AppConfig in UnrealBuildSource.cs
		/// </summary>
		public GauntletCommandLine CommandLineParams { get; set; }


		/// <summary>
		/// Map override to use on a server in case we don't want them all running the same map.
		/// </summary>
		public string MapOverride;

		/// <summary>
		/// List of files to copy to the device.
		/// </summary>
		public List<UnrealFileToCopy> FilesToCopy;

		/// <summary>
		/// Additional UE directories to copy from when saving artifacts
		/// </summary>
		public List<EIntendedBaseCopyDirectory> AdditionalArtifactDirectories;

		/// <summary>
		/// Allows for individual roles to configure their associated Target device before build installation
		/// </summary>
		public PreConfigureDeviceHandler PreConfigureDevice;

		/// <summary>
		/// Allows for individual roles to configure their associated Target device after build installation and AppInstall creation
		/// </summary>
		public ConfigureDeviceHandler ConfigureDevice;

		/// <summary>
		/// Properties we require our build to have
		/// </summary>
		public BuildFlags RequiredBuildFlags;

		/// <summary>
		/// Flavor of the build
		/// </summary>
		public string RequiredFlavor;

		/// <summary>
		/// Should be represented by a null device?
		/// </summary>
		public ERoleModifier RoleModifier;

		/// <summary>
		/// Is this a dummy executable?
		/// </summary>
		public bool IsDummy() { return RoleModifier == ERoleModifier.Dummy; }

		/// <summary>
		/// Whether this role should be responsible only for installing the build and not monitoring a process.
		/// </summary>
		public bool InstallOnly { get; set; }

		/// <summary>
		/// Whether this role will launched by the test node at a later time, typically during TickTest(). By default, all roles are launched immediately.
		/// </summary>
		public bool DeferredLaunch { get; set; }

		/// <summary>
		/// Whether this role will compress screenshots produced as an artifact into a jpeg format
		/// </summary>
		public bool CompressScreenshots { get; set; }
		
		/// <summary>
		/// Whether this role will create a gif from screenshots produced as artifacts
		/// </summary>
		public bool CreateGifFromScreenshots { get; set; }

		/// <summary>
		/// Whether or not this role should skip the cleaning of its device's artifact between each test run
		/// </summary>
		public bool SkipCleanDeviceArtifacts { get; set; }

		/// <summary>
		/// Whether or not this role will save artifacts when running with -dev. 
		/// These can be huge, so be cautious with using this.
		/// </summary>
		public bool ArchiveDevArtifacts { get; set; }

		/// <summary>
		/// Whether to respawn this process automatically if it fails unexpectedly
		/// </summary>
		public bool RespawnProcessOnUnexpectedExit { get; set; }

		/// <summary>
		/// What to do if this role exits unexpectedly.
		/// </summary>
		public OnTryToRestoreProcessDelegate OnTryToRestoreProcess;

		/// <summary>
		/// Is this role Null?
		/// </summary>
		public bool IsNullRole() { return RoleModifier == ERoleModifier.Null; }


		/// <summary>
		/// Constructor taking limited params
		/// </summary>
		/// <param name="InType"></param>
		/// <param name="InPlatform"></param>
		/// <param name="InConfiguration"></param>
		/// <param name="InOptions"></param>
		public UnrealSessionRole(UnrealTargetRole InType, UnrealTargetPlatform? InPlatform, UnrealTargetConfiguration InConfiguration, IConfigOption<UnrealAppConfig> InOptions)
			: this(InType, InPlatform, InConfiguration, null, InOptions)
		{
		}

		/// <summary>
		/// Constructor taking optional params
		/// </summary>
		/// <param name="InType"></param>
		/// <param name="InPlatform"></param>
		/// <param name="InConfiguration"></param>
		/// <param name="InCommandLine"></param>
		/// <param name="InOptions"></param>
		public UnrealSessionRole(UnrealTargetRole InType, UnrealTargetPlatform? InPlatform, UnrealTargetConfiguration InConfiguration, string InCommandLine = null, IConfigOption<UnrealAppConfig> InOptions = null, bool CompressScreenshots = true, bool CreateGifFromScreenshots = true)
		{
			RoleType = InType;

			Platform = InPlatform;
			Configuration = InConfiguration;
			MapOverride = string.Empty;

			if (string.IsNullOrEmpty(InCommandLine))
			{
				CommandLine = string.Empty;
			}
			else
			{
				CommandLine = InCommandLine;
			}

			RequiredBuildFlags = BuildFlags.None;

			if (Globals.IsRunningDev && !RoleType.UsesEditor())
			{
				RequiredBuildFlags |= BuildFlags.CanReplaceExecutable;
			}

			// Enforce build flags for the platform build that support it
			IDeviceBuildSupport TargetBuildSupport = InterfaceHelpers.FindImplementations<IDeviceBuildSupport>().Where(B => B.CanSupportPlatform(InPlatform)).FirstOrDefault();
			if (TargetBuildSupport != null)
			{
				if (Globals.Params.ParseParam("bulk") && TargetBuildSupport.CanSupportBuildType(BuildFlags.Bulk))
				{
					RequiredBuildFlags |= BuildFlags.Bulk;
				}
				else if (TargetBuildSupport.CanSupportBuildType(BuildFlags.NotBulk))
				{
					RequiredBuildFlags |= BuildFlags.NotBulk;
				}

				if (Globals.Params.ParseParam("packaged") && TargetBuildSupport.CanSupportBuildType(BuildFlags.Packaged))
				{
					RequiredBuildFlags |= BuildFlags.Packaged;
				}
				else if (Globals.Params.ParseParam("staged") && TargetBuildSupport.CanSupportBuildType(BuildFlags.Loose))
				{
					RequiredBuildFlags |= BuildFlags.Loose;
				}
			}

			string RoleName = RoleType.ToString().ToLower();
			RequiredFlavor = Globals.Params.ParseValue(RoleName + "flavor", "");

			InstallOnly = false;
			DeferredLaunch = false;
			Options = InOptions;
            FilesToCopy = new List<UnrealFileToCopy>();
			CommandLineParams = new GauntletCommandLine();
			RoleModifier = ERoleModifier.None;
			this.CompressScreenshots = CompressScreenshots;
			this.CreateGifFromScreenshots = CreateGifFromScreenshots;
		}

        /// <summary>
        /// Debugging aid
        /// </summary>
        /// <returns></returns>
        public override string ToString()
		{
			return string.Format("{0} {1} {2} {3}", Platform, Configuration, RoleType, RequiredFlavor);
		}
	}

	/// <summary>
	/// Represents an instance of a running an Unreal session. Basically an aggregate of all processes for
	/// all roles (clients, server, etc
	///
	/// TODO - combine this into UnrealSession
	/// </summary>
	public class UnrealSessionInstance : IDisposable
	{
		/// <summary>
		/// Represents an running role in our session
		/// </summary>
		public class RoleInstance
		{
			public RoleInstance(UnrealSessionRole InRole, IAppInstance InInstance)
			{
				Role = InRole;
				AppInstance = InInstance;
			}

			/// <summary>
			/// Role that is being performed in this session
			/// </summary>
			public UnrealSessionRole Role { get; protected set; }

			/// <summary>
			/// Underlying AppInstance that is running the role
			/// </summary>
			public IAppInstance AppInstance { get; set; }

			/// <summary>
			/// Debugging aid
			/// </summary>
			/// <returns></returns>
			public override string ToString()
			{
				return Role.ToString();
			}
		};

		/// <summary>
		/// All roles
		/// </summary>
		public RoleInstance[] AllRoles { get; protected set; }

		/// <summary>
		/// All running roles
		/// </summary>
		public IEnumerable<RoleInstance> RunningRoles { get { return AllRoles.Where( X => X.AppInstance != null ); } }

		/// <summary>
		/// All deferred roles
		/// </summary>
		public IEnumerable<RoleInstance> DeferredRoles { get { return DeferredRoleToAppInstall.Keys; } }

		/// <summary>
		/// All deferred roles and their associated IAppInstall
		/// </summary>
		public Dictionary<RoleInstance, IAppInstall> DeferredRoleToAppInstall { get; protected set; }

		/// <summary>
		/// Helper for accessing all client processes. May return an empty array if no clients are involved
		/// </summary>
		public IAppInstance[] ClientApps
		{
			get
			{
				return RunningRoles.Where(R => R.Role.RoleType.IsClient()).Select(R => R.AppInstance).ToArray();
			}
		}

		/// <summary>
		/// Helper for accessing server process. May return null if no server is involved
		/// </summary>
		public IAppInstance ServerApp
		{
			get
			{
				return RunningRoles.Where(R => R.Role.RoleType.IsServer()).Select(R => R.AppInstance).FirstOrDefault();
			}
		}

		/// <summary>
		/// Helper for accessing editor process. May return null if no editor is involved
		/// </summary>
		public IAppInstance EditorApp
		{
			get
			{
				return RunningRoles.Where(R => R.Role.RoleType.IsEditor()).Select(R => R.AppInstance).FirstOrDefault();
			}
		}

		/// <summary>
		/// Helper that returns true if clients are currently running
		/// </summary>
		public bool ClientsRunning
		{
			get
			{
				return ClientApps != null && ClientApps.Where(C => C.HasExited).Count() == 0;
			}
		}

		/// <summary>
		/// Helper that returns true if there's a running server
		/// </summary>
		public bool ServerRunning
		{
			get
			{
				return ServerApp != null && ServerApp.HasExited == false;
			}
		}

		/// <summary>
		/// Returns true if any of our roles are still running
		/// </summary>
		public bool IsRunningRoles
		{
			get
			{
				return RunningRoles.Any(R => R.AppInstance.HasExited == false);
			}
		}

		/// <summary>
		/// Constructor. Roles must be passed in
		/// </summary>
		/// <param name="InAllRoles"></param>
		/// <param name="InDeferredRoleToAppInstall"></param>
		public UnrealSessionInstance(RoleInstance[] InAllRoles, Dictionary<RoleInstance, IAppInstall> InDeferredRoleToAppInstall = null)
		{
			AllRoles = InAllRoles;
			DeferredRoleToAppInstall = InDeferredRoleToAppInstall;
		}

		~UnrealSessionInstance()
		{
			Dispose(false);
		}

		#region IDisposable Support
		private bool disposedValue = false; // To detect redundant calls

		protected virtual void Dispose(bool disposing)
		{
			if (!disposedValue)
			{
				if (disposing)
				{
					// TODO: dispose managed state (managed objects).
				}

				Shutdown();
				disposedValue = true;
			}
		}

		// This code added to correctly implement the disposable pattern.
		public void Dispose()
		{
			// Do not change this code. Put cleanup code in Dispose(bool disposing) above.
			Dispose(true);
		}
		#endregion


		/// <summary>
		/// Returns the app install for a given role, where the role was marked as 'DeferredLaunch'
		/// </summary>
		/// <param name="Role"></param>
		public IAppInstall FindInstallForDeferredRole( UnrealSessionRole Role )
		{
			IAppInstall AppInstall = DeferredRoleToAppInstall
				.Where(X => X.Key.Role == Role)
				.Select(X => X.Value)
				.FirstOrDefault();

			if (AppInstall == null)
			{
				Log.Error("Cannot find derferred role {0}", Role.ToString());
			}

			return AppInstall;
		}


		/// <summary>
		/// Launches a role that was previously flagged as 'DeferredLaunch'. Note that this does not handle device failure, marking problem devices etc.
		/// </summary>
		/// <param name="Role"></param>
		/// <returns></returns>
		public bool LaunchDeferredRole( UnrealSessionRole Role )
		{
			return LaunchDeferredRole(Role, out _);
		}

		/// <summary>
		/// Launches a role that was previously flagged as 'DeferredLaunch'. Note that this does not handle device failure, marking problem devices etc.
		/// </summary>
		/// <param name="Role"></param>
		/// <param name="RoleInstance"></param>
		/// <returns></returns>
		public bool LaunchDeferredRole(UnrealSessionRole Role, out RoleInstance RoleInstance)
		{
			RoleInstance = null;
			if (Role.DeferredLaunch == false)
			{
				Log.Error("Cannot start deferred role {0} because it is not marked 'DeferredLaunch'", Role.ToString());
				return false;
			}

			int CurrentRoleInstanceIndex = AllRoles.FindIndex(X => X.Role == Role);
			if (CurrentRoleInstanceIndex < 0)
			{
				Log.Error("Cannot start dereferred role {0} because it cannot be found among the 'all roles' list", Role.ToString());
				return false;
			}

			IAppInstall CurrentInstall = FindInstallForDeferredRole(Role);

			bool Success = false;

			try
			{
				Log.Info("Starting deferred {0} on {1}", Role, CurrentInstall.Device);
				IAppInstance Instance = CurrentInstall.Run();
				IDeviceUsageReporter.RecordStart(Instance.Device.Name, Instance.Device.Platform, IDeviceUsageReporter.EventType.Test);

				if (Instance != null || Globals.CancelSignalled)
				{
					// remove deferred role and update to a running role instance
					DeferredRoleToAppInstall.Remove(AllRoles[CurrentRoleInstanceIndex]);
					RoleInstance = new RoleInstance(Role, Instance);
					AllRoles[CurrentRoleInstanceIndex] = RoleInstance;
				}

				Success = true;
			}
			catch (DeviceException Ex)
			{
				Log.Error("Cannot start deferred role {0} because device {Name} threw an exception during launch. \nException={Exception}", Role.ToString(), CurrentInstall.Device, Ex.Message);
				Success = false;
			}

			return Success;
		}

		/// <summary>
		/// Tries to recover a process for a role that exited unexpectedly.
		/// </summary>
		/// <param name="Role"></param>
		/// <returns></returns>
		public bool RecoverProcessForRole(UnrealSessionRole Role, IAppInstall CurInstall)
		{
			int CurrentRoleInstanceIndex = AllRoles.FindIndex(X => X.Role == Role);
			if (CurrentRoleInstanceIndex < 0)
			{
				Log.Error("Cannot restart role {0} because it cannot be found among the 'all roles' list", Role.ToString());
				return false;
			}
			bool Success = false;

			try
			{
				Log.Info("Attempting to restore {0} on {1}", Role, CurInstall.Device);
				IAppInstance Instance = CurInstall.Run();
			
				if (Instance != null || Globals.CancelSignalled)
				{
					// Update to the new wowrking instance.
					AllRoles[CurrentRoleInstanceIndex].AppInstance = Instance;
				}

				Success = true;
			}
			catch (DeviceException Ex)
			{
				Log.Error("Cannot restart role {0} because device {Name} threw an exception during launch. \nException={Exception}", Role.ToString(), CurInstall.Device, Ex.Message);
				Success = false;
			}

			return Success;
		}

		/// <summary>
		/// Shutdown the session by killing any remaining processes.
		/// </summary>
		/// <returns></returns>
		public void Shutdown(bool GenerateDumpOnKill = false)
		{
			// Kill any remaining client processes
			if (ClientApps != null)
			{
				List<IAppInstance> RunningApps = ClientApps.Where(App =>( App != null && App.HasExited == false)).ToList();

				if (RunningApps.Count > 0)
				{
					Log.Info("Shutting down {0} clients", RunningApps.Count);
					RunningApps.ForEach(App =>
					{
						App.Kill(GenerateDumpOnKill);
						// Apps that are still running have timed out => fail
						IDeviceUsageReporter.RecordEnd(App.Device.Name, App.Device.Platform, IDeviceUsageReporter.EventType.Test, IDeviceUsageReporter.EventState.Success);
					});
				}

				List<IAppInstance> ClosedApps = ClientApps.Where(App => (App != null && App.HasExited == true)).ToList();
				if(ClosedApps.Count > 0)
				{
					ClosedApps.ForEach(App =>
					{
						// Apps that have already exited have 'succeeded'
						IDeviceUsageReporter.RecordEnd(App.Device.Name, App.Device.Platform, IDeviceUsageReporter.EventType.Test, IDeviceUsageReporter.EventState.Failure);
					});
				}
			}

			if (ServerApp != null)
			{
				if (ServerApp.HasExited == false)
				{
					Log.Info("Shutting down server");
					ServerApp.Kill(GenerateDumpOnKill);
				}
			}

			// kill anything that's left
			RunningRoles.Where(R => !R.Role.InstallOnly && R.AppInstance.HasExited == false).ToList().ForEach(R => R.AppInstance.Kill(GenerateDumpOnKill));

			// Wait for it all to end
			RunningRoles.Where(R => !R.Role.InstallOnly).ToList().ForEach(R => R.AppInstance.WaitForExit());

			Thread.Sleep(3000);
		}
	}

	/// <summary>
	/// Represents the set of available artifacts available after an UnrealSessionRole has completed
	/// </summary>
	public class UnrealRoleArtifacts
	{
		/// <summary>
		/// Session role info that created these artifacts
		/// </summary>
		public UnrealSessionRole SessionRole { get; protected set; }

		/// <summary>
		/// AppInstance that was used to run this role
		/// </summary>
		public IAppInstance AppInstance { get; protected set; }

		/// <summary>
		/// Path to artifacts from this role (these are local and were retried from the device).
		/// </summary>
		public string ArtifactPath { get; protected set; }

		/// <summary>
		/// Path to Log from this role
		/// </summary>
		public string LogPath { get; protected set; }

		/// <summary>
		/// Constructor, all values must be provided
		/// </summary>
		/// <param name="InSessionRole"></param>
		/// <param name="InAppInstance"></param>
		/// <param name="InArtifactPath"></param>
		/// <param name="InLogPath"></param>
		public UnrealRoleArtifacts(UnrealSessionRole InSessionRole, IAppInstance InAppInstance, string InArtifactPath, string InLogPath)
		{
			SessionRole = InSessionRole;
			AppInstance = InAppInstance;
			ArtifactPath = InArtifactPath;
			LogPath = InLogPath;
		}
	}


	/// <summary>
	/// Helper class that understands how to launch/monitor/stop an Unreal test (clients + server) based on params contained in the test context and config
	/// </summary>
	public class UnrealSession : IDisposable
	{
		/// <summary>
		/// Device reservation instance of this session
		/// </summary>
		public UnrealDeviceReservation UnrealDeviceReservation { get; private set; }

		/// <summary>
		/// Source of the build that will be launched
		/// </summary>
		protected UnrealBuildSource BuildSource { get; set; }

		/// <summary>
		/// Roles that will be performed by this session
		/// </summary>
		protected UnrealSessionRole[] SessionRoles { get; set; }

		/// <summary>
		/// Running instance of this session
		/// </summary>
		public UnrealSessionInstance SessionInstance { get; protected set; }

		/// <summary>
		/// Sandbox for installed apps
		/// </summary>
		public string Sandbox { get; set; }

		/// <summary>
		/// Whether or not devices should be retained between each test iteration
		/// </summary>
		public bool ShouldRetainDevices { get; set; }

		[AutoParam(false)]
		public bool ReinstallPerPass { get; set; }

		/// <summary>
		/// Number of attempts when launching a session.
		/// Failed installs and runs will trigger a re-try
		/// </summary>
		public int LaunchSessionAttempts { get; set; }

		/// <summary>
		/// Number of attempts when trying to reserve devices
		/// </summary>
		public int DeviceReservationAttempts { get; set; }

		/// <summary>
		/// Number of seconds to wait between failed device reservation attempts
		/// </summary>
		public int DeviceReservationRetryTime { get; set; }

		/// <summary>
		/// Record of each ITargetDevice assigned to a given role
		/// </summary>
		public Dictionary<UnrealSessionRole, ITargetDevice> RolesToDevices { get; private set; }

		/// <summary>
		/// Record of each UnrealAppConfig created for each role
		/// </summary>
		public Dictionary<UnrealSessionRole, UnrealAppConfig> RolesToConfigs { get; private set; }

		/// <summary>
		/// Record of our installations in case we want to re-use them in a later pass
		/// </summary>
		public Dictionary<UnrealSessionRole, IAppInstall> RolesToInstalls { get; private set; }

		/// <summary>
		/// Constructor that takes a build source and a number of roles
		/// </summary>
		/// <param name="InSource"></param>
		/// <param name="InSessionRoles"></param>
		public UnrealSession(UnrealBuildSource InSource, IEnumerable<UnrealSessionRole> InSessionRoles)
		{
			AutoParam.ApplyParamsAndDefaults(this, Globals.Params.AllArguments);

			BuildSource = InSource;
			SessionRoles = InSessionRoles.ToArray();
			ShouldRetainDevices = !Globals.Params.ParseParam("ReacquireDevicesPerPass");

			RolesToDevices = new Dictionary<UnrealSessionRole, ITargetDevice>();
			RolesToConfigs = new Dictionary<UnrealSessionRole, UnrealAppConfig>();
			RolesToInstalls = new Dictionary<UnrealSessionRole, IAppInstall>();

			LaunchSessionAttempts = 3;
			DeviceReservationAttempts = 5;
			DeviceReservationRetryTime = 120;

			if (SessionRoles.Length == 0)
			{
				throw new AutomationException("No roles specified for Unreal session");
			}

			List<string> ValidationIssues = new List<string>();

			if (!CheckRolesArePossible(ref ValidationIssues))
			{
				ValidationIssues.ForEach(S => Log.Error(KnownLogEvents.Gauntlet_BuildDropEvent, S));
				Horde.ReportInvalidatingIssue(Horde.GauntletHordeStepMessage.BuildNotFound, "The test was unable to find the necessary executables.");
				throw new AutomationException("One or more issues occurred when validating build {0} against requested roles", InSource.BuildName);
			}

			UnrealDeviceReservation = new UnrealDeviceReservation();
		}

		/// <summary>
		/// Destructor, terminates any running session
		/// </summary>
		~UnrealSession()
		{
			Dispose(false);
		}

		#region IDisposable Support
		private bool disposedValue = false; // To detect redundant calls

		protected virtual void Dispose(bool disposing)
		{
			if (!disposedValue)
			{
				if (disposing)
				{
					// TODO: dispose managed state (managed objects).
				}

				ShutdownSession();
				disposedValue = true;
			}
		}

		// This code added to correctly implement the disposable pattern.
		public void Dispose()
		{
			// Do not change this code. Put cleanup code in Dispose(bool disposing) above.
			Dispose(true);
		}
		#endregion

		/// <summary>
		/// Helper that reserves and returns a list of available devices based on the passed in roles
		/// </summary>
		/// <returns></returns>
		public bool TryReserveDevices()
		{
			if (ShouldRetainDevices && HasAcquiredDevices())
			{
				return true;
			}

			// Figure out how many of each device we need
			Dictionary<UnrealDeviceTargetConstraint, int> RequiredDeviceTypes = new Dictionary<UnrealDeviceTargetConstraint, int>();
			IEnumerable<UnrealSessionRole> RolesThatRequireDevice = SessionRoles
				.Where(Role => !Role.IsNullRole())
				.Where(RoleNeedsDevice);

			// Get a count of the number of devices required for each platform
			foreach (UnrealSessionRole Role in RolesThatRequireDevice)
			{
				if (RequiredDeviceTypes.ContainsKey(Role.Constraint))
				{
					++RequiredDeviceTypes[Role.Constraint];
				}
				else
				{
					RequiredDeviceTypes.Add(Role.Constraint, 1);
				}
			}

			try
			{
				return UnrealDeviceReservation.TryReserveDevices(RequiredDeviceTypes, RolesThatRequireDevice.Count(), true);
			}
			catch (Exception ex)
			{
				if (ex.InnerException != null && ex.InnerException is WebException
					&& (ex.InnerException as WebException).Response != null
					&& ((ex.InnerException as WebException).Response is HttpWebResponse WebResponse)
					&& WebResponse.StatusCode == HttpStatusCode.Conflict)
				{
					// Reservation failed, usually for lack of devices
					// Return false to let attempt loop to kick in
					Log.Info(ex.Message);
					return false;
				}

				throw;
			}
		}

		public bool TryReserveDevices(int Attempts)
		{
			for (int RemainingAttempts = Attempts; RemainingAttempts > 0; --RemainingAttempts)
			{
				if(Globals.CancelSignalled)
				{
					ReleaseSessionDevices(true);
					return false;
				}

				if(TryReserveDevices())
				{
					return true;
				}
				else
				{
					Thread.Sleep(1000 * DeviceReservationRetryTime);
				}
			}

			Log.Error("Failed to reserve devices after {Attempts} attempts", Attempts);
			ReleaseSessionDevices(true);
			return false;
		}

		/// <summary>
		/// Check that all the current roles can be performed by our build source
		/// </summary>
		/// <param name="Issues"></param>
		/// <returns></returns>
		bool CheckRolesArePossible(ref List<string> Issues)
		{
			bool Success = true;
			foreach (var Role in SessionRoles)
			{
				if (!BuildSource.CanSupportRole(Role, ref Issues))
				{
					Success = false;
				}
			}

			return Success;
		}

		/// <summary>
		/// Installs and launches all of our roles and returns an UnrealSessonInstance that represents the aggregate
		/// of all of these processes. Will perform retries if errors with devices are encountered so this is a "best
		/// attempt" at running with the devices provided
		/// </summary>
		/// <returns></returns>
		public UnrealSessionInstance LaunchSession()
		{
			if(Globals.Params.ParseParam("LegacyLaunchSession"))
			{
				return Legacy_LaunchSession();
			}

			// Clear any existing session from a previous iteration
			SessionInstance = null;

			// When launching, issues with devices may be encountered.
			// When these issues occur, those devices will be marked as problem devices and returned to the pool.
			// A new set of devices will then be reserved and another attempt at launching the processes will be made.
			// If LaunchSession() fails LaunchSessionAttempts amount of times, an exception is thrown.
			for (int RemainingAttempts = LaunchSessionAttempts; RemainingAttempts > 0; --RemainingAttempts)
			{
				// Reserve devices, if needed
				if (!TryReserveDevices(DeviceReservationAttempts))
				{
					if(Globals.CancelSignalled)
					{
						return null;
					}
					// If device reservation fails, the device pool cannot support this launch.
					Horde.ReportInvalidatingIssue(Horde.GauntletHordeStepMessage.NotEnoughDevices, "The test was unable to secure enough devices to run.");
					throw new AutomationException("Failed to acquire all devices for launch. See above for details.");
				}

				if(Globals.CancelSignalled)
				{
					return null;
				}

				if(!TryAssignDevicesToRoles())
				{
					// If device assignment fails, reservations were likely deleted at an unexpected time.
					ReleaseSessionDevices(true);
					continue;
				}

				// All roles should now be assigned a device.
				// Install necessary builds, clear stale test artifacts, and copy any additional files
				try
				{
					ReadyDevicesForSession();
				}
				catch (Exception Ex)
				{
					if(IsOutOfSpaceException(Ex) || IsOverlayException(Ex) || Ex is not DeviceException)
					{
						RemainingAttempts = 0;
						ReleaseSessionDevices(true);
						continue;
					}
					else if (RemainingAttempts > 1)
					{
						Log.Info("A new device will be selected and another attempt at launching session will be made.");
					}

					ReleaseProblemDevices();
					continue;
				}

				if (Globals.CancelSignalled)
				{
					ReleaseSessionDevices(true);
					return null;
				}

				// All roles should now be assigned device with an associated IAppInstall.
				// Launch all the processes!
				try
				{
					SessionInstance = LaunchProcesses();
				}
				catch
				{
					if (RemainingAttempts > 1)
					{
						Log.Info("A new device will be selected and another attempt at launching session will be made.");
					}

					ReleaseProblemDevices();
					continue;
				}

				if (Globals.CancelSignalled)
				{
					ReleaseSessionDevices(true);
					return null;
				}

				return SessionInstance;
			}

			return null;
		}

		/// <summary>
		/// Restarts the current session (if any)
		/// </summary>
		/// <returns></returns>
		public UnrealSessionInstance RestartSession()
		{
			ShutdownSession();

			// AG-TODO - want to preserve device reservations here...

			return LaunchSession();
		}

		///<summary>
		/// Shuts down any running apps
		/// </summary>
		public void ShutdownInstance()
		{
			if (SessionInstance != null)
			{
				SessionInstance.Dispose();
				SessionInstance = null;
			}
		}

		/// <summary>
		/// Shuts down the current session (if any)
		/// </summary>
		/// <returns></returns>
		public void ShutdownSession()
		{
			ShutdownInstance();

			if (!ShouldRetainDevices)
			{
				ReleaseSessionDevices();
			}
		}

		public void ReleaseSessionDevices(bool bForce = false)
		{
			// Terminate any running apps
			if (SessionInstance != null && SessionInstance.RunningRoles != null)
			{
				foreach (UnrealSessionInstance.RoleInstance RunningRole in SessionInstance.RunningRoles)
				{
					Log.Info("Shutting down {0}", RunningRole.AppInstance.Device);
					RunningRole.AppInstance.Kill();
					RunningRole.AppInstance.Device.Disconnect();
				}
			}

			if (UnrealDeviceReservation == null)
			{
				return;
			}

			UnrealDeviceReservation.ReleaseDevices(bForce);

			RolesToDevices.Clear();
			RolesToConfigs.Clear();
			RolesToInstalls.Clear();
		}

		public void ReleaseProblemDevices()
		{
			// Terminate any running apps
			if (SessionInstance != null && SessionInstance.RunningRoles != null)
			{
				foreach (UnrealSessionInstance.RoleInstance RunningRole in SessionInstance.RunningRoles.Where(Role => Role.AppInstance != null))
				{
					Log.Info("Shutting down {0}", RunningRole.AppInstance.Device);
					RunningRole.AppInstance.Kill();
					RunningRole.AppInstance.Device.Disconnect();
				}
			}

			if (UnrealDeviceReservation == null)
			{
				return;
			}

			IEnumerable<ITargetDevice> ProblemDevices = UnrealDeviceReservation.ReleaseProblemDevices();
			IEnumerable<UnrealSessionRole> RolesToClear = RolesToDevices.Keys.Where(Role => ProblemDevices.Contains(RolesToDevices[Role]));

			Log.Info("Released problem devices...");
			foreach(UnrealSessionRole Role in RolesToClear)
			{
				Log.Info("\t {DeviceName}", RolesToDevices[Role].Name);
				RolesToDevices.Remove(Role);
				RolesToConfigs.Remove(Role);
				RolesToInstalls.Remove(Role);
			}
		}

		/// <summary>
		/// Retrieves and saves all artifacts from the provided session role. Artifacts are saved to the destination path
		/// </summary>
		/// <param name="InContext"></param>
		/// <param name="InRunningRole"></param>
		/// <param name="DestinationArtifactPath"></param>
		/// <returns></returns>
		public UnrealRoleArtifacts SaveRoleArtifacts(UnrealTestContext InContext, UnrealSessionInstance.RoleInstance InRunningRole, string DestinationArtifactPath)
		{
			string DummyOutputString = null;
			return SaveRoleArtifacts(InContext, InRunningRole, DestinationArtifactPath, ref DummyOutputString);
		}

		public UnrealRoleArtifacts SaveRoleArtifacts(UnrealTestContext InContext, UnrealSessionInstance.RoleInstance InRunningRole, string DestinationArtifactPath, ref string DirectoryToCleanUp)
		{
			DirectoryInfo SourceDirectory = new DirectoryInfo(InRunningRole.AppInstance.ArtifactPath);
			DirectoryInfo DestinationDirectory = new DirectoryInfo(DestinationArtifactPath);
			DestinationDirectory.Create();

			// Whether this is a Dummy, Client, Server, Editor, etc
			string RoleName = (InRunningRole.Role.IsDummy() ? "Dummy" : "") + InRunningRole.Role.RoleType.ToString();

			// Move over any additional artifacts that a role requested (before they are removed potentially later)
			foreach (EIntendedBaseCopyDirectory AdditionalDirectory in InRunningRole.Role.AdditionalArtifactDirectories)
			{
				Dictionary<EIntendedBaseCopyDirectory, string> PlatformMappings = InRunningRole.AppInstance.Device.GetPlatformDirectoryMappings();
				if (PlatformMappings.ContainsKey(AdditionalDirectory))
				{
					DirectoryInfo AdditionalSourceDirectory = new DirectoryInfo(PlatformMappings[AdditionalDirectory]);
					if (AdditionalSourceDirectory.Exists)
					{
						string TargetDirectory = Path.Combine(DestinationDirectory.FullName, AdditionalSourceDirectory.Name);

						try
						{
							SystemHelpers.CopyDirectory(AdditionalSourceDirectory.FullName, TargetDirectory);
						}
						catch (Exception Exception)
						{
							Log.Warning("Encountered an {Exception} when trying to copy additional artifact directory {BaseCopyDirectory}" +
								" from {AdditionalSourceDirectory} to {TargetDirectory}.",
								Exception, AdditionalDirectory, AdditionalDirectory, TargetDirectory);
						}
					}
				}
			}

			// We only want to move any other artifacts for editor data if there was a crash on a buildmachine.
			// Also, don't move artifacts in dev mode, because peoples saved data could be huuuuuuuge!
			bool IsEditorBuild = InRunningRole.Role.RoleType.UsesEditor();
			bool IsBuildMachine = CommandUtils.IsBuildMachine;
			bool SkipArchivingAssets = (Globals.IsRunningDev && !InRunningRole.Role.ArchiveDevArtifacts) || (IsEditorBuild && (IsBuildMachine == false || InRunningRole.AppInstance.ExitCode == 0));
			bool bRetainArtifacts = InContext.TestParams.ParseParam("RetainDeviceArtifacts");

			// Check if we should copy artifacts
			if (!SkipArchivingAssets)
			{
				if (SourceDirectory.Exists)
				{
					// If a PersistentDownloadDirectory exists in the saved folder, delete it.
					foreach (DirectoryInfo SubDirectory in SourceDirectory.EnumerateDirectories("*", SearchOption.AllDirectories))
					{
						if (SubDirectory.Name.Equals(EIntendedBaseCopyDirectory.PersistentDownloadDir.ToString(), StringComparison.OrdinalIgnoreCase))
						{
							try
							{
								SubDirectory.Delete(true);
							}
							catch (Exception Exception)
							{
								Log.Info("Encountered a {Exception} when attempting to delete PersistentDownloadDirectory {Directory}. The PDD will be present in artifacts", Exception, SubDirectory);
							}
							break;
						}
					}

					// Perform the copy
					try
					{
						// Save screenshots and create a gif when not running a server
						if (!InRunningRole.Role.RoleType.IsServer())
						{
							SaveScreenshots(RoleName, SourceDirectory.FullName, DestinationDirectory.FullName, InRunningRole.Role.CompressScreenshots, InRunningRole.Role.CreateGifFromScreenshots);
						}
						
						// Copy remaining artifacts
						SystemHelpers.CopyDirectory(SourceDirectory.FullName, DestinationDirectory.FullName, SystemHelpers.CopyOptions.Default, TruncateLongPathFilter);
					}
					catch (Exception Exception)
					{
						bRetainArtifacts = true;
						Log.Warning("Encountered an {Exception} when copying saved artifacts from {SourceDirectory} to {DestinationDirectory}. " +
							"Artifacts will not be saved locally, but the source artifacts will not be deleted.", Exception, SourceDirectory, DestinationDirectory);
					}
				}
				else
				{
					Log.Info("Unable to find source artifact directory {SourceDirectory}", SourceDirectory.FullName);
				}
			}
			else
			{
				if (IsEditorBuild)
				{
					Log.Info("Skipping archival of assets for editor {Role}", RoleName);
				}
				else if (Globals.IsRunningDev)
				{
					Log.Info("Skipping archival of assets for dev build");
				}
			}

			// Now write the role's log file
			string ArtifactLogFilePath = Path.GetFullPath(Path.Combine(DestinationDirectory.FullName, RoleName + "Output.log"));
			try
			{
				if (!InRunningRole.AppInstance.WriteOutputToFile(ArtifactLogFilePath))
				{
					ArtifactLogFilePath = string.Empty;
				}
			}
			catch (Exception Ex)
			{
				string Message = "Encountered an {0} when attempting to write the {1} process log. The log will not be present in the artifacts.\n {2}";
				Log.Warning(Message, Ex.GetType().Name, RoleName, Ex.Message);
				ArtifactLogFilePath = string.Empty;
			}

			// By default we delete the source artifacts, but we keep them if requested
			// we also want to do this after the AppInstance has the chance to look at
			// the role's log file to ensure it gets to look at the full log before we 
			// delete this path.
			if (SkipArchivingAssets && !bRetainArtifacts && SourceDirectory.Exists)
			{
				if (DirectoryToCleanUp != null)
				{
					DirectoryToCleanUp = SourceDirectory.FullName;
				}
				else
				{
					// Account for any read-only files.
					void SetAttributesNormal(DirectoryInfo Directory)
					{
						foreach (FileInfo File in Directory.GetFiles())
						{
							File.Attributes = FileAttributes.Normal;
						}
						foreach (DirectoryInfo SubDirectory in Directory.GetDirectories())
						{
							SetAttributesNormal(SubDirectory);
						}
					}
					;
					try
					{
						SetAttributesNormal(SourceDirectory);
					}
					catch
					{
						Log.Info("Could not remove the read-only attribute from {SourceDirectory}.", SourceDirectory);
					}
					try
					{
						SourceDirectory.Delete(true);
					}
					catch (Exception Exception)
					{
						Log.Info("Encountered an exception when deleting source artifacts at {SourceDirectory}. Artifacts will remain on the device.\n\tError message: {Exception}", SourceDirectory, Exception.Message);
					}
				}
			}

			if (!string.IsNullOrEmpty(ArtifactLogFilePath))
			{
				Log.Info($"Wrote {RoleName} Log to {ArtifactLogFilePath}");

				// On build machines, copy all role logs to Horde.
				if (IsBuildMachine && Horde.IsHordeJob)
				{
					// Extract the log path portion that includes the Gauntlet test name. ie: UE.BootTest(Win64_Test_Client)\Client\ClientOutput.log
					// That is to handle situation where multiple tests are run within the same Gauntlet Session and logs get overwritten.
					string LogName = ArtifactLogFilePath.Replace(Path.GetFullPath(InContext.Options.LogDir), "").TrimStart(Path.DirectorySeparatorChar);
					if (Path.IsPathFullyQualified(LogName))
					{
						// The path was expected to be relative to LogDir, however it appeared to be an absolute path.
						// So we revert to default behavior and save the log directly in UAT log folder.
						LogName = RoleName + "Output.log";
					}
					string HordeLogFilePath = Path.GetFullPath(Path.Combine(CommandUtils.CmdEnv.LogFolder, LogName));
					Log.Verbose($"Copy log for Horde to {HordeLogFilePath}");
					string TargetDirectry = Path.GetDirectoryName(HordeLogFilePath);
					if (!Directory.Exists(TargetDirectry)) { Directory.CreateDirectory(TargetDirectry); }
					File.Copy(ArtifactLogFilePath, HordeLogFilePath, true);
				}
			}

			// TODO REMOVEME- this should go elsewhere, likely a utile that can be called or inserted by relevant test nodes.
			SavePSOs(InContext, InRunningRole, DestinationDirectory.FullName);
			// END REMOVEME

			// Save the Artifact filepath
			return new UnrealRoleArtifacts(InRunningRole.Role, InRunningRole.AppInstance, DestinationDirectory.FullName, ArtifactLogFilePath);
		}

		/// <summary>
		/// Saves all artifacts from the provided session to the specified output path.
		/// </summary>
		/// <param name="Context"></param>
		/// <param name="TestInstance"></param>
		/// <param name="OutputPath"></param>
		/// <returns></returns>
		public IEnumerable<UnrealRoleArtifacts> SaveRoleArtifacts(UnrealTestContext Context, UnrealSessionInstance TestInstance, string OutputPath)
		{
			int DummyClientCount = 0;
			Dictionary<UnrealTargetRole, int> RoleCounts = new Dictionary<UnrealTargetRole, int>();
			List<UnrealRoleArtifacts> AllArtifacts = new List<UnrealRoleArtifacts>();

			HashSet<string> DirectoriesToDelete = new();
			foreach (UnrealSessionInstance.RoleInstance App in TestInstance.RunningRoles)
			{
				string RoleName = (App.Role.IsDummy() ? "Dummy" : "") + App.Role.RoleType.ToString();
				string FolderName = RoleName;

				int RoleCount = 1;

				if (App.Role.IsDummy())
				{
					DummyClientCount++;
					RoleCount = DummyClientCount;
				}
				else
				{
					if (!RoleCounts.ContainsKey(App.Role.RoleType))
					{
						RoleCounts.Add(App.Role.RoleType, 1);
					}
					else
					{
						RoleCounts[App.Role.RoleType]++;
					}

					RoleCount = RoleCounts[App.Role.RoleType];
				}

				if (RoleCount > 1)
				{
					FolderName += string.Format("_{0:00}", RoleCount);
				}

				string DestPath = Path.Combine(OutputPath, FolderName);

				if (!App.Role.IsNullRole() && !App.Role.InstallOnly)
				{
					Log.VeryVerbose("Calling SaveRoleArtifacts, Role: {0}  Artifact Path: {1}", App.ToString(), App.AppInstance.ArtifactPath);
					UnrealRoleArtifacts Artifacts = null;
					ITargetDevice device = App.AppInstance.Device;
					IDeviceUsageReporter.RecordStart(device.Name, device.Platform, IDeviceUsageReporter.EventType.SavingArtifacts);
					try
					{
						string DirectoryToDelete = "";
						Artifacts = SaveRoleArtifacts(Context, App, DestPath, ref DirectoryToDelete);
						if (!string.IsNullOrEmpty(DirectoryToDelete))
						{
							DirectoriesToDelete.Add(DirectoryToDelete);
						}

					}
					catch (Exception SaveArtifactsException)
					{
						// Caught an exception -> report failure
						IDeviceUsageReporter.RecordEnd(device.Name, device.Platform, IDeviceUsageReporter.EventType.SavingArtifacts, IDeviceUsageReporter.EventState.Failure);

						// Retry once only, after rebooting device, if artifacts couldn't be saved.
						if (SaveArtifactsException.Message.Contains("A retry should be performed"))
						{
							Log.Info("Rebooting device and retrying save role artifacts once.");
							App.AppInstance.Device.Reboot();
							string DirectoryToDelete = "";
							Artifacts = SaveRoleArtifacts(Context, App, DestPath);
							if (!string.IsNullOrEmpty(DirectoryToDelete))
							{
								DirectoriesToDelete.Add(DirectoryToDelete);
							}
						}
						else
						{
							// Pass exception to the surrounding try/catch in UnrealTestNode while preserving the original callstack
							throw;
						}
					}
					// Did not catch -> successful reporting
					IDeviceUsageReporter.RecordEnd(device.Name, device.Platform, IDeviceUsageReporter.EventType.SavingArtifacts, IDeviceUsageReporter.EventState.Success);

					if (Artifacts != null)
					{
						AllArtifacts.Add(Artifacts);
					}
				}
				else
				{
					Log.Verbose("Skipping SaveRoleArtifacts for Null Role: {0}", App.ToString());
				}
			}

			// Account for any read-only files.
			void SetAttributesNormal(DirectoryInfo Directory)
			{
				foreach (FileInfo File in Directory.GetFiles())
				{
					File.Attributes = FileAttributes.Normal;
				}
				foreach (DirectoryInfo SubDirectory in Directory.GetDirectories())
				{
					SetAttributesNormal(SubDirectory);
				}
			}

			foreach (string DirectoryToDelete in DirectoriesToDelete)
			{
				DirectoryInfo Directory = new DirectoryInfo(DirectoryToDelete);
				try
				{
					SetAttributesNormal(Directory);
				}
				catch
				{
					Log.Info("Could not remove the read-only attribute from {SourceDirectory}.", Directory);
				}
				try
				{
					Directory.Delete(true);
				}
				catch (Exception Exception)
				{
					Log.Info("Encountered an exception when deleting source artifacts at {SourceDirectory}. Artifacts will remain on the device.\n\tError message: {Exception}", Directory, Exception.Message);
				}
			}


			return AllArtifacts;
		}

		private UnrealSessionInstance Legacy_LaunchSession()
		{
			SessionInstance = null;

			// The number of retries when launching session to avoid an endless loop if package can't be installed, network timeouts, etc
			int SessionRetries = 2;

			// tries to find devices and launch our session. Will loop until we succeed, we run out of devices/retries, or
			// something fatal occurs..
			while (SessionInstance == null && Globals.CancelSignalled == false)
			{
				int ReservationRetries = 5;
				int ReservationRetryWait = 120;

				IEnumerable<UnrealSessionRole> RolesNeedingDevices = SessionRoles.Where(R => R.IsNullRole() == false);

				while (UnrealDeviceReservation.ReservedDevices.Count() < RolesNeedingDevices.Count())
				{
					// get devices
					TryReserveDevices();

					if (Globals.CancelSignalled)
					{
						break;
					}

					// if we failed to get enough devices, show a message and wait
					if (UnrealDeviceReservation.ReservedDevices.Count() != SessionRoles.Count())
					{
						if (ReservationRetries == 0)
						{
							throw new AutomationException("Unable to acquire all devices for test.");
						}
						Log.Info("\nUnable to find enough device(s). Waiting {0} secs (retries left={1})\n", ReservationRetryWait, --ReservationRetries);
						Thread.Sleep(ReservationRetryWait * 1000);
					}
				}

				if (Globals.CancelSignalled)
				{
					return null;
				}

				Dictionary<IAppInstall, UnrealSessionRole> InstallsToRoles = new Dictionary<IAppInstall, UnrealSessionRole>();
				Dictionary<IAppInstall, UnrealAppConfig> InstallsToConfig = new Dictionary<IAppInstall, UnrealAppConfig>();

				// create a copy of our list
				IEnumerable<ITargetDevice> DevicesToInstallOn = UnrealDeviceReservation.ReservedDevices.ToArray();

				bool InstallSuccess = true;

				// sort by constraints, so that we pick constrained devices first
				List<UnrealSessionRole> SortedRoles = SessionRoles.OrderBy(R => R.Constraint.IsIdentity() ? 1 : 0).ToList();

				// first install all roles on these devices
				foreach (UnrealSessionRole Role in SortedRoles)
				{
					ITargetDevice Device = null;

					if (Role.IsNullRole() == false)
					{
						Device = DevicesToInstallOn.Where(D => D.IsConnected && D.Platform == Role.Platform
															&& (Role.Constraint.IsIdentity() || DevicePool.Instance.GetConstraint(D) == Role.Constraint)).First();

						DevicesToInstallOn = DevicesToInstallOn.Where(D => D != Device);
					}
					else
					{
						Device = new TargetDeviceNull(string.Format("Null{0}", Role.RoleType));
					}

					Role.PreConfigureDevice?.Invoke(Device, Role);

					IEnumerable<UnrealSessionRole> OtherRoles = SortedRoles.Where(R => R != Role);

					// create a config from the build source (this also applies the role options)
					UnrealAppConfig AppConfig = BuildSource.CreateConfiguration(Role, OtherRoles);

					// todo - should this be elsewhere?
					AppConfig.Sandbox = Sandbox;

					IAppInstall Install = null;
					if (RolesToInstalls == null || !RolesToInstalls.ContainsKey(Role) || ReinstallPerPass)
					{
						// Tag the device for report result
						if (BuildHostPlatform.Current.Platform != Device.Platform)
						{
							AppConfig.CommandLineParams.Add("DeviceTag", Device.Name);
						}

						IDeviceUsageReporter.RecordStart(Device.Name, Device.Platform, IDeviceUsageReporter.EventType.Device, IDeviceUsageReporter.EventState.Success);
						IDeviceUsageReporter.RecordStart(Device.Name, Device.Platform, IDeviceUsageReporter.EventType.Install, IDeviceUsageReporter.EventState.Success, BuildSource.BuildName);
						try
						{
							// Make sure the artifact paths are clean before installation
							if(!Role.SkipCleanDeviceArtifacts)
							{
                                Device.CleanArtifacts(AppConfig);
                            }

							Install = Device.InstallApplication(AppConfig);
							IDeviceUsageReporter.RecordEnd(Device.Name, Device.Platform, IDeviceUsageReporter.EventType.Install, IDeviceUsageReporter.EventState.Success);
						}
						catch (Exception Ex)
						{
							InstallSuccess = false;
							string ErrorMessage = string.Format("Encountered error setting up device {0} for role {1}. {2}", Device, Role, Ex);
							if (ErrorMessage.Contains("not enough space") && Device.Platform == BuildHostPlatform.Current.Platform)
							{
								Horde.ReportInvalidatingIssue(Horde.GauntletHordeStepMessage.NotEnoughDriveSpace, ErrorMessage);
								Log.Error(ErrorMessage);
								// If on desktop platform, we are not retrying.
								// It is unlikely that space is going to be made and InstallBuildParallel has marked the build path as problematic.
								SessionRetries = 0;
								break;
							}

							// Warn, ignore the device, and do not continue
							if (Ex is DeviceException)
							{
								UnrealDeviceReservation.MarkProblemDevice(Device, ErrorMessage);
							}
							else
							{
								Log.Warning(KnownLogEvents.Gauntlet_DeviceEvent, ErrorMessage);
							}

							IDeviceUsageReporter.RecordEnd(Device.Name, Device.Platform, IDeviceUsageReporter.EventType.Install, IDeviceUsageReporter.EventState.Failure);
							break;
						}


						if (Globals.CancelSignalled)
						{
							break;
						}

						// Device has app installed, give role a chance to configure device
						Role.ConfigureDevice?.Invoke(Device);

						InstallsToRoles[Install] = Role;
						InstallsToConfig[Install] = AppConfig;

						if (ReinstallPerPass)
						{
							RolesToInstalls[Role] = Install;
						}
					}
					else
					{
						Install = RolesToInstalls[Role];
						InstallsToRoles[Install] = Role;
						InstallsToConfig[Install] = AppConfig;
						Log.Info("Using previous install of {0} on {1}", Install.Name, Install.Device.Name);
					}
				}

				if (InstallSuccess == false)
				{
					ReleaseSessionDevices(true);

					if (SessionRetries == 0)
					{
						Horde.ReportInvalidatingIssue(Horde.GauntletHordeStepMessage.InvalidPerfSpec, "Unable to install application for session.");
						throw new AutomationException("Unable to install application for session.");
					}

					Log.Info("\nUnable to install application for session (retries left={0})\n", --SessionRetries);
				}

				if (InstallSuccess && Globals.CancelSignalled == false)
				{
					List<UnrealSessionInstance.RoleInstance> AllRoles = new List<UnrealSessionInstance.RoleInstance>();
					Dictionary<UnrealSessionInstance.RoleInstance, IAppInstall> DeferredRoleToAppInstall = new Dictionary<UnrealSessionInstance.RoleInstance, IAppInstall>();

					// Now try to run all installs on their devices
					foreach (var InstallRoleKV in InstallsToRoles)
					{
						IAppInstall CurrentInstall = InstallRoleKV.Key;
						if (InstallRoleKV.Value.InstallOnly)
						{
							AllRoles.Add(new UnrealSessionInstance.RoleInstance(InstallRoleKV.Value, null));
							continue;
						}
						if (InstallRoleKV.Value.DeferredLaunch)
						{
							UnrealSessionInstance.RoleInstance DeferredRoleInstance = new UnrealSessionInstance.RoleInstance(InstallRoleKV.Value, null);

							DeferredRoleToAppInstall.Add(DeferredRoleInstance, CurrentInstall);
							AllRoles.Add(DeferredRoleInstance);
							continue;
						}
						bool Success = false;

						try
						{
							Log.Info("Starting {0} on {1}", InstallRoleKV.Value, CurrentInstall.Device);
							IAppInstance Instance = CurrentInstall.Run();
							IDeviceUsageReporter.RecordStart(Instance.Device.Name, Instance.Device.Platform, IDeviceUsageReporter.EventType.Test);

							if (Instance != null || Globals.CancelSignalled)
							{
								AllRoles.Add(new UnrealSessionInstance.RoleInstance(InstallRoleKV.Value, Instance));
							}

							Success = true;
						}
						catch (DeviceException Ex)
						{
							// mark that device as a problem
							Log.Info(KnownLogEvents.Gauntlet_DeviceEvent, "Failed to start build on {DeviceName}. Marking as problem device and retrying with new set", CurrentInstall.Device.Name);
							UnrealDeviceReservation.MarkProblemDevice(CurrentInstall.Device, $"Device threw an exception during launch.\nException={Ex.Message}");
							Success = false;
						}

						if (Success == false)
						{
							// terminate anything that's running
							foreach (UnrealSessionInstance.RoleInstance RunningRole in AllRoles.Where(X => X.AppInstance != null))
							{
								Log.Info("Shutting down {DeviceName}", RunningRole.AppInstance.Device.Name);
								RunningRole.AppInstance.Kill();
								RunningRole.AppInstance.Device.Disconnect();
							}

							ReleaseSessionDevices(true);

							if (SessionRetries == 0)
							{
								throw new AutomationException("Unable to start application for session, see log for details.");
							}

							Log.Info("\nUnable to start application for session (retries left={0})\n", --SessionRetries);

							break; // do not continue loop
						}
					}

					if (AllRoles.Count() == SessionRoles.Count())
					{
						SessionInstance = new UnrealSessionInstance(AllRoles.ToArray(), DeferredRoleToAppInstall);
					}
				}
			}

			return SessionInstance;
		}

		/// <summary>
		/// Returns true if the number of reserved devices matches the number on non-null session roles
		/// </summary>
		private bool HasAcquiredDevices()
		{
			IEnumerable<UnrealSessionRole> RolesThatRequireDevice = SessionRoles.Where(Role => !Role.IsNullRole());
			return UnrealDeviceReservation != null
				&& UnrealDeviceReservation.ReservedDevices != null
				&& UnrealDeviceReservation.ReservedDevices.Count() == RolesThatRequireDevice.Count();
		}

		/// <summary>
		/// Returns true if the provided session role has not yet been assigned a device
		/// </summary>
		private bool RoleNeedsDevice(UnrealSessionRole Role)
		{
			return !(RolesToDevices.ContainsKey(Role) && RolesToDevices[Role] != null);
		}

		/// <summary>
		/// Returns true if the provided session role has not yet had an install performed on it's assigned device
		/// </summary>
		private bool RoleNeedsInstall(UnrealSessionRole Role)
		{
			return ReinstallPerPass
				|| !(RolesToConfigs.ContainsKey(Role) && RolesToConfigs[Role] != null)
				|| !(RolesToInstalls.ContainsKey(Role) && RolesToInstalls[Role] != null);
		}

		/// <summary>
		/// Returns true if the provided target device matches the constraint requested by the session role
		/// </summary>
		private bool DeviceMatchesRoleConstraint(UnrealSessionRole Role, ITargetDevice Device)
		{
			bool bRoleMatchesConstraint = Role.Constraint.Equals(DevicePool.Instance.GetConstraint(Device));

			return Device.IsConnected
				&& Device.Platform == Role.Platform
				&& Role.Constraint.IsIdentity() || bRoleMatchesConstraint;
		}

		/// <summary>
		/// From the existing reserved device pool, assign each role a device that matches it's requested constraint
		/// This will cache a map of each role and the target device it's using in this UnrealSession
		/// </summary>
		private bool TryAssignDevicesToRoles()
		{
			// Order by constraint. This ensures roles with constraints have their devices selected first.
			IEnumerable<UnrealSessionRole> RolesSortedByConstraint = SessionRoles.OrderBy(R => R.Constraint.IsIdentity() ? 1 : 0);
			IEnumerable<ITargetDevice> ReservedDevices = UnrealDeviceReservation.ReservedDevices;

			foreach (UnrealSessionRole Role in RolesSortedByConstraint)
			{
				if (RoleNeedsDevice(Role))
				{
					ITargetDevice DeviceToAssign = null;

					if (Role.IsNullRole())
					{
						DeviceToAssign = new TargetDeviceNull($"Null{Role.RoleType}");
					}
					else
					{
						try
						{
							DeviceToAssign = ReservedDevices.Where(Device => DeviceMatchesRoleConstraint(Role, Device)).First();
							IDeviceUsageReporter.RecordStart(DeviceToAssign.Name, DeviceToAssign.Platform, IDeviceUsageReporter.EventType.Device, IDeviceUsageReporter.EventState.Success);
						}
						catch (Exception Ex)
						{
							Log.Warning("Failed to assign a reserved device to role {Role}. " +
								"This usually means devices were unexpectedly released mid session\n{Exception}", Role, Ex);
							return false;
						}
					}

					RolesToDevices.Add(Role, DeviceToAssign);
					ReservedDevices = ReservedDevices.Except(Enumerable.Repeat(DeviceToAssign, 1));
				}
			}

			return true;
		}

		/// <summary>
		/// Prepares each device for launch by performing the following
		///		- Install builds
		///		- Clean up old artifacts
		///		- Copy additional files requested by a test
		///		- Specific role configurations
		///	This will create and cache both an UnrealAppConfig and an IAppInstall for future reference
		/// </summary>
		private void ReadyDevicesForSession()
		{
			foreach(UnrealSessionRole Role in SessionRoles)
			{
				UnrealAppConfig AppConfig = null;
				ITargetDevice Device = RolesToDevices[Role];

				if (Globals.CancelSignalled)
				{
					return;
				}

				try
				{
					if (RoleNeedsInstall(Role))
					{
						RolesToConfigs.Remove(Role);
						RolesToInstalls.Remove(Role);

						Role.PreConfigureDevice?.Invoke(Device, Role);

						// Create the app config
						IEnumerable<UnrealSessionRole> OtherRoles = SessionRoles.Where(Other => Other != Role);
						AppConfig = BuildSource.CreateConfiguration(Role, OtherRoles);
						AppConfig.Sandbox = Sandbox;
						AppConfig.CommandLineParams.AddUnique("DeviceTag", Device.Name);
						RolesToConfigs.Add(Role, AppConfig);

						if (AppConfig.FullClean)
						{
							Log.Info("Fully cleaning device before install...");
							Device.FullClean();
						}

						if (!Role.SkipCleanDeviceArtifacts)
						{
							Device.CleanArtifacts(AppConfig);
						}

						if (AppConfig.SkipInstall)
						{
							Log.Info("Skipping install due to SkipInstall");
						}
						else
						{
							// Telemetry
							DateTimeStopwatch Stopwatch = DateTimeStopwatch.Start();
							Log.Info("Installing {BuildName} of type {BuildType} to {DeviceName}...", BuildSource.BuildName, AppConfig.Build.GetType().Name, Device.Name);
							IDeviceUsageReporter.RecordStart(Device.Name, Device.Platform, IDeviceUsageReporter.EventType.Install, IDeviceUsageReporter.EventState.Success, BuildSource.BuildName);

							try
							{
								Device.InstallBuild(AppConfig);
								IDeviceUsageReporter.RecordEnd(Device.Name, Device.Platform, IDeviceUsageReporter.EventType.Install, IDeviceUsageReporter.EventState.Success);
								Log.Info("Installation completed in {InstallTime}", GetInstallTime(Stopwatch.ElapsedTime));
							}
							catch
							{
								IDeviceUsageReporter.RecordEnd(Device.Name, Device.Platform, IDeviceUsageReporter.EventType.Install, IDeviceUsageReporter.EventState.Failure);
								throw;
							}
						}

						IAppInstall Install = Device.CreateAppInstall(AppConfig);
						RolesToInstalls.Add(Role, Install);
					}
					else
					{
						AppConfig = RolesToConfigs[Role];
                        if (!Role.SkipCleanDeviceArtifacts)
                        {
                            Device.CleanArtifacts(AppConfig);
                        }
                    }

					if (RolesToConfigs[Role].Build.SupportsAdditionalFileCopy)
					{
						Device.CopyAdditionalFiles(RolesToConfigs[Role].FilesToCopy);
					}

					Role.ConfigureDevice?.Invoke(Device);
				}
				catch(Exception Ex)
				{
					string Message = $"Encountered {Ex.GetType()} when creating installation.\n{Ex.Message}";

					bool IsHostPlatform = Device.Platform == BuildHostPlatform.Current.Platform;
					EventId EventType = IsHostPlatform ? KnownLogEvents.Gauntlet : KnownLogEvents.Gauntlet_DeviceEvent;

					if (IsOutOfSpaceException(Ex) && IsHostPlatform)
					{
						// If on desktop platform, we are not retrying.
						// It is unlikely that space is going to be made and InstallBuildParallel has marked the build path as problematic.

						Horde.ReportInvalidatingIssue(Horde.GauntletHordeStepMessage.NotEnoughDriveSpace, Message);
						Log.Error(EventType, Message);
						throw;
					}
					else if (IsOverlayException(Ex))
					{
						// Errors with Overlay executables are caused by missing files or improper setup, inform user and exit early

						Horde.ReportInvalidatingIssue(Horde.GauntletHordeStepMessage.OverlayException, Message);
						Log.Error(EventType, "Device {DeviceName} {Message}", Device.Name, Message);
						throw;
					}
					else if (Ex is DeviceException && !IsHostPlatform)
					{
						UnrealDeviceReservation.MarkProblemDevice(Device, Message);
					}
					else
					{
						Log.Warning(EventType, Message);
					}
					throw; // Can consider not throwing here - this would let every device complete setup before releasing the problem devices
				}
			}
		}

		/// <summary>
		/// Launches the Unreal Engine processes for each role requested by this session.
		/// Roles marked InstallOnly will not have a process launched.
		/// Roles marked DeferredLaunch will not have a process launched.
		/// Deferred roles can be launched at anytime (usually in TickTest()) by calling UnrealSessionInstance.LaunchDeferredRole
		/// </summary>
		private UnrealSessionInstance LaunchProcesses()
		{
			List<UnrealSessionInstance.RoleInstance> RoleInstances = new();
			Dictionary<UnrealSessionInstance.RoleInstance, IAppInstall> DeferredRolesToInstalls = new();

			foreach (KeyValuePair<UnrealSessionRole, IAppInstall> RoleInstall in RolesToInstalls)
			{
				UnrealSessionRole Role = RoleInstall.Key;
				IAppInstall Install = RoleInstall.Value;
				UnrealAppConfig Config = RolesToConfigs[Role];

				// InstallOnly roles don't execute a process
				if (Role.InstallOnly)
				{
					RoleInstances.Add(new UnrealSessionInstance.RoleInstance(Role, null));
					continue;
				}

				// DeferredLaunch roles don't immediately execute a process.
				// We cache deferred roles so users can launch the process at the desired time.
				else if (Role.DeferredLaunch)
				{
					UnrealSessionInstance.RoleInstance DeferredRoleInstance = new(Role, null);
					RoleInstances.Add(DeferredRoleInstance);
					DeferredRolesToInstalls.Add(DeferredRoleInstance, Install);
					continue;
				}

				try
				{
					Log.Info("Launching {Install} on {DeviceName}", Install, RolesToDevices[Role].Name);
					IAppInstance AppInstance = Install.Run();

					if (AppInstance == null)
					{
						throw new AutomationException("Failed to create an IAppInstance after attempting Run() on {0}", Install);
					}
					else
					{
						RoleInstances.Add(new UnrealSessionInstance.RoleInstance(Role, AppInstance));
					}
				}
				catch(Exception Ex)
				{
					// Kill any processes that were started
					foreach (UnrealSessionInstance.RoleInstance Instance in RoleInstances.Where(Role => Role.AppInstance != null))
					{
						Log.Info("Shutting down {AppInstance}", Instance.AppInstance);
						Instance.AppInstance.Kill();
					}

					string WarningMessage = $"Encountered exception when attempting to run install: {Ex}";
					if (Ex is DeviceException)
					{
						UnrealDeviceReservation.MarkProblemDevice(Install.Device, WarningMessage);
					}
					else
					{
						Log.Warning(WarningMessage);
					}
					throw;
				}
			}

			return new UnrealSessionInstance(RoleInstances.ToArray(), DeferredRolesToInstalls);
		}

		private void SaveScreenshots(string RoleName, string SourceDirectory, string DestinationDirectory, bool bCompressImages, bool bCreateGif)
		{
			try
			{
				string ScreenshotPath = PathUtils.FindRelevantPath(BasePath: SourceDirectory, "Screenshots");

				if (string.IsNullOrEmpty(ScreenshotPath) || !Directory.Exists(ScreenshotPath))
				{
					Log.Verbose("Could not locate Screenshots subdirectory in artifact path {ArtifactPath}. Skipping GIF creation", SourceDirectory);
					return;
				}

				// Image transformation relies on the image being directly on the host PC
				// This means it can't be on a remote device, or in a network storage.
				// The screenshots directory will be copied to a temp directory for the transformations, and then moved to the final artifact destination.
				DirectoryInfo ScreenshotDirectory = new(ScreenshotPath);
				DirectoryInfo ImageStage = new(Path.Combine(Path.GetTempPath(), "ImageStage", RoleName));
				if (ImageStage.Exists)
				{
					ImageStage.Delete(true);
				}

				// Copy the images to a the staging directory
				SystemHelpers.CopyDirectory(ScreenshotPath, ImageStage.FullName, SystemHelpers.CopyOptions.Mirror);

				// Creates an enumerable containing both the root directory and any sub directories
				IEnumerable<DirectoryInfo> ImageDirectories = ImageStage
					.EnumerateDirectories()
					.Concat(Enumerable.Repeat(ImageStage, 1));

				// Compress the images and convert them to a gif
				foreach (DirectoryInfo ImageDirectory in ImageDirectories)
				{
					FileInfo[] Files = ImageDirectory.GetFiles();
					if (Files.Any())
					{
						if (bCompressImages)
						{
							Image.ConvertImages(ImageDirectory.FullName, ImageDirectory.FullName, "jpg", true);
						}

						string GifPath = GenerateNotTakenFilePath(Path.Combine(DestinationDirectory, RoleName + "Test.gif"));
						if (bCreateGif)
						{
							if (Image.SaveImagesAsGif(ImageDirectory.FullName, GifPath))
							{
								Log.Info("Saved gif to {0}", GifPath);
							}
						}
					}
				}

				// Now, copy the compressed images in the temp staged directory to the desired artifact destination
				string RelativeScreenshotDirectory = ScreenshotPath.Replace(SourceDirectory, string.Empty).Trim('\\').Trim('/');
				string ScreenshotDestination = Path.Combine(DestinationDirectory, RelativeScreenshotDirectory);
				SystemHelpers.CopyDirectory(ImageStage.FullName, ScreenshotDestination, SystemHelpers.CopyOptions.Mirror);

				// Delete the source screenshot directory so they aren't duplicated in the following artifact copy
				ScreenshotDirectory.Delete(true);
				ImageStage.Delete(true);
			}
			catch (Exception Ex)
			{
				Log.Info("Failed to downsize and gif-ify images! {0}", Ex.Message);
			}
		}

		private void SavePSOs(UnrealTestContext InContext, UnrealSessionInstance.RoleInstance InRunningRole, string DestSavedDir)
		{
			if (InRunningRole.Role.RoleType.IsServer())
			{
				return;
			}

			if (!InContext.Options.LogPSO)
			{
				return;
			}

			if (!Directory.Exists(DestSavedDir))
			{
				try
				{
					Directory.CreateDirectory(DestSavedDir);
				}
				catch (Exception Ex)
				{
					Log.Info($"Archive path '{DestSavedDir}' was not found!\n{Ex.Message}");
					return;
				}
			}

			// Copy over PSOs
			try
			{
				foreach (string ThisFile in CommandUtils.FindFiles_NoExceptions(true, "*.rec.upipelinecache", true, DestSavedDir))
				{
					bool Copied = false;
					string JustFile = Path.GetFileName(ThisFile);
					if (!JustFile.StartsWith("++"))
					{
						continue;
					}

					string[] Parts = JustFile.Split(new Char[] { '+', '-' }).Where(A => A != "").ToArray();
					if (Parts.Count() >= 2)
					{
						string ProjectName = Parts[0].ToString();
						string BuildRoot = CommandUtils.CombinePaths(CommandUtils.RootBuildStorageDirectory());

						string SrcBuildPath = CommandUtils.CombinePaths(BuildRoot, ProjectName);
						string SrcBuildPath2 = CommandUtils.CombinePaths(BuildRoot, ProjectName.Replace("Game", "").Replace("game", ""));

						if (!CommandUtils.DirectoryExists(SrcBuildPath))
						{
							SrcBuildPath = SrcBuildPath2;
						}
						if (CommandUtils.DirectoryExists(SrcBuildPath))
						{
							var JustBuildFolder = JustFile.Replace("-" + Parts.Last(), "");

							string PlatformStr = InRunningRole.Role.Platform.ToString();
							string SrcCLMetaPath = CommandUtils.CombinePaths(SrcBuildPath, JustBuildFolder, PlatformStr, "MetaData");
							if (CommandUtils.DirectoryExists(SrcCLMetaPath))
							{
								string SrcCLMetaPathCollected = CommandUtils.CombinePaths(SrcCLMetaPath, "CollectedPSOs");
								if (!CommandUtils.DirectoryExists(SrcCLMetaPathCollected))
								{
									Log.Info("Creating Directory {0}", SrcCLMetaPathCollected);
									CommandUtils.CreateDirectory(SrcCLMetaPathCollected);
								}
								if (CommandUtils.DirectoryExists(SrcCLMetaPathCollected))
								{
									string DestFile = CommandUtils.CombinePaths(SrcCLMetaPathCollected, JustFile);
									CommandUtils.CopyFile_NoExceptions(ThisFile, DestFile, true);
									if (CommandUtils.FileExists(true, DestFile))
									{
										Log.Info("Deleting local file, copied to {0}", DestFile);
										CommandUtils.DeleteFile_NoExceptions(ThisFile, true);
										Copied = true;
									}
								}
							}
						}
					}
					if (!Copied)
					{
						Log.Warning("Could not find anywhere to put this file {0}", JustFile);
					}
				}
			}
			catch (Exception Ex)
			{
				Log.Info("Failed to copy upipelinecaches to the network {0}", Ex);
			}
		}

		private string GenerateNotTakenFilePath(string DesiredPath)
		{
			string ResultPath = null;

			FileInfo PotentialPathFileInfo = new FileInfo(DesiredPath);

			for (int NumericPostfix = 0; (string.IsNullOrEmpty(ResultPath)) && (NumericPostfix < int.MaxValue); NumericPostfix++)
			{
				string PotentialPath = DesiredPath;

				if (NumericPostfix > 0)
				{
					PotentialPath = Path.Combine(
						PotentialPathFileInfo.DirectoryName,
						string.Format("{0}_{1}", Path.GetFileNameWithoutExtension(PotentialPathFileInfo.Name), NumericPostfix));

					if (!string.IsNullOrEmpty(PotentialPathFileInfo.Extension))
					{
						PotentialPath += PotentialPathFileInfo.Extension;
					}
				}

				bool PathIsTaken = File.Exists(PotentialPath);
				if (PathIsTaken)
				{
					Log.VeryVerbose("File already exists at {0}", PotentialPath);
				}
				else
				{
					ResultPath = PotentialPath;
				}
			}

			if (string.IsNullOrEmpty(ResultPath))
			{
				throw new AutomationException("Cannot generate not taken file path for the path {0}", DesiredPath);
			}

			return ResultPath;
		}

		/// <summary>
		/// Filter that Truncate long file paths such as CrashReporter files. //UECC-Windows-F0DD9BB04C3C9250FAF39D8AB4A88556//
		/// These are particularly problematic with testflights which append a long random name to the destination folder, 
		/// easily pushing past 260 chars.
		/// </summary>
		/// <param name="LongFilePath"></param>
		/// <returns></returns>
		private string TruncateLongPathFilter(string LongFilePath)
		{
			Dictionary<string, string> LongCrashReporterStringToIndex = new Dictionary<string, string>();

			Match RegexMatch = Regex.Match(LongFilePath, @"((?i)UECC)-.+-([\dA-Fa-f]+)");

			if (RegexMatch.Success)
			{
				string LongString = RegexMatch.Groups[2].ToString();

				if (!LongCrashReporterStringToIndex.ContainsKey(LongString))
				{
					LongCrashReporterStringToIndex[LongString] = LongCrashReporterStringToIndex.Keys.Count.ToString("D2");
				}

				string ShortSring = LongCrashReporterStringToIndex[LongString];
				LongFilePath = LongFilePath.Replace(LongString, ShortSring);
			}

			return LongFilePath;
		}

		private string GetInstallTime(TimeSpan Time)
		{
			string Hours = Time.Hours > 0 ? string.Format("{0} hrs, ", Time.Hours) : string.Empty;
			string Minutes = Time.Minutes > 0 ? string.Format("{0} mins, ", Time.Minutes) : string.Empty;
			string Seconds = string.Format("{0} secs", Time.Seconds);

			return Hours + Minutes + Seconds;
		}

		private bool IsOutOfSpaceException(Exception Ex)
		{
			return Ex.Message.Contains("not enough space", StringComparison.OrdinalIgnoreCase);
		}

		private bool IsOverlayException(Exception Ex)
		{
			return Ex.Message.Contains("Overlay Error", StringComparison.OrdinalIgnoreCase);
		}
	}

}
