// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using AutomationTool;
using UnrealBuildTool;
using System.Linq;
using System.Text.RegularExpressions;

namespace Gauntlet
{

	public class GauntletCommandLine
	{
		public string Project;
		public string GameMap;
		private Dictionary<string, object> Params;
		private HashSet<string> NonOptionParams;

		public static readonly Regex CommandLineRegex = new Regex("(?:\"[^\"]*\"|[^\\s\"])+", RegexOptions.Compiled);
		public static readonly Regex CommandlineOptionRegex = new Regex(@"-(?<option>-?[\w\d.:!\[\]\/\\?-]+)(=(?<value>(""(?:(?:\\""|[^""])+)"")|(?:\S+)))?", RegexOptions.Compiled);

		// Give external people read-only access
		public IReadOnlyDictionary<string, object> Arguments { get { return Params; } }

		/// <summary>
		/// String containing extra commandline args that do not conform to UE commandline arg specs.
		/// Please do not use this for for standard flags.
		/// </summary>
		public string AdditionalExplicitCommandLineArgs;

		public GauntletCommandLine()
		{
			Project = string.Empty;
			GameMap = string.Empty;
			AdditionalExplicitCommandLineArgs = string.Empty;
			Params = new Dictionary<string, object>(StringComparer.OrdinalIgnoreCase);
			NonOptionParams = new HashSet<string>(StringComparer.OrdinalIgnoreCase);
		}

		// copy constructor
		public GauntletCommandLine(GauntletCommandLine InCopy)
		{
			Project = InCopy.Project;
			GameMap = InCopy.GameMap;
			AdditionalExplicitCommandLineArgs = InCopy.AdditionalExplicitCommandLineArgs;
			Params = new Dictionary<string, object>(InCopy.Params, StringComparer.OrdinalIgnoreCase);
			NonOptionParams = new HashSet<string>(StringComparer.OrdinalIgnoreCase);
		}

		/// <summary>
		/// Breaks down a raw commandline and adds it to the commandline dictionary.
		/// Will override current set values in the dictionary when conflicts arise.
		/// </summary>
		/// <param name="InRawCommandline"></param>
		/// <param name="bOverrideExistingValues"></param>
		public void AddRawCommandline(string InRawCommandline, bool bOverrideExistingValues = true)
		{
			// turn Name(p1,etc) into a collection of Name|(p1,etc) groups
			MatchCollection Matches = CommandlineOptionRegex.Matches(InRawCommandline);

			foreach (Match M in Matches)
			{
				if (M.Groups["option"] == null || string.IsNullOrWhiteSpace(M.Groups["option"].ToString()))
				{
					continue;
				}
				if (bOverrideExistingValues)
				{
					Add(M.Groups["option"].ToString().Trim(), string.IsNullOrWhiteSpace(M.Groups["value"].ToString()) ? null : M.Groups["value"]);
				}
				else
				{
					AddUnique(M.Groups["option"].ToString().Trim(), string.IsNullOrWhiteSpace(M.Groups["value"].ToString()) ? null : M.Groups["value"]);
				}
			}
		}

		/// <summary>
		/// Breaks down a raw commandline and adds it to the commandline dictionary.
		/// Will override current set values in the dictionary when conflicts arise.
		/// </summary>
		/// <param name="InCommandline"></param>
		/// <param name="bOverrideExistingValues"></param>
		public void CombineCommandLines(GauntletCommandLine InCommandline, bool bOverrideExistingValues = true)
		{
			foreach (string Key in InCommandline.Params.Keys)
			{
				Add(Key, InCommandline.Params[Key]);
			}
		}
		/// <summary>
		/// Add a new value to the commandline, returning false if the value already exists on the commandline and would be set
		/// to something other than what is passed in. Execcmds passed in here will still append to an existing value.
		/// </summary>
		/// <param name="ParamName"></param>
		/// <param name="ParamVal"></param>
		/// <returns>Whether the value passed in is now the expected value of the commandline. Returns false if we failed to set.</returns>
		public bool AddUnique(string ParamName, object ParamVal = null)
		{
			if (Params.ContainsKey(ParamName))
			{
				if ((ParamName.ToLower() == "execcmds" || ParamName.ToLower() == "dpcvars") && ParamVal != null)
				{
					AddOrAppendParamValue(ParamName, ParamVal.ToString());
					return true;
				}

				if (Params[ParamName] != null)
				{
					return (Params[ParamName] == ParamVal);
				}
				else
				{
					Params[ParamName] = ParamVal;
					return true;
				}
			}
			Params.Add(ParamName, ParamVal);
			return true;
		}

		/// <summary>
		/// Add new param value (adding with a value of null makes it a flag instead of a param)
		/// Overrides current value of the param if one exists.
		/// Execcmds will be appended to instead of overridden.
		/// </summary>
		/// <param name="ParamName"></param>
		/// <param name="ParamVal"></param>
		/// <param name="IsNonOption"></param>
		public void Add(string ParamName, object ParamVal = null, bool IsNonOption = false)
		{
			if (IsNonOption)
			{
				NonOptionParams.Add(ParamName);
			}

			if (Params.ContainsKey(ParamName))
			{
				if ((ParamName.ToLower() == "execcmds" || ParamName.ToLower() == "dpcvars") && ParamVal != null)
				{
					AddOrAppendParamValue(ParamName, ParamVal.ToString());
					return;
				}

				if (ParamVal == null && Params[ParamName] != null)
				{
					Gauntlet.Log.Info(string.Format("Ignored attempt to convert param {0} from a param with value to a flag", ParamName));
					return;
				}
				else if (ParamVal != null)
				{
					if (Params[ParamName] == null)
					{
						Gauntlet.Log.Info(string.Format("Converting param {0} from a flag to a param with value {1}.", ParamName, ParamVal));
					}
					else if (Params[ParamName] != null && ParamVal.ToString().Trim() != Params[ParamName].ToString().Trim())
					{
						Gauntlet.Log.Info(string.Format("Overriding value of param {0} from {1} to {2}", ParamName, Params[ParamName], ParamVal));
					}
					Params[ParamName] = ParamVal;
				}
			}
			else
			{
				Params.Add(ParamName, ParamVal);
			}
		}
		/// <summary>
		/// Add a new param, or append passed in value to it if a value already exists,
		/// delimited with the Delimiter passed in.
		/// </summary>
		/// <param name="ParamName"></param>
		/// <param name="ParamVal"></param>
		/// <param name="DelimiterToUse"></param>
		public void AddOrAppendParamValue(string ParamName, string ParamVal, string DelimiterToUse = ",")
		{
			if (Params.ContainsKey(ParamName))
			{
				if (Params[ParamName] == null)
				{
					Params[ParamName] = ParamVal;
				}
				else if (Params[ParamName].ToString() != ParamVal)
				{
					Params[ParamName] = Params[ParamName].ToString() + DelimiterToUse + ParamVal;
					Params[ParamName] = Params[ParamName].ToString().Replace("\"", "");
				}
			}
			else
			{
				Params.Add(ParamName, ParamVal);
			}
		}
		/// <summary>
		/// Remove the param or flag by the passed-in name if it exists.
		/// </summary>
		/// <param name="ParamName"></param>
		public void RemoveParam(string ParamName)
		{
			if (Params.ContainsKey(ParamName))
			{
				Params.Remove(ParamName);
			}
		}

		/// <summary>
		/// Returns the current value of the param if it has one. Flags and nonexistent 
		/// entries will return null.
		/// </summary>
		/// <param name="ParamName"></param>
		/// <returns></returns>
		public object GetParamValue(string ParamName)
		{
			if (!Params.ContainsKey(ParamName))
			{
				return null;
			}
			return Params[ParamName];
		}

		/// <summary>
		/// Get a collection of all sub paremeters
		/// Useful for group arguments like ExecCmds
		/// </summary>
		/// <param name="GroupName"></param>
		/// <param name="SubParamDelimeter">Which separator to use when distinguishing the group's sub parameters</param>
		/// <param name="SubValueDelimeter">Which separator to use when splitting the sub parameter from the corresponding value</param>
		/// <returns>A dictionary containing the group's sub parameters of parameter values</returns>
		public Dictionary<string, string> GetGroupParamValues(string GroupName, string SubParamDelimeter = ",", string SubValueDelimeter = " ")
		{
			Dictionary<string, string> Group = new Dictionary<string, string>();

			string FullArgument = GetParamValue(GroupName).ToString();
			string[] SubParams = FullArgument.Split(SubParamDelimeter);
			foreach (string SubParam in SubParams)
			{
				int Index = SubParam.IndexOf(SubValueDelimeter);

				if (Index < 1)
				{
					// No value, this is just a boolean param
					Group.Add(SubParam, string.Empty);
				}
				else
				{
					Group.Add(SubParam.Substring(0, Index), SubParam.Substring(Index + 1));
				}
			}

			return Group;
		}

		/// <summary>
		/// Checks if a parameter is present
		/// </summary>
		/// <param name="ParamName">The name of the parameter</param>
		/// <returns>True if the parameter has already been added</returns>
		public bool HasParam(string ParamName)
		{
			return (Params != null && Params.ContainsKey(ParamName));
		}

		/// <summary>
		/// Checks if a parameter group has a subparameter
		/// </summary>
		/// <param name="GroupName">The name of the param group, like "ExecCmds"</param>
		/// <param name="SubParamName">A component of the group, like "sg.TextureQuality"</param>
		/// <param name="SubParamDelimeter">Which separator to use when distinguishing the group's sub parameters</param>
		/// <param name="SubValueDelimeter">Which separator to use when splitting the sub parameter from the corresponding value</param>
		/// <returns>True if a group parameter exists and contains a matching sub parameter</returns>
		public bool HasGroupParam(string GroupName, string SubParamName, string SubParamDelimeter = ",", string SubValueDelimeter = " ")
		{
			return HasParam(GroupName) && GetGroupParamValues(GroupName, SubParamDelimeter, SubValueDelimeter).ContainsKey(SubParamName);
		}

		/// <summary>
		/// Wipe out the entire set of passed in params.
		/// </summary>
		public void ClearCommandLine()
		{
			Params.Clear();
			AdditionalExplicitCommandLineArgs = string.Empty;
		}

		/// <summary>
		/// Cobbles together the finalized commandline from the passed-in values. Wraps param values w/ spaces
		/// in quotes etc.
		/// This is the function we use, as well, when accessing the Role's commandline itself.
		/// </summary>
		/// <param name="CanAlterCommandArgs">Some tests require that the command is not altered by any Gauntlet logic and only the provided args are used.</param>
		/// <returns></returns>
		public string GenerateFullCommandLine(bool CanAlterCommandArgs = true)
		{
			string FinalCommandline = "";
			if (CanAlterCommandArgs)
			{
				if (!string.IsNullOrEmpty(Project))
				{
					FinalCommandline += string.Format("\"{0}\" ", Project);
				}
				if (!string.IsNullOrEmpty(GameMap))
				{
					FinalCommandline += string.Format("\"{0}\" ", GameMap);
				}
			}
			foreach (string Key in Params.Keys)
			{
				string CurrentArgument;
				if (Params[Key] != null && !string.IsNullOrWhiteSpace(Params[Key].ToString()))
				{
					CurrentArgument = string.Format("{0}{1}={2}", NonOptionParams.Contains(Key) ? "" : "-", Key,
						(Params[Key].ToString().Contains(' ') && !Params[Key].ToString().Contains('\"'))
						? string.Format("\"{0}\"", Params[Key]) : Params[Key]);
				}
				else
				{
					CurrentArgument = string.Format("{0}{1}", NonOptionParams.Contains(Key) ? "" : "-", Key);
				}
				FinalCommandline = string.Format("{0} {1} ", FinalCommandline, CurrentArgument);
			}
			if (!string.IsNullOrEmpty(AdditionalExplicitCommandLineArgs))
			{
				FinalCommandline = string.Format("{0} {1}", FinalCommandline, AdditionalExplicitCommandLineArgs);
			}
			return FinalCommandline;
		}

		public static List<string> GetArgumentList(string Args)
		{
			List<string> ArgList = CommandLineRegex.Matches(Args).Select(Match => Match.Value).ToList();
			for (int Index = 0; Index < ArgList.Count; Index++)
			{
				bool bJsonScope = false;
				int ScopeEndIndex = -1;
				for (int CharIndex = 0; CharIndex < ArgList[Index].Length; CharIndex++)
				{
					// Strip quotes as long as they're not embedded in json
					if (!bJsonScope)
					{
						if (ArgList[Index][CharIndex] == '{')
						{
							ScopeEndIndex = ArgList[Index].LastIndexOf('}');
							if (ScopeEndIndex != -1 && ScopeEndIndex > CharIndex)
							{
								bJsonScope = true;
							}
						}

						if (ArgList[Index][CharIndex] == '"')
						{
							ArgList[Index] = ArgList[Index].Remove(CharIndex, 1);
							CharIndex--;
						}
					}
					else if (CharIndex == ScopeEndIndex)
					{
						bJsonScope = false;
					}
				}
			}

			return ArgList;
		}
	}

	public enum EWindowMode
	{
		/// <summary>
		/// The window is in true fullscreen mode.
		/// </summary>
		Fullscreen,
		/// <summary>
		/// CURRENTLY UNSUPPORTED. Using this value will enable -fullscreen for now. The window has no border and takes up the entire area of the screen.
		/// </summary>
		WindowedFullscreen,
		/// <summary>
		/// The window has a border and may not take up the entire screen area.
		/// </summary>
		Windowed,
		/// <summary>
		/// The total number of supported window modes.
		/// </summary>
		NumWindowModes
	};

	/// <summary>
	/// Generic intent enum for the base of where we would like to copy a file to.
	/// Interpreted properly in TargetDeviceX.cs.
	/// </summary>
	public enum EIntendedBaseCopyDirectory
	{
		Build,
		Binaries,
		Config,
		Content,
		Demos,
		Profiling,
		Saved,
		Platform,
		PersistentDownloadDir
	}

	/// <summary>
	/// What reaching the max duration of this test signifies.
	/// </summary>
	public enum EMaxDurationReachedResult
	{
		Failure,
		Success
	}

	/// <summary>
	/// Delegate for role device pre-configuration
	/// </summary>
	public delegate void PreConfigureDeviceHandler(ITargetDevice Device, UnrealSessionRole Role);

	/// <summary>
	/// Delegate for role device configuration
	/// </summary>
	public delegate void ConfigureDeviceHandler(ITargetDevice Device);

	/// <summary>
	/// Delegate to fire when a process exits unexpectedly.
	/// We pass in the Role Instance instead of the Role so we can get all of the instantiated data.
	/// </summary>
	public delegate void OnTryToRestoreProcessDelegate(UnrealSessionInstance.RoleInstance InRole);

	/// <summary>
	/// This class represents a process-role in a test and defines the type, command line,
	/// and controllers that are needed.
	///
	/// TODO - can this be removed and UnrealSessionRole used directly?
	///
	/// </summary>
	public class UnrealTestRole
	{
		/// <summary>
		/// Constructor. This intentionally takes only a type as it's expected that code creating roles should do so via
		/// the configuration class and take care to append properties.
		/// </summary>
		/// <param name="InType"></param>
		/// <param name="InPlatformOverride"></param>
		public UnrealTestRole(UnrealTargetRole InType, UnrealTargetPlatform? InPlatformOverride)
		{
			Type = InType;
			PlatformOverride = InPlatformOverride;
			CommandLine = string.Empty;
			MapOverride = string.Empty;
			ExplicitClientCommandLine = string.Empty;
			Controllers = new List<string>();
			FilesToCopy = new List<UnrealFileToCopy>();
			RoleConfigurations = new List<IUnrealRoleConfiguration>();
			AdditionalArtifactDirectories = new List<EIntendedBaseCopyDirectory>();
			RoleType = ERoleModifier.None;
			InstallOnly = false;
			DeferredLaunch = false;
			CommandLineParams = new GauntletCommandLine();
			CompressScreenshots = true;
			RespawnProcessOnUnexpectedExit = false;
		}

		public ERoleModifier RoleType { get; set; }

		/// <summary>
		/// Whether this role should be responsible only for installing the build and not monitoring a process.
		/// </summary>
		public bool InstallOnly { get; set; }

		/// <summary>
		/// Whether this role will launched by the test node at a later time, typically during TickTest(). By default, all roles are launched immediately.
		/// </summary>
		public bool DeferredLaunch { get; set; }

		/// <summary>
		/// Type of process this role represents
		/// </summary>
		public UnrealTargetRole Type { get; protected set; }

		/// <summary>
		/// Override for what platform this role is on
		/// </summary>
		public UnrealTargetPlatform? PlatformOverride { get; protected set; }

		/// <summary>
		/// Command line or this role
		/// </summary>
		public string CommandLine
		{
			get
			{
				if (CommandLineParams == null)
				{
					CommandLineParams = new GauntletCommandLine();
				}
				return CommandLineParams.GenerateFullCommandLine();
			}
			set
			{
				if (CommandLineParams == null)
				{
					CommandLineParams = new GauntletCommandLine();
				}

				CommandLineParams.ClearCommandLine();

				CommandLineParams.AddRawCommandline(value);
			}
		}

		/// <summary>
		/// Dictionary of commandline arguments that are turned into a commandline at the end.
		/// For flags, leave the value set to null. Created and then passed through to the Session Role's Commandline Object
		/// in UnrealTestNode.cs
		/// </summary>
		public GauntletCommandLine CommandLineParams { get; set; }

		/// <summary>
		/// Controllers for this role
		/// </summary>
		public List<string> Controllers { get; set; }

		/// <summary>
		/// Collection of modular configurations applied to this role
		/// </summary>
		public List<IUnrealRoleConfiguration> RoleConfigurations { get; set; }

		/// <summary>
		/// Explicit command line for this role. If this is set no other
		/// options from above or configs will be applied!
		/// </summary>
		public string ExplicitClientCommandLine { get; set; }

		public List<UnrealFileToCopy> FilesToCopy { get; set; }

		/// <summary>
		/// Additional directories to 
		/// </summary>
		public List<EIntendedBaseCopyDirectory> AdditionalArtifactDirectories { get; set; }

		/// <summary>
		/// A map value passed in per server in case a test needs multiple servers on different maps.
		/// </summary>
		public string MapOverride { get; set; }

		/// <summary>
		/// Allows for individual roles to configure their associated Target device before build installation
		/// </summary>
		public PreConfigureDeviceHandler PreConfigureDevice;

		/// <summary>
		/// Allows for individual roles to configure their associated Target device after build installation and AppInstall creation
		/// </summary>
		public ConfigureDeviceHandler ConfigureDevice;

		/// <summary>
		/// Whether this role will compress screenshots produced as an artifact into a jpeg format
		/// </summary>
		public bool CompressScreenshots;

		/// <summary>
		/// Whether this role will create a gif from screenshots produced as an artifact
		/// </summary>
		public bool CreateGifFromScreenshots;

		/// <summary>
		/// Whether or not this role should skip the cleaning of its device's artifact between each test run
		/// </summary>
		public bool SkipCleanDeviceArtifacts { get; set; }

		/// <summary>
		/// Whether or not this role will save artifacts when running with -dev. 
		/// These can be huge, so be cautious with using this.
		/// </summary>
		public bool ArchiveDevArtifacts { get; set; }

		/// <summary>
		/// Whether to respawn this process automatically if it fails unexpectedly
		/// </summary>
		public bool RespawnProcessOnUnexpectedExit { get; set; }

		/// <summary>
		/// What to do if this role exits unexpectedly and we try to restore it
		/// </summary>
		public OnTryToRestoreProcessDelegate OnTryToRestoreProcess;

	}

	/// <summary>
	/// Collection of parameters that control how heartbeats coming from the native gauntlet controller for this role should be handled.
	/// To make best use of this, your GauntletTestController should regularly call MarkHeartbeatActive().
	/// Set bExpectHeartbeats to true to enable killing the App Instance when expected heartbeats are not detected.
	/// </summary>
	public class UnrealHeartbeatOptions
	{
		/// <summary>
		/// The amount of time between regular heartbeats. This value is passed along through the command line.
		/// </summary>
		public float HeartbeatPeriod;

		/// <summary>
		/// Set to true to allow the App Instance to be killed when expected heartbeats are not detected. If left false, heartbeat timeouts will not result in any action or timeouts.
		/// </summary>
		public bool bExpectHeartbeats;

		/// <summary>
		/// The max amount of time allowed before the first "active" heartbeat is detected
		/// </summary>
		public float TimeoutBeforeFirstActiveHeartbeat;

		/// <summary>
		/// The max amount of time allowed between "active" heartbeats
		/// </summary>
		public float TimeoutBetweenActiveHeartbeats;

		/// <summary>
		/// The max amount of time allowed between any heartbeats, active or not
		/// </summary>
		public float TimeoutBetweenAnyHeartbeats;

		/// <summary>
		/// The minimum time interval between retrieval of heartbeat logs
		/// </summary>
		public float LogHeartbeatInterval;

		public UnrealHeartbeatOptions(float InHeartbeatPeriod = 30f, bool bShouldExpectHeartbeats = false, float InTimeoutBeforeFirstActiveHeartbeat = 0f, float InTimeoutBetweenActiveHeartbeats = 0f, float InTimeoutBetweenAnyHeartbeats = 90f, float InLogHeartbeatInterval = 0f)
		{
			HeartbeatPeriod = InHeartbeatPeriod;
			bExpectHeartbeats = bShouldExpectHeartbeats;
			TimeoutBeforeFirstActiveHeartbeat = InTimeoutBeforeFirstActiveHeartbeat;
			TimeoutBetweenActiveHeartbeats = InTimeoutBetweenActiveHeartbeats;
			TimeoutBetweenAnyHeartbeats = InTimeoutBetweenAnyHeartbeats;
			LogHeartbeatInterval = InLogHeartbeatInterval;
		}

	}

	/// <summary>
	///	TestConfiguration describes the setup that is required for a specific test. 
	///	
	/// Protected parameters are generally test-wide options that are read from the command line and which tests cannot
	/// control.
	/// 
	/// Public parameters are options that individual tests can configure as appropriate.
	/// 
	///	Each test can (and should) supply its own configuration by overriding TestNode.GetConfiguration. At a minimum a 
	///	test must add one or more roles and the command line or controller necessary to execute the tests.
	///	
	/// Inherited classes should implement ApplyToConfig to apply the options they expose, and should ball the base class
	/// implementation.
	///
	/// </summary>
	public class UnrealTestConfiguration : IConfigOption<UnrealAppConfig>
	{

		// Protected options that are driven from the command line

		/// <summary>
		/// How often to grab a screenshot
		/// </summary>
		/// 
		[AutoParam(0)]
		protected int ScreenshotPeriod { get; set; }

		/// <summary>
		/// Use a nullrhi for tests
		/// </summary>
		/// 
		[AutoParam(false)]
		public bool NullRHI { get; protected set; }

		// Public options that tests can configure

		/// <summary>
		/// Map to use
		/// </summary>
		[AutoParam]
		public string Map = "";

		/// <summary>
		/// If true, explicitly do not set the default resolution of 1280x720 or the window mode. Most tests should not do this.
		/// </summary>
		/// 
		[AutoParam(false)]
		public bool IgnoreDefaultResolutionAndWindowMode { get; set; }

		/// <summary>
		/// The width resolution.
		/// </summary>
		/// 
		[AutoParam(0)]
		public int ResX { get; set; }

		/// <summary>
		/// The height resolution.
		/// </summary>
		/// 
		[AutoParam(0)]
		public int ResY { get; set; }

		/// <summary>
		/// Set to Windowed mode (same as -WindowMode=Windowed);
		/// </summary>
		/// 
		[AutoParam(false)]
		protected bool Windowed { get; set; }

		/// <summary>
		/// Which window mode to use for the PC or Mac or Linux client. Only Windowed and Fullscreen are fully supported.
		/// </summary>
		/// 
		[AutoParam(EWindowMode.Windowed)]
		public EWindowMode WindowMode { get; set; }

		/// <summary>
		/// Do not specify the unattended flag
		/// </summary>
		/// 
		[AutoParam(false)]
		public bool Attended { get; set; }

		/// <summary>
		/// Maximum duration in seconds that this test is expected to run for. Defaults to 600.
		/// </summary>
		[AutoParam(600.0f)]
		public float MaxDuration { get; set; }

		/// <summary>
		/// Max number of retries in case of critical failure
		/// </summary>
		[AutoParam(3)]
		public int MaxRetries { get; set; }

		/// <summary>
		/// Produce test artifacts for Horde build system
		/// </summary>
		[AutoParam]
		public bool WriteTestResultsForHorde = false;

		/// <summary>
		/// Path to store test data for Horde build system
		/// </summary>
		[AutoParam]
		public string HordeTestDataPath = "";

		/// <summary>
		/// Key to store Horde Test Data
		/// </summary>
		[AutoParam]
		public string HordeTestDataKey = "";

		/// <summary>
		/// Path to store test artifacts for Horde build system
		/// </summary>
		[AutoParam]
		public string HordeArtifactPath = "";

		/// <summary>
		/// PreFlight change id
		/// </summary>
		[AutoParam]
		public string PreFlightChange = "";

		/// <summary>
		/// Telemetry Database config to use
		/// </summary>
		[AutoParam]
		public string PublishTelemetryTo = "";

		/// <summary>
		/// Path to Database config file
		/// </summary>
		[AutoParam]
		public string DatabaseConfigPath = "";

		/// <summary>
		/// What the test result should be treated as if we reach max duration.
		/// </summary>
		public EMaxDurationReachedResult MaxDurationReachedResult { get; set; }

		/// <summary>
		/// Whether ensures are considered a failure
		/// </summary>
		[AutoParam(false)]
		public bool FailOnEnsures { get; set; }

		/// <summary>
		/// Whether ensures should be ignored from report
		/// </summary>
		[AutoParam(false)]
		public bool IgnoreEnsures { get; set; }

		/// <summary>
		/// Whether warnings are shown in the summary
		/// </summary>
		[AutoParam(false)]
		public bool ShowWarningsInSummary { get; set; }

		/// <summary>
		/// Whether warnings are shown in the summary
		/// </summary>
		[AutoParam(false)]
		public bool ShowErrorsInSummary { get; set; }

		/// <summary>
		/// Whether the test expects all roles to exit
		/// </summary>
		public bool AllRolesExit { get; set; }

		/// <summary>
		/// The collection of options which define heartbeat behavior
		/// </summary>
		public UnrealHeartbeatOptions HeartbeatOptions { get; set; }

		/// <summary>
		/// Prevents heartbeats timeouts from being checked so that tests will not fail from missed heartbeats
		/// </summary>
		[AutoParam(false)]
		public bool DisableHeartbeatTimeout { get; set; }

		/// <summary>
		/// Enforce Vertical resolution
		/// </summary>
		[AutoParam]
		public int ForceVerticalRes = 0;

		/// <summary>
		/// Consider the package as CookedEditor
		/// </summary>
		[AutoParam(false)]
		public bool CookedEditor { get; set; }

		/// <summary>
		/// Enforce Verbose logging for a list of loggers
		/// </summary>
		[AutoParam]
		public string VerboseLogCategories { get; set; }

		/// <summary>
		/// Apply a tag filter, restricting tests to run
		/// </summary>
		[AutoParam]
		public string TagFilter { get; set; }

		/// <summary>
		/// If true, an individual process role failing (i.e. crash) does not flag the test as a failure
		/// </summary>
		public bool IgnoreRoleFailures { get; set; }

		// Member variables 

		/// <summary>
		/// A map of role types to test roles
		/// </summary>
		public Dictionary<UnrealTargetRole, List<UnrealTestRole>> RequiredRoles { get; private set; }

		/// <summary>
		/// Log channels that should be treated as events for this test. Warnings & Errors in these
		/// channels will be promoted to test warnings and errors. For LogFoo return "Foo".
		/// </summary>
		public List<string> LogCategoriesForEvents { get; protected set; } = new List<string>();

		/// <summary>
		/// Base constructor
		/// </summary>
		public UnrealTestConfiguration()
		{
			MaxDuration = 600;  // 10m

			// create the role structure
			RequiredRoles = new Dictionary<UnrealTargetRole, List<UnrealTestRole>>();

			HeartbeatOptions = new UnrealHeartbeatOptions();

			MaxDurationReachedResult = EMaxDurationReachedResult.Failure;
		}

		/// <summary>
		/// Set this test to use dummy, renderless clients.
		/// </summary>
		/// <param name="Quantity">Number of dummy clients to spawn.</param>
		/// <param name="AdditionalCommandLine"></param>
		public void AddDummyClients(int Quantity, string AdditionalCommandLine = "")
		{
			IEnumerable<UnrealTestRole> DummyClientRole = RequireRoles(UnrealTargetRole.Client, UnrealTargetPlatform.Win64, Quantity, ERoleModifier.Dummy);
			foreach (UnrealTestRole DummyClient in DummyClientRole)
			{
				DummyClient.CommandLine += " -nullrhi " + AdditionalCommandLine;
			}
		}

		/// <summary>
		/// Adds one role of the specified type to this test. With inherited tests this could
		/// return an existing role so care should be added to append commandlines, controllers etc
		/// </summary>
		/// <param name="InRole"></param>
		/// <returns></returns>
		public UnrealTestRole RequireRole(UnrealTargetRole InRole)
		{
			if (InRole.IsEditor())
			{
				return GetEditorRole();
			}
			return RequireRoles(InRole, 1).First();
		}

		public UnrealTestRole RequireRole(UnrealTargetRole InRole, UnrealTargetPlatform PlatformOverride)
		{
			if (InRole.IsEditor())
			{
				InRole = CookedEditor ? UnrealTargetRole.CookedEditor : UnrealTargetRole.Editor;
			}
			return RequireRoles(InRole, PlatformOverride, 1).First();
		}

		public UnrealTestRole GetEditorRole()
		{
			UnrealTargetRole EditorRole = CookedEditor ? UnrealTargetRole.CookedEditor : UnrealTargetRole.Editor;
			return RequireRoles(EditorRole, 1).First();
		}

		/// <summary>
		/// Adds 'Count' of the specified roles to this test
		/// </summary>
		/// <param name="InRole"></param>
		/// <param name="Count"></param>
		/// <returns></returns>
		public List<UnrealTestRole> RequireRoles(UnrealTargetRole InRole, int Count)
		{
			return RequireRoles(InRole, null, Count);
		}

		/// <summary>
		/// Clears all roles from this config. 
		/// </summary>
		public void ClearRoles()
		{
			RequiredRoles.Clear();
		}

		public List<UnrealTestRole> RequireRoles(UnrealTargetRole InRole, UnrealTargetPlatform? PlatformOverride, int Count, ERoleModifier roleType = ERoleModifier.None)
		{
			if (RequiredRoles.ContainsKey(InRole) == false)
			{
				RequiredRoles[InRole] = new List<UnrealTestRole>();
			}

			List<UnrealTestRole> RoleList = new List<UnrealTestRole>();

			RequiredRoles[InRole].ForEach((R) => { if (R.PlatformOverride == PlatformOverride) RoleList.Add(R); });

			for (int i = RoleList.Count; i < Count; i++)
			{
				UnrealTestRole NewRole = new UnrealTestRole(InRole, PlatformOverride);
				NewRole.RoleType = roleType;
				RoleList.Add(NewRole);
				RequiredRoles[InRole].Add(NewRole);
			}

			return RoleList;
		}

		/// <summary>
		/// Returns the number of roles of the specified type that exist for this test
		/// </summary>
		/// <param name="Role"></param>
		/// <returns></returns>
		public int RoleCount(UnrealTargetRole Role)
		{
			int Roles = 0;

			if (RequiredRoles.ContainsKey(Role))
			{
				Roles = RequiredRoles[Role].Count;
			}

			return Roles;
		}

		/// <summary>
		/// Return the list of required roles for the target role
		/// </summary>
		/// <param name="InRole"></param>
		/// <returns></returns>
		public List<UnrealTestRole> GetRequiredRoles(UnrealTargetRole InRole)
		{
			if (RequiredRoles.ContainsKey(InRole))
			{
				return RequiredRoles[InRole];
			}
			return new List<UnrealTestRole>();
		}

		/// <summary>
		/// Return the main required role to execute the test.
		/// </summary>
		/// <returns></returns>
		public UnrealTestRole GetMainRequiredRole()
		{
			var PriorityList = new UnrealTargetRole[] {
				UnrealTargetRole.Client,
				UnrealTargetRole.EditorGame,
				UnrealTargetRole.Server,
				UnrealTargetRole.EditorServer,
				UnrealTargetRole.Host,
				UnrealTargetRole.Editor,
				UnrealTargetRole.CookedEditor
			};
			foreach (UnrealTargetRole TargetRole in PriorityList)
			{
				IEnumerable<UnrealTestRole> Roles = GetRequiredRoles(TargetRole);
				if (Roles.Any())
				{
					return Roles.First();
				}
			}

			if (RequiredRoles.Any())
			{
				var RoleEnumerator = RequiredRoles.Values.GetEnumerator();
				RoleEnumerator.MoveNext();
				return RoleEnumerator.Current.First();
			}

			return new UnrealTestRole(UnrealTargetRole.Unknown, null);
		}

		/// <summary>
		/// 
		/// </summary>
		/// <param name="AppConfig"></param>
		public void ApplyToConfig(UnrealAppConfig AppConfig)
		{
			throw new AutomationException("Unreal tests should use ApplyToConfig(Config, Role, OtherRoles)");
		}

		/// <summary>
		/// Apply our options to the provided app config
		/// </summary>
		/// <param name="AppConfig"></param>
		/// <param name="ConfigRole"></param>
		/// <param name="OtherRoles"></param>
		/// <returns></returns>
		public virtual void ApplyToConfig(UnrealAppConfig AppConfig, UnrealSessionRole ConfigRole, IEnumerable<UnrealSessionRole> OtherRoles)
		{
			if (NullRHI)
			{
				AppConfig.CommandLine += " -nullrhi";
			}
			else if (AppConfig.ProcessType.IsClient())
			{
				if (AppConfig.Platform == UnrealTargetPlatform.Win64 || AppConfig.Platform == UnrealTargetPlatform.Mac || AppConfig.Platform == UnrealTargetPlatform.Linux)
				{
					if (!IgnoreDefaultResolutionAndWindowMode)
					{
						if (ResX != 0)
						{
							AppConfig.CommandLine += String.Format(" -ResX={0} ", ResX);
						}

						if (ResY != 0)
						{
							AppConfig.CommandLine += String.Format(" -ResY={0}", ResY);
						}

						if (WindowMode == EWindowMode.Windowed || Windowed)
						{
							AppConfig.CommandLine += " -windowed";
						}
						else if (WindowMode == EWindowMode.Fullscreen)
						{
							AppConfig.CommandLine += " -fullscreen";
						}
						else if (WindowMode == EWindowMode.WindowedFullscreen) // Proper -windowedfullscreen flag does not exist and some platforms treat both modes as the same.
						{
							AppConfig.CommandLine += " -fullscreen";
						}
						else
						{
							Log.Warning("Test config uses an unsupported WindowMode: {0}! WindowMode not set.", Enum.GetName(typeof(EWindowMode), WindowMode));
						}
					}
				}

				if (ScreenshotPeriod > 0 && NullRHI == false)
				{
					AppConfig.CommandLine += string.Format(" -gauntlet.screenshotperiod={0}", ScreenshotPeriod);
				}
			}

			if (AppConfig.Platform == UnrealTargetPlatform.Linux)
			{
				// due to an issue with dotnet being extremely pedantic we have to drop our locks on files so we can read from the log file
				// https://github.com/dotnet/runtime/issues/34126
				AppConfig.CommandLine += " -noexclusivelockonwrite";
				AppConfig.CommandLine += " -RemoveInvalidKeys";
			}

			// use -log on user machine so we get a window..
			if (!AutomationTool.Automation.IsBuildMachine)
			{
				AppConfig.CommandLine += " -log";
			}

			if (Attended == false)
			{
				AppConfig.CommandLine += " -unattended -nosplash";

				// if we are unattended but still may need access to Vulkan passing renderoffscreen to allow not depending on
				// the X11/Wayland display server to be around and use a dummy/offscreen rendering mode
				//
				// As well as disable sound as there are no audio devices when running through horde
				//
				// Disable cef as it seems to want to talk to an X11 server so unlikely its even working
				if (AppConfig.Platform == UnrealTargetPlatform.Linux)
				{
					AppConfig.CommandLine += " -renderoffscreen -nocef";
				}
			}

			AppConfig.CommandLine += " -stdout -FullStdOutLogOutput";

			float HeartbeatPeriod = Globals.Params.ParseValue("HeartbeatPeriod", HeartbeatOptions.HeartbeatPeriod);
			if (HeartbeatPeriod > 0)
			{
				AppConfig.CommandLine += string.Format(" -gauntlet.heartbeatperiod={0}", HeartbeatPeriod);
			}

			string MapChoice = string.IsNullOrEmpty(ConfigRole.MapOverride) ? Map : ConfigRole.MapOverride;

			if (string.IsNullOrEmpty(MapChoice) == false)
			{
				if (AppConfig.ProcessType.IsServer()
				|| (AppConfig.ProcessType.IsClient() && RoleCount(UnrealTargetRole.Server) == 0))
				{
					AppConfig.CommandLineParams.GameMap = MapChoice;
				}
			}

			if (CommandUtils.IsBuildMachine)
			{
				AppConfig.CommandLineParams.AddUnique("BUILDMACHINE");
			}
		}
	}

}
