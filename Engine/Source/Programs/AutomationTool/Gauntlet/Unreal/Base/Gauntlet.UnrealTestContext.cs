// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.IO;
using AutomationTool;
using UnrealBuildTool;
using System.Linq;
using EpicGames.Core;

namespace Gauntlet
{

	public class TestRequest
	{
		public string TestName { get; set; }
		
		public List<ArgumentWithParams> Platforms = new List<ArgumentWithParams>();

		public Params TestParams { get; set; }

		public override string ToString()
		{
			return string.Format("{0}({1})", TestName, string.Join(",", Platforms.Aggregate(new List<string>(), (L, P) => { L.Add(P.Argument); return L; })));
		}


		static public TestRequest CreateRequest(string InName) { return new TestRequest() { TestName = InName, TestParams = new Params(new string[0]) }; }
	}


	/// <summary>
	/// Represents options that can be configured on the test context. These options are considered
	/// global, e.g they apply to all tests executed in a given session and are not exposed to 
	/// invidiual tests to modify
	/// </summary>
	public class UnrealTestOptions : TestExecutorOptions,  IAutoParamNotifiable
	{
		/// <summary>
		/// Params for this test run
		/// </summary>
		public Params Params { get; protected set; }

		/// <summary>
		/// Name or path to this this project. This will be normalized to the project name
		/// and the path to the project file available via the ProjectPath argument,
		/// </summary>
		[AutoParam]
		public string Project = "";

		/// <summary>
		/// Returns the path to the project file. Created based on the Project argument
		/// </summary>
		public FileReference ProjectPath { get; set; }

		/// <summary>
		/// Reference to the build that is being tested
		/// </summary>
		[AutoParamWithNames("Build", "Builds")]
		public string Build = "";

		/// <summary>
		/// Location of the editor build
		/// </summary>
		[AutoParam]
		public string EditorDir = "";

		/// <summary>
		/// Does this project use 'Game' or Client/Server?
		/// </summary>
		[AutoParam(true)]
		public bool UsesSharedBuildType = true;

		// todo - remove this and pass in BuildSource
		public Type BuildSourceType;

		/// <summary>
		/// Configuration to perform tests on
		/// </summary>
		[AutoParam(UnrealTargetConfiguration.Development)]
		public UnrealTargetConfiguration Configuration;

		/// <summary>
		/// Platforms to perform tests on and their params
		/// </summary>
		public List<ArgumentWithParams> PlatformList = new List<ArgumentWithParams>();

		/// <summary>
		/// Optional set of platform overrides for a given role
		/// </summary>
		public Dictionary<UnrealTargetRole, UnrealTargetPlatform> RolePlatformOverrides = new Dictionary<UnrealTargetRole, UnrealTargetPlatform>();

		/// <summary>
		/// Tests to run and their params.
		/// </summary>
		public List<TestRequest> TestList = new List<TestRequest>();

		/// <summary>
		/// List of devices to use for tests
		/// </summary>
		public List<ArgumentWithParams> DeviceList = new List<ArgumentWithParams>();

		/// <summary>
		/// Service URL of devices to use for tests
		/// </summary>
		[AutoParam("")]
		public string DeviceURL;

		/// <summary>
		/// Maximum number of available local(host platform) devices
		/// </summary>
		[AutoParam(10)]
		public int MaxLocalDevices;

		/// <summary>
		/// Maximum number of available virtual devices of each type (supported on the host platform)
		/// </summary>
		[AutoParam(2)]
		public int MaxVirtualDevices;

		/// <summary>
		/// Details for current job (example: link to CIS, etc)
		/// </summary>
		[AutoParam("")]
		public string JobDetails;

		/// <summary>
		/// Comma-separated list of namespaces to check for tests. E.g.
		/// with test 'Foo' and namespace 'Bar' then Bar.Foo and Foo will
		/// be checked for.
		/// </summary>
		[AutoParam("")]
		public string Namespaces;

		public IEnumerable<string> SearchPaths;

		/// <summary>
		/// Directory to store temp files (including builds that need to be run locally)
		/// </summary>
		[AutoParam("")]
		public string Sandbox { get; set; }

		/// <summary>
		/// Skip any check or copying of builds. Mostly useful when debugging
		/// </summary>
		[AutoParam(false)]
		public bool Attended { get; set; }

		/// <summary>
		/// Skip any check or copying of builds. Mostly useful when debugging
		/// </summary>
		[AutoParam(false)]
		public bool SkipCopy { get; set; }

		/// <summary>
		/// 'Dev' - turns on a few options, most useful of which is that local exes will be used if newer
		/// </summary>
		[AutoParam(false)]
		public bool Dev { get; set; }

        /// <summary>
        /// 'LogPSO' - turns on logpso on the client and uploads the files to P:
        /// </summary>
        [AutoParam(false)]
        public bool LogPSO { get; set; }

        /// <summary>
        /// Use a nullrhi 
        /// </summary>
        [AutoParam(false)]
		public bool NullRHI { get; set; }

		/// <summary>
		/// Location to store temporary files. Defaults to GauntletTemp in the root of the
		/// current drive
		/// </summary>
		[AutoParam("")]
		public string TempDir;

		/// <summary>
		/// Location to store log files. Defaults to TEmpDir/Logs
		/// </summary>
		[AutoParam("")]
		public string LogDir;

		/// <summary>
		/// Custom name for the directory under LogDir where artifacts are stored.
		/// </summary>
		[AutoParam("")]
		public string ArtifactName;

		/// <summary>
		/// Psotfix to apply to the artifact folder (e.g. don't replace the whole thing like ArtifactName, just append this string)
		/// </summary>
		[AutoParam("")]
		public string ArtifactPostfix;

		/// <summary>
		/// Add custom module name/role pair to identify custom project target executable
		/// Format is <name>:<role>+<name>:<role>
		/// </summary>
		[AutoParam("")]
		public string CustomModuleRoles { get; set; }

		/// <summary>
		/// Set the target name of the module name to identify a custom project target executable
		/// </summary>
		[AutoParam("")]
		public string Target { get; set; }

		/// <summary>
		/// Less logging
		/// </summary>
		[AutoParamWithNames(false, "Verbose", "Gauntlet.Verbose")]
		public bool Verbose;

		/// <summary>
		/// Less logging
		/// </summary>
		[AutoParam(false)]
		[AutoParamWithNames(false, "VeryVerbose", "Gauntlet.VeryVerbose")]
		public bool VeryVerbose;

		public UnrealTestOptions()
		{
			// todo, make platform default to the current os
			Configuration = UnrealTargetConfiguration.Development;
			SearchPaths = new string[0];
			BuildSourceType = typeof(UnrealBuildSource);
		}

		/// <summary>
		/// Called after command line params are applied. Perform any checks / fixup
		/// </summary>
		/// <param name="InParams"></param>
		public virtual void ParametersWereApplied(string[] InParams)
		{
			// save params
			this.Params = new Params(InParams);

			if (string.IsNullOrEmpty(TempDir))
			{
				TempDir = Globals.TempDir;
			}
			else
			{
				Globals.TempDir = TempDir;
			}

			if (string.IsNullOrEmpty(LogDir))
			{
				LogDir = Globals.LogDir;
			}
			else
			{
				Globals.LogDir = LogDir;
			}
			// Normalize paths
			LogDir = Path.GetFullPath(LogDir);
			TempDir = Path.GetFullPath(TempDir);

			// normalize the project name and get the path
			if (File.Exists(Project))
			{
				ProjectPath = new FileReference(Project);
				Project = ProjectPath.GetFileNameWithoutExtension();
			}
			else
			{
				if (!string.IsNullOrEmpty(Project))
				{
					ProjectPath = ProjectUtils.FindProjectFileFromName(Project);

					if (ProjectPath == null)
					{
						throw new AutomationException("Could not find project file for {0}", Project);
					}
					Project = ProjectPath.GetFileNameWithoutExtension();
				}
			}

			if (string.IsNullOrEmpty(Sandbox))
			{
				Sandbox = Project;
			}

			// parse platforms. These will be in the format of Win64(param1,param2='foo,bar') etc

			// check for old-style -platform=Win64(params)
			List<string> PlatformArgStrings = Params.ParseValues("Platform=");

			// check for convenience flags of -Win64(params) (TODO - need to think about this..)
			/*foreach (UnrealTargetPlatform Plat in UnrealTargetPlatform.GetValidPlatforms())
			{
				IEnumerable<string> RawPlatformArgs = InParams.Where(P => P.ToLower().StartsWith(Plat.ToString().ToLower()));

				PlatformArgStrings.AddRange(RawPlatformArgs);
			}*/

			// now turn the Plat(param1,param2) etc into an argument/parm pair
			PlatformList = PlatformArgStrings.SelectMany(P => ArgumentWithParams.CreateFromString(P)).ToList();

			List<string> TestArgStrings = Params.ParseValues("test=");

			// clear all tests incase this is called multiple times
			TestList.Clear();

			if (TestArgStrings.Count == 0)
			{
				TestArgStrings = Params.ParseValues("tests=");
			}

			foreach (string TestArg in TestArgStrings)
			{
				foreach (ArgumentWithParams TestWithParms in ArgumentWithParams.CreateFromString(TestArg))
				{
					TestRequest Test = new TestRequest() { TestName = TestWithParms.Argument, TestParams = new Params(TestWithParms.AllArguments) };

					// parse any specified platforms
					foreach (string PlatformArg in Test.TestParams.ParseValues("Platform="))
					{
						List<ArgumentWithParams> PlatParams = ArgumentWithParams.CreateFromString(PlatformArg);
						Test.Platforms.AddRange(PlatParams);

						// register platform in test options 
						PlatParams.ForEach(TestPlat => { if (PlatformList.Where(Plat => Plat.Argument == TestPlat.Argument).Count() == 0) PlatformList.Add(TestPlat); });						
					}

					TestList.Add(Test);
				}

			}

			if (PlatformList.Count == 0)
			{
				// Default to local platform
				PlatformList.Add(new ArgumentWithParams(BuildHostPlatform.Current.Platform.ToString(), new string[0]));
			}

			// do we have any tests? Need to check the global test list
			bool HaveTests = TestList.Count > 0 || PlatformList.Where(Plat => Plat.ParseValues("test").Count() > 0).Count() > 0;

			List<string> DeviceArgStrings = Params.ParseValues("device=");

			if (DeviceArgStrings.Count == 0)
			{
				DeviceArgStrings = Params.ParseValues("devices=");
			}

			DeviceList = DeviceArgStrings.SelectMany(D => ArgumentWithParams.CreateFromString(D)).ToList();

			if (DeviceList.Count == 0)
			{
				// Add the default test
				DeviceList.Add(new ArgumentWithParams("default", new string[0]));
			}

			// remote all -test and -platform arguments from our params. Nothing else should be looking at these now...
			string[] CleanArgs = Params.AllArguments
				.Where(Arg => !Arg.StartsWith("test=", StringComparison.OrdinalIgnoreCase) 
					&& !Arg.StartsWith("platform=", StringComparison.OrdinalIgnoreCase)
					&& !Arg.StartsWith("device=", StringComparison.OrdinalIgnoreCase))
				.ToArray();
			Params = new Params(CleanArgs);

			// Custom Module name/role
			if (!string.IsNullOrEmpty(CustomModuleRoles))
			{
				foreach (var Pair in CustomModuleRoles.Split("+"))
				{
					var SplittedPair = Pair.Split(":");
					if (SplittedPair.Length > 1)
					{
						string Name = SplittedPair[0];
						UnrealTargetRole Role = UnrealTargetRole.Unknown;
						if (Enum.TryParse(SplittedPair[1], out Role))
						{
							UnrealHelpers.AddCustomModuleName(Name, Role);
						}
						else
						{
							throw new AutomationException(string.Format("Target Role '{0}' for Custom Module '{1}' is unknown.", SplittedPair[1], Name));
						}
					}
					else
					{
						Gauntlet.Log.Warning("CustomModuleRoles is poorly formatted. Expected <name>:<role> pair. Got '{0}'", Pair);
					}
				}
			}
			if(!string.IsNullOrEmpty(Target) && Target != Project)
			{
				bool IsEditor = Build.Equals("Editor", StringComparison.InvariantCultureIgnoreCase) || Globals.Params.ParseParam("editor");
				UnrealTargetRole Role = IsEditor? UnrealTargetRole.Editor : UnrealTargetRole.Client;
				UnrealHelpers.AddCustomModuleName(IsEditor? string.Format("{0}Editor", Target): Target, Role);
			}
		}
	}
	

	/// <summary>
	/// This class describes the "context" that is used for a given role. E.g. if the test specifies it needs a Client role
	/// then this class determines what that actually means, and even whether it's really a client or a client emulated
	/// by running the editor with -game
	/// 
	/// This mappinf of role->context is performed by the TestContext so this structure holds the absolute truth of how
	/// a client is configured
	/// 
	/// </summary>
	public class UnrealTestRoleContext : ICloneable
	{
		/// <summary>
		/// Role of this role :)
		/// </summary>
		public UnrealTargetRole		Type;

		/// <summary>
		/// Platform that this role is run under
		/// </summary>
		public UnrealTargetPlatform		Platform;

		/// <summary>
		/// Configuration that this role is run under
		/// </summary>
		public UnrealTargetConfiguration Configuration;

		/// <summary>
		/// Extra args to use for this role
		/// </summary>
		public string ExtraArgs;

		/// <summary>
		/// Are these roles actually skipped during tests? Useful if you need to manually start
		/// a role when debugging etc
		/// </summary>
		public bool Skip;

		public object Clone()
		{
			// no deep copy needed
			return this.MemberwiseClone();
		}

		public override string ToString()
		{
			string Description = string.Format("{0} {1} {2}", Platform, Configuration, Type);
			return Description;
		}
	};

	/// <summary>
	/// This class contains the Context - build and global options - that one or more tests are going to
	/// be executed under.
	/// </summary>
	public class UnrealTestContext : ITestContext, ICloneable
	{
		// Begin ITestContext implementation
		public UnrealBuildSource BuildInfo { get; private set; }

		// Worker Job ID (generates unique node results and logs in parallel runs)
		public string WorkerJobID;

		/// <summary>
		/// Global options for this test
		/// </summary>
		public UnrealTestOptions Options { get; set; }

		public Params TestParams { get; set; }

		public UnrealTestRoleContext GetRoleContext(UnrealTargetRole Role)
		{
			return RoleContext[Role];
		}

		public Dictionary<UnrealTargetRole, UnrealTestRoleContext> RoleContext { get; set; }

		/// <summary>
		/// Target constraint that this test is run under
		/// </summary>
		public UnrealDeviceTargetConstraint Constraint;

		public UnrealTestContext(UnrealBuildSource InBuildInfo, Dictionary<UnrealTargetRole, UnrealTestRoleContext> InRoleContexts, UnrealTestOptions InOptions)
		{
			//BuildRepository = InBuildRepository;
			BuildInfo = InBuildInfo;
			Options = InOptions;
			TestParams = new Params(new string[0]);

			RoleContext = InRoleContexts;
		}

		public object Clone()
		{
			UnrealTestContext Copy = (UnrealTestContext)this.MemberwiseClone();

			// todo - what else shou;d be unique?
			Copy.RoleContext = this.RoleContext.ToDictionary(entry => entry.Key,
											   entry => (UnrealTestRoleContext)entry.Value.Clone());

			return Copy;
		}

		public override string ToString()
		{
			return ToString(false);
		}

		public string ToString(bool bWithServerType = false)
		{
			string Description = string.Format("{0}", RoleContext[UnrealTargetRole.Client]);

			if (WorkerJobID != null)
			{
				Description += " " + WorkerJobID;
			}
				
			if (bWithServerType)
			{
				Description += ", " +  RoleContext[UnrealTargetRole.Server].ToString();
			}

			return Description;
		}
	}

}