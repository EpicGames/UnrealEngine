// Copyright Epic Games, Inc. All Rights Reserved.

using EpicGames.Core;
using System;
using System.Collections.Generic;
using System.Linq;

namespace Gauntlet
{
	/// <summary>
	/// Unreal-specific implementation of a TestEvent
	/// </summary>
	public class UnrealTestEvent : ITestEvent
	{
		/// <summary>
		/// Level of severity that this event represents
		/// </summary>
		public EventSeverity Severity { get; protected set; }

		// Time at which the event was fired
		public DateTime Time { get; }

		// Title/Summary of event
		public string Summary { get; protected set; }

		// Event details
		public IEnumerable<string> Details { get; protected set; }

		// Callstack 
		public IEnumerable<string> Callstack { get; protected set; }

		// True if this is an ensure (Gauntlet does not define a level for this, but we log differently).
		public bool IsEnsure { get; protected set; }

		// True if this is a Sanitizer error report.
		public bool IsSanReport { get; protected set; }

		// True if the event severity is Error or Fatal
		public bool IsError => Severity == EventSeverity.Error || Severity == EventSeverity.Fatal;

		// True if the event severity is Warning
		public bool IsWarning => Severity == EventSeverity.Warning;

		// Constructor that requires all properties
		public UnrealTestEvent(DateTime InTime, EventSeverity InSeverity, string InSummary, IEnumerable<string> InDetails, UnrealLog.CallstackMessage InCallstack = null)
		{
			Time = InTime;
			Severity = InSeverity;
			Summary = InSummary;
			Details = InDetails.ToArray();

			if (InCallstack != null)
			{
				IsEnsure = InCallstack.IsEnsure;
				IsSanReport = InCallstack.IsSanReport;
				Callstack = InCallstack.Callstack;
			}
			else
			{
				IsEnsure = false;
				IsSanReport = false;
				Callstack = Enumerable.Empty<string>();
			}
		}

		// Constructor that requires all properties but DateTime
		public UnrealTestEvent(EventSeverity InSeverity, string InSummary, IEnumerable<string> InDetails, UnrealLog.CallstackMessage InCallstack = null)
			: this(DateTime.UtcNow, InSeverity, InSummary, InDetails, InCallstack) { }

		// Constructor that use structure logging standard inputs
		public UnrealTestEvent(EventSeverity InSeverity, string InFormat, params object[] Args)
			: this(InSeverity, string.Empty, Enumerable.Empty<string>())
		{
			Dictionary<string, object> Properties = new();
			MessageTemplate.ParsePropertyValues(InFormat, Args, Properties);
			Summary = MessageTemplate.Render(InFormat, Properties);
		}
	}
}
