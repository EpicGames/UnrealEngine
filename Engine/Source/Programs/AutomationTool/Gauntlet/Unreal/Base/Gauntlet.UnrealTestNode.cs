// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Concurrent;
using System.Collections.Generic;
using System.IO;
using AutomationTool;
using UnrealBuildTool;
using System.Text.RegularExpressions;
using System.Linq;
using EpicGames.Core;
using EpicGames.LogMatchers;
using Microsoft.Extensions.Logging;
using static Gauntlet.HordeReport;
using static Gauntlet.UnrealSessionInstance;
using Gauntlet.Utils;
using static Gauntlet.HordeReport.AutomatedTestSessionData;
using Gauntlet.TestTracking;
using Gauntlet.Report;

namespace Gauntlet
{
	/// <summary>
	/// Implementation of a Gauntlet TestNode that is capable of executing tests on an Unreal "session" where multiple
	/// Unreal instances may be involved. This class leans on UnrealSession to do the work of spinning up, monitoring, and
	/// shutting down instances. Those operations plus basic validation of Unreal's functionality are used to provide the
	/// required ITestNode interfaces
	/// </summary>
	/// <typeparam name="TConfigClass"></typeparam>
	public abstract class UnrealTestNode<TConfigClass> : BaseTest, IDisposable
		where TConfigClass : UnrealTestConfiguration, new()
	{
		[Flags]
		public enum BehaviorFlags
		{
			None = 0,
			PromoteErrors = 1,              // Promote errors from Unreal instances to regular test errors. (By default only fatal errors are errors)
			PromoteWarnings = 2,            // Promote warnings from Unreal instances to regular test warnings.	(By default only ensures are warnings)
		}

		/// <summary>
		/// Test Metadata for audit
		/// </summary>
		public ITestAuditMetadata TestMetadata;

#nullable enable
		/// <summary>
		/// Unique key to identify and cross reference test across test sessions 
		/// </summary>
		protected virtual string ReportTestKey
		{
			get
			{
				return Type;
			}
		}
		protected virtual string HordeReportTestKey => ReportTestKey;

		/// <summary>
		/// The test name used in report
		/// </summary>
		protected virtual string ReportTestName
		{
			get
			{
				string ReportName = Name;
				if (ReportName.Split('.').Length > 1)
				{
					ReportName = ReportName.Split('.').Last();
				}

				return ReportName;
			}
		}
		protected virtual string HordeReportTestName => ReportTestName;

		/// <summary>
		/// The harness used to execute the test
		/// </summary>
		protected virtual TestHarness? TestHarness => null;

		/// <summary>
		/// The test profile (unit test, functional, performance)
		/// </summary>
		protected virtual TestProfile? TestProfile => null;

		/// <summary>
		/// The team for which the test is for
		/// </summary>
		protected virtual TestTeam? TestTeam => null;

		/// <summary>
		/// Short description of the test intent
		/// </summary>
		protected virtual string? TestIntent => null;

		/// <summary>
		/// The tags associated with this test
		/// </summary>
		protected virtual TestTag[]? TestTags => null;
#nullable disable

		/// <summary>
		/// Returns an identifier for this test
		/// </summary>
		public override string Name { get { return this.GetType().FullName; } }

		/// <summary>
		/// Returns the test suite. Default to the project name
		/// </summary>
		public virtual string Suite { get { return Context.BuildInfo.ProjectName; } }

		/// <summary>
		///Returns an identifier for this type of test
		/// </summary>
		public virtual string Type { get { return string.Format("{0}({1})", this.GetType().FullName, Suite); } }

		/// <summary>
		/// This class will log its own warnings and errors as part of its summary
		/// </summary>
		public override bool LogWarningsAndErrorsAfterSummary { get; protected set; } = false;

		/// <summary>
		/// How long this test should run for, set during LaunchTest based on results of GetConfiguration
		/// </summary>
		public override float MaxDuration { get; protected set; }

		/// Behavior flags for this test
		/// </summary>
		public BehaviorFlags Flags { get; protected set; }
		/// <summary>
		/// Priority of this test
		/// </summary>
		public override TestPriority Priority { get { return GetPriority(); } }

		/// <summary>
		/// Delegate type to populate test phase in report
		/// </summary>
		/// <param name="TestReport"></param>
		protected delegate void PopulateTestPhasesReport(AutomatedTestSessionData TestReport);
		/// <summary>
		/// Call when populating Horde test report
		/// </summary>
		protected PopulateTestPhasesReport OnPopulateTestPhases;

		/// <summary>
		/// Phase tracker
		/// </summary>
		protected PhaseQueue Phases;

		/// <summary>
		/// Returns a list of all log channels the heartbeat tick should look for.
		/// </summary>
		public virtual IEnumerable<string> GetHeartbeatLogCategories()
		{
			return Enumerable.Empty<string>();
		}

		/// <summary>
		/// Returns Warnings found during tests. By default only ensures and are considered
		/// </summary>
		public override IEnumerable<string> GetWarnings()
		{
			IEnumerable<string> WarningList = TestNodeEvents.Where(E => E.IsWarning).Select(E => E.Summary);

			if (RoleResults != null)
			{
				WarningList = WarningList.Union(RoleResults.SelectMany(R => R.Events.Where(E => E.IsWarning)).Select(E => E.Summary));
			}

			return WarningList.ToArray();
		}

		/// <summary>
		/// Returns Errors found during tests. By default fatal and error severities are considered.
		/// returning lists of event summaries
		/// </summary>
		public override IEnumerable<string> GetErrors()
		{
			IEnumerable<string> ErrorList = TestNodeEvents.Where(E => E.IsError).Select(E => E.Summary);

			if (RoleResults != null)
			{
				ErrorList = ErrorList.Union(RoleResults.SelectMany(R => R.Events.Where(E => E.IsError)).Select(E => E.Summary));
			}

			return ErrorList.ToArray();
		}

		/// <summary>
		/// Returns Errors found during tests. Including Abnormal Exit reasons
		/// </summary>
		public virtual IEnumerable<string> GetErrorsAndAbnornalExits()
		{
			IEnumerable<string> Errors = GetErrors();
			if (RoleResults == null)
			{
				return Errors;
			}

			return Errors.Union(RoleResults.Where(R => R.ProcessResult != UnrealProcessResult.ExitOk).Select(
				R => string.Format("Abnormal Exit: Reason={0}, ExitCode={1}, Log={2}", R.Summary, R.ExitCode, Path.GetFileName(R.Artifacts.LogPath))
			));
		}

		/// <summary>
		/// Report an error - Legacy.  Instead, please consider using ReportInterrupt for functional test problems, or ReportFailure for actual test failures
		/// </summary>
		/// <param name="Message"></param>
		/// <param name="Args"></param>
		public virtual void ReportError(string Message, params object[] Args)
		{
			Reporter.Instance.Failure(Message, Args);
		}

		/// <summary>
		/// Report a test-halting error that is NOT directly related to the behavior that is being tested.
		/// This method is deprecated. Please use Gauntlet.Reporter.Instance.Interrupt instead of this method.
		/// </summary>
		/// <param name="Message"></param>
		/// <param name="Args"></param>
		public virtual void ReportInterrupt(string Message, params object[] Args)
        {
			Reporter.Instance.Interrupt(Message, Args);
		}

		/// <summary>
		/// Report a test failure.  Specifically, the behavior that is targeted by the test case did not function as expected.
		/// This method is deprecated. Please use Gauntlet.Reporter.Instance.Failure instead of this method.
		/// </summary>
		/// <param name="Message"></param>
		/// <param name="Args"></param>
		public virtual void ReportFailure(string Message, params object[] Args)
        {
			Reporter.Instance.Failure(Message, Args);
		}

		/// <summary>
		/// Report a warning.
		/// This method is deprecated. Please use Gauntlet.Reporter.Instance.Warning instead of this method.
		/// </summary>
		/// <param name="Message"></param>
		/// <param name="Args"></param>
		public virtual void ReportWarning(string Message, params object[] Args)
		{
			Reporter.Instance.Warning(Message, Args);
		}

		/// <summary>
		/// Report information.
		/// This method is deprecated. Please use Gauntlet.Reporter.Instance.Info instead of this method.
		/// </summary>
		/// <param name="Message"></param>
		/// <param name="Args"></param>
		public virtual void ReportInfo(string Message, params object[] Args)
		{
			Reporter.Instance.Info(Message, Args);
		}

		// Begin UnrealTestNode properties and members

		/// <summary>
		/// Our context that holds environment wide info about the required conditions for the test
		/// </summary>
		public UnrealTestContext Context { get; private set; }

		/// <summary>
		/// When the test is running holds all running Unreal processes (clients, servers etc).
		/// </summary>
		public UnrealSessionInstance TestInstance { get; private set; }

		/// <summary>
		/// Path to the directory that logs and other artifacts are copied to after the test run.
		/// </summary>
		public string ArtifactPath { get; private set; }

		/// <summary>
		/// Describes the post-test results for a role.
		/// </summary>
		public class UnrealRoleResult
		{
			/// <summary>
			/// High-level description of how the process ended
			/// </summary>
			public UnrealProcessResult ProcessResult;

			/// <summary>
			/// Exit code for the process. Unreal makes limited use of exit codes so in most cases
			/// this will be 0 / -1
			/// </summary>
			public int ExitCode;

			/// <summary>
			/// Human-readable of the process result. (E.g 'process encountered a fatal error')
			/// </summary>
			public string Summary;

			/// <summary>
			/// A summary of information such as entries, warnings, errors, ensures, etc etc extracted from the log
			/// </summary>
			public UnrealLog LogSummary;

			/// <summary>
			/// Artifacts for this role. 
			/// </summary>
			public UnrealRoleArtifacts Artifacts;

			/// <summary>
			/// Events that occurred during the test pass. Asserts/Ensures/Errors/Warnings should all be in here
			/// </summary>
			public IEnumerable<UnrealTestEvent> Events;

			/// <summary>
			/// Constructor. All members are required
			/// </summary>
			public UnrealRoleResult(UnrealProcessResult InResult, int InExitCode, string InSummary, UnrealLog InLog, UnrealRoleArtifacts InArtifacts, IEnumerable<UnrealTestEvent> InEvents)
			{
				ProcessResult = InResult;
				ExitCode = InExitCode;
				Summary = InSummary;
				LogSummary = InLog;
				Artifacts = InArtifacts;
				Events = InEvents;
			}
		};

		/// <summary>
		/// After the test completes holds artifacts for each process (clients, servers etc).
		/// </summary>
		public IEnumerable<UnrealRoleResult> RoleResults { get; private set; }

		/// <summary>
		/// After the test completes holds artifacts for each process (clients, servers etc).
		/// </summary>
		public IEnumerable<UnrealRoleArtifacts> SessionArtifacts { get; private set; }

		/// <summary>
		/// Collection of events thrown by gauntlet itself during the test run.
		/// </summary>
		protected IReadOnlyCollection<UnrealTestEvent> TestNodeEvents => _TestNodeEvents;
		private ConcurrentQueue<UnrealTestEvent> _TestNodeEvents { get; set; } = new();

		public override void AddTestEvent(UnrealTestEvent InEvent)
		{
			_TestNodeEvents.Enqueue(InEvent);
		}

		/// <summary>
		/// UnrealSession for this instance
		/// </summary>
		protected UnrealSession UnrealApp { get; set; }

		/// <summary>
		/// Used to track each running app's new log output
		/// </summary>
		private Dictionary<RoleInstance, UnrealLogStreamParser> RoleParsers = new Dictionary<RoleInstance, UnrealLogStreamParser>();

		/// <summary>
		/// Returns the Role Log Parser from a given RoleInstance key.
		/// If the key does not exist it will be created before returning it.
		/// </summary>
		private UnrealLogStreamParser GetParserForRole(RoleInstance Role)
		{
			if(!RoleParsers.ContainsKey(Role))
			{
				RoleParsers.Add(Role, new UnrealLogStreamParser(Role.AppInstance?.GetLogBufferReader()));
			}
			return RoleParsers[Role];
		}

		private int CurrentPass;

		private int NumPasses;

		static protected DateTime SessionStartTime = DateTime.MinValue;

		public int Retries { get; protected set; } = 0;
		private int MaxRetries = 3;
		public virtual bool CanRetry() { return Retries < MaxRetries; }
		public virtual bool SetToRetryIfPossible()
		{
			if (!CanRetry()) 
			{
				Log.Warning(KnownLogEvents.Gauntlet_TestEvent, $"Max retries have been exceeded, attempted retry {Retries}/{MaxRetries} times.");
				return false; 
			}
			if (IsSetToRetry()) { return true; }

			Retries++;
			SetUnrealTestResult(TestResult.WantRetry);
			Log.Warning(KnownLogEvents.Gauntlet_TestEvent, $"Test node is set to retry, test will now retry (Attempt {Retries}/{MaxRetries})");
			return true;
		}

		/// <summary>
		/// PLEASE READ BEFORE OVERRIDING:
		/// This checks a condition defined in a test node at the end of the test to determine whether or not the test should run again.
		/// This should only be used in very specific instances to retry for user defined dependency/service failure string.
		/// Example: Useful for unexpected unattended log outs during testing
		/// </summary>
		protected virtual bool CheckShouldRetry()
		{
			return false;
		}

		public virtual bool IsSetToRetry()
		{
			return GetTestResult() == TestResult.WantRetry;
		}

		/// <summary>
		/// Standard semantic versioning for tests. Should be overwritten within individual tests, and individual test maintainers
		/// are responsible for updating their own versions. See https://semver.org/ for more info on maintaining semantic versions.
		/// </summary>

		protected Version TestVersion;

		/// <summary>
		/// Our test result. May be set directly, or by overriding GetUnrealTestResult()
		/// </summary>
		private TestResult UnrealTestResult;

		protected TConfigClass CachedConfig = null;

		protected DateTime TimeOfFirstMissingProcess;

		protected int TimeToWaitForProcesses { get; set; }

		protected bool bDisableHeartbeatLogging = false;

		protected DateTime LastHeartbeatTime = DateTime.MinValue;
		protected DateTime LastActiveHeartbeatTime = DateTime.MinValue;
		private DateTime LastHeartbeatLogTime = DateTime.MinValue;

		// End  UnrealTestNode properties and members

		// artifact paths that have been used in this run
		static protected HashSet<string> ReservedArtifcactPaths = new HashSet<string>();

		/// <summary>
		/// Help doc-style list of parameters supported by this test. List can be divided into test-specific and general arguments.
		/// </summary>
		public List<GauntletParamDescription> SupportedParameters = new List<GauntletParamDescription>();

		/// <summary>
		/// Optional list of provided commandlines to be displayed to users who want to look at test help docs.
		/// Key should be the commandline to use, value should be the description for that commandline.
		/// </summary>
		protected List<KeyValuePair<string, string>> SampleCommandlines = new List<KeyValuePair<string, string>>();

		public void AddSampleCommandline(string Commandline, string Description)
		{
			SampleCommandlines.Add(new KeyValuePair<string, string>(Commandline, Description));
		}

		/// <summary>
		/// List of log event categories to track with the heartbeat system.
		/// </summary>
		private List<string> HeartbeatLogCategories = new List<string>();

		/// <summary>
		/// Constructor. A context of the correct type is required
		/// </summary>
		/// <param name="InContext"></param>
		public UnrealTestNode(UnrealTestContext InContext)
		{
			Context = InContext;

			UnrealTestResult = TestResult.Invalid;
			TimeToWaitForProcesses = 5;
			CurrentPass = 0;
			NumPasses = 0;
			TestVersion = new Version("1.0.0");
			ArtifactPath = string.Empty;
			Phases = new();
			Reporter.Instance.UpdateReferences(Phases, SetUnrealTestResult, this);
			PopulateCommandlineInfo();
			// We format warnings ourselves so don't show these
			LogWarningsAndErrorsAfterSummary = false;

			TestMetadata = new UnrealTestMetadata
			(
				() => HordeReportTestKey,
				() => HordeReportTestName,
				() => GetType().FullName,
				() => TestVersion,
				() => MaxDuration / 60,
				() => GetRunLocalCommand(Globals.BuildCommand),
				() => TestHarness,
				() => TestProfile,
				() => TestTeam,
				() => TestIntent,
				() => TestTags,
				() => GetDistilledTestParams()
						.ToLookup(A => A.Key, A => A.Value) /* use a lookup here to avoid duplicate param key */
						.ToDictionary(A => A.Key, A => A.Last())
			);
		}

		~UnrealTestNode()
		{
			Dispose(false);
		}

		#region IDisposable Support
		private bool disposedValue = false; // To detect redundant calls

		protected virtual void Dispose(bool disposing)
		{
			if (!disposedValue)
			{
				if (disposing)
				{
					// TODO: dispose managed state (managed objects).
				}

				CleanupTest();

				if (RoleResults != null)
				{
					foreach (var RoleResult in RoleResults)
					{
						RoleResult?.LogSummary?.Dispose();
					}
				}

				disposedValue = true;
			}
		}

		// This code added to correctly implement the disposable pattern.
		public void Dispose()
		{
			// Do not change this code. Put cleanup code in Dispose(bool disposing) above.
			Dispose(true);
		}
		#endregion

		public override String ToString()
		{
			if (Context == null)
			{
				return Name;
			}

			return string.Format("{0} ({1})", Name, GetMainRoleContextString());
		}

		public string GetMainRoleContextString()
		{
			if (Context == null)
			{
				return string.Empty;
			}
			// Using CachedConfiguration so GetConfiguration does not get looped over. If it has not been cached
			// yet, will call test's GetConfiguration method.
			var Config = CachedConfig;
			return (Config is UnrealTestConfiguration) ? Context.GetRoleContext(Config.GetMainRequiredRole().Type).ToString() : Context.ToString();
		}

		/// <summary>
		/// Sets the context that tests run under. Called once during creation
		/// </summary>
		/// <param name="InContext"></param>
		/// <returns></returns>
		public override void SetContext(ITestContext InContext)
		{
			Context = InContext as UnrealTestContext;
		}


		/// <summary>
		/// Returns information about how to configure our Unreal processes. For the most part the majority
		/// of Unreal tests should only need to override this function
		/// </summary>
		/// <returns></returns>
		public virtual TConfigClass GetConfiguration()
		{
			if (CachedConfig == null)
			{
				CachedConfig = new TConfigClass();
				AutoParam.ApplyParamsAndDefaults(CachedConfig, this.Context.TestParams.AllArguments);
			}
			return CachedConfig;
		}

		/// <summary>
		/// Returns the cached version of our config. Avoids repeatedly calling GetConfiguration() on derived nodes
		/// </summary>
		/// <returns></returns>
		protected TConfigClass GetCachedConfiguration()
		{
			if (CachedConfig == null)
			{
				CachedConfig = GetConfiguration();
			}

			return CachedConfig;
		}

		/// <summary>
		/// Returns a priority value for this test
		/// </summary>
		/// <returns></returns>
		protected TestPriority GetPriority()
		{
			IEnumerable<UnrealTargetPlatform> DesktopPlatforms = UnrealBuildTool.Utils.GetPlatformsInClass(UnrealPlatformClass.Desktop);

			UnrealTestRoleContext ClientContext = Context.GetRoleContext(UnrealTargetRole.Client);

			// because these device build need deployed we want them in flight asap
			IDeviceBuildSupport DeviceBuildSupport = Gauntlet.Utils.InterfaceHelpers.FindImplementations<IDeviceBuildSupport>().Where(D => D.CanSupportPlatform(ClientContext.Platform)).FirstOrDefault(); ;
			if (DeviceBuildSupport != null && DeviceBuildSupport.NeedBuildDeployed())
			{
				return TestPriority.High;
			}

			return TestPriority.Normal;
		}

		protected virtual IEnumerable<RoleInstance> FindAnyMissingRoles()
		{
			return TestInstance.RunningRoles.Where(R => R.AppInstance.HasExited);
		}

		protected virtual bool TryRestoreAllMissingRoles(IEnumerable<UnrealSessionInstance.RoleInstance> MissingRoles)
		{
			// We want to make sure we don't start respawning some processses if vital ones are missing
			foreach (UnrealSessionInstance.RoleInstance RoleInstance in MissingRoles)
			{
				if (!RoleInstance.Role.RespawnProcessOnUnexpectedExit)
				{
					return false;
				}
			}
			foreach (UnrealSessionInstance.RoleInstance RoleInstance in MissingRoles)
			{
				if (!UnrealApp.RolesToInstalls.ContainsKey(RoleInstance.Role))
				{
					return false;
				}
				if (RoleInstance.Role.OnTryToRestoreProcess != null)
				{
					RoleInstance.Role.OnTryToRestoreProcess.Invoke(RoleInstance);
				}
				IAppInstall DesiredInstall = UnrealApp.RolesToInstalls[RoleInstance.Role];
				UnrealApp.SessionInstance.RecoverProcessForRole(RoleInstance.Role, DesiredInstall);
			}

			return true;
		}

		/// <summary>
		/// Checks whether the test is still running. The default implementation checks whether all of our processes
		/// are still alive.
		/// </summary>
		/// <returns></returns>
		public virtual bool IsTestRunning()
		{
			var MissingRoles = FindAnyMissingRoles().ToList();

			if (MissingRoles.Count == 0)
			{
				// nothing missing, keep going.
				return true;
			}

			// if all roles are gone, we're done
			if (MissingRoles.Count == TestInstance.RunningRoles.Count())
			{
				return false;
			}

			// This test only ends when all roles are gone
			if (GetCachedConfiguration().AllRolesExit)
			{
				return true;
			}

			if (TryRestoreAllMissingRoles(MissingRoles))
			{
				return true;
			}

			if (TimeOfFirstMissingProcess == DateTime.MinValue)
			{
				TimeOfFirstMissingProcess = DateTime.Now;
				Log.Verbose("Role {Role} exited. Waiting {Duration} seconds for others to exit", MissingRoles.First().ToString(), TimeToWaitForProcesses);
			}

			if ((DateTime.Now - TimeOfFirstMissingProcess).TotalSeconds < TimeToWaitForProcesses)
			{
				// give other processes time to exit normally
				return true;
			}

			Log.Info("Ending {Name} due to exit of Role {Role}. {Count} processes still running", Name, MissingRoles.First().ToString(), TestInstance.RunningRoles.Count());

			// Done!
			return false;
		}

		protected bool PrepareUnrealApp()
		{
			// Get our configuration
			TConfigClass Config = GetCachedConfiguration();

			if (Config == null)
			{
				Horde.ReportInvalidatingIssue(Horde.GauntletHordeStepMessage.InvalidConfig, "Null config returned for this test.");
				throw new AutomationException("Test {0} returned null config!", this);
			}

			if (UnrealApp != null)
			{
				throw new AutomationException("Node already has an UnrealApp, was PrepareUnrealSession called twice?");
			}

			// pass through any arguments such as -TestNameArg or -TestNameArg=Value
			var TestName = this.GetType().Name;
			var ShortName = TestName.Replace("Test", "");

			var PassThroughArgs = Context.TestParams.AllArguments
				.Where(A => A.StartsWith(TestName, System.StringComparison.OrdinalIgnoreCase) || A.StartsWith(ShortName, System.StringComparison.OrdinalIgnoreCase))
				.Select(A =>
				{
					A = "-" + A;

					var EqIndex = A.IndexOf("=");

					// no =? Just a -switch then
					if (EqIndex == -1)
					{
						return A;
					}

					var Cmd = A.Substring(0, EqIndex + 1);
					var Args = A.Substring(EqIndex + 1);

					// if no space in the args, just leave it
					if (Args.IndexOf(" ") == -1)
					{
						return A;
					}

					return string.Format("{0}\"{1}\"", Cmd, Args);
				});

			List<UnrealSessionRole> SessionRoles = new List<UnrealSessionRole>();

			// Go through each type of role that was required and create a session role
			foreach (var TypesToRoles in Config.RequiredRoles)
			{
				// get the actual context of what this role means.
				UnrealTestRoleContext RoleContext = Context.GetRoleContext(TypesToRoles.Key);

				foreach (UnrealTestRole TestRole in TypesToRoles.Value)
				{
					// If a config has overridden a platform then we can't use the context constraints from the commandline
					bool UseContextConstraint = TestRole.Type == UnrealTargetRole.Client && TestRole.PlatformOverride == null;

					// important, use the type from the ContextRolke because Server may have been mapped to EditorServer etc
					UnrealTargetPlatform SessionPlatform = TestRole.PlatformOverride ?? RoleContext.Platform;

					// Apply all role configurations
					foreach (IUnrealRoleConfiguration RoleConfiguration in TestRole.RoleConfigurations)
					{
						RoleConfiguration.ApplyConfigToRole(TestRole);
					}

					// Verify all role configurations
					foreach (IUnrealRoleConfiguration RoleConfiguration in TestRole.RoleConfigurations)
					{
						RoleConfiguration.VerifyRoleConfig(TestRole);
					}

					UnrealSessionRole SessionRole = new UnrealSessionRole(RoleContext.Type, SessionPlatform, RoleContext.Configuration, TestRole.CommandLine);
					SessionRole.InstallOnly = TestRole.InstallOnly;
					SessionRole.DeferredLaunch = TestRole.DeferredLaunch;
					SessionRole.CommandLineParams = TestRole.CommandLineParams;
					SessionRole.RoleModifier = TestRole.RoleType;
					SessionRole.Constraint = UseContextConstraint ? Context.Constraint : new UnrealDeviceTargetConstraint(SessionPlatform);
					SessionRole.SkipCleanDeviceArtifacts = TestRole.SkipCleanDeviceArtifacts;
					SessionRole.ArchiveDevArtifacts = TestRole.ArchiveDevArtifacts;
					SessionRole.RespawnProcessOnUnexpectedExit = TestRole.RespawnProcessOnUnexpectedExit;
					SessionRole.OnTryToRestoreProcess = TestRole.OnTryToRestoreProcess;

					Log.Verbose("Created SessionRole {Role} from RoleContext {Context} (RoleType={Type})", SessionRole, RoleContext, TypesToRoles.Key);

					// TODO - this can all / mostly go into UnrealTestConfiguration.ApplyToConfig

					// Deal with command lines
					if (string.IsNullOrEmpty(TestRole.ExplicitClientCommandLine) == false)
					{
						SessionRole.CommandLine = TestRole.ExplicitClientCommandLine;
					}
					else
					{
						// start with anything from our context
						SessionRole.CommandLine += RoleContext.ExtraArgs;

						// did the test ask for anything?
						if (string.IsNullOrEmpty(TestRole.CommandLine) == false)
						{
							SessionRole.CommandLine += TestRole.CommandLine;
						}

						// add controllers
						SessionRole.CommandLineParams.Add("gauntlet",
							TestRole.Controllers.Count > 0 ? string.Join(",", TestRole.Controllers) : null);

						if (PassThroughArgs.Count() > 0)
						{
							SessionRole.CommandLine += " " + string.Join(" ", PassThroughArgs);
						}

						// add options
						SessionRole.Options = Config;
					}

					if (RoleContext.Skip)
					{
						SessionRole.RoleModifier = ERoleModifier.Null;
					}

					// copy over relevant settings from test role
					SessionRole.FilesToCopy = TestRole.FilesToCopy;
					SessionRole.AdditionalArtifactDirectories = TestRole.AdditionalArtifactDirectories;
					SessionRole.PreConfigureDevice = TestRole.PreConfigureDevice;
					SessionRole.ConfigureDevice = TestRole.ConfigureDevice;
					SessionRole.MapOverride = TestRole.MapOverride;
					SessionRole.CompressScreenshots = TestRole.CompressScreenshots;
					SessionRole.CreateGifFromScreenshots = TestRole.CreateGifFromScreenshots;
					SessionRole.RespawnProcessOnUnexpectedExit = TestRole.RespawnProcessOnUnexpectedExit;
					SessionRole.OnTryToRestoreProcess = TestRole.OnTryToRestoreProcess;
					SessionRoles.Add(SessionRole);
				}
			}

			UnrealApp = new UnrealSession(Context.BuildInfo, SessionRoles) { Sandbox = Context.Options.Sandbox };
			return true;
		}

		public override bool IsReadyToStart()
		{
			Reporter.Instance.UpdateReferences(Phases, SetUnrealTestResult, this);

			if (UnrealApp == null)
			{
				PrepareUnrealApp();
			}

			return true;
		}

		/// <summary>
		/// Generates a unique path for the test artifacts and reserves it
		/// This will create the target directory, deleting it first if it exists already
		/// </summary>
		/// <returns>The path to the created artifact directory</returns>
		protected string ReserveArtifactPath()
		{
			// Either use the ArtifactName param or name of this test
			string TestFolder = string.IsNullOrEmpty(Context.Options.ArtifactName) ? this.ToString() : Context.Options.ArtifactName;

			if (string.IsNullOrEmpty(Context.Options.ArtifactPostfix) == false)
			{
				TestFolder += "_" + Context.Options.ArtifactPostfix;
			}

			TestFolder = TestFolder.Replace(" ", "_");
			TestFolder = TestFolder.Replace(":", "_");
			TestFolder = TestFolder.Replace("|", "_");
			TestFolder = TestFolder.Replace(",", "");

			ArtifactPath = Path.Combine(Context.Options.LogDir, TestFolder);

			// if doing multiple passes, put each in a subdir
			if (NumPasses > 1)
			{
				ArtifactPath = Path.Combine(ArtifactPath, string.Format("Pass_{0}_of_{1}", CurrentPass + 1, NumPasses));
			}

			if (Retries > 0)
			{
				ArtifactPath = Path.Combine(ArtifactPath, string.Format("Retry_{0}", Retries));
			}

			// When running with -parallel we could have several identical tests (same test, configurations) in flight so
			// we need unique artifact paths. We also don't overwrite dest directories from the build machine for the same
			// reason of multiple tests for a build. Really though these should use ArtifactPrefix to save to
			// SmokeTest_HighQuality etc
			int ArtifactNumericPostfix = 0;
			bool ArtifactPathIsTaken = false;

			do
			{
				string PotentialPath = ArtifactPath;

				if (ArtifactNumericPostfix > 0)
				{
					PotentialPath = string.Format("{0}_{1}", ArtifactPath, ArtifactNumericPostfix);
				}

				ArtifactPathIsTaken = ReservedArtifcactPaths.Contains(PotentialPath) || (CommandUtils.IsBuildMachine && Directory.Exists(PotentialPath));

				if (ArtifactPathIsTaken)
				{
					Log.Info("Directory already exists at {Path}", PotentialPath);
					ArtifactNumericPostfix++;
				}
				else
				{
					ArtifactPath = PotentialPath;
				}

			} while (ArtifactPathIsTaken);

			DirectoryInfo ArtifactDirectory = new DirectoryInfo(ArtifactPath);
			if (ArtifactDirectory.Exists)
			{
				SystemHelpers.Delete(ArtifactDirectory, true, true);
			}

			Log.Info("Creating artifact path {Path}", ArtifactPath);
			Directory.CreateDirectory(ArtifactPath);

			ReservedArtifcactPaths.Add(ArtifactPath);
			return ArtifactPath;
		}

		/// <summary>
		/// Called by the test executor to start our test running. After this
		/// Test.Status should return InProgress or greater
		/// </summary>
		/// <returns></returns>
		public override bool StartTest(int Pass, int InNumPasses)
		{
			if (InNumPasses > 1
				&& Pass + 1 < InNumPasses)
			{
				if (UnrealApp == null)
				{
					throw new AutomationException("Node already has a null UnrealApp, was PrepareUnrealSession or IsReadyToStart called?");
				}
			}

			// ensure we reset things
			SessionArtifacts = Enumerable.Empty<UnrealRoleArtifacts>();
			RoleResults = Enumerable.Empty<UnrealRoleResult>();
			UnrealTestResult = TestResult.Invalid;
			CurrentPass = Pass;
			NumPasses = InNumPasses;
			LastHeartbeatTime = DateTime.MinValue;
			LastActiveHeartbeatTime = DateTime.MinValue;
			Phases.Clear();

			Reporter.Instance.UpdateReferences(Phases, SetUnrealTestResult, this);

			TConfigClass Config = GetCachedConfiguration();

			ReserveArtifactPath();

			// Launch the test
			TestInstance = UnrealApp.LaunchSession();
			if (TestInstance == null)
			{
				if (InNumPasses > 1 && DevicePool.DeviceReservationBlock)
				{
					// We throw here because we don't want different devices used for a multi-iteration test when using a reservation block
					throw new AutomationException("Failed to start test iteration {0} when using a device reservation block. Terminating the test.", Pass);
				}
				return false;
			}

			// Add info from test context to device usage log
			foreach (IAppInstance AppInstance in TestInstance.ClientApps)
			{
				if (AppInstance != null)
				{
					IDeviceUsageReporter.RecordComment(AppInstance.Device.Name, AppInstance.Device.Platform, IDeviceUsageReporter.EventType.Device, Context.Options.JobDetails);
					IDeviceUsageReporter.RecordComment(AppInstance.Device.Name, AppInstance.Device.Platform, IDeviceUsageReporter.EventType.Test, this.GetType().Name);
				}
			}

			// track the overall session time
			if (SessionStartTime == DateTime.MinValue)
			{
				SessionStartTime = DateTime.Now;
			}

			// Update these for the executor
			MaxDuration = Config.MaxDuration;
			MaxDurationReachedResult = Config.MaxDurationReachedResult;
			MaxRetries = Config.MaxRetries;
			MarkTestStarted();

			// Caching what log categories heartbeat should track
			HeartbeatLogCategories.Add("Gauntlet");
			{
				// get the categories used to monitor process (this needs rethought).
				IEnumerable<string> HeartbeatCategories = GetHeartbeatLogCategories().Union(GetCachedConfiguration().LogCategoriesForEvents);
				HeartbeatLogCategories.AddRange(HeartbeatCategories);
			}
			HeartbeatLogCategories = HeartbeatLogCategories.Distinct().ToList();

			return true;
		}

		/// <summary>
		/// Used to Add a URL to a Horde Report.
		/// </summary>
		/// <param name="Report">The curret ITestReport.</param>
		/// <param name="ReportName">The name you want to show up on the Horde Child Report.</param>
		/// <param name="URLLink">The URL Link you want to populate on the Horde Child Report.</param>
		/// <param name="TextLink">The Text you want to show as URL link. Default is "Open link".</param>
		public void AddLinkToReport(ITestReport Report, string ReportName, string URLLink, string TextLink = null)
		{
			if (Report == null)
			{
				Log.Warning($"No Test report to associated URL Link {URLLink} with.");
			}
			else
			{
				if (Report is AutomatedTestSessionData SessionReport)
				{
					TestEventStream Stream = SessionReport.GetEventStream();
					Stream.AddURLLink(DateTime.UtcNow, Microsoft.Extensions.Logging.LogLevel.Information, ReportName, URLLink);
				}
			}

			if (string.IsNullOrEmpty(TextLink))
			{
				TextLink = "Open link";
			}

			Horde.OnStepSummaryFinalization += () => $"* **{ReportName}**: [{TextLink}]({URLLink})";
			Horde.OnJobSummary += () => $"* **{ReportName}**: [{TextLink}]({URLLink})";
		}

		public virtual void PopulateCommandlineInfo()
		{
			SupportedParameters.Add(new GauntletParamDescription()
			{
				ParamName = "nomcp",
				TestSpecificParam = false,
				ParamDesc = "Run test without an mcp backend",
				DefaultValue = "false"
			});
			SupportedParameters.Add(new GauntletParamDescription()
			{
				ParamName = "ResX",
				InputFormat = "1280",
				Required = false,
				TestSpecificParam = false,
				ParamDesc = "Horizontal resolution for the game client.",
				DefaultValue = "1280"
			});
			SupportedParameters.Add(new GauntletParamDescription()
			{
				ParamName = "ResY",
				InputFormat = "720",
				Required = false,
				TestSpecificParam = false,
				ParamDesc = "Vertical resolution for the game client.",
				DefaultValue = "720"
			});
			SupportedParameters.Add(new GauntletParamDescription()
			{
				ParamName = "FailOnEnsure",
				Required = false,
				TestSpecificParam = false,
				ParamDesc = "Consider the test a fail if we encounter ensures."
			});
			SupportedParameters.Add(new GauntletParamDescription()
			{
				ParamName = "MaxDuration",
				InputFormat = "600",
				Required = false,
				TestSpecificParam = false,
				ParamDesc = "Time in seconds for test to run before it is timed out",
				DefaultValue = "Test-defined"
			});
		}

		protected virtual IEnumerable<KeyValuePair<string, object>> GetDistilledTestParams()
		{
			HashSet<string> ArgsToNotDisplay =
			new List<string>{
					"test",
					"tests",
					"tempdir",
					"logdir",
					"branch",
					"changelist",
					"JobDetails",
					"RecordDeviceUsage",
					"uploadreport",
					"skipdashboardsubmit",
					"ReportURL",
					"ReportExportPath",
					"WriteTestResultsForHorde",
					"HordeTestDataKey",
					"PublishTelemetryTo",
					"ECBranch",
					"ECChangelist",
					"PreFlightChange",
					"AssetRegistryCacheRootFolder",
					"DeactivatedTestConfigPath",
					"SkipInstall",
					"destlocalinstalldir",
					"deviceurl",
					"devicepool",
					"VerifyLogin",
					"cleardevices",
					"reboot",
					"RequireBoot",
					"fullclean",
					"BuildName",
					"PerfReportServer",
					"sandbox",
			}.Select(I => I.ToLower()).ToHashSet();

			return Context.TestParams.AllArguments.Select(A =>
			{
				var Arg = A.Split('=', 2);
				return new KeyValuePair<string, object>(Arg[0], Arg.Length > 1 ? (Arg[1].Equals("false", StringComparison.OrdinalIgnoreCase)? false : Arg[1]) : true);
			}).Where(A => !ArgsToNotDisplay.Contains(A.Key.ToLower()));
		}

		public override string GetRunLocalCommand(string LaunchingBuildCommand)
		{

			string WrapParameterInQuotes(string ParamStrName, string ParamStrValue)
			{
				if (ParamStrValue.Contains(' '))
				{
					return string.Format("{0}=\"{1}\"", ParamStrName, ParamStrValue);
				}
				else
				{
					return string.Format("{0}={1}", ParamStrName, ParamStrValue);
				}
			}

			var CleanedArgs = GetDistilledTestParams().Select(Arg => Arg.Value.Equals(true)? Arg.Key : WrapParameterInQuotes(Arg.Key, Arg.Value.ToString()));

			string StringOfCleanedArguments = string.Format("-{0}", string.Join(" -", CleanedArgs));
			string CommandToRunLocally =
				string.Format("RunUAT {0} -Test=\"{1}\" {2}", LaunchingBuildCommand, GetType(), StringOfCleanedArguments);

			return CommandToRunLocally;
		}

		/// <summary>
		/// Cleanup all resources
		/// </summary>
		/// <returns></returns>
		public override void CleanupTest()
		{
			Reporter.Instance.UpdateReferences(Phases, SetUnrealTestResult, this);

			if (TestInstance != null)
			{
				TestInstance.Dispose();
				TestInstance = null;
			}

			if (UnrealApp != null)
			{
				if (CurrentPass + 1 == NumPasses)
				{
					UnrealApp.Dispose();
					UnrealApp = null;
				}
				else
				{
					UnrealApp.ShutdownInstance();
				}
			}
		}

		/// <summary>
		/// Restarts the provided test. Only called if one of our derived
		/// classes requests it via the Status result
		/// </summary>
		/// <returns></returns>
		public override bool RestartTest()
		{
			//Reset/Increment artifact output
			SessionArtifacts = Enumerable.Empty<UnrealRoleArtifacts>();
			RoleResults = Enumerable.Empty<UnrealRoleResult>();

			//Reset log parser for all apps
			RoleParsers.Clear();

			LastHeartbeatTime = DateTime.MinValue;
			LastActiveHeartbeatTime = DateTime.MinValue;

			//Reset phases
			Phases.Clear();

			ReserveArtifactPath();

			// Relaunch the test
			TestInstance = UnrealApp.RestartSession();

			bool bWasRestarted = (TestInstance != null);
			if (bWasRestarted)
			{
				MarkTestStarted();
			}

			return bWasRestarted;
		}

		/// <summary>
		/// Periodically called while the test is running. A chance for tests to examine their
		/// health, log updates etc. Base classes must call this or take all responsibility for
		/// setting Status as necessary
		/// </summary>
		/// <param name="InInstance">The Unrealnstance being tested</param>
		public virtual void TickTest(UnrealSessionInstance InInstance)
		{
			// Retrieve and log heartbeat
			LogHeartbeat(InInstance);

			// Detect missed heartbeats and fail the test
			CheckHeartbeat();
		}

		/// <summary>
		/// Periodically called while the test is running. A chance for tests to examine their
		/// health, log updates etc. Base classes must call this or take all responsibility for
		/// setting Status as necessary
		/// </summary>
		/// <returns></returns>
		public override void TickTest()
		{
			Reporter.Instance.UpdateReferences(Phases, SetUnrealTestResult, this);

			TickTest(TestInstance);

			// Check status and health after updating logs
			if (GetTestStatus() == TestStatus.InProgress && IsTestRunning() == false)
			{
				MarkTestComplete();
			}
		}

		/// <summary>
		/// This class is here to provide compatibility 
		/// </summary>
		/// <param name="WasCancelled"></param>
		protected virtual void StopTest(bool WasCancelled)
		{
			StopTest(WasCancelled ? StopReason.MaxDuration : StopReason.Completed);
		}

		/// <summary>
		/// Called when a test has completed. By default saves artifacts and calls CreateReport
		/// </summary>
		/// <param name="InReason"></param>
		/// <returns></returns>
		public override void StopTest(StopReason InReason)
		{
			if (Context.TestParams.ParseParam("FailurePrtScr") && (GetTestResult() != TestResult.Passed || InReason != StopReason.Completed))
			{
				foreach (ITargetDevice TargetDevice in UnrealApp.UnrealDeviceReservation.ReservedDevices)
				{
					TargetDevice.TakeScreenshot("FailureScreenshot.png");
				}
			}
			base.StopTest(InReason);

			// Warn if there are still deferred roles that have not been launched when the test is finished
			foreach (UnrealSessionInstance.RoleInstance DeferredRole in TestInstance.DeferredRoles)
			{
				Log.Warning("Deferred role {Role} was not started before the test was stopped", DeferredRole);
			}

			// Shutdown the instance so we can access all files, but do not null it or shutdown the UnrealApp because we still need
			// access to these objects and their resources! Final cleanup is done in CleanupTest()
			TestInstance.Shutdown(InReason == StopReason.MaxDuration || GetTestResult() == TestResult.TimedOut);

			try
			{
				Log.Info("Saving artifacts to {Path}", ArtifactPath);
				// run create dir again just in case the already made dir was cleaned up by another buildfarm job or something similar.
				Directory.CreateDirectory(ArtifactPath);
				Utils.SystemHelpers.MarkDirectoryForCleanup(ArtifactPath);

				SessionArtifacts = SaveRoleArtifacts(ArtifactPath);

				// call legacy version
				SaveArtifacts_DEPRECATED(ArtifactPath);
			}
			catch (Exception Ex)
			{
				Log.Warning("Failed to save artifacts. {Exception}", Ex);
			}

			if (UnrealApp != null && CurrentPass + 1 == NumPasses)
			{
				try
				{
					// Artifacts have been saved, release devices back to pool for other tests to use
					UnrealApp.ReleaseSessionDevices();
				}
				catch (Exception Ex)
				{
					Log.Warning(KnownLogEvents.Gauntlet_DeviceEvent, "Failed to release devices. {Exception}", Ex);
				}
			}


			// Create results from all roles from these artifacts
			RoleResults = CreateRoleResultsFromArtifacts(InReason, SessionArtifacts);

			if (RoleResults.Any(R => R.ProcessResult == UnrealProcessResult.EncounteredFatalError))
			{
				UnrealRoleResult CrashedRole = RoleResults.FirstOrDefault(R => R.ProcessResult == UnrealProcessResult.EncounteredFatalError);
				if (CrashedRole != null)
				{
					if (CrashedRole.Artifacts.SessionRole.RoleType == UnrealTargetRole.Client)
					{
						Horde.UpdateStepMessage(Horde.GauntletHordeStepMessage.ClientCrashDetected);
					}
					else if (CrashedRole.Artifacts.SessionRole.RoleType == UnrealTargetRole.Server)
					{
						Horde.UpdateStepMessage(Horde.GauntletHordeStepMessage.ServerCrashDetected);
					}
					else if (CrashedRole.Artifacts.SessionRole.RoleType == UnrealTargetRole.Editor)
					{
						Horde.UpdateStepMessage(Horde.GauntletHordeStepMessage.EditorCrashDetected);
					}
				}
			}

			// Update Test node result from Role processed results when they don't match
			if (GetTestResult() == TestResult.Passed && RoleResults.Any(R => R.ProcessResult != UnrealProcessResult.ExitOk))
			{
				if (RoleResults.Any(R => R.ProcessResult == UnrealProcessResult.TimeOut))
				{
					Horde.UpdateStepMessage(Horde.GauntletHordeStepMessage.TestTimeoutExceeded);
					SetTestResult(TestResult.TimedOut);
				}
				else if (RoleResults.Any(R => R.ProcessResult == UnrealProcessResult.EncounteredFatalError))
				{
					SetTestResult(TestResult.Failed);
				}
				else if (!GetCachedConfiguration().IgnoreRoleFailures)
				{
					SetTestResult(TestResult.OperationalException);
				}
			}

			string Message = string.Empty;
			ITestReport Report = null;

			try
			{
				// Check if the deprecated signature is overridden, call it anyway if that the case.
				var OldSignature = new[] { typeof(TestResult), typeof(UnrealTestContext), typeof(UnrealBuildSource), typeof(IEnumerable<UnrealRoleResult>), typeof(string) };
				var Simplifiedignature = new[] { typeof(TestResult) };
				if (!Utils.InterfaceHelpers.HasOverriddenMethod(this.GetType(), "CreateReport", Simplifiedignature) && Utils.InterfaceHelpers.HasOverriddenMethod(this.GetType(), "CreateReport", OldSignature))
				{
					Report = CreateReport(GetTestResult(), Context, Context.BuildInfo, RoleResults, ArtifactPath);
				}
				else
				{
					Report = CreateReport(GetTestResult());
				}
			}
			catch (Exception Ex)
			{
				CreateReportFailed = true;
				Message = Ex.Message;
				Log.Warning("Failed to save completion report. {Exception}", Ex);
			}

			if (CreateReportFailed)
			{
				try
				{
					HandleCreateReportFailure(Context, Message);
				}
				catch (Exception Ex)
				{
					Log.Warning("Failed to handle completion report failure. {Exception}", Ex);
				}
			}

			try
			{
				if (Report != null)
				{
					SubmitToDashboard(Report);
				}

			}
			catch (Exception Ex)
			{
				Log.Warning("Failed to submit results to dashboard. {Exception}", Ex);
			}
		}

		/// <summary>
		/// Called when report creation fails, by default logs warning with failure message
		/// </summary>
		protected virtual void HandleCreateReportFailure(UnrealTestContext Context, string Message = "")
		{
			if (string.IsNullOrEmpty(Message))
			{
				Message = string.Format("See Gauntlet.log for details");
			}

			Log.Warning("CreateReport Failed: {Failure}", Message);
		}

		/// <summary>
		/// Whether creating the test report failed
		/// </summary>
		public bool CreateReportFailed { get; protected set; }

		/// <summary>
		/// Add all of our process results to the gauntlet event summary list.
		/// </summary>
		public virtual void AddProcessResultEventsFromTestNode()
		{
			if (RoleResults != null)
			{
				// First, make sure we have all of our process exit events.
				// Iterate through ProcessResults to see if we need to add any messages to summary.
				HashSet<UnrealProcessResult> DistinctResults = new HashSet<UnrealProcessResult>();
				foreach (UnrealRoleResult roleResult in RoleResults)
				{
					DistinctResults.Add(roleResult.ProcessResult);
				}
				if (DistinctResults.Contains(UnrealProcessResult.InitializationFailure))
				{
					AddTestEvent(new UnrealTestEvent(EventSeverity.Fatal, "Engine Initialization Failed", new List<string> { "Engine failed to initialize in one or more roles. This is likely a bad build." }));
				}
				if (DistinctResults.Contains(UnrealProcessResult.LoginFailed))
				{
					AddTestEvent(new UnrealTestEvent(EventSeverity.Fatal, "Login Unsuccessful", new List<string> { "User account never successfully finished logging in." }));
				}
				if (DistinctResults.Contains(UnrealProcessResult.EncounteredFatalError))
				{
					AddTestEvent(new UnrealTestEvent(EventSeverity.Fatal, "Fatal Error Encountered", new List<string> { "Test encountered a fatal error. Check ClientOutput.log and ServerOutput.log for details." }));
				}
				if (DistinctResults.Contains(UnrealProcessResult.EncounteredEnsure))
				{
					AddTestEvent(new UnrealTestEvent(EventSeverity.Error, "Ensure Found", new List<string> { "Test encountered one or more ensures. See above for details." }));
				}
				if (DistinctResults.Contains(UnrealProcessResult.TestFailure))
				{
					AddTestEvent(new UnrealTestEvent(EventSeverity.Error, "Test Failure Encountered", new List<string> { "Test encountered a failure. See above for more info." }));
				}
				if (DistinctResults.Contains(UnrealProcessResult.TimeOut))
				{
					AddTestEvent(new UnrealTestEvent(EventSeverity.Error, "Test Timed Out", new List<string> { "Test terminated due to timeout. Check ClientOutput.log and ServerOutput.log for more info." }));
				}
			}
		}

		/// <summary>
		/// Display all of the defined commandline information for this test.
		/// Will display generic gauntlet params as well if -help=full is passed in.
		/// </summary>
		public override void DisplayCommandlineHelp()
		{
			Log.Info(string.Format("--- Available Commandline Parameters for {0} ---", Name));
			Log.Info("--------------------");
			Log.Info("TEST-SPECIFIC PARAMS");
			Log.Info("--------------------");
			foreach (GauntletParamDescription ParamDesc in SupportedParameters)
			{
				if (ParamDesc.TestSpecificParam)
				{
					Log.Info(ParamDesc.ToString());
				}
			}
			if (Context.TestParams.ParseParam("listallargs"))
			{
				Log.Info("\n--------------");
				Log.Info("GENERIC PARAMS");
				Log.Info("--------------");
				foreach (GauntletParamDescription ParamDesc in SupportedParameters)
				{
					if (!ParamDesc.TestSpecificParam)
					{
						Log.Info(ParamDesc.ToString());
					}
				}
			}
			else
			{
				Log.Info("\nIf you need information on base Gauntlet arguments, use -listallargs\n\n");
			}
			if (SampleCommandlines.Count > 0)
			{
				Log.Info("\n-------------------");
				Log.Info("SAMPLE COMMANDLINES");
				Log.Info("-------------------");
				foreach (KeyValuePair<string, string> SampleCommandline in SampleCommandlines)
				{
					Log.Info(SampleCommandline.Key);
					Log.Info("");
					Log.Info(SampleCommandline.Value);
					Log.Info("-------------------");
				}
			}

		}

		/// <summary>
		/// Optional function that is called on test completion and gives an opportunity to create a report. The returned class will later be passed to
		/// SubmitToDashboard if submission of results is enabled
		/// </summary>
		/// <param name="Result">Test result</param>
		/// <param name="Context">Context that describes the environment of the test</param>
		/// <param name="Build">Build being tested</param>
		/// <param name="RoleResults">Results from each role in the test</param>
		/// <param name="ArtifactPath">Path to where artifacts from each role are saved</param>
		/// <returns></returns>
		public virtual ITestReport CreateReport(TestResult Result, UnrealTestContext Context, UnrealBuildSource Build, IEnumerable<UnrealRoleResult> RoleResults, string ArtifactPath)
		{
			if ((CheckShouldRetry() && SetToRetryIfPossible()) || IsSetToRetry()) { return null; }
			if (GetCachedConfiguration().WriteTestResultsForHorde)
			{
				// write test report for Horde
				return CreateSimpleReportForHorde(Result);
			}

			return null;
		}

		/// <summary>
		/// Optional function that is called on test completion and gives an opportunity to create a report
		/// </summary>
		/// <param name="Result"></param>
		/// <returns>ITestReport</returns>
		public virtual ITestReport CreateReport(TestResult Result)
		{
			if (GetCachedConfiguration().WriteTestResultsForHorde)
			{
				// write test report for Horde
				return CreateSimpleReportForHorde(Result);
			}

			return null;
		}

		private string GetHordeTestDataKey_legacy()
		{
			if (string.IsNullOrEmpty(GetCachedConfiguration().HordeTestDataKey))
			{
				return Name + " " + GetMainRoleContextString();
			}

			return GetCachedConfiguration().HordeTestDataKey;
		}

		/// <summary>
		/// Generate a Simple Test Report from the results of this test
		/// </summary>
		/// <param name="Result"></param>
		protected virtual ITestReport CreateSimpleReportForHorde(TestResult Result)
		{
			if (Phases.Any())
			{
				OnPopulateTestPhases += PopulatePhaseReport;
				OnPopulateTestPhases += (Report) => PopulateClosurePhase(Report);
			}

			string ArtifactPathForReport = ArtifactPath;
			Horde.OnStepSummary += () => Horde.GenerateArtifactLink(ArtifactPathForReport);
			BaseHordeReport HordeTestReport = null;
			if (Horde.UseTestDataV1)
			{
				// Legacy
				SimpleTestReport SimpleReport = CreateSimpleReportForHorde_legacy(Result);
				HordeTestReport = SimpleReport;
			}
			else
			{
				AutomatedTestSessionData TestReport = new AutomatedTestSessionData(TestMetadata);
				DateTime SessionTime = SessionStartTime.ToUniversalTime();
				float TimeElapse = (float)(DateTime.Now - SessionStartTime).TotalSeconds;
				TestReport.SetSessionTiming(SessionTime, TimeElapse);
				// Simple report has one phase
				var InitPhase = TestReport.AddPhase("Initialization", "Initialization");
				InitPhase.SetTiming(SessionTime, 0); // Unfortunately we are not tracking initialization duration at this point
				if (RoleResults != null)
				{
					foreach (UnrealRoleResult Item in RoleResults)
					{
						string RoleType = Item.Artifacts.SessionRole.RoleType.ToString();
						if (Item.ProcessResult == UnrealProcessResult.InitializationFailure || Item.ProcessResult == UnrealProcessResult.LoginFailed)
						{
							var RelevantStream = InitPhase.GetStream();
							foreach (UnrealTestEvent Event in Item.Events)
							{
								RelevantStream.AddEvent(Event);
							}
							// Add Process result
							if (Item.ProcessResult != UnrealProcessResult.ExitOk)
							{
								RelevantStream.AddError($"{RoleType}: {Item.Summary}");
								InitPhase.SetOutcome(TestPhaseOutcome.Interrupted);
							}
						}
						// Since we are going through the Role, lets add the related device
						var ReportDevice = AddRoleDeviceToSessionReport(Item, TestReport);
						InitPhase.deviceKeys.Add(ReportDevice.key);
					}
				}
				if (OnPopulateTestPhases != null)
				{
					// Some delegates have been setup
					OnPopulateTestPhases(TestReport);
				}
				else
				{
					// Default phase if no delegate were registered
					PopulateClosurePhase(TestReport, "Main", SessionTime, TimeElapse);
				}
				HordeTestReport = TestReport;
				// Metadata
				SetReportMetadata(HordeTestReport, new UnrealTestRole[] { GetCachedConfiguration().GetMainRequiredRole() });
			}
			string HordeArtifactPath = string.IsNullOrEmpty(GetCachedConfiguration().HordeArtifactPath) ? DefaultArtifactsDir : GetCachedConfiguration().HordeArtifactPath;
			string ArtifactPathLabel = new DirectoryInfo(ArtifactPath).Name;
			HordeTestReport.SetOutputArtifactPath(Path.Combine(HordeArtifactPath, ArtifactPathLabel));
			if (SessionArtifacts != null)
			{
				foreach (UnrealRoleResult RoleResult in RoleResults)
				{
					string LogPath = RoleResult.Artifacts.LogPath;
					if (!string.IsNullOrEmpty(LogPath) && File.Exists(LogPath))
					{
						string LogName = FileUtils.ConvertPathToUnix(Path.GetRelativePath(Path.GetFullPath(ArtifactPath), Path.GetFullPath(LogPath)));
						if (HordeTestReport is BaseHordeReport Report)
						{
							string DeviceKey = RoleResult.Artifacts.AppInstance.Device.Name.Replace(".", "-");
							Report.AttachDeviceLog(DeviceKey, LogPath, LogName);
						}
						else
						{
							HordeTestReport.AttachArtifact(LogPath, LogName);
						}

					}
				}
			}

			return HordeTestReport;
		}

		/// <summary>
		/// Populate report phases from test execution
		/// </summary>
		/// <param name="SessionData"></param>
		protected virtual void PopulatePhaseReport(AutomatedTestSessionData SessionData)
		{
			// Assume all devices were used to run all test cases
			HashSet<string> DeviceKeys = SessionData.GetDevices().Select(D => D.key).ToHashSet();
			// Go through all phases
			foreach (Phase CurPhase in Phases)
			{
				if (string.IsNullOrEmpty(CurPhase.Name)) continue;
				TestPhase ReportedPhase = CurPhase.AddToReport(SessionData);

				if (ReportedPhase.outcome != TestPhaseOutcome.NotRun
					&& ReportedPhase.outcome != TestPhaseOutcome.Skipped)
				{
					ReportedPhase.deviceKeys = DeviceKeys;
				}
			}
		}

		/// <summary>
		/// Default populate closure phase
		/// </summary>
		/// <param name="TestReport"></param>
		/// <param name="PhaseName"></param>
		protected virtual void PopulateClosurePhase(AutomatedTestSessionData TestReport, string PhaseName = "Closure", DateTime? SessionTime = null, float TimeElapseSeconds = 0)
		{
			var ClosurePhase = TestReport.AddPhase(PhaseName);
			ClosurePhase.SetTiming(SessionTime ?? DateTime.UtcNow, TimeElapseSeconds);
			// Propagate events
			if (RoleResults != null)
			{
				foreach (UnrealRoleResult Item in RoleResults)
				{
					string RoleType = Item.Artifacts.SessionRole.RoleType.ToString();
					if (Item.ProcessResult != UnrealProcessResult.InitializationFailure && Item.ProcessResult != UnrealProcessResult.LoginFailed)
					{
						var RelevantStream = ClosurePhase.GetStream();
						foreach (UnrealTestEvent Event in Item.Events)
						{
							RelevantStream.AddEvent(Event);
						}
						// Add Process result
						if (Item.ProcessResult != UnrealProcessResult.ExitOk)
						{
							RelevantStream.AddError($"{RoleType}: {Item.Summary}");
						}
					}
				}
				ClosurePhase.deviceKeys = TestReport.GetDevices().Select(D => D.key).ToHashSet();
			}
			var MainStream = ClosurePhase.GetStream();
			foreach (UnrealTestEvent Event in TestNodeEvents)
			{
				MainStream.AddEvent(Event);
			}
			// Outcome
			TestResult Result = GetTestResult();
			if (Result != TestResult.Passed)
			{
				ClosurePhase.SetOutcome(Result);
			}
		}

		private AutomatedTestSessionData.TestDevice AddRoleDeviceToSessionReport(UnrealRoleResult RoleResult, AutomatedTestSessionData TestReport)
		{
			ITargetDevice TargetDevice = RoleResult.Artifacts.AppInstance.Device;
			string DeviceName = TargetDevice.Platform == BuildHostPlatform.Current.Platform ? System.Environment.MachineName : TargetDevice.Name;
			string DeviceKey = TargetDevice.Name.Replace(".", "-"); // Key must not contains dot
			string RoleType = RoleResult.Artifacts.SessionRole.RoleType.ToString();
			AutomatedTestSessionData.TestDevice ReportDevice = TestReport.AddDevice(DeviceName, DeviceKey);
			ReportDevice.metadata.Add("Platform", TargetDevice.Platform.ToString());
			ReportDevice.metadata.Add("Role", RoleType);

			return ReportDevice;
		}

		private SimpleTestReport CreateSimpleReportForHorde_legacy(TestResult Result)
		{
			SimpleTestReport SimpleReport = new SimpleTestReport(UnrealTestResult, Context, GetCachedConfiguration());
			SimpleReport.TestName = HordeReportTestName;
			SimpleReport.ReportCreatedOn = DateTime.UtcNow.ToString();
			SimpleReport.TotalDurationSeconds = (float)(DateTime.Now - SessionStartTime).TotalSeconds;
			SimpleReport.Description = GetMainRoleContextString();
			SimpleReport.Errors.AddRange(GetErrorsAndAbnornalExits());
			if (!string.IsNullOrEmpty(CancellationReason))
			{
				SimpleReport.Errors.Add(CancellationReason);
			}
			SimpleReport.Warnings.AddRange(GetWarnings());
			SimpleReport.HasSucceeded = !(Result == TestResult.Failed || Result == TestResult.TimedOut || SimpleReport.Errors.Count > 0);
			if (SimpleReport.Errors.Count > 0 && Result == TestResult.Passed)
			{
				// Make sure the test is marked complete before setting the test result
				MarkTestComplete();
				SetUnrealTestResult(TestResult.Failed);
			}
			SimpleReport.Status = GetTestResult().ToString();
			// Metadata
			SetReportMetadata(SimpleReport, GetCachedConfiguration().RequiredRoles.Values.SelectMany(V => V));

			return SimpleReport;
		}

		/// <summary>
		/// Generate report from Unreal Automated Test Results
		/// </summary>
		/// <param name="UnrealAutomatedTestReportPath"></param>
		/// <param name="ReportURL"></param>
		/// <returns>ITestReport</returns>
		public virtual ITestReport CreateUnrealEngineTestPassReport(string UnrealAutomatedTestReportPath, string ReportURL)
		{
			if (IsSetToRetry()) { return null; }

			BaseHordeReport HordeTestPassResults = null;
			// Handle initialization phase
			if (!Horde.UseTestDataV1)
			{
				AutomatedTestSessionData SessionReport = new AutomatedTestSessionData(TestMetadata);
				string ArtifactPathLabel = new DirectoryInfo(ArtifactPath).Name;
				string HordeArtifactPath = Path.Combine(GetCachedConfiguration().HordeArtifactPath, ArtifactPathLabel);
				SessionReport.SetOutputArtifactPath(HordeArtifactPath);
				DateTime SessionTime = SessionStartTime.ToUniversalTime();
				// Elapse time will be adjusted later if the initialization passed
				float TimeElapse = (float)(DateTime.Now - SessionStartTime).TotalSeconds;
				SessionReport.SetSessionTiming(SessionTime, TimeElapse);
				var InitPhase = SessionReport.AddPhase("Initialization", "Initialization");
				InitPhase.SetTiming(SessionTime, TimeElapse);
				if (RoleResults != null)
				{
					foreach (UnrealRoleResult Item in RoleResults)
					{
						// Add the related devices to initialization phase
						var ReportDevice = AddRoleDeviceToSessionReport(Item, SessionReport);
						InitPhase.deviceKeys.Add(ReportDevice.key);
					}
				}
				HordeTestPassResults = SessionReport;
			}

			// With UE Test Automation, we care only for one role.
			var MainRole = GetCachedConfiguration().GetMainRequiredRole();
			var RoleList = new List<UnrealTestRole>() { MainRole };
			string MainDeviceInstanceName = string.Empty;

			string JsonReportPath = Path.Combine(UnrealAutomatedTestReportPath, "index.json");
			bool IsFileReportExist = File.Exists(JsonReportPath);
			if (IsFileReportExist)
			{
				Log.Verbose("Reading json Unreal Automated test report from {Path}", JsonReportPath);
				UnrealAutomatedTestPassResults JsonTestPassResults = UnrealAutomatedTestPassResults.LoadFromJson(JsonReportPath);
				MainDeviceInstanceName = JsonTestPassResults.Devices.Last().Instance;
				// Convert test results for Horde
				if (HordeTestPassResults is AutomatedTestSessionData SessionReport)
				{ // Populate v2 if set before
					var InitPhase = SessionReport.GetPhase("Initialization");
					if (InitPhase != null)
					{
						if (JsonTestPassResults.Tests != null && JsonTestPassResults.Tests.Any())
						{ // Adjust initialization timing since the json report was generated (initialization must have completed successfully)
							DateTime FirstTestTime = UnrealAutomationEntry.GetTimestampAsDateTime(JsonTestPassResults.Tests.First().DateTime);
							DateTime SessionTime = SessionStartTime.ToUniversalTime();
							float TimeElapse = (float)(FirstTestTime - SessionTime).TotalSeconds;
							InitPhase.timeElapseSec = TimeElapse;
						}
						else
						{
							bool bAllowZeroTestResults = Globals.Params.ParseParam("AllowZeroTestResults");

							var Stream = InitPhase.GetStream();
							if (bAllowZeroTestResults)
							{
								Stream.AddWarning("No tests were executed, check test filter!");
							}
							else
							{
								Stream.AddError("No tests were executed!");
							}
							if (RoleResults != null)
							{
								// Add any relevant events collected through log parsing
								foreach (UnrealRoleResult RoleResult in RoleResults)
								{
									foreach (UnrealTestEvent Item in RoleResult.Events)
									{
										Stream.AddEvent(Item);
									}
								}
							}
						}
					}
					SessionReport.PopulateFromUnrealAutomatedTests(JsonTestPassResults, UnrealAutomatedTestReportPath);
				}
				else if (HordeTestPassResults == null)
				{
					// Legacy report
					string HordeArtifactPath = GetCachedConfiguration().HordeArtifactPath;
					AutomatedTestSessionData_legacy LegacyTestPassResults = AutomatedTestSessionData_legacy.FromUnrealAutomatedTests(
						JsonTestPassResults, Type, Suite, UnrealAutomatedTestReportPath, HordeArtifactPath
					);
					// Pre Flight information
					LegacyTestPassResults.PreFlightChange = GetCachedConfiguration().PreFlightChange;
					HordeTestPassResults = LegacyTestPassResults;
					// Check dependencies
					foreach (var Item in LegacyTestPassResults.GetReportDependencies())
					{
						HordeTestPassResults.AttachDependencyReport(Item.Value, Item.Key);
					}
					// Make a copy of the report in the old way - until we decide the transition is over
					UnrealEngineTestPassResults CopyTestPassResults = UnrealEngineTestPassResults.FromUnrealAutomatedTests(JsonTestPassResults, ReportURL);
					CopyTestPassResults.CopyTestResultsArtifacts(UnrealAutomatedTestReportPath, HordeArtifactPath);
					SetReportMetadata(CopyTestPassResults, RoleList); // Set the metadata to the old report
					HordeTestPassResults.AttachDependencyReport(CopyTestPassResults, GetHordeTestDataKey_legacy());
				}
			}
			else
			{
				Log.Warning("Could not find Unreal Automated test report at {FilePath}. Test report will be partial.", JsonReportPath);
				if (HordeTestPassResults is AutomatedTestSessionData SessionReport)
				{
					var InitPhase = SessionReport.GetPhase("Initialization");
					InitPhase.GetStream().AddWarning("Could not find Unreal Automated test report at {FilePath}. No test report details available.", JsonReportPath);
					if (RoleResults != null)
					{
						// Check if there is any non-OK exit to add more context
						foreach (UnrealRoleResult RoleResult in RoleResults.Where(R => R.ProcessResult != UnrealProcessResult.ExitOk && R.ProcessResult != UnrealProcessResult.EncounteredEnsure))
						{
							string RoleType = RoleResult.Artifacts.SessionRole.RoleType.ToString();
							var Stream = InitPhase.GetStream();
							Stream.AddError("{Role}: {Summary}", RoleType, RoleResult.Summary);
							foreach (UnrealTestEvent Item in RoleResult.Events)
							{
								Stream.AddEvent(Item);
							}

							// In case of a fatal error or initialization failure, it is an interrupted outcome
							if (RoleResult.ProcessResult == UnrealProcessResult.EncounteredFatalError || RoleResult.ProcessResult == UnrealProcessResult.InitializationFailure)
							{
								if (RoleResult.ProcessResult == UnrealProcessResult.InitializationFailure)
								{
									InitPhase.GetStream().AddError("No tests were executed!");
								}
								InitPhase.SetOutcome(TestPhaseOutcome.Interrupted);
							}
						}
					}
				}
			}

			if (HordeTestPassResults != null)
			{ // Populate generic data and artifacts
				// Metadata
				SetReportMetadata(HordeTestPassResults, RoleList);
				// Attached test log artifacts
				if (SessionArtifacts != null)
				{
					foreach (UnrealRoleArtifacts Artifact in SessionArtifacts)
					{
						string DeviceKey = Artifact.AppInstance.Device.Name.Replace(".", "-"); // Key must not contains dot
						if (!string.IsNullOrEmpty(Artifact.LogPath))
						{
							string LogName = FileUtils.ConvertPathToUnix(Path.GetRelativePath(Path.GetFullPath(Horde.UseTestDataV1 ? Context.Options.LogDir : ArtifactPath), Path.GetFullPath(Artifact.LogPath)));
							if (HordeTestPassResults is BaseHordeReport Report)
							{
								Report.AttachDeviceLog(DeviceKey, Artifact.LogPath, LogName);
								if (Artifact.SessionRole.RoleType == MainRole.Type && !string.IsNullOrEmpty(MainDeviceInstanceName))
								{ // UE json report has a different Instance name than Gauntlet device
									Report.AttachDeviceLog(MainDeviceInstanceName, Artifact.LogPath, LogName);
								}
							}
							else
							{
								HordeTestPassResults.AttachArtifact(Artifact.LogPath, LogName);
							}
						}
					}
				}
			}

			return HordeTestPassResults;
		}

		/// <summary>
		/// Set Metadata on ITestReport
		/// </summary>
		/// <param name="Report"></param>
		/// <param name="Roles"></param>
		protected virtual void SetReportMetadata(ITestReport Report, IEnumerable<UnrealTestRole> Roles)
		{
			var AllRoleTypes = Roles.Select(R => R.Type);
			var AllRoleContexts = Roles.Select(R => Context.GetRoleContext(R.Type));
			Report.SetMetadata("Platform", string.Join("+", AllRoleContexts.Select(R => R.Platform.ToString()).Distinct().OrderBy(P => P)));
			Report.SetMetadata("BuildTarget", string.Join("+", AllRoleTypes.Select(R => R.ToString()).Distinct().OrderBy(B => B)));
			Report.SetMetadata("Configuration", string.Join("+", AllRoleContexts.Select(R => R.Configuration.ToString()).Distinct().OrderBy(C => C)));
			Report.SetMetadata("Project", Context.BuildInfo.ProjectName);

			var TagFilter = Globals.Params.ParseValue("TagFilter", string.Empty);
			if (!string.IsNullOrEmpty(TagFilter))
			{
				Report.SetMetadata("TagFilter", TagFilter);
			}

			// Additional metadata passed through the command line arguments
			foreach (var Meta in Globals.Params.ParseValues("Metadata"))
			{
				var Entry = Meta.Split(":", 2);
				if (Entry.Count() > 1)
				{
					Report.SetMetadata(Entry[0], Entry[1]);
				}
			}

			// Additional metadata based on test config
			if (GetCachedConfiguration() is UnrealTestConfiguration TestConfig)
			{
				if (!string.IsNullOrEmpty(TestConfig.Map))
				{
					Report.SetMetadata("Map", Path.GetFileName(TestConfig.Map));
				}
			}
		}

		void SubmitToHorde(ITestReport Report)
		{
			if (!GetCachedConfiguration().WriteTestResultsForHorde)
			{
				return;
			}

			// write test data collection for Horde
			string FileName = FileUtils.SanitizeFilename(string.IsNullOrEmpty(Context.Options.ArtifactName) ? Name : Context.Options.ArtifactName);
			string HordeTestDataFilePath = Path.Combine(
				string.IsNullOrEmpty(GetCachedConfiguration().HordeTestDataPath) ? HordeReport.DefaultTestDataDir : GetCachedConfiguration().HordeTestDataPath,
				FileName + ".TestData.json"
			);
			HordeReport.TestDataCollection HordeTestDataCollection = new HordeReport.TestDataCollection();
			HordeTestDataCollection.AddNewTestReport(Report, GetHordeTestDataKey_legacy());
			HordeTestDataCollection.WriteToJson(HordeTestDataFilePath, !AutomationTool.Automation.IsBuildMachine);
		}

		/// <summary>
		/// Optional function that is called on test completion and gives an opportunity to submit a report to a Dashboard
		/// </summary>
		/// <param name="Report"></param>
		public virtual void SubmitToDashboard(ITestReport Report)
		{
			SubmitToHorde(Report);

			if (!string.IsNullOrEmpty(GetCachedConfiguration().PublishTelemetryTo) && Report is ITelemetryReport Telemetry)
			{
				IEnumerable<TelemetryData> DataRows = Telemetry.GetAllTelemetryData();
				if (DataRows != null)
				{
					IDatabaseConfig<TelemetryData> DBConfig = DatabaseConfigManager<TelemetryData>.GetConfigByName(GetCachedConfiguration().PublishTelemetryTo);
					if (DBConfig != null)
					{
						DBConfig.LoadConfig(GetCachedConfiguration().DatabaseConfigPath);
						IDatabaseDriver<TelemetryData> DB = DBConfig.GetDriver();
						Log.Verbose("Submitting telemetry data to {Database}", DB.ToString());

						UnrealTelemetryContext TestContext = new UnrealTelemetryContext();
						TestContext.SetProperty("ProjectName", Context.BuildInfo.ProjectName);
						TestContext.SetProperty("Branch", Context.BuildInfo.Branch);
						TestContext.SetProperty("Changelist", Context.BuildInfo.Changelist);
						var RoleType = GetCachedConfiguration().GetMainRequiredRole().Type;
						var Role = Context.GetRoleContext(RoleType);
						TestContext.SetProperty("Platform", Role.Platform);
						TestContext.SetProperty("Configuration", string.Format("{0} {1}", RoleType, Role.Configuration));

						DB.SubmitDataItems(DataRows, TestContext);
					}
					else
					{
						Log.Warning("Got telemetry data, but database configuration is unknown '{Config}'.", GetCachedConfiguration().PublishTelemetryTo);
					}
				}
			}
		}

		/// <summary>
		/// Called to request that the test save all artifacts from the completed test to the specified 
		/// output path. By default the app will save all logs and crash dumps
		/// </summary>
		/// <param name="OutputPath"></param>
		/// <returns></returns>
		public virtual void SaveArtifacts_DEPRECATED(string OutputPath)
		{
			// called for legacy reasons
		}

		/// <summary>
		/// Called to request that the test save all artifacts from the completed test to the specified 
		/// output path. By default the app will save all logs and crash dumps
		/// </summary>
		/// <param name="OutputPath"></param>
		/// <returns></returns>
		public virtual IEnumerable<UnrealRoleArtifacts> SaveRoleArtifacts(string OutputPath)
		{
			if (UnrealApp != null)
			{
				return UnrealApp.SaveRoleArtifacts(Context, TestInstance, ArtifactPath);
			}

			return Enumerable.Empty<UnrealRoleArtifacts>(); 
		}

		/// <summary>
		/// Parses the provided artifacts to determine the cause of an exit and whether it was abnormal
		/// </summary>
		/// <param name="InReason"></param>
		/// <param name="InLog"></param>
		/// <param name="InArtifacts"></param>
		/// <param name="ExitReason"></param>
		/// <param name="ExitCode"></param>
		/// <returns></returns>
		protected virtual UnrealProcessResult GetExitCodeAndReason(StopReason InReason, UnrealLog InLog, UnrealRoleArtifacts InArtifacts, out string ExitReason, out int ExitCode)
		{
			// first check for fatal issues
			if (InLog.FatalError != null)
			{
				ExitReason = "Process encountered fatal error";
				ExitCode = -1;
				return UnrealProcessResult.EncounteredFatalError;
			}

			// Catch failed engine init. Early issues can result in the engine exiting with hard to diagnose reasons
			if (InLog.EngineInitialized == false)
			{
				ExitReason = string.Format("Engine initialization failed");
				ExitCode = -1;
				return UnrealProcessResult.InitializationFailure;
			}

			// If the test considers ensures as fatal, fail here
			if (CachedConfig.FailOnEnsures && InLog.Ensures.Count() > 0)
			{
				ExitReason = string.Format("Process encountered {0} Ensures", InLog.Ensures.Count());
				ExitCode = -1;
				return UnrealProcessResult.EncounteredEnsure;
			}

			// Gauntlet killed the process. This can be valid in many scenarios (e.g. shutting down an ancillary 
			// process, but if there was a timeout it will be handled at a higher level
			if (InArtifacts.AppInstance.WasKilled)
			{
				if (InReason == StopReason.MaxDuration)
				{
					ExitReason = "Process was killed by Gauntlet due to a timeout";
					ExitCode = -1;
					return UnrealProcessResult.TimeOut;
				}
				else
				{
					ExitReason = string.Format("Process was killed by Gauntlet [Reason={0}]", InReason.ToString());
					ExitCode = 0;
					return UnrealProcessResult.ExitOk;
				}
			}

			// If we found a valid exit code with test markup, return it
			if (InLog.HasTestExitCode)
			{
				ExitReason = string.Format("Tests exited with error code {0}", InLog.TestExitCode);
				ExitCode = InLog.TestExitCode;
				return ExitCode == 0 ? UnrealProcessResult.ExitOk : UnrealProcessResult.TestFailure;
			}

			// Engine exit was requested with no visible fatal error
			if (InLog.RequestedExit)
			{
				// todo - need join cleanup with UE around RE due to errors
				ExitReason = string.Format("Exit was requested: {0}", InLog.RequestedExitReason);
				ExitCode = 0;
				return UnrealProcessResult.ExitOk;
			}

			bool WasGauntletTest = InArtifacts.SessionRole.CommandLine.ToLower().Contains("-gauntlet=");
			// ok, process was using a gauntlet controller so see if there's a result divine a result...
			if (WasGauntletTest)
			{
				if (InLog.HasTestExitCode == false)
				{
					Log.Verbose("Role {Role} had 0 exit code but used Gauntlet and no TestExitCode was found. Assuming failure", InArtifacts.SessionRole.RoleType);
					ExitReason = "Process terminated prematurely! No test result from Gauntlet controller";
					ExitCode = -1;
					return UnrealProcessResult.TestFailure;
				}
			}

			// AG TODO - do we still need this?
			// Normal exits from server are not ok if we had clients running!
			/*if (ExitCode == 0 && InArtifacts.SessionRole.RoleType.IsServer())
			{
				bool ClientsKilled = SessionArtifacts.Any(A => A.AppInstance.WasKilled && A.SessionRole.RoleType.IsClient());

				if (ClientsKilled)
				{
					ExitCode = -1;
					ExitReason = "Server exited while clients were running";
				}
			}*/

			// The process is gone but we don't know why. This is likely bad and signifies an unhandled or undiagnosed error
			ExitReason = "app exited for unhandled reason";
			ExitCode = -1;
			return UnrealProcessResult.Unknown;
		}

		/// <summary>
		/// Creates an EventList from the artifacts for the specified role. By default this will be asserts (errors), ensures (warnings)
		/// from the log, plus any log entries from categories that are the list returned by GetMonitoredLogCategories(). Nodes can also set
		/// their behavior flags to elevate *all* warnings/errors
		/// </summary>
		/// <param name="InReason"></param>
		/// <param name="InRoleArtifacts"></param>
		/// <param name="InLog"></param>
		/// <param name="ProcessResult"></param>
		/// <returns></returns>
		protected virtual IEnumerable<UnrealTestEvent> CreateEventListFromArtifact(StopReason InReason, UnrealRoleArtifacts InRoleArtifacts, UnrealLog InLog, UnrealProcessResult ProcessResult)
		{
			List<UnrealTestEvent> EventList = new List<UnrealTestEvent>();

			// Create events for any fatal errors in the log
			if (InLog.FatalError != null)
			{
				UnrealTestEvent FatalEvent = new UnrealTestEvent(EventSeverity.Fatal, InLog.FatalError.Message, Enumerable.Empty<string>(), InLog.FatalError);
				EventList.Add(FatalEvent);
			}

			if (!GetCachedConfiguration().IgnoreEnsures)
			{
				// Create events for any ensures
				foreach (UnrealLog.CallstackMessage Ensure in InLog.Ensures)
				{
					UnrealTestEvent EnsureEvent = new UnrealTestEvent(EventSeverity.Warning, Ensure.Message, Enumerable.Empty<string>(), Ensure);
					EventList.Add(EnsureEvent);
				}
			}

			HashSet<string> MonitoredCategorySet = new HashSet<string>();

			foreach (string Category in GetCachedConfiguration().LogCategoriesForEvents)
			{
				MonitoredCategorySet.Add(Category);
			}

			bool TrackAllWarnings = GetCachedConfiguration().ShowWarningsInSummary || Flags.HasFlag(BehaviorFlags.PromoteWarnings);
			// if we are getting an initialization failure it is still not clear what the reason was, may differ from one case to another,
			// so we would like to enforce all-error tracking in this situation, even if it is disabled in the config
			bool TrackAllErrors = GetCachedConfiguration().ShowErrorsInSummary || Flags.HasFlag(BehaviorFlags.PromoteErrors)
				|| ProcessResult == UnrealProcessResult.InitializationFailure;

			// now look at the log. Add events for warnings/errors if the category is monitored or if this test is flagged to 
			// promote all warnings/errors
			foreach (UnrealLog.LogEntry Entry in InLog.LogEntries)
			{
				bool IsMonitored = MonitoredCategorySet.Contains(Entry.Category);

				if (Entry.Level == UnrealLog.LogLevel.Warning &&
					(IsMonitored || TrackAllWarnings))
				{
					EventList.Add(new UnrealTestEvent(EventSeverity.Warning, Entry.ToString(), Enumerable.Empty<string>()));
				}

				if (Entry.Level == UnrealLog.LogLevel.Error &&
					(IsMonitored || TrackAllErrors))
				{
					EventList.Add(new UnrealTestEvent(EventSeverity.Error, Entry.ToString(), Enumerable.Empty<string>()));
				}
			}

			return EventList;
		}


		/// <summary>
		/// Returns a RoleResult, a representation of this roles result from the test, for the provided artifact
		/// </summary>
		/// <param name="InReason"></param>
		/// <param name="InRoleArtifacts"></param>
		/// <returns></returns>
		protected virtual UnrealRoleResult CreateRoleResultFromArtifact(StopReason InReason, UnrealRoleArtifacts InRoleArtifacts)
		{
			int ExitCode;
			string ExitReason;

			UnrealLog LogSummary = CreateLogSummaryFromArtifact(InRoleArtifacts);

			// Give ourselves (and derived classes) a chance to analyze what happened
			UnrealProcessResult ProcessResult = GetExitCodeAndReason(InReason, LogSummary, InRoleArtifacts, out ExitReason, out ExitCode);

			IEnumerable<UnrealTestEvent> EventList = CreateEventListFromArtifact(InReason, InRoleArtifacts, LogSummary, ProcessResult);

			// if the test is stopping for a reason other than completion, mark this as failing in case derived classes
			// don't do the right thing
			if (InReason == StopReason.MaxDuration)
			{
				ProcessResult = UnrealProcessResult.TimeOut;
				ExitCode = -1;
			}

			return new UnrealRoleResult(ProcessResult, ExitCode, ExitReason, LogSummary, InRoleArtifacts, EventList);
		}

		/// <summary>
		/// Returns a log summary from the provided artifacts. 
		/// </summary>
		/// <param name="InArtifacts"></param>
		/// <returns></returns>
		protected virtual UnrealLog CreateLogSummaryFromArtifact(UnrealRoleArtifacts InArtifacts)
		{
			return new UnrealLogParser(InArtifacts.AppInstance.GetLogReader()).GetSummary();
		}

		/// <summary>
		/// Returns a list of all results for the roles involved in this test by calling CreateRoleResultFromArtifact for all
		/// artifacts in the list
		/// </summary>
		/// <param name="InReason"></param>
		/// <param name="InAllArtifacts"></param>
		/// <returns></returns>
		protected virtual IEnumerable<UnrealRoleResult> CreateRoleResultsFromArtifacts(StopReason InReason, IEnumerable<UnrealRoleArtifacts> InAllArtifacts)
		{
			return InAllArtifacts.Select(A => CreateRoleResultFromArtifact(InReason, A)).ToArray();
		}


		private void LogHeartbeat(UnrealSessionInstance InInstance)
		{
			if (CachedConfig == null ||
				GetTestStatus() != TestStatus.InProgress)
			{
				return;
			}

			UnrealHeartbeatOptions HeartbeatOptions = CachedConfig.HeartbeatOptions;
			if (DateTime.Now.Subtract(LastHeartbeatLogTime).TotalSeconds < HeartbeatOptions.LogHeartbeatInterval)
			{
				return;
			}

			void LogHeartbeatCategories(UnrealLogStreamParser Parser, string AppPrefix, bool bUpdateHeartbeatTime)
			{
				if (!bDisableHeartbeatLogging)
				{
					try
					{
						Parser.ReadStream();
					}
					catch (OutOfMemoryException OOMEx)
					{
						Log.Warning("{App} encountered an {ExceptionType} when attempting to read the process output. " +
							"This is usually caused by the output being >4GB in size due to log spam or deformed encoding. " +
							"Heartbeat logging will be disabled for the remainder of the test.\n" +
							"{Exception}", AppPrefix, OOMEx.GetType().Name, OOMEx.Message);

						bDisableHeartbeatLogging = true;
						CachedConfig.HeartbeatOptions.bExpectHeartbeats = false; // prevents CheckHeartbeat from timing out tests after heartbeats are disabled

						return;
					}

					foreach (string TestLine in Parser.GetLogFromShortNameChannels(HeartbeatLogCategories))
					{
						Log.Info("{App}: {Message}", AppPrefix, TestLine);

						if (bUpdateHeartbeatTime)
						{
							if (Regex.IsMatch(TestLine, @".*GauntletHeartbeat\: Active.*"))
							{
								LastHeartbeatTime = DateTime.Now;
								LastActiveHeartbeatTime = DateTime.Now;
							}
							else if (Regex.IsMatch(TestLine, @".*GauntletHeartbeat\: Idle.*"))
							{
								LastHeartbeatTime = DateTime.Now;
							}
						}
					}
				}
			}

			foreach (RoleInstance Role in InInstance.RunningRoles)
			{
				bool bUpdateHeartbeat = true;
				string RoleName = Role.Role.RoleType.ToString();

				if (Role.Role.RoleType.IsServer())
				{
					bUpdateHeartbeat = InInstance.ClientApps == null;
				} 
				else if (Role.Role.RoleType.IsEditor())
				{
					bUpdateHeartbeat = false;
				} 
				else if (Role.Role.RoleType.IsClient())
				{
					RoleName += " " + Role.AppInstance.Device.Name;
				}

				UnrealLogStreamParser RoleParser = GetParserForRole(Role);
				LogHeartbeatCategories(RoleParser, RoleName, bUpdateHeartbeat);
			}

			LastHeartbeatLogTime = DateTime.Now;
		}

		private void CheckHeartbeat()
		{
			if (CachedConfig == null
				|| CachedConfig.DisableHeartbeatTimeout
				|| CachedConfig.HeartbeatOptions.bExpectHeartbeats == false
				|| GetTestStatus() != TestStatus.InProgress)
			{
				return;
			}

			UnrealHeartbeatOptions HeartbeatOptions = CachedConfig.HeartbeatOptions;

			// First active heartbeat has not happened yet and timeout before first active heartbeat is enabled
			if (LastActiveHeartbeatTime == DateTime.MinValue && HeartbeatOptions.TimeoutBeforeFirstActiveHeartbeat > 0)
			{
				double SecondsSinceSessionStart = DateTime.Now.Subtract(SessionStartTime).TotalSeconds;
				if (SecondsSinceSessionStart > HeartbeatOptions.TimeoutBeforeFirstActiveHeartbeat)
				{

					Horde.UpdateStepMessage(Horde.GauntletHordeStepMessage.HeartbeatTimeout);
					Log.Error(KnownLogEvents.Gauntlet_TestEvent, "{Time} seconds have passed without detecting the first active Gauntlet heartbeat.", HeartbeatOptions.TimeoutBeforeFirstActiveHeartbeat);
					MarkTestComplete();
					SetUnrealTestResult(TestResult.TimedOut);
				}
			}

			// First active heartbeat has happened and timeout between active heartbeats is enabled
			if (LastActiveHeartbeatTime != DateTime.MinValue && HeartbeatOptions.TimeoutBetweenActiveHeartbeats > 0)
			{
				double SecondsSinceLastActiveHeartbeat = DateTime.Now.Subtract(LastActiveHeartbeatTime).TotalSeconds;
				if (SecondsSinceLastActiveHeartbeat > HeartbeatOptions.TimeoutBetweenActiveHeartbeats)
				{
					Horde.UpdateStepMessage(Horde.GauntletHordeStepMessage.HeartbeatTimeout);
					Log.Error(KnownLogEvents.Gauntlet_TestEvent, "{Time} seconds have passed without detecting any active Gauntlet heartbeats.", HeartbeatOptions.TimeoutBetweenActiveHeartbeats);
					MarkTestComplete();
					SetUnrealTestResult(TestResult.TimedOut);
				}
			}

			// First heartbeat has happened and timeout between heartbeats is enabled
			if (LastHeartbeatTime != DateTime.MinValue && HeartbeatOptions.TimeoutBetweenAnyHeartbeats > 0)
			{
				double SecondsSinceLastHeartbeat = DateTime.Now.Subtract(LastHeartbeatTime).TotalSeconds;
				if (SecondsSinceLastHeartbeat > HeartbeatOptions.TimeoutBetweenAnyHeartbeats)
				{
					Horde.UpdateStepMessage(Horde.GauntletHordeStepMessage.HeartbeatTimeout);
					Log.Error(KnownLogEvents.Gauntlet_TestEvent, "{Time} seconds have passed without detecting any Gauntlet heartbeats.", HeartbeatOptions.TimeoutBetweenAnyHeartbeats);
					MarkTestComplete();
					SetUnrealTestResult(TestResult.TimedOut);
				}
			}
		}

		/// <summary>
		/// Returns a hash that represents the results of a role. Should be 0 if no fatal errors or ensures
		/// </summary>
		/// <param name="InResult"></param>
		/// <returns></returns>
		protected virtual string GetRoleResultHash(UnrealRoleResult InResult)
		{
			const int MaxCallstackLines = 10;

			UnrealLog LogSummary = InResult.LogSummary;

			string TotalString = "";

			//Func<int, string> ComputeHash = (string Str) => { return Hasher.ComputeHash(Encoding.UTF8.GetBytes(Str)); };

			if (LogSummary.FatalError != null)
			{
				TotalString += string.Join("\n", LogSummary.FatalError.Callstack.Take(MaxCallstackLines));
				TotalString += "\n";
			}

			foreach (var Ensure in LogSummary.Ensures)
			{
				TotalString += string.Join("\n", Ensure.Callstack.Take(MaxCallstackLines));
				TotalString += "\n";
			}

			string Hash = Hasher.ComputeHash(TotalString);

			return Hash;
		}

		/// <summary>
		/// Returns a hash that represents the failure results of this test. If the test failed this should be an empty string
		/// </summary>
		/// <returns></returns>
		protected virtual string GetTestResultHash()
		{
			IEnumerable<string> RoleHashes = RoleResults.Select(R => GetRoleResultHash(R)).OrderBy(S => S);

			RoleHashes = RoleHashes.Where(S => S.Length > 0 && S != "0");

			string Combined = string.Join("\n", RoleHashes);

			string CombinedHash = Hasher.ComputeHash(Combined);

			return CombinedHash;
		}

		/// <summary>
		/// Deprecated
		/// </summary>
		/// <param name="InRoleResult"></param>
		/// <returns></returns>
		protected virtual string GetFormattedRoleSummary(UnrealRoleResult InRoleResult)
		{
			return "";
		}

		protected struct DecomposedAssert
		{
			public EventId EventType;
			public string Message;
			public object[] FormatArguments;
		}

		/// <summary>
		/// Decompose an assert event so that it can be easily logged.
		/// </summary>
		/// <param name="Event"></param>
		/// <param name="MaxCallstackLines"></param>
		/// <param name="Assert"></param>
		protected static LogEvent DecomposeAssert(UnrealTestEvent Event, int MaxCallstackLines = 0)
		{
			List<string> Callstack = new List<string>();
			if (Event.Callstack.Any())
			{
				if (CommandUtils.IsBuildMachine || MaxCallstackLines <= 0)
				{
					Callstack = Event.Callstack.ToList();
				}
				else
				{
					Callstack = Event.Callstack.Take(MaxCallstackLines).ToList();
					if (Event.Callstack.Count() > MaxCallstackLines)
					{
						Callstack.Add("See log for full callstack");
					}
				}
			}
			else
			{
				Callstack.Add("Could not parse callstack. See log for full callstack");
			}
			string CallstackString = string.Join("\n", Callstack.Select(C => "    " + C));
			Dictionary<string, object> Properties = new Dictionary<string, object>() { { "Summary", Event.Summary } };
			string Message = " * Fatal Error: {Summary}\n{Callstack}";
			EventId EventType = KnownLogEvents.Gauntlet_FatalEvent;
			if (Event.IsSanReport && CommandUtils.IsBuildMachine
				&& SanitizerEventMatcher.AddSanitizerSummaryProperties(ref CallstackString, Properties))
			{
				CallstackString = CallstackString.Replace("{", "{{").Replace("}", "}}");
				Message = $" * Fatal Error: {{Summary}}\n{CallstackString}";
				EventType = SanitizerEventMatcher.ConvertSanitizerNameToEventId(Properties.GetValueOrDefault("SanitizerName")?.ToString());
			}
			else
			{
				Properties.Add("Callstack", CallstackString);
			}
			string RenderedMessage = MessageTemplate.Render(Message, Properties);
			return new LogEvent(DateTime.Now,
				Microsoft.Extensions.Logging.LogLevel.Critical,
				EventType,
				RenderedMessage,
				Message,
				Properties,
				null
			);
		}

		/// <summary>
		/// Log a formatted summary of the role that's suitable for displaying
		/// </summary>
		/// <param name="InRoleResult"></param>
		/// <returns></returns>
		protected virtual void LogRoleSummary(UnrealRoleResult InRoleResult)
		{

			const int MaxLogLines = 10;
			const int MaxCallstackLines = 20;

			UnrealLog LogSummary = InRoleResult.LogSummary;

			UnrealRoleArtifacts RoleArtifacts = InRoleResult.Artifacts;

			Log.Info(" #### Role: {0} ({1} {2})", RoleArtifacts.SessionRole.RoleType, RoleArtifacts.SessionRole.Platform, RoleArtifacts.SessionRole.Configuration);

			bool HasFailed = InRoleResult.ProcessResult != UnrealProcessResult.ExitOk && InRoleResult.ProcessResult != UnrealProcessResult.EngineTestError;
			string RoleState = HasFailed ? "failed:" : "completed:";
			string StatusMessage = string.Format(" #### {0} {1} {2} ({3}, ExitCode={4})", RoleArtifacts.SessionRole.RoleType, RoleState, InRoleResult.Summary, InRoleResult.ProcessResult, InRoleResult.ExitCode);
			if (HasFailed && !GetCachedConfiguration().IgnoreRoleFailures)
			{
				Log.Error(KnownLogEvents.Gauntlet_TestEvent, StatusMessage);
			}
			else
			{
				Log.Info(StatusMessage);
			}

			// log command line up here for visibility

			Log.Info(string.Join("\n", new string[] {
				string.Format(" * CommandLine: {0}", RoleArtifacts.AppInstance.CommandLine),
				string.Format(" * Log: {0}", RoleArtifacts.LogPath),
				string.Format(" * SavedDir: {0}", RoleArtifacts.ArtifactPath),
				LogSummary.FatalError != null ? " * Fatal Errors: 1" : null,
				LogSummary.Ensures.Count() > 0 ? string.Format(" * Ensures: {0}", LogSummary.Ensures.Count()) : null,
				LogSummary.Errors.Count() > 0 ? string.Format(" * Log Errors: {0}", LogSummary.Errors.Count()) : null,
				LogSummary.Warnings.Count() > 0 ? string.Format(" * Log Warnings: {0}", LogSummary.Warnings.Count()) : null
			}.Where(L => !string.IsNullOrEmpty(L))));
			Log.Info("");

			// Separate the events we want to report on
			IEnumerable<UnrealTestEvent> Asserts = InRoleResult.Events.Where(E => E.Severity == EventSeverity.Fatal);
			IEnumerable<UnrealTestEvent> Errors = InRoleResult.Events.Where(E => E.Severity == EventSeverity.Error);
			IEnumerable<UnrealTestEvent> Ensures = InRoleResult.Events.Where(E => E.IsEnsure);
			IEnumerable<UnrealTestEvent> Warnings = InRoleResult.Events.Where(E => E.Severity == EventSeverity.Warning && !E.IsEnsure);

			foreach (UnrealTestEvent Event in Asserts)
			{
				LogEvent AssertData = DecomposeAssert(Event, MaxCallstackLines);
				
				if (GetCachedConfiguration().IgnoreRoleFailures)
				{
					Log.Info(AssertData.Id, AssertData.Format ?? AssertData.Message, ((Dictionary<string, object>)AssertData.Properties)?.Values.ToArray());
				}
				else
				{
					Log.Error(AssertData.Id, AssertData.Format ?? AssertData.Message, ((Dictionary<string, object>)AssertData.Properties)?.Values.ToArray());
				}
					
				Log.Info("");
			}

			foreach (UnrealTestEvent Event in Ensures.Distinct())
			{
				List<string> Callstack = new List<string>();
				if (Event.Callstack.Any())
				{
					if (CommandUtils.IsBuildMachine)
					{
						Callstack = Event.Callstack.ToList();
					}
					else
					{
						Callstack = Event.Callstack.Take(MaxCallstackLines).ToList();
						if (Event.Callstack.Count() > MaxCallstackLines)
						{
							Callstack.Add("See log for full callstack");
						}
					}
				}
				else
				{
					Callstack.Add("Could not parse callstack. See log for full callstack");
				}

				if (GetCachedConfiguration().IgnoreRoleFailures)
				{
					Log.Info(KnownLogEvents.Gauntlet_TestEvent, " * Ensure: {Summary}\n{Callstack}", Event.Summary, string.Join("\n", Callstack.Select(C => "    " + C)));
				}
				else
				{
					Log.Warning(KnownLogEvents.Gauntlet_TestEvent, " * Ensure: {Summary}\n{Callstack}", Event.Summary, string.Join("\n", Callstack.Select(C => "    " + C)));
				}
					
				Log.Info("");
			}

			if (Errors.Any())
			{
				var ErrorList = Errors.Select(E => E.Summary).Distinct();
				var PrintedErrorList = ErrorList;

				string TrimStatement = "";

				// too many warnings. If there was an abnormal exit show the last ones as they may be relevant
				if (ErrorList.Count() > MaxLogLines)
				{
					if (LogSummary.HasAbnormalExit)
					{
						PrintedErrorList = ErrorList.Skip(ErrorList.Count() - MaxLogLines);
						TrimStatement = string.Format("  (Last {0} of {1} errors)", MaxLogLines, ErrorList.Count());
					}
					else
					{
						PrintedErrorList = ErrorList.Take(MaxLogLines);
						TrimStatement = string.Format("  (First {0} of {1} errors)", MaxLogLines, ErrorList.Count());
					}
				}

				foreach (var Error in PrintedErrorList)
				{
					if (GetCachedConfiguration().IgnoreRoleFailures)
					{
						Log.Info(KnownLogEvents.Gauntlet_TestEvent, " * {Message}", Error);
					}
					else
					{
						Log.Error(KnownLogEvents.Gauntlet_TestEvent, " * {Message}", Error);
					}	
				}
				if (!string.IsNullOrEmpty(TrimStatement))
				{
					Log.Info(KnownLogEvents.Gauntlet_TestEvent, TrimStatement);
				}
				Log.Info("");
			}

			if (Warnings.Any())
			{
				var WarningList = Warnings.Select(E => E.Summary).Distinct();
				var PrintedWarningList = WarningList;

				string TrimStatement = "";

				// too many warnings. If there was an abnormal exit show the last ones as they may be relevant
				if (WarningList.Count() > MaxLogLines)
				{
					if (LogSummary.HasAbnormalExit)
					{
						PrintedWarningList = WarningList.Skip(WarningList.Count() - MaxLogLines);
						TrimStatement = string.Format(" (Last {0} of {1} warnings)", MaxLogLines, WarningList.Count());
					}
					else
					{
						PrintedWarningList = WarningList.Take(MaxLogLines);
						TrimStatement = string.Format(" (First {0} of {1} warnings)", MaxLogLines, WarningList.Count());
					}
				}

				foreach (var Warning in PrintedWarningList)
				{
					Log.Warning(KnownLogEvents.Gauntlet_TestEvent, " * {Message}", Warning);
				}
				if (!string.IsNullOrEmpty(TrimStatement))
				{
					Log.Info(KnownLogEvents.Gauntlet_TestEvent, TrimStatement);
				}
				Log.Info("");
			}
		}

		/// <summary>
		/// Returns a formatted summary of the events that were thrown by the test node itself during run.
		/// </summary>
		/// <returns>An HTML string containing the properly formatted list of results thrown by the Test Node.</returns>
		protected virtual string CreateFormattedEventListFromTestNode()
		{

			AddProcessResultEventsFromTestNode();

			MarkdownBuilder MB = new MarkdownBuilder();

			if (TestNodeEvents.Count == 0)
			{
				MB.H4("No Gauntlet Events were fired during this test!");
				return MB.ToString();
			}

			// Separate the events we want to report on
			IEnumerable<UnrealTestEvent> Fatals = TestNodeEvents.Where(E => E.Severity == EventSeverity.Fatal);
			IEnumerable<UnrealTestEvent> Errors = TestNodeEvents.Where(E => E.Severity == EventSeverity.Error);
			IEnumerable<UnrealTestEvent> Ensures = TestNodeEvents.Where(E => E.IsEnsure);
			IEnumerable<UnrealTestEvent> Warnings = TestNodeEvents.Where(E => E.Severity == EventSeverity.Warning && !E.IsEnsure);
			IEnumerable<UnrealTestEvent> Infos = TestNodeEvents.Where(E => E.Severity == EventSeverity.Info);

			foreach (UnrealTestEvent Event in Fatals)
			{
				MB.H4(Event.Summary);
				MB.UnorderedList(Event.Details);
				MB.NewLine();
			}

			foreach (UnrealTestEvent Event in Ensures.Distinct())
			{
				MB.H4(Event.Summary);
				MB.UnorderedList(Event.Details);
				MB.NewLine();
			}

			if (Errors.Any())
			{
				MB.H3("Errors:");
				MB.HorizontalLine();
				foreach (UnrealTestEvent errorEvent in Errors)
				{
					MB.H5(errorEvent.Summary);
					MB.UnorderedList(errorEvent.Details);
					MB.NewLine();
				}
			}

			if (Warnings.Any())
			{
				MB.H3("Warnings:");
				MB.HorizontalLine();
				foreach (UnrealTestEvent warnEvent in Warnings)
				{
					MB.H5(warnEvent.Summary);
					MB.UnorderedList(warnEvent.Details);
					MB.NewLine();
				}
			}
			if (Infos.Any())
			{
				MB.H3("Info:");
				MB.HorizontalLine();
				MB.NewLine();
				foreach (UnrealTestEvent infoEvent in Infos)
				{
					MB.H5(infoEvent.Summary);
					MB.UnorderedList(infoEvent.Details);
					MB.NewLine();
				}
			}
			return MB.ToString();
		}

		/// <summary>
		/// Returns the current 0-based Pass count for this node.
		/// </summary>
		public int GetCurrentPass()
		{
			return CurrentPass;
		}

		/// <summary>
		/// Returns the current total Pass count for this node
		/// </summary>
		/// <returns></returns>
		public int GetNumPasses()
		{
			return NumPasses;
		}

		/// <summary>
		/// Result of the test once completed. Nodes inheriting from us should override
		/// this if custom results are necessary
		/// </summary>
		public sealed override TestResult GetTestResult()
		{
            if (UnrealTestResult == TestResult.Invalid)
			{
				UnrealTestResult = GetUnrealTestResult();
			}

			return UnrealTestResult;
		}

		/// <summary>
		/// Result of the test once completed. Nodes inheriting from us should override
		/// </summary>
		public override void SetTestResult(TestResult testResult)
		{
            UnrealTestResult = testResult;
		}

		/// <summary>
		/// Allows tests to set this at anytime. If not called then GetUnrealTestResult() will be called when
		/// the framework first calls GetTestResult()
		/// </summary>
		/// <param name="Result"></param>
		/// <returns></returns>
		protected void SetUnrealTestResult(TestResult Result)
		{
            if (GetTestStatus() != TestStatus.Complete)
			{
				throw new Exception("SetUnrealTestResult() called while test is incomplete!");
			}

			UnrealTestResult = Result;
		}

		/// <summary>
		/// Return all artifacts that are exited abnormally. An abnormal exit is termed as a fatal error,
		/// crash, assert, or other exit that does not appear to have been caused by completion of a process
		/// </summary>
		/// <returns></returns>
		protected virtual IEnumerable<UnrealRoleResult> GetRolesThatExitedAbnormally()
		{
			if (RoleResults == null)
			{
				Log.Warning("RoleResults was null, unable to check for failures");
				return Enumerable.Empty<UnrealRoleResult>();
			}

			return RoleResults.Where(R => R.ProcessResult != UnrealProcessResult.ExitOk && R.LogSummary.HasAbnormalExit);
		}

        /// <summary>
        /// Return all artifacts that are considered to have caused the test to fail
        /// </summary>
        /// <returns></returns>
        protected virtual IEnumerable<UnrealRoleResult> GetRolesThatFailed()
        {
            if (RoleResults == null)
            {
                Log.Warning("RoleResults was null, unable to check for failures");
                return Enumerable.Empty<UnrealRoleResult>();
            }

            return RoleResults.Where(R => R.ProcessResult == UnrealProcessResult.TestFailure);
        }

		/// <summary>
		/// Return all artifacts that are considered to have caused a test Interrupt
		/// </summary>
		/// <returns></returns>
		protected virtual IEnumerable<UnrealRoleResult> GetRolesThatInterrupted()
		{
			if (RoleResults == null)
			{
				Log.Warning("RoleResults was null, unable to check for Interrupts");
				return Enumerable.Empty<UnrealRoleResult>();
			}

			return RoleResults.Where(R => R.ProcessResult != UnrealProcessResult.ExitOk && R.ProcessResult != UnrealProcessResult.TestFailure);
		}

		/// <summary>
		/// THe base implementation considers  considers Classes can override this to implement more custom detection of success/failure than our
		/// log parsing. Not guaranteed to be called if a test is marked complete
		/// </summary>
		/// <returns></returns>in
		protected virtual TestResult GetUnrealTestResult()
		{
			int ExitCode = 0;

            // Let the test try and diagnose things as best it can
            List<UnrealRoleResult> InterruptRoles = GetRolesThatInterrupted().ToList();
            List<UnrealRoleResult> FailedRoles = GetRolesThatFailed().ToList();

			bool bInterruptFound = false;

			// Prioritizing Failed tests over Interrupted tests, because Failed tests are more likely to be the result of a product bug.
			if (FailedRoles.Any())
			{
				foreach (var Role in FailedRoles)
				{
					Log.Info("Failing test because {Role} exited with {ExitCode}. ({Result})", Role.Artifacts.SessionRole, Role.ExitCode, Role.Summary);
				}
				ExitCode = FailedRoles.FirstOrDefault().ExitCode;
			}
			else if (InterruptRoles.Any())
            {
				bInterruptFound = true;
                foreach (var Role in InterruptRoles)
                {
                    Log.Info("Test Interrupt because {Role} exited with {ExitCode}. ({Result})", Role.Artifacts.SessionRole, Role.ExitCode, Role.Summary);
                }
                ExitCode = InterruptRoles.FirstOrDefault().ExitCode;
            }

			// This is a bit of an extrapolation from the original code, which looked for a combination of a good exit code but with a "cancelled" flag, 
			// and then marked that combination as "Failed".  Since we're now splitting "failures" from other problems, we're going a bit more granular.
			// Since we have a "Cancelled" test result available, we'll make use of it, and chalk any other interrupt up as a general "OperationalException".
			if (bInterruptFound)
			{
				return (ExitCode == 0 && WasCancelled) ? TestResult.Cancelled : TestResult.OperationalException;
			}

			return ExitCode == 0 ? TestResult.Passed : TestResult.Failed;
		}


		/// <summary>
		/// Deprecated
		/// </summary>
		/// <returns></returns>
		protected virtual string GetTestSummaryHeader()
		{
			return "";
		}

		/// <summary>
		/// Log header for the test summary. The header is the first block of text and will be
		/// followed by the summary of each individual role in the test
		/// </summary>
		/// <returns></returns>
		protected virtual void LogTestSummaryHeader()
		{
			int FatalErrors = 0;
			int Ensures = 0;
			int Errors = 0;
			int Warnings = 0;

			bool TestHadErrors = GetTestResult() != TestResult.Passed;

			// Good/Bad news upfront
			string Prefix = TestHadErrors ? "Error: " : "";
			string WarningStatement = (HasWarnings && !TestHadErrors) ? " With Warnings" : "";
			string ResultString = string.Format(" ### {0}{1} {2}{3} ***\n", Prefix, this.Name, GetTestResult(), WarningStatement);
			Log.Info(ResultString);

			IEnumerable<UnrealRoleResult> SortedRoles = RoleResults.OrderBy(R => R.ProcessResult == UnrealProcessResult.ExitOk);

			// create a quick summary of total failures, ensures, errors, etc. Don't write out errors etc for roles, those will be
			// displayed individually by GetFormattedRoleSummary
			foreach (var RoleResult in SortedRoles)
			{
				string RoleName = RoleResult.Artifacts.SessionRole.RoleType.ToString();
				string LogMessage = string.Format(" {0} Role: {1} ({2}, ExitCode={3})", RoleName, RoleResult.Summary, RoleResult.ProcessResult, RoleResult.ExitCode);
				Log.Info(LogMessage);

				FatalErrors += RoleResult.LogSummary.FatalError != null ? 1 : 0;
				Ensures += RoleResult.LogSummary.Ensures.Count();
				Errors += RoleResult.LogSummary.Errors.Count();
				Warnings += RoleResult.LogSummary.Warnings.Count();
			}

			Log.Info("");
			Log.Info(string.Join("\n", new string[] {
				string.Format(" * Main Context: {0}", GetMainRoleContextString()),
				FatalErrors > 0 ? string.Format(" * FatalErrors: {0}", FatalErrors) : null,
				Ensures > 0 ? string.Format(" * Ensures: {0}", Ensures) : null,
				Errors > 0 ? string.Format(" * Log Errors: {0}", Errors) : null,
				Warnings > 0 ? string.Format(" * Log Warnings: {0}", Warnings) : null,
				string.Format(" * Result: {0}", GetTestResult())
			}.Where(L => !string.IsNullOrEmpty(L))));

			if (TestHadErrors && GetRolesThatFailed().Where(R => R.ProcessResult == UnrealProcessResult.Unknown).Any())
			{
				Log.Info("");
				Log.Error(KnownLogEvents.Gauntlet_TestEvent, " * {Name} failed due to undiagnosed reasons", this.Name);
			}
			Log.Info("\n See 'Role Report' below for more details on each role\n");
		}

		/// <summary>
		/// Returns a summary of this test
		/// </summary>
		/// <returns></returns>
		public override string GetTestSummary()
		{
			MarkdownBuilder ReportBuilder = new MarkdownBuilder();

			// Handle case where there are session artifacts
			if (SessionArtifacts != null)
			{
				// Sort roles so problem ones are at the bottom, just above the summary
				var SortedRoles = RoleResults.OrderBy(R =>
				{
					int Score = 0;

					if (R.ProcessResult != UnrealProcessResult.ExitOk)
					{
						Score += 1000000;
					}

					Score += R.LogSummary.Errors.Count() * 10;
					Score += R.LogSummary.Warnings.Count();

					return Score;
				});

				// add Summary
				string HorizontalLine = " " + new string('-', 80);
				Log.Info(HorizontalLine);
				Log.Info(" # Test Summary: " + this.Name);
				Log.Info(HorizontalLine);
				string HeaderSummary = GetTestSummaryHeader();
				if (!string.IsNullOrEmpty(HeaderSummary))
				{
					Log.Info(HeaderSummary);
				}
				else
				{
					LogTestSummaryHeader();
				}

				Log.Info(HorizontalLine);
				Log.Info(" # Role(s): {Name}", this.Name);
				Log.Info(HorizontalLine);

				// Add a summary of each 
				foreach (var Role in SortedRoles)
				{
					string RoleSummary = GetFormattedRoleSummary(Role);
					if (!string.IsNullOrEmpty(RoleSummary))
					{
						Log.Info(RoleSummary);
					}
					else
					{
						LogRoleSummary(Role);
					}
				}
			}

			ReportBuilder.HorizontalLine();
			ReportBuilder.H1(string.Format("Gauntlet summary events:"));
			ReportBuilder.HorizontalLine();
			ReportBuilder.Append(CreateFormattedEventListFromTestNode());
			return ReportBuilder.ToString();
		}
	}
}