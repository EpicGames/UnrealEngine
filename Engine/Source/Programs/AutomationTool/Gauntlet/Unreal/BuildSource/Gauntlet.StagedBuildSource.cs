// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.IO;
using AutomationTool;
using UnrealBuildTool;
using System.Threading;
using System.Text.RegularExpressions;
using System.Linq;
using Gauntlet.Utils;

namespace Gauntlet
{

	public class EditorBuild : IBuild
	{
		public int PreferenceOrder { get { return 0; } }

		public UnrealTargetPlatform Platform { get { return BuildHostPlatform.Current.Platform; } }

		public UnrealTargetConfiguration Configuration { get; protected set; }

		public BuildFlags Flags { get { return BuildFlags.CanReplaceCommandLine | BuildFlags.Loose; } }

		public bool SupportsAdditionalFileCopy => true;

		public string Flavor { get { return ""; } }

		public bool CanSupportRole(UnrealTargetRole InRoleType) { return InRoleType.UsesEditor(); }

		public string ExecutablePath { get; protected set; }

		public EditorBuild(string InExecutablePath, UnrealTargetConfiguration InConfig)
		{
			ExecutablePath = InExecutablePath;
			Configuration = InConfig;
		}
	}

	public class PackagedBuild : IBuild
	{
		public virtual int PreferenceOrder { get { return 0; } }

		public UnrealTargetPlatform Platform { get; protected set; }

		public UnrealTargetConfiguration Configuration { get; protected set; }

		public UnrealTargetRole Role { get; protected set; }

		public BuildFlags Flags { get; protected set; }

		public string Flavor { get; protected set; }

		public bool SupportsAdditionalFileCopy => false;

		public string BuildPath { get; protected set; }

		public virtual bool CanSupportRole(UnrealTargetRole InRoleType)
		{
			if (InRoleType.IsEditor())
			{
				return Role.IsCookedEditor();
			}
			return InRoleType == Role;
		}

		public PackagedBuild(UnrealTargetPlatform InPlatform, UnrealTargetConfiguration InConfig, UnrealTargetRole InRole, string InBuildPath, BuildFlags InFlags, string InFlavor="")
		{
			Platform = InPlatform;
			Configuration = InConfig;
			Role = InRole;
			BuildPath = InBuildPath;
			Flags = InFlags | BuildFlags.Packaged;  // always add this.
			Flavor = InFlavor;
		}
	}


	public class StagedBuild : IBuild
	{
		public virtual int PreferenceOrder { get { return 1; } }

		public UnrealTargetPlatform Platform { get; protected set; }

		public UnrealTargetConfiguration Configuration { get; protected set; }

		public UnrealTargetRole Role { get; protected set; }

		public BuildFlags Flags { get; protected set; }

		public string Flavor { get; protected set; }

		public bool SupportsAdditionalFileCopy { get; }

		public string BuildPath { get; protected set; }

		public string ExecutablePath { get; protected set; }

		public virtual bool CanSupportRole(UnrealTargetRole InRoleType)
		{
			if (InRoleType.IsEditor())
			{
				return Role.IsCookedEditor();
			}
			return InRoleType == Role;
		}

		public virtual IStagedBuildFilter GetInstallFilter(UnrealAppConfig AppConfig)
		{
			return IStagedBuildFilter.GetFilterForPlatform(AppConfig.Platform.Value);
		}

		public StagedBuild(UnrealTargetPlatform InPlatform, UnrealTargetConfiguration InConfig, UnrealTargetRole InRole, string InBuildPath, string InExecutablePath, string InFlavor="")
		{
			Platform = InPlatform;
			Configuration = InConfig;
			Role = InRole;
			BuildPath = InBuildPath;
			ExecutablePath = InExecutablePath;
			Flags = BuildFlags.CanReplaceCommandLine | BuildFlags.CanReplaceExecutable | BuildFlags.Loose;
			Flavor = InFlavor;
			SupportsAdditionalFileCopy = true; // NativeStagedBuilds may want this to be false
		}

		enum InstallStatus
		{
			Error,
			Installing,
			Installed
		}

		class Install
		{
			public InstallStatus Status;
			public string Path;
			public UnrealTargetConfiguration Configuration;

			public Install(InstallStatus InStatus, string InPath, UnrealTargetConfiguration InConfiguration = UnrealTargetConfiguration.Unknown)
			{
				Status = InStatus;
				Path = InPath;
				Configuration = InConfiguration;
			}
		}

		static Dictionary<StagedBuild, Install> LocalInstalls = new Dictionary<StagedBuild, Install>();

		/// <summary>
		/// When running parallel tests using staged builds, local desktop devices only need one copy of client/server
		/// this method is used to coordinate the copy across multiple local devices
		/// </summary>
		public static string InstallBuildParallel(UnrealAppConfig AppConfig, StagedBuild InBuild, string BuildPath, string DestPath, string Desc)
		{
			// In parallel tests, we only want to copy the client/server once
			bool Install = false;
			InstallStatus Status = InstallStatus.Error;

			lock (Globals.MainLock)
			{
				if (!LocalInstalls.ContainsKey(InBuild))
				{
					Install = true;
					Status = InstallStatus.Installing;
					LocalInstalls[InBuild] = new Install(Status, DestPath, AppConfig.Configuration);
				}
				else
				{
					Status = LocalInstalls[InBuild].Status;
					DestPath = LocalInstalls[InBuild].Path;
				}
			}

			// check if we've already errored in another thread
			if (Status == InstallStatus.Error)
			{
				throw new AutomationException("Parallel build error installing {0} to {1}", BuildPath, DestPath);
			}

			if (Install)
			{
				try
				{
					Log.Info("Installing {0} to {1}", AppConfig.Name, Desc);

					IStagedBuildFilter Filter = InBuild.GetInstallFilter(AppConfig);
					if (Filter != null)
					{
						foreach (FileInfo File in Filter.GetFilteredFiles(AppConfig, InBuild))
						{
							string Source = File.FullName;
							string Destination = Source.Replace(BuildPath, DestPath);

							Log.Verbose("\tCopying {0} to {1}", Source, Destination);
							CommandUtils.CopyFile(Source, Destination);
						}
					}
					else
					{
						Log.Verbose("\tCopying {0} to {1}", BuildPath, DestPath);
						SystemHelpers.CopyDirectory(BuildPath, DestPath, SystemHelpers.CopyOptions.Mirror);
					}
				}
				catch
				{
					lock (Globals.MainLock)
					{
						LocalInstalls[InBuild].Status = InstallStatus.Error;
					}

					throw;
				}

				lock (Globals.MainLock)
				{
					LocalInstalls[InBuild].Status = InstallStatus.Installed;
				}

			}
			else
			{
				DateTime StartTime = DateTime.Now;
				while (true)
				{
					if ((DateTime.Now - StartTime).TotalMinutes > 60)
					{
						throw new AutomationException("Parallel build error installing {0} to {1}, timed out after an hour", BuildPath, DestPath);
					}

					Thread.Sleep(1000);

					lock (Globals.MainLock)
					{
						Status = LocalInstalls[InBuild].Status;
					}

					// install process failed in other thread, disk full, etc
					if (Status == InstallStatus.Error)
					{
						throw new AutomationException("Error installing parallel build from {0} to {1}", BuildPath, DestPath);
					}

					if (Status == InstallStatus.Installed)
					{
						Log.Verbose("Parallel build successfully installed from {0} to {1}", BuildPath, DestPath);
						break;
					}
				}
			}

			return DestPath;
		}

		public static IEnumerable<T> CreateFromPath<T>(UnrealTargetPlatform InPlatform, string InProjectName, string InPath, string InExecutableExtension, StagedBuildSource<T> VerificationBuildSource = null)
			where T : StagedBuild
		{
			string BuildPath = InPath;

			List<T> DiscoveredBuilds = new List<T>();

			// Turn FooGame into just Foo as we need to check for client/server builds too
			string ShortName = Regex.Replace(InProjectName, "Game", "", RegexOptions.IgnoreCase);

			string ContentPath = Path.Combine(InPath, InProjectName, "Content");

			if (Directory.Exists(ContentPath))
			{
				string EngineBinaryPath = Path.Combine(InPath, "Engine", "Binaries", InPlatform.ToString());
				string GameBinaryPath = Path.Combine(InPath, InProjectName, "Binaries", InPlatform.ToString());

				// Executable will either be Project*.exe or for content-only UnrealGame.exe
				List<string> ExecutableMatches = new List<string>
				{
					ShortName + "*" + InExecutableExtension,
					"UnrealGame*" + InExecutableExtension,
				};
				foreach (KeyValuePair<string, UnrealHelpers.ModuleConfig> Module in UnrealHelpers.CustomModules)
				{
					ExecutableMatches.Add(string.Format("{0}*{1}", Module.Key, InExecutableExtension));
				}

				// check
				// 1) Path/Project/Binaries/Platform
				// 2) Path (content only builds on some platforms write out a stub exe here)
				// 3) path/Engine/Binaries/Platform

				string[] ExecutablePaths = new string[]
				{
					Path.Combine(InPath, InProjectName, "Binaries", InPlatform.ToString()),
					Path.Combine(InPath, "Engine", "Binaries", InPlatform.ToString()),
					InPath,
				};

				List<FileSystemInfo> Binaries = new List<FileSystemInfo>();

				foreach (var BinaryPath in ExecutablePaths)
				{
					if (Directory.Exists(BinaryPath))
					{
						DirectoryInfo Di = new DirectoryInfo(BinaryPath);

						foreach (var FileMatch in ExecutableMatches)
						{
							// Look at files & directories since apps on Mac are bundles
							FileSystemInfo[] AppFiles = Di.GetFileSystemInfos(FileMatch);
							if(string.IsNullOrEmpty(InExecutableExtension))
							{
								// Special case for empty extension (linux), filter out files with extension
								AppFiles = AppFiles.Where(F => string.IsNullOrEmpty(F.Extension)).ToArray();
							}
							Binaries.AddRange(AppFiles);
						}
					}
				}

				foreach (FileSystemInfo App in Binaries)
				{
					UnrealHelpers.ConfigInfo ConfigInfo = UnrealHelpers.GetUnrealConfigFromFileName(InProjectName, App.Name);
					UnrealTargetConfiguration Config = ConfigInfo.Configuration;
					UnrealTargetRole Role = ConfigInfo.RoleType;
					string Flavor = ConfigInfo.Flavor;

					// if we have a verificatio BuildSource object, verify the build is usable before we even create the build
					if (VerificationBuildSource == null || VerificationBuildSource.ShouldMakeBuildAvailable(App.FullName))
					{
						if (Config != UnrealTargetConfiguration.Unknown && Role != UnrealTargetRole.Unknown && !DiscoveredBuilds.Any(B => B.Configuration == Config && B.Role == Role && B.Flavor == Flavor))
						{
							// store the exe path as relative to the staged dir path
							T NewBuild = Activator.CreateInstance(typeof(T), new object[] { InPlatform, Config, Role, InPath, Utils.SystemHelpers.MakePathRelative(App.FullName, InPath), Flavor }) as T;

							if (App.Name.StartsWith("UnrealGame", StringComparison.OrdinalIgnoreCase))
							{
								NewBuild.Flags |= BuildFlags.ContentOnlyProject;
							}

							DiscoveredBuilds.Add(NewBuild);
						}
					}
				}
			}

			return DiscoveredBuilds;
		}

		public static void CleanupInstalls()
		{
			lock (Globals.MainLock)
			{
				LocalInstalls.Clear();
			}
		}
	}

	public class NativeStagedBuild : StagedBuild
	{
		public NativeStagedBuild(UnrealTargetPlatform InPlatform, UnrealTargetConfiguration InConfig, UnrealTargetRole InRole, string InBuildPath, string InExecutablePath, string InFlavor = "")
			: base(InPlatform, InConfig, InRole, InBuildPath, InExecutablePath, InFlavor)
		{
			Flags = BuildFlags.CanReplaceCommandLine | BuildFlags.Loose;
		}
	}

	public abstract class StagedBuildSource<T> : IFolderBuildSource 
		where T : StagedBuild
	{
		public abstract string BuildName { get; }

		public abstract UnrealTargetPlatform Platform { get; }

		public abstract string PlatformFolderPrefix { get; }

		virtual public bool CanSupportPlatform(UnrealTargetPlatform InPlatform)
		{
			return InPlatform == Platform; 
		}

		virtual public bool ShouldMakeBuildAvailable(string AppPath)
		{
			return true;
		}

		virtual public string ExecutableExtension
		{
			get { return AutomationTool.Platform.GetExeExtension(Platform); }
		}

		protected virtual string[] ValidPlatformFolders => [PlatformFolderPrefix];


		public virtual List<IBuild> GetBuildsAtPath(string InProjectName, string InPath, int MaxRecursion = 3)
		{
			List<DirectoryInfo> AllDirs = new List<DirectoryInfo>();
			List<IBuild> Builds = new List<IBuild>();
			DirectoryInfo PathDI = new DirectoryInfo(InPath);

			if (PathDI.Exists)
			{
				// Find all directories that begin with the platform name
				foreach (string PlatformFolder in ValidPlatformFolders)
				{
					if (PathDI.Name.IndexOf(PlatformFolder, StringComparison.OrdinalIgnoreCase) >= 0)
					{
						AllDirs.Add(PathDI);
					}
					AllDirs.AddRange(PathDI.GetDirectories(PlatformFolder + "*", SearchOption.TopDirectoryOnly));
				}

				List<DirectoryInfo> DirsToRecurse = AllDirs;

				// now get subdirs
				while (MaxRecursion-- > 0)
				{
					List<DirectoryInfo> DiscoveredDirs = new List<DirectoryInfo>();

					DirsToRecurse.ToList().ForEach((D) =>
					{
						DiscoveredDirs.AddRange(D.GetDirectories("*", SearchOption.TopDirectoryOnly));
					});

					AllDirs.AddRange(DiscoveredDirs);
					DirsToRecurse = DiscoveredDirs;
				}

				// every directory that contains a valid build should have at least a ProjectName folder
				AllDirs = AllDirs.Where(D =>
				{
					var SubDirs = D.GetDirectories();
					return SubDirs.Any(SD => SD.Name.Equals(InProjectName, StringComparison.OrdinalIgnoreCase));

				}).ToList();

				foreach (DirectoryInfo Di in AllDirs)
				{
					IEnumerable<IBuild> FoundBuilds = StagedBuild.CreateFromPath<T>(Platform, InProjectName, Di.FullName, ExecutableExtension, this);

					if (FoundBuilds != null)
					{
						Builds.AddRange(FoundBuilds);
					}
				}
			}

			return Builds;
		}
	}

	public interface IStagedBuildFilter
	{
		public UnrealTargetPlatform Platform { get; }

		public List<FileInfo> GetFilteredFiles(UnrealAppConfig AppConfig, StagedBuild Build);

		public static IStagedBuildFilter GetFilterForPlatform(UnrealTargetPlatform Platform)
		{
			return InterfaceHelpers.FindImplementations<IStagedBuildFilter>().FirstOrDefault(FilterType => FilterType.Platform.Equals(Platform));
		}
	}
}
