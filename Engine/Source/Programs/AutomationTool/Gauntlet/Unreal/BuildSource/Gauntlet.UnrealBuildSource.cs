// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using AutomationTool;
using UnrealBuildTool;
using System.Text.RegularExpressions;
using System.IO;
using System.Linq;
using EpicGames.Core;
using System.Text.Json.Serialization;
using System.Text.Json;

namespace Gauntlet
{
	public class UnrealBuildSource : IBuildSource
	{
		public DirectoryReference UnrealPath { get; protected set; }

		public string ProjectName { get; protected set; }

		public FileReference ProjectPath { get; protected set; }

		public bool UsesSharedBuildType { get; protected set; }

		public string BuildName { get; protected set; }

		public IEnumerable<string> BuildPaths { get; protected set; }

		public string Branch { get; protected set; }

		public int Changelist { get; protected set; }

		public int Preflight { get; protected set; }

		public bool EditorValid { get; protected set; }

		protected Dictionary<UnrealTargetPlatform, List<IBuild>> DiscoveredBuilds;

		protected class ArtifactMetadata
		{
			[JsonPropertyName("change")]
			public string Change { get; set; }
			[JsonPropertyName("preflight")]
			public string Preflight { get; set; }
			[JsonPropertyName("branch")]
			public string Branch { get; set; }
			[JsonPropertyName("buildName")]
			public string BuildName { get; set; }
			[JsonPropertyName("configuration")]
			public string Configuration { get; set; }
			[JsonPropertyName("platform")]
			public string Platform { get; set; }
			[JsonPropertyName("type")]
			public string Type { get; set; }

			public static ArtifactMetadata LoadFromJson(string FilePath)
			{
				JsonSerializerOptions Options = new JsonSerializerOptions
				{
					PropertyNameCaseInsensitive = true
				};
				string JsonString = File.ReadAllText(FilePath);
				ArtifactMetadata JsonTestPassResults = JsonSerializer.Deserialize<ArtifactMetadata>(JsonString, Options);
				return JsonTestPassResults;
			}
		}

		public UnrealBuildSource(string InProjectName, FileReference InProjectPath, DirectoryReference InUnrealPath, bool InUsesSharedBuildType, string BuildReference)
		{
			InitBuildSource(InProjectName, InProjectPath, InUnrealPath, InUsesSharedBuildType, BuildReference, null);
		}

		public UnrealBuildSource(string InProjectName, FileReference InProjectPath, DirectoryReference InUnrealPath, bool InUsesSharedBuildType, string BuildReference, Func<string, string> ResolutionDelegate)
		{
			InitBuildSource(InProjectName, InProjectPath, InUnrealPath, InUsesSharedBuildType, BuildReference, ResolutionDelegate);
		}

		public UnrealBuildSource(string InProjectName, FileReference InProjectPath, DirectoryReference InUnrealPath, bool InUsesSharedBuildType, string BuildReference, IEnumerable<string> InSearchPaths)
		{
			InitBuildSource(InProjectName, InProjectPath, InUnrealPath, InUsesSharedBuildType, BuildReference, (string BuildRef) =>
			{
				foreach (string SearchPath in InSearchPaths)
				{
					string AggregatedPath = Path.Combine(SearchPath, BuildRef);
					if (AggregatedPath.Length > 0)
					{
						DirectoryInfo SearchDir = new DirectoryInfo(AggregatedPath);

						if (SearchDir.Exists)
						{
							return SearchDir.FullName;
						}
					}
				}

				return null;
			});
		}

		public bool CanSupportPlatform(UnrealTargetPlatform Platform)
		{
			return UnrealTargetPlatform.GetValidPlatforms().Contains(Platform);
		}


		protected void InitBuildSource(string InProjectName, FileReference InProjectPath, DirectoryReference InUnrealPath, bool InUsesSharedBuildType, string InBuildArgument, Func<string, string> ResolutionDelegate)
		{
			UnrealPath = InUnrealPath;
			UsesSharedBuildType = InUsesSharedBuildType;

			ProjectPath = InProjectPath;
			ProjectName = InProjectName;

			// Resolve the build argument into something meaningful
			string ResolvedBuildName;
			IEnumerable<string> ResolvedPaths = null;

			if (!ResolveBuildReference(InBuildArgument, ResolutionDelegate, out ResolvedPaths, out ResolvedBuildName))
			{
				throw new AutomationException("Unable to resolve {0} to a valid build", InBuildArgument);
			}

			BuildName = ResolvedBuildName;
			BuildPaths = ResolvedPaths;

			// any Branch/CL info?
			Match M = Regex.Match(BuildName, @"(\+\+.+)-CL-(\d+)");

			if (M.Success)
			{
				Branch = M.Groups[1].Value.Replace("+", "/");
				Changelist = Convert.ToInt32(M.Groups[2].Value);
			}
			else
			{
				Branch = "";
				Changelist = 0;
			}

			// Preflight?
			M = Regex.Match(BuildName, @"-PF-(\d+)");
			Preflight = M.Success? Convert.ToInt32(M.Groups[1].Value) : 0;

			// Look if the build has an artifact metadata file
			string ArtifactMetadataFilePath = Path.Combine(InBuildArgument, "artifactmetadata.json");
			if (File.Exists(ArtifactMetadataFilePath))
			{
				ArtifactMetadata Metadata = null;
				try
				{
					Metadata = ArtifactMetadata.LoadFromJson(ArtifactMetadataFilePath);
				}
				catch (Exception Ex)
				{
					Log.Warning("Could not parse ArtifactMetadata file '{File}': {Exception}", ArtifactMetadataFilePath, Ex.Message);
				}
				if (Metadata != null)
				{
					if (!string.IsNullOrEmpty(Metadata.Branch))
					{
						Branch = Metadata.Branch;
					}
					if (!string.IsNullOrEmpty(Metadata.Change))
					{
						int MetaChange;
						if (int.TryParse(Metadata.Change, out MetaChange))
						{
							Changelist = MetaChange;
						}
					}
					if (!string.IsNullOrEmpty(Metadata.Preflight))
					{
						int MetaPreflight;
						if (int.TryParse(Metadata.Preflight, out MetaPreflight))
						{
							Preflight = MetaPreflight;
						}
					}
					if (!string.IsNullOrEmpty(Metadata.BuildName))
					{
						BuildName = Metadata.BuildName;
					}
				}
			}

			// allow user overrides (TODO - centralize all this!)
			Branch = Globals.Params.ParseValue("branch", Branch);
			Changelist = Convert.ToInt32(Globals.Params.ParseValue("changelist", Changelist.ToString()));
			Preflight = Convert.ToInt32(Globals.Params.ParseValue("preflight", Preflight.ToString()));

			// We resolve these on demand
			DiscoveredBuilds = new Dictionary<UnrealTargetPlatform, List<IBuild>>();
		}

		virtual protected bool ResolveBuildReference(string InBuildReference, Func<string, string> ResolutionDelegate, out IEnumerable<string> OutBuildPaths, out string OutBuildName)
		{
			OutBuildName = null;
			// start as null. It's valid for some references to return empty paths so we use null to verify
			// that a resolution did happen
			OutBuildPaths = null;

			if (string.IsNullOrEmpty(InBuildReference))
			{
				return false;
			}

			if (InBuildReference.Equals("AutoP4", StringComparison.InvariantCultureIgnoreCase))
			{
				if (!CommandUtils.P4Enabled)
				{
					throw new AutomationException("-Build=AutoP4 requires -P4");
				}
				if (CommandUtils.P4Env.Changelist < 1000)
				{
					throw new AutomationException("-Build=AutoP4 requires a CL from P4 and we have {0}", CommandUtils.P4Env.Changelist);
				}

				string BuildRoot = CommandUtils.CombinePaths(CommandUtils.RootBuildStorageDirectory());
				string CachePath = InternalUtils.GetEnvironmentVariable("UE-BuildCachePath", "");

				string SrcBuildPath = CommandUtils.CombinePaths(BuildRoot, ProjectName);
				string SrcBuildPath2 = CommandUtils.CombinePaths(BuildRoot, ProjectName.Replace("Game", "").Replace("game", ""));

				string SrcBuildPath_Cache = CommandUtils.CombinePaths(CachePath, ProjectName);
				string SrcBuildPath2_Cache = CommandUtils.CombinePaths(CachePath, ProjectName.Replace("Game", "").Replace("game", ""));

				if (!InternalUtils.SafeDirectoryExists(SrcBuildPath))
				{
					if (!InternalUtils.SafeDirectoryExists(SrcBuildPath2))
					{
						throw new AutomationException("-Build=AutoP4: Neither {0} nor {1} exists.", SrcBuildPath, SrcBuildPath2);
					}
					SrcBuildPath = SrcBuildPath2;
					SrcBuildPath_Cache = SrcBuildPath2_Cache;
				}
				string SrcCLPath = CommandUtils.CombinePaths(SrcBuildPath, CommandUtils.EscapePath(CommandUtils.P4Env.Branch) + "-CL-" + CommandUtils.P4Env.Changelist.ToString());
				string SrcCLPath_Cache = CommandUtils.CombinePaths(SrcBuildPath_Cache, CommandUtils.EscapePath(CommandUtils.P4Env.Branch) + "-CL-" + CommandUtils.P4Env.Changelist.ToString());
				if (!InternalUtils.SafeDirectoryExists(SrcCLPath))
				{
					throw new AutomationException("-Build=AutoP4: {0} does not exist.", SrcCLPath);
				}

				if (InternalUtils.SafeDirectoryExists(SrcCLPath_Cache))
				{
					InBuildReference = SrcCLPath_Cache;
				}
				else
				{
					InBuildReference = SrcCLPath;
				}
				Log.Verbose("Using AutoP4 path {0}", InBuildReference);
			}

			// BuildParam could be a path, a name that we should resolve to a path, Staged, or Editor
			DirectoryInfo BuildDir = new DirectoryInfo(InBuildReference);

			if (BuildDir.Exists)
			{
				// Easy option first - is this a full path?
				OutBuildName = BuildDir.Name;
				OutBuildPaths = new string[] { BuildDir.FullName };
			}
			else if (BuildDir.Name.Equals("editor", StringComparison.OrdinalIgnoreCase))
			{
				// Second special case - "Editor" means run using the editor, no path needed
				OutBuildName = "Editor";
				OutBuildPaths = Enumerable.Empty<string>();
			}
			else if (BuildDir.Name.Equals("local", StringComparison.OrdinalIgnoreCase) || BuildDir.Name.Equals("staged", StringComparison.OrdinalIgnoreCase))
			{
				// First special case - "Staged" means use whats locally staged
				OutBuildName = "Local";
				string StagedPath = Path.Combine(ProjectPath.Directory.FullName, "Saved", "StagedBuilds");

				if (Directory.Exists(StagedPath) == false)
				{
					Log.Error("BuildReference was Staged but staged directory {0} not found", StagedPath);
					return false;
				}

				// include binaries path for packaged builds if it exists
				string BinariesPath = Path.Combine(ProjectPath.Directory.FullName, "Binaries");
				OutBuildPaths = Directory.Exists(BinariesPath) ? new string[] { StagedPath, BinariesPath } : new string[] { StagedPath };
			}
			else if (BuildDir.Name.Equals("LatestGood", StringComparison.OrdinalIgnoreCase)
				|| BuildDir.Name.Equals("LKG", StringComparison.OrdinalIgnoreCase))
			{
				string RequestedValidator = Globals.Params.ParseValue("BuildValidator", null);
				IBuildValidator Validator = Utils.InterfaceHelpers.FindImplementations<IBuildValidator>(true)
					.Where(Validator => Validator.CanSupportProject(ProjectName))
					.Where(Validator => string.IsNullOrEmpty(RequestedValidator) || Validator.Name.Equals(RequestedValidator, StringComparison.OrdinalIgnoreCase))
					.FirstOrDefault();

				if(Validator == null)
				{
					Log.Error("No build validator that can support project {ProjectName} was found.", ProjectName);
					return false;
				}

				string LatestGoodBuild = Validator.GetLatestGoodBuild();
				if(string.IsNullOrEmpty(LatestGoodBuild))
				{
					Log.Error("No latest good build was able to be found!");
					return false;
				}

				OutBuildPaths = new[] { LatestGoodBuild };
				OutBuildName = Path.GetFileName(LatestGoodBuild);

				Log.Info("{Validator} selected {Build} as the latest good build. Proceeding with this build", Validator.GetType().Name, OutBuildName);
				return true;
			}
			else
			{
				// todo - make this more generic
				if (BuildDir.Name.Equals("usesyncedbuild", StringComparison.OrdinalIgnoreCase))
				{
					BuildVersion Version;
					if (BuildVersion.TryRead(BuildVersion.GetDefaultFileName(), out Version))
					{
						InBuildReference = Version.BranchName + "-CL-" + Version.Changelist.ToString();
					}
				}

				// See if it's in the passed locations
				if (ResolutionDelegate != null)
				{
					string FullPath = ResolutionDelegate(InBuildReference);

					if (string.IsNullOrEmpty(FullPath) == false)
					{
						DirectoryInfo Di = new DirectoryInfo(FullPath);

						if (Di.Exists == false)
						{
							throw new AutomationException("Resolution delegate returned non existent path");
						}

						OutBuildName = Di.Name;
						OutBuildPaths = new string[] { Di.FullName };
					}
				}
			}

			if (string.IsNullOrEmpty(OutBuildName) || OutBuildPaths == null)
			{
				Log.Error("Unable to resolve build argument '{BuildReference}'", InBuildReference);
				Horde.UpdateStepMessage(Horde.GauntletHordeStepMessage.BuildNotFound);
				return false;
			}

			return true;
		}

		/// <summary>
		/// Adds the provided build to our list (calls ShouldMakeBuildAvailable to verify).
		/// </summary>
		/// <param name="NewBuild"></param>
		virtual protected void AddBuild(IBuild NewBuild)
		{
			NewBuild = ShouldMakeBuildAvailable(NewBuild);

			if (NewBuild != null)
			{
				if (!DiscoveredBuilds.ContainsKey(NewBuild.Platform))
				{
					DiscoveredBuilds[NewBuild.Platform] = new List<IBuild>();
				}

				string Flavor = string.IsNullOrEmpty(NewBuild.Flavor) ? "None" : NewBuild.Flavor;
				Log.Info("Adding {Platform} {BuildType} | Configuration: {Configuration} | Flavor: {Flavor} | Flags: {Flags} | PreferenceOrder: {PreferenceOrder}",
					NewBuild.Platform, NewBuild.GetType().Name, NewBuild.Configuration, Flavor, NewBuild.Flags, NewBuild.PreferenceOrder);
				DiscoveredBuilds[NewBuild.Platform].Add(NewBuild);
			}
		}

		/// <summary>
		/// Allows derived classes to nix or modify builds as they are discovered
		/// </summary>
		/// <param name="InBuild"></param>
		/// <returns></returns>
		virtual protected IBuild ShouldMakeBuildAvailable(IBuild InBuild)
		{
			return InBuild;
		}

		/// <summary>
		/// Adds an Editor build to our list of available builds if one exists
		/// </summary>
		/// <param name="InUnrealPath"></param>
		/// <param name="InConfiguration"></param>
		virtual protected IBuild CreateEditorBuild(DirectoryReference InUnrealPath, UnrealTargetConfiguration InConfiguration = UnrealTargetConfiguration.Development)
		{
			if (InUnrealPath != null)
			{
				// check for the editor
				string EditorExe  = Path.Combine(InUnrealPath.FullName, GetRelativeExecutablePath(UnrealTargetRole.Editor, BuildHostPlatform.Current.Platform, InConfiguration));
				string OverlayExe = Path.Combine(InUnrealPath.FullName, GetModuleOverlayExecutablePath(UnrealTargetRole.Editor, BuildHostPlatform.Current.Platform));

				if (Utils.SystemHelpers.ApplicationExists(EditorExe))
				{
					EditorBuild NewBuild = new EditorBuild(EditorExe, InConfiguration);

					return NewBuild;
				}

				if (Utils.SystemHelpers.ApplicationExists(OverlayExe))
				{
					return new EditorBuild(OverlayExe, InConfiguration);
				}

				Log.Info("No editor binaries found at {0}. Unable to create an editor build source.", EditorExe);
			}
			else
			{
				Log.Info("No path to Unreal found. Unable to create an editor build source.");
			}

			return null;
		}


		/// <summary>
		/// True/false on whether we've tried to discover builds for the specified platform
		/// </summary>
		/// <param name="InPlatform"></param>
		/// <returns></returns>
		bool HaveDiscoveredBuilds(UnrealTargetPlatform InPlatform)
		{
			return DiscoveredBuilds.ContainsKey(InPlatform);
		}

		/// <summary>
		/// Discover all builds for the specified platform. Nop if this has already been run
		/// for the provided platform
		/// </summary>
		/// <param name="InPlatform"></param>
		/// <param name="InConfiguration"></param>
		virtual protected void DiscoverBuilds(UnrealTargetPlatform InPlatform, UnrealTargetConfiguration InConfiguration = UnrealTargetConfiguration.Development)
		{
			if (!HaveDiscoveredBuilds(InPlatform))
			{
				Log.Info("Discovering builds for {0}", InPlatform);
				DiscoveredBuilds[InPlatform] = new List<IBuild>();

				// Add an editor build if this is our current platform.
				if (InPlatform == BuildHostPlatform.Current.Platform)
				{
					IBuild EditorBuild = CreateEditorBuild(UnrealPath, InConfiguration);

					if (EditorBuild == null)
					{
						Log.Info("Could not create editor build for project. Binaries are likely missing");
					}
					else
					{
						AddBuild(EditorBuild);
					}
				}

				if (BuildPaths.Any())
				{
					foreach (string Path in BuildPaths)
					{
						IEnumerable<IFolderBuildSource> BuildSources = Gauntlet.Utils.InterfaceHelpers.FindImplementations<IFolderBuildSource>(true)
																			.Where(BS => BS.CanSupportPlatform(InPlatform));

						foreach (var BS in BuildSources)
						{
							if (!BS.BuildName.Contains(InPlatform.ToString(), StringComparison.OrdinalIgnoreCase))
							{
								continue;
							}

							IEnumerable<IBuild> Builds = BS.GetBuildsAtPath(ProjectName, Path);

							foreach (IBuild Build in Builds)
							{
								AddBuild(Build);
							}
						}
					}
				}
			}
		}

		/// <summary>
		/// Returns how many builds are available for the specified platform
		/// </summary>
		/// <param name="InPlatform"></param>
		/// <param name="InFlags"></param>
		/// <returns></returns>
		public int GetBuildCount(UnrealTargetPlatform InPlatform, BuildFlags InFlags = BuildFlags.None)
		{
			if (!HaveDiscoveredBuilds(InPlatform))
			{
				DiscoverBuilds(InPlatform);
			}

			return DiscoveredBuilds[InPlatform].Where(B => (B.Flags & InFlags) == InFlags).Count();
		}

		/// <summary>
		/// Returns all builds that match the specified parameters. If no builds have been discovered then that is performed first
		/// </summary>
		/// <param name="InRole"></param>
		/// <param name="InPlatform"></param>
		/// <param name="InConfiguration"></param>
		/// <param name="InFlags"></param>
		/// <param name="InFlavor">Optional special flavor of the build, e.g. asan/ubsan/clang/etc..., which can be added on top of a standard configuration.</param>
		/// <returns></returns>
		IEnumerable<IBuild> GetMatchingBuilds(UnrealTargetRole InRole, UnrealTargetPlatform? InPlatform, UnrealTargetConfiguration InConfiguration, BuildFlags InFlags, string InFlavor="")
		{
			// can't get a build with no platform or if we have none
			if (InPlatform == null)
			{
				return new IBuild[0];
			}

			if (!HaveDiscoveredBuilds(InPlatform.Value))
			{
				DiscoverBuilds(InPlatform.Value, InConfiguration);
			}

			IEnumerable<IBuild> PlatformBuilds = DiscoveredBuilds[InPlatform.Value];

			IEnumerable<IBuild> MatchingBuilds = PlatformBuilds.Where((B) =>
			{
				if (B.CanSupportRole(InRole)
					&& B.Configuration == InConfiguration
					&& (B.Flags & InFlags) == InFlags
					&& (B.Flavor.Equals(InFlavor, StringComparison.OrdinalIgnoreCase)))
				{
					return true;
				}

				return false;
			});

			if (MatchingBuilds.Count() > 0)
			{
				return MatchingBuilds;
			}

			MatchingBuilds = PlatformBuilds.Where((B) =>
			{
				if ((InFlags & BuildFlags.CanReplaceExecutable) == BuildFlags.CanReplaceExecutable)
				{
					if (B.CanSupportRole(InRole)
						&& (B.Flags & InFlags) == InFlags
						&& (B.Flavor.Equals(InFlavor, StringComparison.OrdinalIgnoreCase)))
					{
						Log.Warning("Build did not have configuration {0} for {1}, but selecting due to presence of -dev flag",
							InConfiguration, InPlatform);
						return true;
					}
				}

				return false;
			});

			return MatchingBuilds;
		}

		/// <summary>
		/// Checks if we are able to support the specified role. This will trigger build discovery if it has not yet
		/// happened for the specified platform
		/// </summary>
		/// <param name="Role"></param>
		/// <param name="Reasons"></param>
		/// <returns></returns>
		public bool CanSupportRole(UnrealSessionRole Role, ref List<string> Reasons)
		{
			if (Role.RoleType.UsesEditor() && UnrealPath == null)
			{
				Reasons.Add(string.Format("Role {0} wants editor but no path to Unreal exists", Role));
				return false;
			}

			// null platform. Need a better way of specifying this
			if (Role.IsNullRole())
			{
				return true;
			}

			// Query our build list
			if (Role.Platform != null)
			{
				var MatchingBuilds = GetMatchingBuilds(Role.RoleType, Role.Platform.Value, Role.Configuration, Role.RequiredBuildFlags, Role.RequiredFlavor);

				if (MatchingBuilds.Count() > 0)
				{
					return true;
				}
			}

			Reasons.Add(string.Format("No build at {0} that matches {1} (RequiredFlags={2})", string.Join(",", BuildPaths), Role.ToString(), Role.RequiredBuildFlags.ToString()));

			return false;
		}

		virtual public UnrealAppConfig CreateConfiguration(UnrealSessionRole Role)
		{
			return CreateConfiguration(Role, new UnrealSessionRole[] { });
		}

		virtual public UnrealAppConfig CreateConfiguration(UnrealSessionRole Role, IEnumerable<UnrealSessionRole> OtherRoles)
		{
			List<string> Issues = new List<string>();

			Log.Verbose("Creating configuration Role {0}", Role);
			if (!CanSupportRole(Role, ref Issues))
			{
				Issues.ForEach(S => Log.Error(S));
				return null;
			}

			UnrealTestConfiguration TestConfig = Role.Options as UnrealTestConfiguration;

			UnrealAppConfig Config = new UnrealAppConfig();

			Config.Name = this.BuildName;
			Config.ProjectFile = this.ProjectPath;
			Config.ProjectName = ProjectName;
			Config.ProcessType = Role.RoleType;
			Config.Platform = Role.Platform;
			Config.Configuration = Role.Configuration;
			Config.CommandLineParams = new GauntletCommandLine(Role.CommandLineParams);
            Config.FilesToCopy = new List<UnrealFileToCopy>();
            Config.MaxDuration = (int)(TestConfig?.MaxDuration ?? 0);

			// new system of retrieving and encapsulating the info needed to install/launch. Android & Mac
			Config.Build = GetMatchingBuilds(Role.RoleType, Role.Platform, Role.Configuration, Role.RequiredBuildFlags, Role.RequiredFlavor).OrderBy(B => B.PreferenceOrder).FirstOrDefault();

			if (Config.Build == null)
			{
				if (Role.IsNullRole())
				{
					Log.Warning("No supported build found, however role is Null and not configure to run anything.");
				}
				else
				{
					var SupportedBuilds = String.Join("\n", DiscoveredBuilds.Select(B => B.ToString()));

					Log.Info("Available builds:\n{0}", SupportedBuilds);
					throw new AutomationException("No build found that can support a role of {0}.", Role);
				}
			}
			else
			{
				Log.Info("Selected build {Build} for test run.", Config.Build.ToString());
			}

			if (Role.Options != null)
			{
				UnrealTestConfiguration ConfigOptions = Role.Options as UnrealTestConfiguration;
				ConfigOptions.ApplyToConfig(Config, Role, OtherRoles);
			}

			// Cleanup the commandline
			Log.Info("Processing CommandLine {0}", Config.CommandLine);
			Config.CommandLine = GenerateProcessedCommandLine(Config.CommandLine);

			// Now add the project (the above code doesn't handle arguments without a leading - so do this last
			bool IsContentOnlyProject = (Config.Build != null) && ((Config.Build.Flags & BuildFlags.ContentOnlyProject) == BuildFlags.ContentOnlyProject);

			// Add in editor - TODO, should this be in the editor build?
			if (Role.RoleType.UsesEditor() || IsContentOnlyProject)
			{
				// add in -game or -server
				if (Role.RoleType.IsClient())
				{
					Config.CommandLineParams.Add("game");
				}
				else if (Role.RoleType.IsServer())
				{
					Config.CommandLineParams.Add("server");
				}

				string ProjectParam = ProjectPath.FullName;

				// if content only we need to provide a relative path to the uproject.
				if (IsContentOnlyProject && !Role.RoleType.UsesEditor())
				{
					ProjectParam = string.Format("../../../{0}/{0}.uproject", ProjectName);
				}
				Config.CommandLineParams.Project = ProjectParam;
			}

			// Detect json log line output
			if (Config.CommandLineParams.HasParam("JsonStdOut")
				|| (!string.IsNullOrEmpty(Environment.GetEnvironmentVariable("UE_LOG_JSON_TO_STDOUT"))
					&& (Role.Platform == BuildHostPlatform.Current.Platform || CanRunPlatformVirtualized(Role.Platform))))
			{
				Config.FilterLoggingDelegate = (string M, bool IsErr) => UnrealLogParser.SanitizeJsonOutputLine(M);
			}

            if (Role.FilesToCopy != null)
            {
                Config.FilesToCopy = Role.FilesToCopy;
            }

			if(Globals.IsRunningDev)
			{
				Config.OverlayExecutable = new OverlayExecutable(Role, Config.ProjectName, Config.ProjectFile);
			}

			return Config;
		}

		private bool CanRunPlatformVirtualized(UnrealTargetPlatform? Platform)
		{
			return Utils.InterfaceHelpers.FindImplementations<IVirtualLocalDevice>()
				.Any(F => F.GetPlatform() == Platform && F.CanRunVirtualFromPlatform(BuildHostPlatform.Current.Platform));
		}

		/// <summary>
		/// Remove all duplicate flags and combine any execcmd strings that might be floating around in the commandline.
		/// </summary>
		/// <param name="InCommandLine"></param>
		/// <returns></returns>
		private string GenerateProcessedCommandLine(string InCommandLine)
		{
			// Break down Commandline into individual tokens
			Dictionary<string, List<string>> CommandlineTokens = new Dictionary<string, List<string>>(StringComparer.OrdinalIgnoreCase);
			// turn Name(p1,etc) into a collection of Name|(p1,etc) groups
			MatchCollection Matches = GauntletCommandLine.CommandlineOptionRegex.Matches(InCommandLine);

			foreach (Match M in Matches)
			{
				if (M.Groups["option"] == null || string.IsNullOrWhiteSpace(M.Groups["option"].ToString()))
				{
					Log.Warning("Unable to parse option in commandline. Please check syntax/regex. This should never be hit.");
					continue;
				}

				string Name = M.Groups["option"].ToString().Trim();

				string Params = M.Groups["value"] != null ? M.Groups["value"].ToString() : string.Empty;

				if (CommandlineTokens.ContainsKey(Name))
				{

					if (string.IsNullOrWhiteSpace(Params))
					{
						Log.Info(string.Format("Duplicate flag {0} found and ignored. Please fix this. ", Name));
					}
					else if (Name.ToLower() == "execcmds")
					{
						// Special cased as execcmds is something that is totally able to be appended to. Requote everything when we're done.
						CommandlineTokens[Name][0] = string.Format("\"{0}, {1}\"", CommandlineTokens[Name][0].Replace("\"", ""), Params.Replace("\"", ""));
					}
					else if (Name.ToLower() == "dpcvars")
					{
						// dpcvars is also appendable but is slightly different in syntax. It doesn't use quotes and cannot contain spaces
						CommandlineTokens[Name][0] = string.Format("{0},{1}", CommandlineTokens[Name][0], Params);
					}
					else
					{
						bool bFoundDuplicateValue = false;
						foreach (string ExistingValue in CommandlineTokens[Name])
						{
							if (ExistingValue == Params)
							{
								Log.Info(string.Format("Duplicate flag {0}={1} found and ignored. Please fix this. ", Name, Params));
								bFoundDuplicateValue = true;
								break;
							}
						}

						if (!bFoundDuplicateValue)
						{
							// dpcvars are also a special parameter because the pattern is to repeat it like: -dpcvars=var1=val1 -dpcvars=var2=val2 etc
							if (Name.ToLower() != "dpcvars")
							{
								Log.Warning(string.Format("Multiple values for flag {0} found: {1} and {2}. The former value will be discarded. ", Name, CommandlineTokens[Name], Params));
								CommandlineTokens[Name][0] = Params.Trim();
							}
							else
							{
								CommandlineTokens[Name].Add(Params);
							}
						}
					}
				}
				else
				{
					CommandlineTokens.Add(Name, new List<string>{ Params });
				}
			}

			string CommandlineToReturn = "";
			foreach (string DictKey in CommandlineTokens.Keys)
			{
				foreach (string DictValue in CommandlineTokens[DictKey])
				{
					CommandlineToReturn += string.Format("-{0}{1} ",
						DictKey,
						string.IsNullOrWhiteSpace(DictValue) ? "" : string.Format("={0}", DictValue)
						);
				}
			}
			Gauntlet.Log.Verbose(string.Format("Pre-formatting Commandline: {0}", InCommandLine));
			Gauntlet.Log.Verbose(string.Format("Post-formatting Commandline: {0}", CommandlineToReturn));

			return CommandlineToReturn;
		}


		/// <summary>
		/// Get the module overlay binary path for a particular configuration
		/// </summary>
		private string GetModuleOverlayExecutablePath(UnrealTargetRole role, UnrealTargetPlatform platform)
		{
			return Path.Join(
				"Engine", "Binaries", platform.ToString(),
				UnrealHelpers.CustomModules.FirstOrDefault(Module => Module.Value.Role == role).Key
			);
		}

		/// <summary>
		/// Given a role, platform, and config, returns the path to the binary for that config. E.g. Binaries\Win64\FooServer-Win64-Shipping.exe
		/// </summary>
		/// <param name="TargetRole"></param>
		/// <param name="TargetPlatform"></param>
		/// <param name="TargetConfiguration"></param>
		/// <returns></returns>
		virtual public string GetRelativeExecutablePath(UnrealTargetRole TargetRole, UnrealTargetPlatform TargetPlatform, UnrealTargetConfiguration TargetConfiguration)
		{
			string ExePath;

			if (TargetRole.UsesEditor() || TargetRole.IsEditor())
			{
				bool HasCustomTarget = UnrealHelpers.CustomModules.Any(Module => Module.Value.Role == UnrealTargetRole.Editor);
				string EditorTarget = HasCustomTarget ? UnrealHelpers.CustomModules.FirstOrDefault(Module => Module.Value.Role == UnrealTargetRole.Editor).Key : string.Empty;
				FileSystemReference EditorExe = null;

				if (!HasCustomTarget)
				{
					try
					{
						EditorExe = ProjectUtils.GetProjectTarget(ProjectPath, UnrealBuildTool.TargetType.Editor, TargetPlatform, TargetConfiguration);
					}
					catch (Exception Ex)
					{
						string Message = string.Format("The project config is overriding build targets.\n"
								+ "But no suitable editor build for {0} configuration found from target file.\n"
								+ "{1}", TargetConfiguration, Ex.Message);

						if (BuildName.Equals("Editor", StringComparison.OrdinalIgnoreCase))
						{
							// An editor is being explicitly requested but no executable could be found from target file
							// Hightlight the issue in the log and return an empty string to avoid taking an inappropriate editor
							Log.Warning(Message);
							return string.Empty;
						}
						Log.Info(Message);
					}
				}

				if (EditorExe != null)
				{
					ExePath = EditorExe.FullName;
					if (!string.IsNullOrEmpty(Globals.Params.ParseValue("EditorDir", null)))
					{
						// Trim the Editor absolute path from what the target file provided as the editor dir is being overriden
						// https://regex101.com/r/7BttxH/1
						ExePath = Regex.Replace(ExePath, @"(.+?)[/\\]((Engine[/\\])?Binaries[/\\].+)", "$2");
					}
				}
				else
				{
					string ExeFileName = HasCustomTarget ? EditorTarget : "UnrealEditor";
					if (TargetConfiguration != UnrealTargetConfiguration.Development)
					{
						ExeFileName += string.Format("-{0}-{1}", TargetPlatform.ToString(), TargetConfiguration.ToString());
					}

					ExeFileName += Platform.GetExeExtension(TargetPlatform);

					string BasePath = HasCustomTarget ? Globals.Params.ParseValue("EditorDir", ProjectPath.Directory.FullName) : "Engine";
					ExePath = string.Format("{0}/Binaries/{1}/{2}", BasePath, BuildHostPlatform.Current.Platform, ExeFileName);
				}
			}
			else
			{
				string BuildType = "";

				if (TargetRole == UnrealTargetRole.Client)
				{
					if (!UsesSharedBuildType)
					{
						BuildType = "Client";
					}
				}
				else if (TargetRole == UnrealTargetRole.Server)
				{
					if (!UsesSharedBuildType)
					{
						BuildType = "Server";
					}
				}

				// Turn FooGame into Foo
				bool IsRunningDev = Globals.IsRunningDev;
				string ExeBase = ProjectName.Replace("Game", "");

				if (TargetPlatform == UnrealTargetPlatform.Android)
				{
					// use the archive results for android.
					//var AndroidSource = new AndroidBuild(ProjectName, GetPlatformPath(TargetType, TargetPlatform), TargetConfiguration);

					// We always (currently..) need to be able to replace the command line
					BuildFlags Flags = BuildFlags.CanReplaceCommandLine;
					if (IsRunningDev)
					{
						Flags |= BuildFlags.CanReplaceExecutable;
					}
					if (Globals.Params.ParseParam("bulk"))
					{
						Flags |= BuildFlags.Bulk;
					}
					else if(Globals.Params.ParseParam("notbulk"))
					{
						Flags |= BuildFlags.NotBulk;
					}

					var Build = GetMatchingBuilds(TargetRole, TargetPlatform, TargetConfiguration, Flags, "").FirstOrDefault();

					if (Build != null)
					{
						AndroidBuild AndroidBuild = Build as AndroidBuild;
						ExePath = AndroidBuild.SourceApkPath;
					}
					else
					{
						throw new AutomationException("No suitable build for {0} found at {1}", TargetPlatform, string.Join(",", BuildPaths));
					}
				}
				else
				{
					string ExeFileName = string.Format("{0}{1}", ExeBase, BuildType);
					string ExeFileName2 = ProjectName;

					if (TargetConfiguration != UnrealTargetConfiguration.Development)
					{
						ExeFileName += string.Format("-{0}-{1}", TargetPlatform.ToString(), TargetConfiguration.ToString());
						ExeFileName2 += string.Format("-{0}-{1}", TargetPlatform.ToString(), TargetConfiguration.ToString());
					}

					if (TargetPlatform != UnrealTargetPlatform.IOS)
					{
						ExeFileName += Platform.GetExeExtension(TargetPlatform);
						ExeFileName2 += Platform.GetExeExtension(TargetPlatform);
					}

					string BasePath = GetPlatformPath(TargetRole, TargetPlatform);
					string ProjectBinary = string.Format("{0}\\Binaries\\{1}\\{2}", ProjectName, TargetPlatform.ToString(), ExeFileName);
					string ProjectBinary2 = string.Format("{0}\\Binaries\\{1}\\{2}", ExeBase, TargetPlatform.ToString(), ExeFileName2);
					string StubBinary = Path.Combine(BasePath, ExeFileName);
					string StubBinary2 = Path.Combine(BasePath, ExeFileName2);
					string DevBinary = Path.Combine(Environment.CurrentDirectory, ProjectBinary);
					string DevBinary2 = Path.Combine(Environment.CurrentDirectory, ProjectBinary2);

					string NonCodeProjectName = "UnrealGame" + Platform.GetExeExtension(TargetPlatform);
					string NonCodeProjectBinary = Path.Combine(BasePath, "Engine", "Binaries", TargetPlatform.ToString());
					NonCodeProjectBinary = Path.Combine(NonCodeProjectBinary, NonCodeProjectName);

					// check the project binaries folder
					if (File.Exists(Path.Combine(BasePath, ProjectBinary)))
					{
						ExePath = ProjectBinary;
					}
					else if (File.Exists(StubBinary))
					{
						ExePath = Path.Combine(BasePath, ExeFileName);
					}
					else if (IsRunningDev && File.Exists(DevBinary))
					{
						ExePath = DevBinary;
					}
					else if (File.Exists(NonCodeProjectBinary))
					{
						ExePath = NonCodeProjectBinary;
					}
					else if (File.Exists(Path.Combine(BasePath, ProjectBinary2)))
					{
						ExePath = Path.Combine(BasePath, ProjectBinary2);
					}
					else if (File.Exists(StubBinary2))
					{
						ExePath = Path.Combine(BasePath, ExeFileName2);
					}
					else if (IsRunningDev && File.Exists(DevBinary2))
					{
						ExePath = DevBinary2;
					}
					else
					{
						List<string> CheckedFiles = new List<String>() { Path.Combine(BasePath, ProjectBinary), StubBinary, NonCodeProjectBinary, Path.Combine(BasePath, ProjectBinary2), StubBinary2 };
						if (IsRunningDev)
						{
							CheckedFiles.Add(DevBinary);
							CheckedFiles.Add(DevBinary2);
						}

						throw new AutomationException("Executable not found, upstream compile job may have failed.  Could not find executable {0} within {1}, binaries checked: {2}", ExeFileName, BasePath, String.Join(" - ", CheckedFiles));
					}

				}
			}

			return Utils.SystemHelpers.CorrectDirectorySeparators(ExePath);
		}

		public string GetPlatformPath(UnrealTargetRole Type, UnrealTargetPlatform Platform)
		{
			if (Type.UsesEditor())
			{
				return UnrealPath.FullName;
			}

			string BuildPath = BuildPaths.ElementAt(0);

			if (string.IsNullOrEmpty(BuildPath))
			{
				return null;
			}

			string PlatformPath = Path.Combine(BuildPath, UnrealHelpers.GetPlatformName(Platform, Type, UsesSharedBuildType));

			// On some builds we stage the actual loose files into a "Staged" folder
			if (Directory.Exists(PlatformPath) && Directory.Exists(Path.Combine(PlatformPath, "staged")))
			{
				PlatformPath = Path.Combine(PlatformPath, "Staged");
			}

			// Urgh - build share uses a different style...
			if (Platform == UnrealTargetPlatform.Android && BuildName.Equals("Local", StringComparison.OrdinalIgnoreCase) == false)
			{
				PlatformPath = PlatformPath.Replace("Android_ETC2Client", "Android\\FullPackages");
			}

			return PlatformPath;
		}
	}
}
