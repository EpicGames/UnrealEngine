// Copyright Epic Games, Inc. All Rights Reserved.

using AutomationTool;
using Gauntlet;
using System;
using System.IO;
using System.Data;
using System.Linq;
using System.Text.Json.Serialization;
using System.Collections.Generic;
using UnrealBuildTool;
using UnrealBuildBase;

namespace EpicConfig
{
	/// <summary>
	/// Production UE Telemetry config
	/// </summary>
	public class UETelemetry : MySQLConfig<Gauntlet.TelemetryData>
	{
		protected virtual string TableName { get { return "test_records_prod"; } }
		protected virtual string BuildTableName { get { return "test_builds_prod"; } }
		private int BuildID = -1;
		public override void LoadConfig(string ConfigFilePath)
		{
			string RootConfig;
			if (string.IsNullOrEmpty(ConfigFilePath))
			{
				RootConfig = @"\\epicgames.net\root\Builds\Automation\UE5\Config";
				if (HostPlatform.Current.HostEditorPlatform == UnrealTargetPlatform.Linux)
				{
					RootConfig = "/mnt/Builds/Automation/UE5/Config";
				}
				else if (HostPlatform.Current.HostEditorPlatform == UnrealTargetPlatform.Mac)
				{
					RootConfig = "/Volumes/Root/Builds/Automation/UE5/Config";
				}
				ConfigFilePath = Path.Combine(RootConfig, "ue-automation-telemetry-mysql-config.txt");
			} else
			{
				RootConfig = Path.GetDirectoryName(ConfigFilePath);
			}
			base.LoadConfig(ConfigFilePath);
		}
		/// <summary>
		/// Get Target Table name based on data type
		/// </summary>
		public override string GetTableName()
		{
			return TableName;
		}
		/// <summary>
		/// Get Target Table columns based on data type
		/// </summary>
		public override IEnumerable<string> GetTableColumns()
		{
			return new List<string>() {
				"buildId",
				"system",
				"dateTime",
				"platform",
				"config",
				"testName",
				"context",
				"dataPoint",
				"recordedValue",
				"unit",
				"baseline",
				"jobLink"
			};
		}
		/// <summary>
		/// Format the data for target table based on data type
		/// </summary>
		/// <returns></returns>
		public override IEnumerable<string> FormatDataForTable(Gauntlet.TelemetryData Data, ITelemetryContext Context)
		{
			return new List<string>() {
				BuildID.ToString(),
				Unreal.MachineName,
				DateTime.Now.ToString("yyyy-MM-dd HH:mm:ss"),
				Context.GetProperty("Platform").ToString(),
				Context.GetProperty("Configuration").ToString(),
				Data.TestName,
				Data.Context,
				Data.DataPoint,
				Data.Measurement.ToString(),
				Data.Unit,
				Data.Baseline.ToString(),
				Context.GetProperty("JobLink").ToString()
			};
		}
		/// <summary>
		/// Get Build Table name
		/// </summary>
		public virtual string GetBuildTableName()
		{
			return BuildTableName;
		}
		/// <summary>
		/// Query DB to get build id or add it and store the build id.
		/// </summary>
		/// <param name="Driver"></param>
		/// <param name="DataRows"></param>
		/// <param name="Context"></param>
		public override bool PreSubmitQuery(IDatabaseDriver<Gauntlet.TelemetryData> Driver, IEnumerable<Gauntlet.TelemetryData> DataRows, ITelemetryContext Context)
		{
			string Changelist = Context.GetProperty("Changelist").ToString();
			string ChangelistDateTime = ((DateTime)Context.GetProperty("ChangelistDateTime")).ToString("yyyy-MM-dd HH:mm:ss");
			string Stream = Context.GetProperty("Branch").ToString();
			string Project = Context.GetProperty("ProjectName").ToString();

			string SqlQuery = @$"INSERT INTO `{DatabaseName}`.{BuildTableName} (changelist, changelistDateTime, stream, project)
									SELECT '{Changelist}', '{ChangelistDateTime}', '{Stream}', '{Project}' FROM DUAL
										WHERE NOT EXISTS (SELECT 1 FROM `{DatabaseName}`.{BuildTableName} WHERE changelist = '{Changelist}' AND stream = '{Stream}' AND project = '{Project}');
									SELECT id FROM `{DatabaseName}`.{BuildTableName} WHERE changelist = '{Changelist}' AND stream = '{Stream}' AND project = '{Project}';";

			DataSet Set = Driver.ExecuteQuery(SqlQuery);
			// Get value from first row from first column
			if (Set.Tables.Count == 0 || Set.Tables[0].Rows.Count == 0)
			{
				Log.Error("No data return from {0} insert Query.", BuildTableName);
				return false;
			}
			BuildID = (int)Set.Tables[0].Rows[0][0];

			return true;
		}

		/// <summary>
		/// Return the query to fetch data
		/// </summary>
		/// <param name="InContext"></param>
		/// <returns></returns>
		public override string GetFetchDataQuery(ITelemetryContext InContext)
        {
			string Project = InContext.GetProperty("ProjectName")?.ToString();
			string Configuration = InContext.GetProperty("Configuration")?.ToString();
			string Platform = InContext.GetProperty("Platform")?.ToString();
			string Branch = InContext.GetProperty("Branch")?.ToString();
			string TestName = InContext.GetProperty("TestName")?.ToString();
			string DataPoint = InContext.GetProperty("DataPoint")?.ToString();
			string TestContext = InContext.GetProperty("Context")?.ToString();

			string WhereClause = "";
			if(!string.IsNullOrEmpty(Project))
            {
				WhereClause += $"AND T1.project = '{Project}' ";
            }
			if (!string.IsNullOrEmpty(Branch))
			{
				WhereClause += $"AND T1.stream = '{Branch}' ";
			}
			if (!string.IsNullOrEmpty(Configuration))
			{
				WhereClause += $"AND T2.config = '{Configuration}' ";
			}
			if (!string.IsNullOrEmpty(Platform))
			{
				WhereClause += $"AND T2.platform = '{Platform}' ";
			}
			if (!string.IsNullOrEmpty(TestName))
			{
				WhereClause += $"AND T2.testName IN ('{string.Join("','", TestName.Split(','))}') ";
			}
			if (!string.IsNullOrEmpty(DataPoint))
			{
				WhereClause += $"AND T2.dataPoint IN ('{string.Join("','", DataPoint.Split(','))}')  ";
			}
			if (!string.IsNullOrEmpty(TestContext))
			{
				WhereClause += $"AND T2.context IN ('{string.Join("','", TestContext.Split(','))}') ";
			}

			string ChangelistDateTime = ((DateTime)InContext.GetProperty("ChangelistDateTime")).ToString("yyyy-MM-dd HH:mm:ss");

			string SqlQuery = @$"SELECT T2.testName AS TestName, T2.context AS Context, T2.dataPoint AS DataPoint, T2.recordedValue AS Measurement, T2.unit AS Unit, T2.baseline AS Baseline, T1.changelist AS Changelist, UNIX_TIMESTAMP(T1.changelistDateTime) AS ChangelistDataTime, T1.stream AS Stream, T1.project AS Project, T2.platform AS Platform, T2.config AS Config, T2.id AS ID
									FROM `{DatabaseName}`.{BuildTableName} AS T1
										RIGHT JOIN (`{DatabaseName}`.{TableName} AS T2) ON (T1.id = T2.buildId)
											WHERE T1.changelistDateTime > '{ChangelistDateTime}' {WhereClause} ORDER BY T1.id Desc;";

			return SqlQuery;
		}
	}
	/// <summary>
	/// Staging UE Telemetry Config
	/// </summary>
	public class UETelemetryStaging : UETelemetry
	{
		protected override string TableName { get { return "test_records_staging"; } }
		protected override string BuildTableName { get { return "test_builds_staging"; } }
	}

}
