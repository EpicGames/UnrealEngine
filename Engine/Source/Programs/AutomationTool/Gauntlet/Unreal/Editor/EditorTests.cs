// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Data;
using System.IO;
using System.Linq;
using AutomationTool;
using Gauntlet;
using EpicGames.Core;
using Log = Gauntlet.Log;

namespace UEPerf
{
	interface IEditorConfig
	{
		string EnablePlugins { get; set; }

		string TraceFile { get; set; }

		bool NoLoadLevelAtStartup { get; set; }

		bool NoShaderDistrib { get; set; }

		bool VerboseShaderLogging { get; set; }

		bool Benchmarking { get; set; }

		bool NoDDCCleanup { get; set; }
	}

	class ConfigUtils
	{
		static public void ApplyEditorConfig(UnrealAppConfig AppConfig, IEditorConfig EditorConfig)
		{
			if (EditorConfig.EnablePlugins != string.Empty)
			{
				AppConfig.CommandLineParams.Add("EnablePlugins", EditorConfig.EnablePlugins);
			}

			if (EditorConfig.TraceFile != string.Empty)
			{
				AppConfig.CommandLineParams.Add("tracefile", EditorConfig.TraceFile);
				AppConfig.CommandLineParams.Add("tracefiletrunc"); // replace existing
				AppConfig.CommandLineParams.Add("trace", "default,counters,stats,loadtime,savetime,assetloadtime");
				AppConfig.CommandLineParams.Add("statnamedevents");
			}

			if (EditorConfig.NoLoadLevelAtStartup)
			{
				AppConfig.CommandLineParams.Add("ini:EditorPerProjectUserSettings:[/Script/UnrealEd.EditorLoadingSavingSettings]:LoadLevelAtStartup=None");
			}

			if (EditorConfig.NoShaderDistrib)
			{
				AppConfig.CommandLineParams.Add("noxgeshadercompile");
			}

			if (EditorConfig.VerboseShaderLogging)
			{
				AppConfig.CommandLineParams.Add("ini:Engine:[Core.Log]:LogShaderCompilers=Verbose");
			}

			if (EditorConfig.Benchmarking)
			{
				AppConfig.CommandLineParams.Add("BENCHMARK");
				AppConfig.CommandLineParams.Add("Deterministic");
			}

			if (EditorConfig.NoDDCCleanup)
			{
				AppConfig.CommandLineParams.Add("NODDCCLEANUP");
			}
		}
	}

	/// <summary>
	/// Default set of options for testing Editor. Options that tests can configure
	/// should be public, external command-line driven options should be protected/private
	/// </summary>
	public class EditorTestConfig : UE.AutomationTestConfig, IEditorConfig
	{
		/// <summary>
		/// Use Simple Horde Report instead of Unreal Automated Tests
		/// </summary>
		public override bool SimpleHordeReport { get; set; } = true;

		/// <summary>
		/// Force some specific plugins to load, comma delimited (at time of writing)
		/// </summary>
		[AutoParam]
		public string EnablePlugins { get; set; } = string.Empty;

		/// <summary>
		/// The file to trace profile data to
		/// </summary>
		[AutoParam]
		public string TraceFile { get; set; } = string.Empty;

		/// <summary>
		/// Control for interpretation of log warnings as test failures
		/// </summary>
		[AutoParam]
		public bool SuppressLogWarnings { get; set; } = false;

		/// <summary>
		/// Control for interpretation of log errors as test failures
		/// </summary>
		[AutoParam]
		public bool SuppressLogErrors { get; set; } = false;

		/// <summary>
		/// Modify the game instance lost timeout interval
		/// </summary>
		[AutoParam]
		public string GameInstanceLostTimerSeconds { get; set; } = string.Empty;

		/// <summary>
		/// Disable loading a level at startup (for profiling the map load)
		/// </summary>
		[AutoParam]
		public bool NoLoadLevelAtStartup { get; set; } = false;

		/// <summary>
		/// Disable distribution of shader builds (but use worker processes still)
		/// </summary>
		[AutoParam]
		public bool NoShaderDistrib { get; set; } = false;

		/// <summary>
		/// Enable Verbose Shader logging so we don't time out compiling lots of shaders
		/// </summary>
		[AutoParam]
		public bool VerboseShaderLogging { get; set; } = false;

		/// <summary>
		/// Enable benchmarking features in the engine
		/// </summary>
		[AutoParam]
		public bool Benchmarking { get; set; } = false;

		/// <summary>
		/// Enable No DDC Cleanup
		/// </summary>
		[AutoParam]
		public bool NoDDCCleanup { get; set; } = false;

		/// <summary>
		/// Applies these options to the provided app config
		/// </summary>
		/// <param name="AppConfig"></param>
		/// <param name="ConfigRole"></param>
		/// <param name="OtherRoles"></param>
		public override void ApplyToConfig(UnrealAppConfig AppConfig, UnrealSessionRole ConfigRole, IEnumerable<UnrealSessionRole> OtherRoles)
		{
			base.ApplyToConfig(AppConfig, ConfigRole, OtherRoles);
			ConfigUtils.ApplyEditorConfig(AppConfig, this);

			if (SuppressLogWarnings)
			{
				AppConfig.CommandLineParams.Add("ini:Engine:[/Script/AutomationController.AutomationControllerSettings]:bSuppressLogWarnings=true");
			}

			if (SuppressLogErrors)
			{
				AppConfig.CommandLineParams.Add("ini:Engine:[/Script/AutomationController.AutomationControllerSettings]:bSuppressLogErrors=true");
			}

			if (GameInstanceLostTimerSeconds != string.Empty)
			{
				AppConfig.CommandLineParams.Add($"ini:Engine:[/Script/AutomationController.AutomationControllerSettings]:GameInstanceLostTimerSeconds={GameInstanceLostTimerSeconds}");
			}

		}
	}

	public class EditorTests : UE.AutomationNodeBase<EditorTestConfig>
	{
		public EditorTests(UnrealTestContext InContext) : base(InContext)
		{
		}

		public override EditorTestConfig GetConfiguration()
		{
			if (CachedConfig != null)
			{
				return CachedConfig;
			}
			// just need a single role
			EditorTestConfig Config = base.GetConfiguration();
			Config.RequireRole(UnrealTargetRole.Editor);	
			return Config;
		}

		protected override string HordeReportTestName
		{
			get
			{
				return GetConfiguration().RunTest.Replace(".", " ");
			}
		}
	}

	public class EditorGauntletTestControllerConfig : UnrealTestConfiguration, IEditorConfig
	{

		/// <summary>
		/// Force some specific plugins to load, comma delimited (at time of writing)
		/// </summary>
		[AutoParam]
		public string EnablePlugins { get; set; } = string.Empty;

		/// <summary>
		/// The file to trace profile data to
		/// </summary>
		[AutoParam]
		public string TraceFile { get; set; } = string.Empty;

		/// <summary>
		/// Disable loading a level at startup (for profiling the map load)
		/// </summary>
		[AutoParam]
		public bool NoLoadLevelAtStartup { get; set; } = false;

		/// <summary>
		/// Disable distribution of shader builds (but use worker processes still)
		/// </summary>
		[AutoParam]
		public bool NoShaderDistrib { get; set; } = false;

		/// <summary>
		/// Enable Verbose Shader logging so we don't time out compiling lots of shaders
		/// </summary>
		[AutoParam]
		public bool VerboseShaderLogging { get; set; } = false;

		/// <summary>
		/// Enable benchmarking features in the engine
		/// </summary>
		[AutoParam]
		public bool Benchmarking { get; set; } = false;

		/// <summary>
		/// Enable No DDC Cleanup
		/// </summary>
		[AutoParam]
		public bool NoDDCCleanup { get; set; } = false;

		/// <summary>
		/// Set Gauntlet controller
		/// </summary>
		[AutoParam]
		public string Controller { get; set; } = string.Empty;

		/// <summary>
		/// Set DDC configuration
		/// </summary>
		[AutoParam]
		public string DDC { get; set; } = string.Empty;

		/// <summary>
		/// Set RHI configuration
		/// </summary>
		[AutoParam]
		public string RHI { get; set; } = string.Empty;

		/// <summary>
		/// Log Idle timeout in second
		/// </summary>
		[AutoParam]
		public int LogIdleTimeoutSec { get; set; } = 0;

		public override void ApplyToConfig(UnrealAppConfig AppConfig, UnrealSessionRole ConfigRole, IEnumerable<UnrealSessionRole> OtherRoles)
		{
			if (string.IsNullOrEmpty(Controller))
			{
				throw new AutomationException("No -Controller= is specified.");
			}

			base.ApplyToConfig(AppConfig, ConfigRole, OtherRoles);
			ConfigUtils.ApplyEditorConfig(AppConfig, this);

			if (!string.IsNullOrEmpty(DDC))
			{
				AppConfig.CommandLineParams.Add("ddc", DDC);
			}
			if (!string.IsNullOrEmpty(RHI))
			{
				AppConfig.CommandLineParams.Add(RHI.ToLower());
			}

			if (!string.IsNullOrEmpty(Map))
			{
				AppConfig.CommandLineParams.GameMap = Map;
			}

			// Enforcing PIE to allow the Gauntlet controller to start
			AppConfig.CommandLineParams.Add("PIE");
		}
	}

	public class EditorGauntletTestController : UnrealTestNode<EditorGauntletTestControllerConfig>
	{
		private UnrealLogStreamParser LogReader = null;
		private DateTime LastAutomationEntryTime = DateTime.MinValue;
		private bool ValidateResolveMap = false;
		private float IdleTimeoutSec = 5 * 60;
		public EditorGauntletTestController(UnrealTestContext InContext) : base(InContext)
		{
		}

		public override EditorGauntletTestControllerConfig GetConfiguration()
		{
			if (CachedConfig != null)
			{
				return CachedConfig;
			}
			// just need a single role
			EditorGauntletTestControllerConfig Config = base.GetConfiguration();
			var EditorRole = Config.RequireRole(UnrealTargetRole.Editor);

			if (!string.IsNullOrEmpty(Config.Controller))
			{
				EditorRole.Controllers.Add(Config.Controller);
			}

			return Config;
		}

		protected override string HordeReportTestName
		{
			get
			{
				string TestName = "EditorGauntletTestController";
				EditorGauntletTestControllerConfig Config = GetConfiguration();
				if (!string.IsNullOrEmpty(Config.Controller))
				{
					TestName = Config.Controller;
				}
				if (!string.IsNullOrEmpty(Config.Map))
				{
					TestName += $" {Path.GetFileName(Config.Map)}";
				}

				return TestName;
			}
		}

		public override bool StartTest(int Pass, int InNumPasses)
		{
			LogReader = null;
			EditorGauntletTestControllerConfig Config = GetConfiguration();
			if (!string.IsNullOrEmpty(Config.Map))
			{
				ValidateResolveMap = true;
			}

			if (Config.LogIdleTimeoutSec > 0)
			{
				IdleTimeoutSec = Config.LogIdleTimeoutSec;
			}


			return base.StartTest(Pass, InNumPasses);
		}

		public override void TickTest(UnrealSessionInstance InInstance)
		{
			// We are only interested in what the editor is doing
			var App = InInstance.EditorApp;
			if (App != null)
			{
				if (LogReader == null)
				{
					LogReader = new UnrealLogStreamParser(App.GetLogBufferReader());
				}
				LogReader.ReadStream();

				IEnumerable<string> ChannelEntries = LogReader.GetLogFromShortNameChannels(UnrealLog.EditorBusyChannels.Append("Gauntlet"));

				// Any new entries?
				if (ChannelEntries.Any())
				{
					// log new entries so people have something to look at
					ChannelEntries.ToList().ForEach(S => Log.Info(S));
					LastAutomationEntryTime = DateTime.Now;
				}
				else
				{
					// Check for timeouts
					if (LastAutomationEntryTime == DateTime.MinValue)
					{
						LastAutomationEntryTime = DateTime.Now;
					}

					double ElapsedTime = (DateTime.Now - LastAutomationEntryTime).TotalSeconds;

					// Check for timeout
					if (ElapsedTime > IdleTimeoutSec)
					{
						Log.Warning(KnownLogEvents.Gauntlet_TestEvent, "No activity observed in last {Time:0.00} minutes. Aborting test", IdleTimeoutSec / 60);
						MarkTestComplete();
						SetUnrealTestResult(TestResult.TimedOut);
					}
				}

				if (ValidateResolveMap)
				{
					string Map = GetConfiguration().Map;
					string ResolvedMap = LogReader.GetLogLinesContaining($"to resolve {Map}.").FirstOrDefault();

					if (!string.IsNullOrEmpty(ResolvedMap))
					{
						if (ResolvedMap.Contains($"failed to resolve"))
						{
							MarkTestComplete();
							ReportError(ResolvedMap);
							ReportError($"Forcing test exits because {Map} failed to be found.");
						}

						ValidateResolveMap = false;
					}
				}
			}
		}

		protected override UnrealProcessResult GetExitCodeAndReason(StopReason InReason, UnrealLog InLog, UnrealRoleArtifacts InArtifacts, out string ExitReason, out int ExitCode)
		{
			if (InArtifacts.AppInstance.WasKilled && GetTestResult() == TestResult.Failed)
			{
				ExitReason = "Process was killed by Gauntlet due to test failure.";
				ExitCode = -1;
				return UnrealProcessResult.TestFailure;
			}

			return base.GetExitCodeAndReason(InReason, InLog, InArtifacts, out ExitReason, out ExitCode);
		}
	}
}

namespace UE
{
	/// <summary>
	/// Test to run any editor command. Validate only if the Editor was initialized and exited normally or when string in found in the log.
	/// Exit needs to be explicitly call by the command line argument or executed code, or use -CompletionString= to trigger exit on log string match
	/// </summary>
	public class EditorCommand : BootTest
	{
		private string CompletionString;

		public EditorCommand(Gauntlet.UnrealTestContext InContext)
			: base(InContext)
		{
		}

		public override UnrealTestConfiguration GetConfiguration()
		{
			if (CachedConfig == null)
			{
				UnrealTestConfiguration Config = base.GetConfiguration();
				Config.ClearRoles();
				UnrealTestRole EditorRole = Config.RequireRole(Config.CookedEditor ? UnrealTargetRole.CookedEditor : UnrealTargetRole.Editor);

				// Take all the command line arguments, filter out those we know about that are not useful for the editor and pass the rest through.
				CompletionString = Globals.Params.ParseValue("CompletionString", string.Empty);
				List<string> ToFilterOut = new List<string>
				{
					"CompletionString=",
					"project=",
					"test=",
					"configuration=",
					"build=",
					"platform=",
					"MaxDuration="
				};
				IEnumerable<string> ParamList = null;
				ParamList = Globals.Params.AllArguments
								.Where(P => !ToFilterOut.Any(L => P.StartsWith(L, StringComparison.OrdinalIgnoreCase)));
				EditorRole.CommandLineParams.AddRawCommandline("-" + string.Join(" -", ParamList));

				CachedConfig = Config;
			}

			return CachedConfig;
		}

		protected override string GetCompletionString()
		{
			return CompletionString;
		}
	}
}