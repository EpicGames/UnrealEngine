// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.IO;
using System.Collections.Generic;
using System.Linq;
using AutomationTool;
using UnrealBuildTool;
using UnrealBuildBase;
using EpicGames.Core;

namespace Gauntlet
{

	namespace UnrealTest
	{
		/// <summary>
		/// Test runner for excuting Unreal Tests in Gauntlet
		/// </summary>
		public class RunUnrealTests : RunUnreal
		{
			public override string DefaultTestName { get { return "EngineTest"; } }

			public override ExitCode Execute()
			{
				Globals.Params = new Gauntlet.Params(this.Params);

				UnrealTestOptions ContextOptions = new UnrealTestOptions();

				AutoParam.ApplyParamsAndDefaults(ContextOptions, Globals.Params.AllArguments);

				if (string.IsNullOrEmpty(ContextOptions.Project))
				{
					throw new AutomationException("No project specified. Use -project=EngineTest etc");
				}

				ContextOptions.Namespaces = "Gauntlet.UnrealTest,UnrealGame,UnrealEditor";
				ContextOptions.UsesSharedBuildType = true;

				return RunTests(ContextOptions);
			}
		}

		public class EngineTest : EngineTestBase<EngineTestConfig>
		{
			public EngineTest(UnrealTestContext InContext) : base(InContext)
			{
			}
		}

		/// <summary>
		/// Runs automated tests on a platform
		/// </summary>
		public abstract class EngineTestBase<TConfigClass> : UnrealTestNode<TConfigClass>
		where TConfigClass : EngineTestConfig, new()
		{
			private UnrealLogStreamParser LogParser = null;

			private DateTime LastAutomationEntryTime = DateTime.MinValue;

			public EngineTestBase(Gauntlet.UnrealTestContext InContext)
				: base(InContext)
			{
				LogWarningsAndErrorsAfterSummary = false;
			}

			/// <summary>
			/// Returns Errors found during tests. By default only fatal errors are considered
			/// </summary>
			public override IEnumerable<string> GetErrors()
			{
				List<string> AllErrors = base.GetErrors().ToList();

				foreach (var Role in GetRolesThatFailed())
				{
					if (Role.Artifacts.SessionRole.RoleType == UnrealTargetRole.Editor)
					{
						AutomationLogParser Parser = new AutomationLogParser(Role.LogSummary);
						AllErrors.AddRange(
							Parser.GetResults().Where(R => R.HasFailed)
							.SelectMany(R => R.Entries
								.Where(E => E.Event.Type == EventType.Error).Select(E => string.Format("[test={0}] {1}", R.TestDisplayName, E))
								)
							);
					}
				}

				return AllErrors;
			}

			/// <summary>
			/// Define the configuration for this test. Most options are applied by the test config above
			/// </summary>
			/// <returns></returns>
			public override TConfigClass GetConfiguration()
			{
				TConfigClass Config = base.GetConfiguration();

				if (Config.UseEditor)
				{
					Config.RequireRole(UnrealTargetRole.Editor);
				}
				else
				{
					Config.RequireRole(UnrealTargetRole.Editor);
					Config.RequireRole(UnrealTargetRole.Client);
				}

				Config.MaxDuration = Context.TestParams.ParseValue("MaxDuration", 3600);

				return Config;
			}

			/// <summary>
			/// Override our name to include the filter we're testing
			/// </summary>
			public override string Name
			{
				get
				{
					string BaseName = base.Name;

					var Config = GetConfiguration();

					if (!string.IsNullOrEmpty(Config.TestFilter))
					{
						BaseName += " " + Config.TestFilter;
					}

					return BaseName;
				}
			}

			/// <summary>
			/// Override TickTest to log interesting things and make sure nothing has stalled
			/// </summary>
			public override void TickTest()
			{
				const float IdleTimeout = 30 * 60;

				if (LogParser == null)
				{
					LogParser = new UnrealLogStreamParser(TestInstance.EditorApp.GetLogBufferReader());
				}
				LogParser.ReadStream();
				IEnumerable<string> ChannelEntries = LogParser.GetLogFromEditorBusyChannels();
				if (ChannelEntries.Any())
				{
					// log new entries so people have something to look at
					ChannelEntries.ToList().ForEach(S => Log.Info("{0}", S));
					LastAutomationEntryTime = DateTime.Now;
				}
				else
				{
					if (LastAutomationEntryTime == DateTime.MinValue)
					{
						LastAutomationEntryTime = DateTime.Now;
					}

					double ElapsedTime = (DateTime.Now - LastAutomationEntryTime).TotalSeconds;

					// Check for timeout
					if (ElapsedTime > IdleTimeout)
					{
						Log.Error(KnownLogEvents.Gauntlet_TestEvent, "No activity observed in last {0:0.00} minutes. Aborting test", IdleTimeout / 60);
						MarkTestComplete();
						SetUnrealTestResult(TestResult.TimedOut);
					}
				}

				base.TickTest();
			}

			/// <summary>
			/// Optional function that is called on test completion and gives an opportunity to create a report
			/// </summary>
			/// <param name="Result"></param>
			/// <returns>ITestReport</returns>
			public override ITestReport CreateReport(TestResult Result)
			{
				// Save test result data for Horde build system
				bool WriteTestResultsForHorde = GetConfiguration().WriteTestResultsForHorde;
				if (WriteTestResultsForHorde)
				{
					if (GetConfiguration().SimpleHordeReport)
					{
						return base.CreateReport(Result);
					}
					else
					{
						string ReportPath = GetConfiguration().ReportExportPath;
						if (!string.IsNullOrEmpty(ReportPath))
						{
							return CreateUnrealEngineTestPassReport(ReportPath, GetConfiguration().ReportURL);
						}
					}
				}

				return null;
			}

			/// <summary>
			/// Override GetExitCodeAndReason to provide additional checking of success / failure based on what occurred
			/// </summary>
			/// <param name="InReason"></param>
			/// <param name="InLog"></param>
			/// <param name="InArtifacts"></param>
			/// <param name="ExitReason"></param>
			/// <param name="ExitCode"></param>
			/// <returns></returns>
			protected override UnrealProcessResult GetExitCodeAndReason(StopReason InReason, UnrealLog InLog, UnrealRoleArtifacts InArtifacts, out string ExitReason, out int ExitCode)
			{
				UnrealProcessResult UnrealResult = base.GetExitCodeAndReason(InReason, InLog, InArtifacts, out ExitReason, out ExitCode);

				// The editor is an additional arbiter of success
				if (InArtifacts.SessionRole.RoleType == UnrealTargetRole.Editor
					&& InLog.HasAbnormalExit == false)
				{
					// if no fatal errors, check test results
					if (InLog.FatalError == null)
					{
						AutomationLogParser Parser = new AutomationLogParser(InLog);

						IEnumerable<UnrealAutomatedTestResult> TotalTests = Parser.GetResults();
						IEnumerable<UnrealAutomatedTestResult> FailedTests = TotalTests.Where(R => R.HasFailed);

						// Tests failed so list that as our primary cause of failure
						if (FailedTests.Any())
						{
							ExitReason = string.Format("{0} of {1} test(s) failed", FailedTests.Count(), TotalTests.Count());
							ExitCode = -1;
							return UnrealProcessResult.TestFailure;
						}

						// If no tests were run then that's a failure (possibly a bad RunTest argument?)
						if (!TotalTests.Any())
						{
							ExitReason = "No tests were executed!";
							ExitCode = -1;
							return UnrealProcessResult.TestFailure;
						}
					}
				}

				return UnrealResult;
			}

			/// <summary>
			/// Override the summary report so we can insert a link to our report and the failed tests
			/// </summary>
			/// <returns></returns>
			protected override void LogTestSummaryHeader()
			{
				base.LogTestSummaryHeader();

				var EditorRole = RoleResults.Where(R => R.Artifacts.SessionRole.RoleType == UnrealTargetRole.Editor).FirstOrDefault();

				if (EditorRole != null)
				{
					AutomationLogParser Parser = new AutomationLogParser(EditorRole.LogSummary);

					IEnumerable<UnrealAutomatedTestResult> AllTests = Parser.GetResults();
					IEnumerable<UnrealAutomatedTestResult> FailedTests = AllTests.Where(R => R.IsComplete && R.HasFailed);
					IEnumerable<UnrealAutomatedTestResult> IncompleteTests = AllTests.Where(R => !R.IsComplete);

					if (AllTests.Count() == 0)
					{
						bool bAllowZeroTestResults = Globals.Params.ParseParam("AllowZeroTestResults");

						if (bAllowZeroTestResults)
						{
							Log.Warning(KnownLogEvents.Gauntlet_UnrealEngineTestEvent, " * No tests were executed, check test filter!");
						}
						else
						{
							Log.Error(KnownLogEvents.Gauntlet_UnrealEngineTestEvent, " * No tests were executed!");
						}
					}
					else
					{
						int PassedTests = AllTests.Count() - (FailedTests.Count() + IncompleteTests.Count());
						Log.Info(" * {0} of {1} tests passed", PassedTests, AllTests.Count());

						if (FailedTests.Count() > 0)
						{
							Log.Info(" ### The following tests failed:");

							foreach (UnrealAutomatedTestResult Result in FailedTests)
							{
								Log.Error(KnownLogEvents.Gauntlet_UnrealEngineTestEvent, " * {0} failed", Result.FullTestPath);
								foreach (var Event in Result.ErrorEvents)
								{
									Log.Error(KnownLogEvents.Gauntlet_UnrealEngineTestEvent, "    "+Event.Message);
								}
							}
						}

						if (IncompleteTests.Count() > 0)
						{
							Log.Info(" ### The following tests timed out:");

							foreach (UnrealAutomatedTestResult Result in IncompleteTests)
							{
								Log.Error(KnownLogEvents.Gauntlet_UnrealEngineTestEvent, " * {0} timed out", Result.FullTestPath);
								foreach (var Event in Result.ErrorEvents)
								{
									Log.Error(KnownLogEvents.Gauntlet_UnrealEngineTestEvent, "    " + Event.Message);
								}
							}
						}

						string ReportLink = GetConfiguration().ReportURL;
						string ReportPath = GetConfiguration().ReportExportPath;

						if (!string.IsNullOrEmpty(ReportLink) || !string.IsNullOrEmpty(ReportPath))
						{
							Log.Info(" ### Links");

							if (string.IsNullOrEmpty(ReportLink) == false)
							{
								Log.Info("  View results here: {URL}", ReportLink);
							}

							if (string.IsNullOrEmpty(ReportPath) == false)
							{
								Log.Info("  Open results in UnrealEd from {Path}", ReportPath);
							}
						}


					}
				}
			}
		}

		/// <summary>
		/// Define a Config class for this test that contains the available options
		/// </summary>
		public class EngineTestConfig : UE.AutomationTestConfig
		{
			/// <summary>
			/// Set to true if the editor executes all these tests in is own process and PIE
			/// </summary>
			[AutoParam]
			public virtual bool UseEditor { get; set; } = false;

			/// <summary>
			/// Filter or groups of tests to apply (alias for RunTest) 
			/// </summary>
			[AutoParam]
			public virtual string TestFilter { get; set; } = "";

			/// <summary>
			/// Apply this config to the role that is passed in
			/// </summary>
			/// <param name="AppConfig"></param>
			/// <param name="ConfigRole"></param>
			/// <param name="OtherRoles"></param>
			public override void ApplyToConfig(UnrealAppConfig AppConfig, UnrealSessionRole ConfigRole, IEnumerable<UnrealSessionRole> OtherRoles)
			{
				if (string.IsNullOrEmpty(RunTest))
				{
					RunTest = TestFilter;
				}

				base.ApplyToConfig(AppConfig, ConfigRole, OtherRoles);
			}
		}
	}
}
