// Copyright Epic Games, Inc. All Rights Reserved.

using AutomationTool;
using Gauntlet;
using System;
using System.Collections.Generic;
using System.Data;
using System.Diagnostics;
using System.IO;
using System.Linq;
using System.Net;
using System.Net.NetworkInformation;
using System.Runtime.InteropServices;
using System.Security.Principal;
using UnrealBuildBase;
using UnrealBuildTool;

namespace EpicGame
{

	/// <summary>
	/// An additional set of options that pertain to internal epic games.
	/// </summary>
	public class EpicGameTestConfig : UnrealGame.UnrealTestConfig, IAutoParamNotifiable
	{
		/// <summary>
		/// Should this test skip mcp?
		/// </summary>
		[AutoParam]
		public bool NoMCP = false;

		/// <summary>
		/// Tell the server not to authenticate u
		/// </summary>
		[AutoParam]
		public bool DeviceAuthSkip = false;

		[AutoParam]
		public bool FastCook = false;

		/// <summary>
		/// Which backend to use for matchmaking
		/// </summary>
		[AutoParam]
		public string EpicApp = "DevLatest";

		/// <summary>
		/// Unique buildid to avoid matchmaking collisions
		/// </summary>
		[AutoParam]
		protected string BuildIDOverride = "";

		/// <summary>
		/// Initial value for a unique server port to avoid matchmaking collisions
		/// </summary>
		[AutoParam]
		int ServerPortStart = 7777;

		/// <summary>
		/// Unique server port for the current Config object
		/// </summary>
		protected int ServerPort { get; private set; }

		/// <summary>
		/// Initial value for a unique server beacon port to avoid matchmaking collisions
		/// </summary>
		[AutoParam]
		int BeaconPortStart = 15000;

		/// <summary>
		/// Unique beacon port for the current Config object
		/// </summary>
		protected int BeaconPort { get; private set; }
		
		/// <summary>
		/// Make sure the client gets -logpso when we are collecting them
		/// </summary>
		[AutoParam]
        public bool LogPSO = false;

		/// <summary>
		/// Which Mempro tags we want to track if we need them. Note, should only be used in short runs.
		/// </summary>
		[AutoParam]
		public string MemPro;


		/// <summary>
		/// Should this test assign a random test account?
		/// </summary>
		[AutoParam]
		public bool PreAssignAccount = true;

		/// <summary>
		/// Does the current test require a user to be logged in to function correctly?
		/// </summary>
		[AutoParam]
		public bool RequiresLogin = false;

		/// <summary>
		/// If true, do not apply args to client and server as if they were running on the same host
		/// </summary>
		[AutoParam]
		public bool RemoteServer = false;

		/// <summary>
		/// Optional specifier for client mcp region
		/// </summary>
		[AutoParam]
		public string McpRegion = string.Empty;

		/// <summary>
		/// Optional specifier for client mcp subregion
		/// </summary>
		[AutoParam]
		public string McpSubRegion = string.Empty;

		/// <summary>
		/// Comma-separated list of Unreal Insights trace channels to collect for Editor roles, saved to a file named EditorTrace{ID#}.utrace
		/// </summary>
		[AutoParam]
		public string EditorTraces { get; set; }

		/// <summary>
		/// Comma-separated list of Unreal Insights trace channels to collect for Client roles, saved to a file named ClientTrace{ID#}.utrace
		/// </summary>
		[AutoParam]
		public string ClientTraces { get; set; }

		/// <summary>
		/// Comma-separated list of Unreal Insights trace channels to collect for Server roles, saved to a file named ServerTrace{ID#}.utrace
		/// </summary>
		[AutoParam]
		public string ServerTraces { get; set; }

		/// <summary>
		/// Enables Windows Performance Recorder on the local machine, saved to a file named WinPerfReportTrace{ID#}.etl
		/// Can produce around 1 GB/minute, avoid for long-running tests.
		/// Requires running with Admin privileges.
		/// </summary>
		[AutoParam]
		public bool WprTraces = false;

		/// <summary>
		/// Filepaths to trace logging artifacts, copied after test execution ends
		/// </summary>
		public readonly List<string> TraceLogPaths = [];

		// Allows a more derived test config to take over mcp setup.
		protected bool SkipMcpSetup = false;

		// incrementing value to ensure we can assign unique values to ports etc
		private readonly int CurrentConfigIndex;
		static private int NumberOfConfigsCreated = 0;
		static private int NumberOfEditorsConfigured = 0;
		static private int NumberOfClientsConfigured = 0;
		static private int NumberOfServersConfigured = 0;

		public static readonly string AutomationToolLogsDir = Path.Combine(Unreal.EngineDirectory.FullName, "Programs", "AutomationTool", "Saved", "Logs");

		public EpicGameTestConfig()
		{
			CurrentConfigIndex = NumberOfConfigsCreated;
			NumberOfConfigsCreated++;
		}

		public void ParametersWereApplied(string[] Params)
		{
			
			if (string.IsNullOrEmpty(BuildIDOverride))
			{
				// pick a default buildid that's the last 4 digits of our IP
				string LocalIP = Dns.GetHostEntry(Dns.GetHostName()).AddressList.Where(o => o.AddressFamily == System.Net.Sockets.AddressFamily.InterNetwork).First().ToString();
				LocalIP = LocalIP.Replace(".", "");
				BuildIDOverride = LocalIP.Substring(LocalIP.Length - 4);
			}

			ServerPort = ServerPortStart;
			BeaconPort = BeaconPortStart;

			// techinically this doesn't matter for mcp because the server will pick a free port and tell the backend what its using, but
			// nomcp requires us to know the port and thus we need to make sure ones we pick haven't been previously assigned or grabbed
			if (NumberOfConfigsCreated > 1)
			{
				BuildIDOverride += string.Format("{0}", NumberOfConfigsCreated);
				ServerPort = (ServerPortStart + NumberOfConfigsCreated);
				BeaconPort = (BeaconPortStart + NumberOfConfigsCreated);
			}
		}

		public override void ApplyToConfig(UnrealAppConfig AppConfig, UnrealSessionRole ConfigRole, IEnumerable<UnrealSessionRole> OtherRoles)
		{
			base.ApplyToConfig(AppConfig, ConfigRole, OtherRoles);

			if (!SkipMcpSetup && (ConfigRole.RoleType.IsClient() || ConfigRole.RoleType.IsServer() || RequiresLogin))
			{
				string McpString = "";
				bool bIsBuildMachine = CommandUtils.IsBuildMachine; 

				if (ConfigRole.RoleType.IsServer() && !RemoteServer)
				{
					// set explicit server and beacon port for online services
					// this is important when running tests in parallel to avoid matchmaking collisions
					McpString += string.Format(" -port={0}", ServerPort);
					McpString += string.Format(" -beaconport={0}", BeaconPort);

					AppConfig.CommandLine += " -net.forcecompatible";

					if (!bIsBuildMachine || !NoMCP)
					{
						AppConfig.CommandLineParams.Add("UseLocalIPs");
					}
				}

				// Default to the first address with a valid prefix
				var LocalAddress = Dns.GetHostEntry(Dns.GetHostName()).AddressList
					.Where(o => o.AddressFamily == System.Net.Sockets.AddressFamily.InterNetwork
						&& o.GetAddressBytes()[0] != 169)
					.FirstOrDefault();

				var ActiveInterfaces = NetworkInterface.GetAllNetworkInterfaces()
					.Where(I => I.OperationalStatus == OperationalStatus.Up);

				bool MultipleInterfaces = ActiveInterfaces.Count() > 1;

				if (MultipleInterfaces)
				{
					// Now, lots of Epic PCs have virtual adapters etc, so see if there's one that's on our network and if so use that IP
					var PreferredInterface = ActiveInterfaces
						.Where(I => I.GetIPProperties().DnsSuffix.Equals("epicgames.net", StringComparison.OrdinalIgnoreCase))
						.SelectMany(I => I.GetIPProperties().UnicastAddresses)
						.Where(A => A.Address.AddressFamily == System.Net.Sockets.AddressFamily.InterNetwork)
						.FirstOrDefault();

					if (PreferredInterface != null)
					{
						LocalAddress = PreferredInterface.Address;
					}
				}

				if (LocalAddress == null)
				{
					throw new AutomationException("Could not find local IP address");
				}

				string RequestedServerIP = Globals.Params.ParseValue("serverip", "");
				string RequestedClientIP = Globals.Params.ParseValue("clientip", "");
				string ServerIP = string.IsNullOrEmpty(RequestedServerIP) ? LocalAddress.ToString() : RequestedServerIP;
				string ClientIP = string.IsNullOrEmpty(RequestedClientIP) ? LocalAddress.ToString() : RequestedClientIP;


				// Do we need to add the -multihome argument to bind to specific IP?
				if (ConfigRole.RoleType.IsServer() && !RemoteServer && (MultipleInterfaces || !string.IsNullOrEmpty(RequestedServerIP)))
				{
					AppConfig.CommandLine += string.Format(" -multihome={0} -multihomehttp={0}", ServerIP);
				}

				// client too, but only desktop platforms
				if (ConfigRole.RoleType.IsClient() && !RemoteServer && (MultipleInterfaces || !string.IsNullOrEmpty(RequestedClientIP)))
				{
					if (ConfigRole.Platform == UnrealTargetPlatform.Win64 || ConfigRole.Platform == UnrealTargetPlatform.Mac)
					{
						AppConfig.CommandLine += string.Format(" -multihome={0} -multihomehttp={0}", ClientIP);
					}
				}

				if (NoMCP)
				{
					if(RemoteServer)
					{
						throw new AutomationException("Attempted to use a remote server when running a test with nomcp, this is not supported.");
					}

					McpString += " -nomcp -notimeouts";

					// if this is a client, and there is a server role, find our PC's IP address and tell it to connect to us
					if (ConfigRole.RoleType.IsClient() &&
							(RequiredRoles.ContainsKey(UnrealTargetRole.Server) || RequiredRoles.ContainsKey(UnrealTargetRole.EditorServer)))
					{
						McpString += string.Format(" -ExecCmds=\"open {0}:{1}\"", ServerIP, ServerPort);
					}
				}
				else
				{
					if (Globals.Params.ParseParam("nobuildid"))
					{
						McpString += string.Format(" -epicapp={0} ", EpicApp);
					}
					else
					{
						McpString += string.Format(" -epicapp={0} -buildidoverride={1}", EpicApp, BuildIDOverride);
					}

					bool bRequestRegion = !string.IsNullOrEmpty(McpRegion);
					if (ConfigRole.RoleType.IsClient() && bRequestRegion)
					{
						McpString += string.Format(" -McpRegion={0}", McpRegion);

						if (bRequestRegion && !string.IsNullOrEmpty(McpSubRegion))
						{
							McpString += string.Format(" -McpSubRegion={0}", McpSubRegion);
						}
					}
				}

				if (FastCook)
				{
					McpString += " -FastCook";
				}

				AppConfig.CommandLine += McpString;
			}

			if (ConfigRole.RoleType.IsClient() || RequiresLogin)
			{
				bool bNoAccountOverride = DataDrivenPlatformInfo.GetDataDrivenInfoForPlatform(ConfigRole.Platform.ToString())?.bNoAccountOverride ?? false;
				// select an account
				if (!NoMCP)
				{
					if (!bNoAccountOverride && PreAssignAccount)
					{
						Account UserAccount = AccountPool.Instance.ReserveAccount();
						UserAccount.ApplyToConfig(AppConfig);
					}
				}
			}

			if (ConfigRole.RoleType.IsClient())
			{
                if (LogPSO)
                {
                    AppConfig.CommandLine += " -logpso";
                }

				if (!string.IsNullOrEmpty(MemPro))
				{
					AppConfig.CommandLineParams.AddOrAppendParamValue("memprotags", MemPro);
					AppConfig.CommandLineParams.AddUnique("mempro");
					AppConfig.CommandLineParams.AddUnique("llm");
					AppConfig.CommandLineParams.AddUnique("llmcsv");
					AppConfig.CommandLineParams.AddUnique("nothreadtimeout");
				}

                if (ConfigRole.Platform == UnrealTargetPlatform.Win64)
				{
					// turn off skill-based matchmaking, turn off porta;
					AppConfig.CommandLine += " -noepicportal";
				}

				// turn off crashlytics so we get symbolicated tombstone crashes on Android
				if (ConfigRole.Platform == UnrealTargetPlatform.Android)
				{
					AppConfig.CommandLine += " -nocrashlytics";
				}

				if (!string.IsNullOrEmpty(ClientTraces))
				{
					AppConfig.CommandLine += $" -trace={ClientTraces} -tracefile={SetupInsightsTraceFile(ConfigRole.RoleType, NumberOfClientsConfigured)} -traceautostart=1";
				}

				++NumberOfClientsConfigured;
			}

			if (ConfigRole.RoleType.IsServer())
			{
				if (!string.IsNullOrEmpty(ServerTraces))
				{
					AppConfig.CommandLine += $" -trace={ServerTraces} -tracefile={SetupInsightsTraceFile(ConfigRole.RoleType, NumberOfServersConfigured)} -traceautostart=1 -nofakeforking";
				}

				++NumberOfServersConfigured;
			}

			if (ConfigRole.RoleType.IsEditor())
			{
				if (!string.IsNullOrEmpty(EditorTraces))
				{
					AppConfig.CommandLine += $" -trace={EditorTraces} -tracefile={SetupInsightsTraceFile(ConfigRole.RoleType, NumberOfEditorsConfigured)} -traceautostart=1";
				}

				++NumberOfEditorsConfigured;
			}
		}

		private string SetupInsightsTraceFile(UnrealTargetRole RoleType, int RoleCount)
		{
			string GeneratedTraceFile = Path.Combine(AutomationToolLogsDir, $"{RoleType}Trace_{RoleCount}.utrace");
			TraceLogPaths.Add(GeneratedTraceFile);
			if (File.Exists(GeneratedTraceFile))
			{
				// unexpected: the Logs directory is cleared each time UAT executes
				Log.Warning($"Removing previous Unreal Insights capture at {GeneratedTraceFile}");
				try
				{
					File.Delete(GeneratedTraceFile);
				}
				catch(Exception ex)
				{
					Log.Warning($"Unable to delete previous Unreal Insights capture, a new result from this run will not be captured. Failure reason: {ex}");
				}
			}
			return GeneratedTraceFile;
		}

		/// <summary>
		/// Path to generate Windows Performance Recorder artifacts
		/// </summary>
		public string CurrentConfigWprPath() => Path.Join(EpicGameTestConfig.AutomationToolLogsDir, $"WinPerfReportTrace{CurrentConfigIndex}.etl");
	}

	/// <summary>
	/// Generic TestNode class for Epic Games internal projects.
	/// </summary>
	public abstract class EpicGameTestNode<TConfigClass> : UnrealTestNode<TConfigClass>
		where TConfigClass : EpicGameTestConfig, new()
	{
		public EpicGameTestNode(UnrealTestContext InContext) : base(InContext)
		{

		}

		private static bool IsWindowsAdmin()
		{
			if (RuntimeInformation.IsOSPlatform(OSPlatform.Windows))
			{
				using var Identity = WindowsIdentity.GetCurrent();
				return new WindowsPrincipal(Identity).IsInRole(WindowsBuiltInRole.Administrator);
			}
			return false;
		}

		public override bool StartTest(int Pass, int InNumPasses)
		{
			if(CachedConfig.WprTraces)
			{
				StartWprCapture();
			}
			return base.StartTest(Pass, InNumPasses);
		}

		private void StartWprCapture()
		{
			if (!IsWindowsAdmin())
			{
				Log.Warning("Windows admin privilege is required for Windows Performance Recorder. Capture not enabled.");
			}
			else
			{
				SafeCancelWprCapture();
				Log.Info($"Starting capture from Windows Performance Recorder");
				// Light profiles limit artifact size, WPR profiles are verbose by default which can capture multiple gigabytes per minute
				var WprStartInfo = new ProcessStartInfo("wpr", "-start CPU.light -start FileIO.light -start DiskIO.light -filemode")
				{
					// prevent wpr from modifying our io streams to display secondary perf statistics
					UseShellExecute = false,
					RedirectStandardInput = true,
					RedirectStandardOutput = true,
					RedirectStandardError = true,
				};

				Globals.PostAbortHandlers.Add(() =>
				{
					Log.Warning("Cancelling Windows Performance Recorder capture, please wait while this operation completes");
					SafeCancelWprCapture();
				});

				Process WprProcess = Process.Start(WprStartInfo);
				if (!WprProcess.WaitForExit(TimeSpan.FromSeconds(10)))
				{
					// unexpected, should immediately return
					Log.Warning("Windows Performance Recorder unexpectedly failed to exit while starting trace collection, capture may be unavailable");
				}
				else
				{
					if (WprProcess.ExitCode != 0)
					{
						Log.Warning($"Windows Performance Recorder returned exit code '{WprProcess.ExitCode}' while starting trace collection, capture may be unavailable");
					}
				}
			}
		}

		private void SafeCancelWprCapture()
		{
			if (IsWindowsAdmin())
			{
				Log.Verbose("Cancelling WPR capture");
				var WprCancelArgs = new ProcessStartInfo("wpr", $"-cancel")
				{
					// prevent wpr from modifying our io streams to report progress
					UseShellExecute = false,
					RedirectStandardInput = true,
					RedirectStandardOutput = true,
					RedirectStandardError = true,
				};
				Process WprProcess = Process.Start(WprCancelArgs);
				if (!WprProcess.WaitForExit(TimeSpan.FromSeconds(10)))
				{
					// unexpected, should immediately return
					Log.Warning("Windows Performance Recorder unexpectedly failed to exit while cancelling trace collection");
				} // ignore exit code, cancel errors when there is no active capture
			}
		}

		private void EndWprCapture(string EtlOutputPath)
		{
			if (IsWindowsAdmin())
			{
				Log.Verbose($"Stopping WPR to output results into: {EtlOutputPath}");
				CachedConfig.TraceLogPaths.Add(EtlOutputPath);

				var EtlDescription = $"Gauntlet Test '{Name}'";
				var WprStopArgs = new ProcessStartInfo("wpr", $"-stop \"{EtlOutputPath}\" \"{EtlDescription}\" -skipPdbGen")
				{
					// prevent wpr from modifying our io streams to report progress
					UseShellExecute = false,
					RedirectStandardInput = true,
					RedirectStandardOutput = true,
					RedirectStandardError = true,
				};
				Process WprProcess = Process.Start(WprStopArgs);
				if (!WprProcess.WaitForExit(TimeSpan.FromMinutes(10))) // generation can take a few minutes
				{
					Log.Error("Windows Performance Recorder unexpectedly failed to exit while stopping trace collection");
				}
			}
		}

		public override IEnumerable<UnrealRoleArtifacts> SaveRoleArtifacts(string OutputPath)
		{
			if(CachedConfig.WprTraces)
			{
				EndWprCapture(CachedConfig.CurrentConfigWprPath());
			}
			
			string OutputFolderName = new DirectoryInfo(OutputPath).Name;
			foreach (string SourceTracePath in CachedConfig.TraceLogPaths)
			{
				if (!File.Exists(SourceTracePath))
				{
					Log.Warning($"Skipping save of missing trace artifact: {SourceTracePath}");
					continue;
				}

				string OutputTracePath = Path.Combine(OutputPath, Path.GetFileName(SourceTracePath));
				Log.Verbose($"Moving trace file to {OutputTracePath}");
				try
				{
					Directory.CreateDirectory(Path.GetDirectoryName(OutputTracePath));
					File.Move(SourceTracePath, OutputTracePath, true);
				}
				catch (Exception Ex)
				{
					Log.Warning($"Failed to move {SourceTracePath} to Role Artifacts. Cached file will be overwritten on the next invocation of Unreal Automation Tool. Exception: {Ex}");
				}
			}

			return base.SaveRoleArtifacts(OutputPath);
		}
	}

}