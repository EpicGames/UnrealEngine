// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Linq;
using System.Text;
using System.Threading.Tasks;
using Gauntlet;
using System.IO;
using System.IO.Compression;

namespace UnrealGame
{

	/// <summary>
	/// Runs automated tests on a platform
	/// </summary>
	public class ElementalDemoTest : DefaultTest
	{
		public ElementalDemoTest(Gauntlet.UnrealTestContext InContext)
			: base(InContext)
		{
		}

		public override UnrealTestConfig GetConfiguration()
		{
			// just need a single client
			UnrealTestConfig Config = base.GetConfiguration();

			UnrealTestRole ClientRole = Config.RequireRole(UnrealTargetRole.Client);
			ClientRole.CommandLine += " -unattended";
			Config.MaxDuration = 5 * 600;		// 5min should be plenty
			return Config;
		}

		public override ITestReport CreateReport(TestResult Result, UnrealTestContext Contex, UnrealBuildSource Build, IEnumerable<UnrealRoleResult> RoleResults, string ArtifactPath)
		{
			UnrealRoleArtifacts ClientArtifacts = RoleResults.Where(R => R.Artifacts.SessionRole.RoleType == UnrealTargetRole.Client).Select(R => R.Artifacts).FirstOrDefault();

			var SnapshotSummary = new UnrealSnapshotSummary<UnrealHealthSnapshot>(ClientArtifacts.AppInstance.GetLogReader());

			Log.Info("Elemental Performance Report");
			Log.Info(SnapshotSummary.ToString());

			return base.CreateReport(Result, Contex, Build, RoleResults, ArtifactPath);
		}

		public override void SaveArtifacts_DEPRECATED(string OutputPath)
		{
			string UploadFolder = Globals.Params.ParseValue("uploadfolder", "");
			if (UploadFolder.Count() > 0 && Directory.CreateDirectory(UploadFolder).Exists)
			{
				string PlatformString = TestInstance.ClientApps[0].Device.Platform.ToString();
				string ArtifactDir = TestInstance.ClientApps[0].ArtifactPath;
				string ProfilingDir = Path.Combine(ArtifactDir, "Profiling");
				string FPSChartsDir = Path.Combine(ProfilingDir, "FPSChartStats").ToLower();
				string FpsChartsZipPath = Path.Combine(TestInstance.ClientApps[0].ArtifactPath, "FPSCharts.zip").ToLower();
				if (Directory.Exists(FPSChartsDir))
				{
					ZipFile.CreateFromDirectory(FPSChartsDir, FpsChartsZipPath);
					string DestFileName = "ElementalDemoTest-" + PlatformString + ".zip";
					string DestZipFile = Path.Combine(UploadFolder, DestFileName);
					File.Copy(FpsChartsZipPath, DestZipFile);
				}
			}
			else
			{
				Log.Info("Not uploading CSV Result UploadFolder: '" + UploadFolder + "'");
			}
		}
	}
}
