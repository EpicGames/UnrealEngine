// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Linq;
using System.Text;
using System.Threading.Tasks;
using Gauntlet;

namespace UnrealGame
{
	/// <summary>
	/// Implements a default node that should boot most projects and supports n number of clients
	/// via -numclients=<n> and a server via -server
	/// </summary>
	public class DefaultTest : UnrealTestNode<UnrealTestConfig>
	{
		public DefaultTest(UnrealTestContext InContext) : base(InContext)
		{
		}

		public override UnrealTestConfig GetConfiguration()
		{
			// just need a single client
			UnrealTestConfig Config = base.GetConfiguration();

			int ClientCount = Context.TestParams.ParseValue("numclients", 1);
			bool WithServer = Context.TestParams.ParseParam("server");

			if (ClientCount > 0)
			{
				Config.RequireRoles(UnrealTargetRole.Client, ClientCount);
			}

			if (WithServer)
			{
				Config.RequireRoles(UnrealTargetRole.Server, 1);
			}

			return Config;
		}
	}
}
