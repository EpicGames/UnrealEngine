// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Linq;
using System.Text;
using System.Threading.Tasks;
using Gauntlet;

namespace UnrealGame
{
	/// <summary>
	/// Default set of options for testing "DefaultGame". Options that tests can configure
	/// should be public, external command-line driven options should be protected/private
	/// </summary>
	public class UnrealTestConfig : UnrealTestConfiguration
	{
		/// <summary>
		/// Applies these options to the provided app config
		/// </summary>
		/// <param name="AppConfig"></param>
		/// <param name="ConfigRole"></param>
		/// <param name="OtherRoles"></param>
		public override void ApplyToConfig(UnrealAppConfig AppConfig, UnrealSessionRole ConfigRole, IEnumerable<UnrealSessionRole> OtherRoles)
		{
			base.ApplyToConfig(AppConfig, ConfigRole, OtherRoles);
	
		}
	}
}
