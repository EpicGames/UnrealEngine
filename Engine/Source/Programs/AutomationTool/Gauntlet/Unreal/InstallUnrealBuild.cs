// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using AutomationTool;
using UnrealBuildBase;


namespace Gauntlet
{
	/// <summary>
	/// Main class for Unreal Installer script
	/// </summary>
	[Help("Install Unreal Build onto one or more connected devices using Gauntlet")]
	[ParamHelp("platform=<platform>", "Platform to use")]
	[ParamHelp("path=<path>", "Path to a build to install")]
	[ParamHelp("device=<device>", "Device to install a build on")]
	[ParamHelp("devices=<devices>", "List of devices, separated with commas, to install a build on")]
	[ParamHelp("commandline=\"-arg1 -arg2 -etc\"", "Commandline to write to UECommandLine.txt on the device")]
	[ParamHelp("parallel=[1:4]", "Number of installs to run in parallel",
		ParamType = typeof(string), Choices = new string[] { "1", "2", "3", "4" }, DefaultValue = "1")]
	[ParamHelp("project=<project>", "Name of the project", Required = true)]
	[ParamHelp("-sandbox=<sandbox>", "String to specficy the name of a staged install directory.")]

	public class InstallUnrealBuild : BuildCommand
	{

		/// <summary>
		/// Entrance point for Unreal Installer script
		/// </summary>
		/// <returns></returns>
		public override ExitCode Execute()
		{
			string PlatformParam = ParseParamValue("platform", string.Empty);

			string BuildPath = ParseParamValue("build", string.Empty);
			BuildPath = ParseParamValue("path", BuildPath);

			string ProjectName = ParseParamValue("project", string.Empty);
			string CommandLine = ParseParamValue("commandline", string.Empty);
			string DevicesArg = ParseParamValue("device", string.Empty);
			DevicesArg = ParseParamValue("devices", DevicesArg);

			//Set default to -1, to know that a value has not been passed in and can be overriden.
			int ParallelTasks = Convert.ToInt32(ParseParamValue("parallel", "-1"));

			bool Success = false;

			try
			{
				Success = InstallUnreal.RunInstall(PlatformParam, BuildPath, ProjectName, CommandLine, DevicesArg, ParallelTasks);
			}
			catch (Exception Ex)
			{
				Log.Error("Failed to install build from {Path} to {Device}. {Exception}", BuildPath, DevicesArg, Ex.Message);
			}

			return Success ? ExitCode.Success : ExitCode.Error_Unknown;
		}
	}
}
