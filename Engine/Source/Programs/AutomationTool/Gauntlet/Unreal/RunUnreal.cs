// Copyright Epic Games, Inc. All Rights Reserved.

using AutomationTool;
using AutomationTool.DeviceReservation;
using EpicGames.Core;
using Microsoft.Extensions.Logging;
using System;
using System.Collections.Generic;
using System.IO;
using System.Linq;
using System.Text;
using UnrealBuildBase;
using UnrealBuildTool;

namespace Gauntlet
{
	/*
	 	public class RunUnreal<SourceType, OptionType> : BuildCommand
		where SourceType : UnrealBuildSource,
		where OptionType : UnrealTestOptions
	 */

	/// <summary>
	/// Base class for executing Unreal tests.
	/// For a full list of options see UnrealTestContextOption
	/// </summary>
	[Help("Run Unreal tests using Gauntlet")]
	[ParamHelp("Tests", "List of gauntlet tests to run", Required = true, MultiSelectSeparator = ",")]
	[ParamHelp("ExecCmds", "List commands to execute. NOTE: To provide ExecCmds to specific roles, use -EditorExecCmds, -ClientExecCmds, etc.", MultiSelectSeparator = "+")]
	[ParamHelp("Build", "Reference to the build that is being tested")]
	[ParamHelp("Configuration", "Configuration to perform tests on", Choices = new string[] { "Debug", "DebugGame", "Development", "Test", "Shipping"})]
	[ParamHelp("Platform", "Platforms to perform tests on and their params")]
	[ParamHelp("Packaged", "Run packaged build instead of staged", ParamType = typeof(bool))]
	[ParamHelp("Dev", "Run in Dev mode", ParamType = typeof(bool))]
	[ParamHelp("p4", "Enable p4v support", ParamType = typeof(bool))]
	[ParamHelp("MaxDuration", "Maximum duration for test in sections", ParamType = typeof(int), DefaultValue = 3600)]
	[ParamHelp("NoTimeout", "No maximum timeout", ParamType = typeof(bool))]
	[ParamHelp("TestIterations", "Number of iterations to repeat this test", ParamType = typeof(int), DefaultValue = 1)]
	[ParamHelp("CookedEditor", "Restricts usage of uncooked editor role", ParamType = typeof(bool))]
	[ParamHelp("CookedEditorFlavor", "Specifies RequiredFlavor for the cooked editor role", ParamType = typeof(string))]
	[ParamHelp("Devices", "List of devices to use for tests using the syntax: <platform>:<address>,<platform>:<address>,...", Action = ParamHelpAttribute.ParamAction.Append)]
	[ParamHelp("DeviceName", "Device name to constrain reservation", Action = ParamHelpAttribute.ParamAction.Append)]
	[ParamHelp("PrefSpec", "Device perfomance specification to constrain reservation (Minimum,Recommended,High)", Action = ParamHelpAttribute.ParamAction.Append)]
	[ParamHelp("PrefModel", "Device model to constrain reservation", Action = ParamHelpAttribute.ParamAction.Append)]
	[ParamHelp("NumClients", "Number of clients to run test with", ParamType = typeof(int), DefaultValue = 1)]
	[ParamHelp("Server", "Run test with server", ParamType = typeof(bool))]
	[ParamHelp("NullRHI", "Null Rendering Hardware Interface (run headless)", ParamType = typeof(bool))]
	[ParamHelp("Windowed", "Run in Windowed mode", ParamType = typeof(bool))]
	[ParamHelp("ResX", "Horizontal resolution", ParamType = typeof(int), DefaultValue = 1920)]
	[ParamHelp("ResY", "Vertical resolution", ParamType = typeof(int), DefaultValue = 1080)]
	[ParamHelp("Unattended", "Run in Unattended mode", ParamType = typeof(bool))]
	[ParamHelp("Reboot", "Reboot device before starting test", ParamType = typeof(bool))]
	[ParamHelp("SkipDeploy", "Skip deployment of build packages to devices", ParamType = typeof(bool))]
	[ParamHelp("Log", "Output Logs", ParamType = typeof(bool))]
	[ParamHelp("LogDir", "Location to store log files. Defaults to TempDir/Logs")]
	[ParamHelp("Timestamp", "Print timestamp prefix to Log lines", ParamType = typeof(bool))]
	[ParamHelp("TempDir", "Location to store temporary files. Defaults to GauntletTemp")]
	[ParamHelp("Verbose", "Verbose logging", ParamType = typeof(bool))]
	[ParamHelp("VeryVerbose", "Very Verbose logging", ParamType = typeof(bool))]
	[ParamHelp("HeartbeatPeriod", "Gauntlet heartbeat period", ParamType = typeof(float))]
	[ParamHelp("Args", "Extra arguments to pass to all role(s)")]
	[ParamHelp("ClientArgs", "Extra arguments passed to client role(s)")]
	[ParamHelp("ServerArgs", "Extra arguments passed to server role(s)")]
	[ParamHelp("EditorArgs", "Extra arguments passed to editor role(s)")]
	[ParamHelp("CookedEditorArgs", "Extra arguments passed to cooked editor role(s)")]
	[ParamHelp("EditorGameArgs", "Extra arguments passed to editor(s) when acting in the game role")]
	[ParamHelp("EditorServerArgs", "Extra arguments passed to editor(s) when acting in the server role")]
	[ParamHelp("Namespaces", "Comma-separated list of namespaces to check for tests.", MultiSelectSeparator = ",")]
	[ParamHelp("AdditionalArgs", "Any additional arguments to pass directly to the Gauntlet command line", IsArgument = true)]
	public class RunUnreal : BuildCommand
	{
		/// <summary>
		/// Test node to create if none were specified
		/// </summary>
		public virtual string DefaultTestName { get { return "DefaultTest"; } }

		/// <summary>
		/// Constant for environment variable that can be set to trigger the cleanup of Options.TempDir upon completion
		/// </summary>
		private const string CleanEnvironmentVariable = "UE_GauntletCleanTempDir";

		/// <summary>
		/// Main UAT entrance point. Custom games can derive from RunUnrealTests to run custom setup steps or
		/// directly set params on ContextOptions to remove the need for certain command line params (e.g.
		/// -project=FooGame
		/// </summary>
		/// <returns></returns>
		public override ExitCode Execute()
		{
			Globals.Params = new Gauntlet.Params(this.Params);
			Globals.BuildCommand = ToString().Split(".").Last();

			UnrealTestOptions ContextOptions = new UnrealTestOptions();

			AutoParam.ApplyParamsAndDefaults(ContextOptions, Globals.Params.AllArguments);

			if (string.IsNullOrEmpty(ContextOptions.Project))
			{
				throw new AutomationException("No project specified. Use -project=ShooterGame etc");
			}

			ContextOptions.Namespaces = "Gauntlet.UnrealTest,UnrealGame,UnrealEditor";
			ContextOptions.UsesSharedBuildType = true;

			return RunTests(ContextOptions);
		}

		/// <summary>
		/// Execute all tests according to the provided context
		/// </summary>
		/// <param name="ContextOptions"></param>
		/// <returns></returns>
		public virtual ExitCode RunTests(UnrealTestOptions ContextOptions)
		{
			if (ContextOptions.Verbose)
			{
				Gauntlet.Log.Level = Gauntlet.LogLevel.Verbose;
			}

			if (ContextOptions.VeryVerbose)
			{
				Gauntlet.Log.Level = Gauntlet.LogLevel.VeryVerbose;
			}

			// Use this flag if iterating in horde to get a test immediately declared invalid.
			if (ParseParam("forcestepfail"))
			{
				string ErrorMessage = "-forcestepfail specified on commandline.";
				Horde.ReportInvalidatingIssue(Horde.GauntletHordeStepMessage.TestStepMessage, ErrorMessage);
				throw new AutomationException(ErrorMessage);
			}
			if (ParseParam("log"))
			{
				if (!Directory.Exists(ContextOptions.LogDir))
				{
					Directory.CreateDirectory(ContextOptions.LogDir);
				}

				// include test names and timestamp in log filename as multiple (parallel or sequential) Gauntlet tests may be outputting to same directory
				string LogPath = Path.Combine(ContextOptions.LogDir, string.Format("GauntletLog{0}-{1}.txt", ContextOptions.TestList.Aggregate(new StringBuilder(), (SB, T) => SB.AppendFormat("-{0}", T.ToString())).ToString(), DateTime.Now.ToString(@"yyyy.MM.dd.HH.mm.ss")));
				Gauntlet.Log.Verbose("Writing Gauntlet log to {0}", LogPath);
				Gauntlet.Log.SaveToFile(LogPath);
			}

			// prune our temp folder
			Utils.SystemHelpers.CleanupMarkedDirectories(ContextOptions.TempDir, 7);

			if (string.IsNullOrEmpty(ContextOptions.Build))
			{
				string ErrorMessage = "No build specified. Use -build=p:\\path\\to\\build";
				Horde.ReportInvalidatingIssue(Horde.GauntletHordeStepMessage.BuildNotSpecified, ErrorMessage);
				throw new AutomationException(ErrorMessage);
			}

			if (typeof(UnrealBuildSource).IsAssignableFrom(ContextOptions.BuildSourceType) == false)
			{
				string ErrorMessage = "Provided BuildSource type does not inherit from UnrealBuildSource";
				Horde.ReportInvalidatingIssue(Horde.GauntletHordeStepMessage.InvalidBuildSource, ErrorMessage);
				throw new AutomationException(ErrorMessage);
			}

			// make -test=none implicit if no test is supplied

			if (ContextOptions.TestList.Count == 0)
			{
				Gauntlet.Log.Info("No test specified, using default '{0}'", DefaultTestName);
				ContextOptions.TestList.Add(TestRequest.CreateRequest(DefaultTestName));
			}

			bool EditorForAllRoles = Globals.Params.ParseParam("editor") || string.Equals(ContextOptions.Build, "editor", StringComparison.OrdinalIgnoreCase);

			if (EditorForAllRoles)
			{
				Gauntlet.Log.Verbose("Will use Editor for all roles");
			}

			Dictionary<UnrealTargetRole, UnrealTestRoleContext> RoleContexts = new Dictionary<UnrealTargetRole, UnrealTestRoleContext>();

			// Default platform to the current os
			UnrealTargetPlatform DefaultPlatform = BuildHostPlatform.Current.Platform;
			UnrealTargetConfiguration DefaultConfiguration = UnrealTargetConfiguration.Development;

			DirectoryReference UnrealPath = new DirectoryReference(!string.IsNullOrEmpty(ContextOptions.EditorDir) ? ContextOptions.EditorDir : Environment.CurrentDirectory);

			// todo, pass this in as a BuildSource and remove the ContextOption params specific to finding builds
			UnrealBuildSource BuildInfo = (UnrealBuildSource)Activator.CreateInstance(ContextOptions.BuildSourceType, new object[] { ContextOptions.Project, ContextOptions.ProjectPath, UnrealPath, ContextOptions.UsesSharedBuildType, ContextOptions.Build, ContextOptions.SearchPaths });

			// Tag the the job with metadata about the build 
			Horde.AddJobBuildMetaData(BuildInfo);

			// Setup accounts
			SetupAccounts();

			List<ITestNode> AllTestNodes = new List<ITestNode>();

			bool InitializedDevices = false;

			// for all platforms we want to test...
			foreach (ArgumentWithParams PlatformWithParams in ContextOptions.PlatformList)
			{
				string PlatformString = PlatformWithParams.Argument;

				// combine global and platform-specific params
				Params CombinedParams = new Params(ContextOptions.Params.AllArguments.Concat(PlatformWithParams.AllArguments).ToArray());

				UnrealTargetPlatform PlatformType = UnrealTargetPlatform.Parse(PlatformString);

				SetupPlatformConfigurationProfiles(PlatformType, ContextOptions);

				if (!InitializedDevices)
				{
					// Setup the devices and assign them to the executor
					SetupDevices(PlatformType, ContextOptions);
					InitializedDevices = true;
				}

				//  Create a context for each process type to operate as
				foreach (UnrealTargetRole Type in Enum.GetValues(typeof(UnrealTargetRole)))
				{
					UnrealTestRoleContext Role = new UnrealTestRoleContext();

					// Default to these
					Role.Type = Type;
					Role.Platform = DefaultPlatform;
					Role.Configuration = DefaultConfiguration;

					// globally, what was requested (e.g -platform=Win64 -configuration=Shipping)
					UnrealTargetPlatform RequestedPlatform = PlatformType;
					UnrealTargetConfiguration RequestedConfiguration = ContextOptions.Configuration;

					// look for FooConfiguration, FooPlatform overrides.
					// e.g. ServerConfiguration, ServerPlatform
					string PlatformRoleString = Globals.Params.ParseValue(Type.ToString() + "Platform", null);
					string ConfigString = Globals.Params.ParseValue(Type.ToString() + "Configuration", null);

					if (string.IsNullOrEmpty(PlatformRoleString) == false)
					{
						RequestedPlatform = UnrealTargetPlatform.Parse(PlatformRoleString);
						ContextOptions.RolePlatformOverrides[Type] = RequestedPlatform;
					}

					if (string.IsNullOrEmpty(ConfigString) == false)
					{
						RequestedConfiguration = (UnrealTargetConfiguration)Enum.Parse(typeof(UnrealTargetConfiguration), ConfigString, true);
					}

					// look for-args= and then -clientargs= and -editorargs etc
					List<string> ArgsParams = Globals.Params.ParseValues("Args", false /* bCommaSeparated */);
					List<string> RoleArgsParams = Globals.Params.ParseValues(Type.ToString() + "Args", false /* bCommaSeparated */);
					ArgsParams.AddRange(RoleArgsParams);
					Role.ExtraArgs = string.Join(' ', ArgsParams);

					// look for -clientexeccmds=, -editorexeccmds= etc, these are separate from clientargs for sanity
					string ExecCmds = Globals.Params.ParseValue("ExecCmds", "");
					string RoleExecCmds= Globals.Params.ParseValue(Type.ToString() + "ExecCmds", "");
					string TagFilter = Globals.Params.ParseValue("TagFilter", string.Empty);
					if (!string.IsNullOrEmpty(ExecCmds))
					{
						Role.ExtraArgs += string.Format(" -ExecCmds=\"{0}\"", ExecCmds);
					}

					if (!string.IsNullOrEmpty(RoleExecCmds))
					{
						Role.ExtraArgs += string.Format(" -ExecCmds=\"{0}\"", RoleExecCmds);
					}

					bool UsesEditor = EditorForAllRoles || Globals.Params.ParseParam("Editor" + Type.ToString());

					if (UsesEditor)
					{
						Gauntlet.Log.Verbose("Will use Editor for role {0}", Type);
					}

					Role.Skip = Globals.Params.ParseParam("Skip" + Type.ToString());

					if (Role.Skip)
					{
						Gauntlet.Log.Verbose("Will use NullPlatform to skip role {0}", Type);
					}

					// TODO - the below is a bit rigid, but maybe that's good enough since the "actually use the editor.." option
					// is specific to clients and servers

					bool bPlatformOverride = ContextOptions.RolePlatformOverrides.ContainsKey(Type);
					if (bPlatformOverride)
					{
						Role.Platform = ContextOptions.RolePlatformOverrides[Type];
					}
					
					if (Type.IsClient())
					{
						if(!bPlatformOverride)
						{
							Role.Platform = RequestedPlatform;
						}
						Role.Configuration = RequestedConfiguration;

						if (UsesEditor)
						{
							Role.Type = UnrealTargetRole.EditorGame;
							Role.Platform = DefaultPlatform;
							if (Role.Configuration > UnrealTargetConfiguration.Development)
							{
								Role.Configuration = UnrealTargetConfiguration.Development;
							}
						}
					}
					else if (Type.IsServer())
					{
						Role.Configuration = RequestedConfiguration;

						if (UsesEditor)
						{
							Role.Type = UnrealTargetRole.EditorServer;
							Role.Platform = DefaultPlatform;
							if (Role.Configuration > UnrealTargetConfiguration.Development)
							{
								Role.Configuration = UnrealTargetConfiguration.Development;
							}
						}
					}
					else if (Type.IsEditor())
					{
						Role.Configuration = RequestedConfiguration;
						if (Role.Configuration > UnrealTargetConfiguration.Development)
						{
							Role.Configuration = UnrealTargetConfiguration.Development;
						}
					}

					Gauntlet.Log.Verbose("Mapped Role {0} to RoleContext {1}", Type, Role);

					RoleContexts[Type] = Role;
				}

				UnrealTestContext Context = new UnrealTestContext(BuildInfo, RoleContexts, ContextOptions);

				IEnumerable<ITestNode> TestNodes = CreateTestList(Context, CombinedParams, PlatformWithParams);

				AllTestNodes.AddRange(TestNodes);
			}

			bool AllTestsPassed = ExecuteTests(ContextOptions, AllTestNodes);

			// Tear down accounts
			TeardownAccounts();

			// Generate Horde summary for CIS test
			Horde.GenerateStepSummary();
			Horde.GenerateJobSummary();

			return AllTestsPassed ? ExitCode.Success : ExitCode.Error_TestFailure;
		}

		bool ExecuteTests(UnrealTestOptions Options, IEnumerable<ITestNode> TestList)
		{
			// Create the test executor
			var Executor = new TestExecutor(ToString());

			try
			{
				bool Result = Executor.ExecuteTests(Options, TestList);

				return Result;
			}
			catch (System.Exception ex)
			{
				Gauntlet.Log.Info("");
				Gauntlet.Log.Error("{0}.\r\n\r\n{1}", ex.Message, ex.StackTrace);

				return false;
			}
			finally
			{
				Executor.Dispose();

				DevicePool.Instance.Dispose();

				foreach (ITestNode Node in TestList)
				{
					if (Node is IDisposable DisposableNode)
					{
						DisposableNode.Dispose();
					}
				}

				bool bCleanFromArgument = ParseParam("Clean");
				bool bCleanFromEnvironmentVariable = bool.TryParse(Environment.GetEnvironmentVariable(CleanEnvironmentVariable), out bool bCleanEnvvar) && bCleanEnvvar;
				bool bCleanFromEnvvarAllowed = !DevicePool.DeviceReservationBlock || DevicePool.DeviceReservationBlockFinalStep;
				if (bCleanFromArgument || (bCleanFromEnvironmentVariable && bCleanFromEnvvarAllowed))
				{
					// Flush any undisposed file streams that may be holding onto tempdir files
					GC.Collect();
					GC.WaitForPendingFinalizers();

					Logger.LogInformation("Cleaning TempDir '{GauntletTempDir}' by request...", Options.TempDir);
					DeleteDirectory(Options.TempDir);
				}

				GC.Collect();
			}
		}

		/// <summary>
		/// Create the list of tests specified by the context. 
		/// </summary>
		/// <param name="Context"></param>
		/// <param name="DefaultParams"></param>
		/// <param name="PlatformParams"></param>
		/// <returns></returns>
		IEnumerable<ITestNode> CreateTestList(UnrealTestContext Context, Params DefaultParams, ArgumentWithParams PlatformParams = null)
		{
			List<ITestNode> NodeList = new List<ITestNode>();

			IEnumerable<string> Namespaces = Context.Options.Namespaces.Split(',').Select(S => S.Trim());
			
			List<string> BuildIssues = new List<string>();

			UnrealTargetPlatform UnrealPlatform = UnrealTargetPlatform.Parse(PlatformParams.Argument);

			//List<string> Platforms = Globals.Params.ParseValue("platform")

			// Create an instance of each test and add it to the executor
			foreach (var Test in Context.Options.TestList)
			{
				// create a copy of the context for this test
				UnrealTestContext TestContext = (UnrealTestContext)Context.Clone();

				// if test specifies platforms, filter for this context
				if (Test.Platforms.Count() > 0 && Test.Platforms.Where(Plat => Plat.Argument == PlatformParams.Argument).Count() == 0)
				{
					continue;
				}

				if (Denylist.Instance.IsTestDenylisted(Test.TestName, UnrealPlatform, TestContext.BuildInfo.Branch))
				{
					Gauntlet.Log.Info("Test {0} is currently denylisted on {1} in branch {2}", Test.TestName, UnrealPlatform, TestContext.BuildInfo.Branch);
					continue;
				}

				// combine global and test-specific params
				Params CombinedParams = new Params(DefaultParams.AllArguments.Concat(Test.TestParams.AllArguments).ToArray());

				// parse any target constraints
				List<string> PerfSpecArgs = CombinedParams.ParseValues("PerfSpec", false);
				string PerfSpecArg = PerfSpecArgs.Count > 0 ? PerfSpecArgs.Last() : "Unspecified";
				EPerfSpec PerfSpec;
				if (!Enum.TryParse<EPerfSpec>(PerfSpecArg, true, out PerfSpec))
				{
					string ErrorMessage = string.Format("Unable to convert perfspec '{0}' into an EPerfSpec", PerfSpec);
					Horde.ReportInvalidatingIssue(Horde.GauntletHordeStepMessage.InvalidPerfSpec, ErrorMessage);
					throw new AutomationException(ErrorMessage);
				}

				// parse hardware model
				List<string> ModelArgs = CombinedParams.ParseValues("PerfModel", false);
				string Model = ModelArgs.Count > 0 ? ModelArgs.Last() : string.Empty;

				List<string> DeviceNameArgs = Globals.Params.ParseValues("DeviceName", false);
				string DeviceName = DeviceNameArgs.Count > 0 ? DeviceNameArgs.Last() : string.Empty;

				Reservation.Tags DeviceTags = new Reservation.Tags();
				DeviceTags.AddTags(Globals.Params.ParseValues("RequiredDeviceTags", true), Reservation.Tags.Type.Required);
				DeviceTags.AddTags(Globals.Params.ParseValues("PreferredDeviceTags", true), Reservation.Tags.Type.Preferred);
				DeviceTags.AddTags(Globals.Params.ParseValues("UndesiredDeviceTags", true), Reservation.Tags.Type.Undesired);
				DeviceTags.AddTags(Globals.Params.ParseValues("BlockedDeviceTags", true), Reservation.Tags.Type.Blocked);

				TestContext.Constraint = new UnrealDeviceTargetConstraint(UnrealPlatform, PerfSpec, Model, DeviceName, DeviceTags);

				// parse worker job id
				List<string> WorkerJobIDArgs = CombinedParams.ParseValues("WorkerJobID", false);
				TestContext.WorkerJobID = WorkerJobIDArgs.Count > 0 ? WorkerJobIDArgs.Last() : null;

				TestContext.TestParams = CombinedParams;

				// This will throw if the test cannot be created
				ITestNode NewTest = Utils.TestConstructor.ConstructTest<ITestNode, UnrealTestContext>(Test.TestName, TestContext, Namespaces);

				if (CombinedParams.ParseParam("listargs") || CombinedParams.ParseParam("listallargs"))
				{
					NewTest.DisplayCommandlineHelp();
				}
				else
				{
					NodeList.Add(NewTest);
				}
			}

			return NodeList;
		}

		/// <summary>
		/// Setup the account pool for tests that use it. Internally our derived classes (e.g.
		/// RunOrionTests) fill these with the usernames and passwords for all of our various
		/// test accounts. This is left here as an example.
		/// </summary>
		/// <returns></returns>
		protected virtual void SetupAccounts()
		{
			// Set up account manager before we set up accounts.
			AccountPool.Initialize();

			string Username = Globals.Params.ParseValue("username", null);
			string Password = Globals.Params.ParseValue("password", null);

			if (!string.IsNullOrEmpty(Username) && !string.IsNullOrEmpty(Password))
			{
				AccountPool.Instance.RegisterAccount(new EpicAccount(Username, Password));
			}
		}

		/// <summary>
		/// TearDown the account pool that was SetUp for tests.
		/// </summary>
		/// <returns></returns>
		protected virtual void TeardownAccounts()
		{
			AccountPool.Shutdown();
		}

		protected void SetupDevices(UnrealTargetPlatform DefaultPlatform, UnrealTestOptions Options)
		{
			Reservation.ReservationDetails = Options.JobDetails;

			DevicePool.Instance.SetLocalOptions(Options.TempDir, Options.Parallel > 1, Options.DeviceURL);
			DevicePool.Instance.AddLocalDevices(Options.MaxLocalDevices);
			DevicePool.Instance.AddVirtualDevices(Options.MaxVirtualDevices);

			foreach (var DeviceWithParams in Options.DeviceList)
			{
				UnrealTargetPlatform Platform = DefaultPlatform;

				// see if one of the params is a platform
				foreach (var Param in DeviceWithParams.AllArguments)
				{
					if (UnrealTargetPlatform.TryParse(Param, out Platform))
					{
						break;
					}
				}

				DevicePool.Instance.AddDevices(Platform, DeviceWithParams.Argument);
			}
		}

		protected void SetupPlatformConfigurationProfiles(UnrealTargetPlatform PlatformType, UnrealTestOptions Options)
		{
			string EngineDeviceConfigDir = Path.Combine(Globals.UnrealRootDir, "Engine", "Build", "DeviceConfigProfiles");
			string ProjectDeviceConfigDir = Path.Combine(Options.ProjectPath.Directory.FullName, "Build", "DeviceConfigProfiles");

			DeviceConfigurationCache.Instance.DiscoverConfigurationProfiles(PlatformType, "Engine", EngineDeviceConfigDir);
			DeviceConfigurationCache.Instance.DiscoverConfigurationProfiles(PlatformType, Options.Project, ProjectDeviceConfigDir);
		}
	}
}
