// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Linq;
using System.Text;
using AutomationTool;
using AutomationTool.DeviceReservation;
using UnrealBuildTool;
using System.IO;
using EpicGames.Core;
using UnrealBuildBase;
using Microsoft.Extensions.Logging;
using Gauntlet;

namespace Gauntlet
{
	public class CheckBuildSize : BuildCommand
	{
		public override ExitCode Execute()
		{
			string ProjectName = Globals.Params.ParseValue("project", "");
			FileReference ProjectFileRef = ProjectUtils.FindProjectFileFromName(ProjectName);
			string UnrealPath = Environment.CurrentDirectory;
			DirectoryReference UnrealPathRef = new DirectoryReference(UnrealPath);
			string BuildDir = Globals.Params.ParseValue("build", "");
			UnrealTargetPlatform.TryParse(Globals.Params.ParseValue("platform", ""), out UnrealTargetPlatform TargetPlatform);
			UnrealTargetConfiguration.TryParse(Globals.Params.ParseValue("configuration", ""), out UnrealTargetConfiguration TargetConfiguration);

			Gauntlet.UnrealBuildSource BuildSource = new Gauntlet.UnrealBuildSource(ProjectName, ProjectFileRef, UnrealPathRef, false, BuildDir);
			string PlatformPath = BuildSource.GetPlatformPath(UnrealTargetRole.Client, TargetPlatform);
			string ExecutablePath = BuildSource.GetRelativeExecutablePath(UnrealTargetRole.Client, TargetPlatform, TargetConfiguration);
			string OBBPath = Path.GetDirectoryName(ExecutablePath);

			// Get the size of the executable file
			long ExecutableSize = new FileInfo(ExecutablePath).Length;
			Gauntlet.Log.Info("Executable size: {0} bytes", ExecutableSize);

			// Iterate through all OBB files summing their file sizes
			long ObbTotalSize = 0;
			string[] ObbFiles = Directory.GetFiles(OBBPath, "*.obb");
			foreach (string ObbFile in ObbFiles)
			{
				long ObbSize = new FileInfo(ObbFile).Length;
				ObbTotalSize += ObbSize;
			}
			Gauntlet.Log.Info("Total OBB size: {0} bytes", ObbTotalSize);

			long PackageSize = (ExecutableSize + ObbTotalSize) / 1024;
			long DesiredSize = Globals.Params.ParseValue("packagesize", 0);

			if (PackageSize <= DesiredSize)
			{
				return ExitCode.Success;
			}

			Gauntlet.Log.Error("Error: Package size of {0} kilo-bytes exceeds desired size of {1} kilo-bytes", PackageSize, DesiredSize);
			return ExitCode.Error_TestFailure;
		}
	}
}
