// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.IO;
using System.Linq;
using System.Text;
using System.Threading.Tasks;
using UnrealBuildBase;
using UnrealBuildTool;

namespace Gauntlet
{
	/// <summary>
	/// Runs through every device provided (either singularly, a comma-separated list, or a device file) and removes any Gauntlet-installed
	/// builds older than MaxDays, and ay local crashdump info (if applicable) older than MaxDays
	/// </summary>
	public class CleanDevices : AutomationTool.BuildCommand
	{

		[AutoParamWithNames("", "device", "devices")]
		public string Devices = "";

		[AutoParam("")]
		public string DeviceURL;

		[AutoParam("")]
		public string TempDir;

		[AutoParam(7)]
		public int MaxDays;


		public override ExitCode Execute()
		{
			AutoParam.ApplyParamsAndDefaults(this, Environment.GetCommandLineArgs());

			Gauntlet.Log.Level = Gauntlet.LogLevel.VeryVerbose;

			if (string.IsNullOrEmpty(TempDir) == false)
			{
				Globals.TempDir = TempDir;
			}

			// add devices. We're quick so can ignore constraints
			DevicePool.Instance.SetLocalOptions(TempDir, false, DeviceURL);
			DevicePool.Instance.AddDevices(UnrealTargetPlatform.Win64, Devices, false);

			IEnumerable<UnrealTargetPlatform> SupportedPlatforms = UnrealTargetPlatform.GetValidPlatforms();

			foreach (UnrealTargetPlatform Platform in SupportedPlatforms)
			{
				DevicePool.Instance.EnumerateDevices(Platform, Device =>
				{
					try
					{
						CleanDevice(Device);
					}
					catch (Exception Ex)
					{
						Gauntlet.Log.Warning("Exception cleaning device: {0}", Ex);
					}

					return true;
				});
			}

			DevicePool.Instance.Dispose();


			return ExitCode.Success;
		}

		protected void CleanDevice(ITargetDevice Device)
		{
			Gauntlet.Log.Info("Cleaning {0}", Device.Name);

			if (Device.IsOn == false)
			{
				Device.PowerOn();
			}

			if (Device.IsAvailable == false)
			{
				Gauntlet.Log.Info("{0} is not available, skipping", Device.Name);
				return;
			}

			Device.Connect();

			if (Device.IsConnected == false)
			{
				Gauntlet.Log.Warning("Failed to connect to {0}", Device.Name);
				return;
			}

			// disconnect and power down
			Gauntlet.Log.Info("Powering down and disconnecting from {0}", Device.Name);
			Device.Disconnect();
			// turns out this may be a bad idea.. sorry environment.
			//Device.PowerOff();
		}
	}
}
