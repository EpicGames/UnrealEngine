// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.IO;
using System.Collections.Generic;
using System.Linq;
using System.Text;
using System.Text.RegularExpressions;
using System.Threading.Tasks;

namespace Gauntlet
{
	/// <summary>
	/// Helper class for parsing AutomationTest results from either an UnrealLogParser or log contents
	/// </summary>
	public class AutomationLogParser
	{
		protected UnrealLogParser Parser;

		public string AutomationReportPath { get; protected set; } = "";

		public string AutomationReportURL { get; protected set; } = "";

		/// <summary>
		/// Returns entries in the log file related to automation
		/// </summary>
		public IEnumerable<UnrealLog.LogEntry> AutomationLogEntries { get; protected set; }
	
		/// <summary>
		/// Returns warning/errors in the logfile related to automation
		/// </summary>
		public IEnumerable<UnrealLog.LogEntry> AutomationWarningsAndErrors
		{
			get
			{
				return AutomationLogEntries.Where(E => E.Level == UnrealLog.LogLevel.Error || E.Level == UnrealLog.LogLevel.Warning);
			}
		}

		/// <summary>
		/// Constructor that uses an existing log parser
		/// </summary>
		/// <param name="InParser"></param>
		public AutomationLogParser(UnrealLogParser InParser)
		{
			Parser = InParser;

			IEnumerable<Match> ReportPathMatch = Parser.GetAllMatches(@"LogAutomationController.+Report can be opened.+'(.+)'");

			if (ReportPathMatch.Any())
			{
				AutomationReportPath = Path.GetFullPath(ReportPathMatch.First().Groups[1].ToString());
			}
			IEnumerable<Match> ReportUrlMatch = Parser.GetAllMatches(@"LogAutomationController.+Report can be viewed.+'(.+)'");

			if (ReportUrlMatch.Any())
			{
				AutomationReportURL = ReportUrlMatch.First().Groups[1].ToString();
			}

			AutomationLogEntries = Parser.LogEntries.Where(
										E => E.Category.StartsWith("Automation", StringComparison.OrdinalIgnoreCase)
										|| E.Category.StartsWith("FunctionalTest", StringComparison.OrdinalIgnoreCase)
										)
									.ToArray();
		}

		/// <summary>
		/// Constructor that takes raw log contents
		/// </summary>
		/// <param name="InContents"></param>
		public AutomationLogParser(string InContents)
				: this(new UnrealLogParser(InContents))
		{
		}

		/// <summary>
		/// Constructor that takes UnrealLog instance
		/// </summary>
		/// <param name="InLog"></param>
		public AutomationLogParser(UnrealLog InLog)
				: this(InLog.GetParser())
		{
		}

		/// <summary>
		/// Returns all results found in our construction content.
		/// </summary>
		/// <returns></returns>
		public IEnumerable<UnrealAutomatedTestResult> GetResults()
		{
			IEnumerable<Match> TestStarts = Parser.GetAllMatches(@"LogAutomationController.+Test Started. Name={(.+?)}\s+Path={(.+?)}");

			// Find all automation results that succeeded/failed
			// [00:10:54.148]   LogAutomationController: Display: Test Started. Name={ST_PR04} Path={Project.Functional Tests./Game/Tests/Rendering/PlanarReflection.ST_PR04}
			// [2019.04.30-18.49.51:329][244]LogAutomationController: Display: Test Completed With Success. Name={ST_PR04} Path={Project.Functional Tests./Game/Tests/Rendering/PlanarReflection.ST_PR04}
			// [2019.04.30-18.49.51:330] [244] LogAutomationController: BeginEvents: Project.Functional Tests./Game/Tests/Rendering/PlanarReflection.ST_PR04
			// [2019.04.30 - 18.49.51:331][244] LogAutomationController: Screenshot 'ST_PR04' was similar!  Global Difference = 0.001377, Max Local Difference = 0.037953
			// [2019.04.30 - 18.49.51:332][244]LogAutomationController: EndEvents: Project.Functional Tests./Game/Tests/Rendering/PlanarReflection.ST_PR04
			IEnumerable<Match> TestResults = Parser.GetAllMatches(@"LogAutomationController.+Test Completed. Result={(.+?)}\s+Name={(.+?)}\s+Path={(.+?)}");

			string[] AutomationChannel = Parser.GetLogChannel("AutomationController").ToArray();

			Func<string, string> SantizeLine = (L) =>
			{
				L = L.Replace(": Error: ", ": ");
				L = L.Replace(": Warning: ", ": ");
				L = L.Replace("LogAutomationController: ", "");

				return L;
			};

			// Convert these lines into results by parsing out the details and then adding the events
			IEnumerable<UnrealAutomatedTestResult> Results = TestStarts.Select(TestMatch =>
			{
				UnrealAutomatedTestResult Result = new UnrealAutomatedTestResult();
				string DisplayName = TestMatch.Groups[1].ToString();
				string LongName = TestMatch.Groups[2].ToString(); ;
				TestStateType State = TestStateType.InProcess;
				
				Match ResultMatch = TestResults.Where(M => M.Groups[3].ToString() == LongName).FirstOrDefault();

				if (ResultMatch != null)
				{
					switch(ResultMatch.Groups[1].ToString().ToLower())
					{
					case "skipped":
						State = TestStateType.Skipped;
						break;
					case "success":
						State = TestStateType.Success;
						break;
					default:
						State = TestStateType.Fail;
						break;
					}

					string EventName = string.Format("BeginEvents: {0}", LongName);
					int EventIndex = Array.FindIndex(AutomationChannel, S => S.Contains(EventName)) + 1;

					if (EventIndex > 0)
					{
						while (EventIndex < AutomationChannel.Length)
						{
							string Event = AutomationChannel[EventIndex++];

							if (Event.Contains("EndEvents"))
							{
								break;
							}

							EventType EventType = Event.Contains(": Error: ") ? EventType.Error : Event.Contains(": Warning: ") ? EventType.Warning : EventType.Info;

							Result.AddEvent(EventType, SantizeLine(Event));
						}
					}
				}
				else
				{
					Result.AddEvent(EventType.Error, string.Format("Test {0} incomplete.", DisplayName));
				}

				Result.TestDisplayName = DisplayName;
				Result.FullTestPath = LongName;
				Result.State = State;

				return Result;
			});

			return Results;
		}
	}
}