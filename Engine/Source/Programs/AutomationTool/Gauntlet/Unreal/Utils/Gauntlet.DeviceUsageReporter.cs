// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;
using Gauntlet.Utils;
using System;
using System.Collections.Generic;

namespace Gauntlet
{
	public abstract class IDeviceUsageReporter
	{
		public enum EventType
		{
			  Device
			, Install
			, Test
			, SavingArtifacts
		};

		public enum EventState : int
		{
			  Failure = 0
			, Success = 1
		};

		public static void RecordStart(string deviceName, UnrealTargetPlatform? platform, EventType et, EventState state = EventState.Success, string comment = "")
		{
			if (platform.HasValue)
			{
				RecordToAll(deviceName, platform.Value, et, true, state, comment);
			}
		}

		public static void RecordEnd(string deviceName, UnrealTargetPlatform? platform, EventType et, EventState state = EventState.Success, string comment = "")
		{
			if (platform.HasValue)
			{
				RecordToAll(deviceName, platform.Value, et, false, state, comment);
			}
		}

		public static void RecordComment(string deviceName, UnrealTargetPlatform? platform, EventType et, string comment)
		{
			if (platform.HasValue)
            {
				CommentToAll(deviceName, platform.Value, et, comment);
			}
		}

		private static List<IDeviceUsageReporter> GetEnabledReporters()
		{
			List<IDeviceUsageReporter> ValidReporters = new List<IDeviceUsageReporter>();

			bool bFoundAnyReporters = false;
			bool bFoundEnabledReporters = false;

			foreach (IDeviceUsageReporter reporter in InterfaceHelpers.FindImplementations<IDeviceUsageReporter>(true))
			{
				// Just in case we get handed an abstract one
				if(reporter.GetType().IsAbstract)
				{
					Gauntlet.Log.Warning("Got abstract IDeviceUsageReporter from InterfaceHelpers.FindImplementations - {0}", reporter.GetType().Name);
					continue;
				}

				bFoundAnyReporters = true;

				if (reporter.IsEnabled())
				{
					bFoundEnabledReporters = true;
					ValidReporters.Add(reporter);
				}
			}

			if (!bFoundAnyReporters)
			{
				Gauntlet.Log.VeryVerbose("Skipped reporting DeviceUsage event - no reporter implementations found!");
			}
			else if (!bFoundEnabledReporters)
			{
				Gauntlet.Log.VeryVerbose("Skipped reporting DeviceUsage event - reporter implementations were found, but none were enabled!");
			}

			return ValidReporters;
		}

		private static void RecordToAll(string deviceName, UnrealTargetPlatform platform, EventType ev, bool bStarting, EventState state, string comment = "")
		{
			foreach (IDeviceUsageReporter reporter in GetEnabledReporters())
			{
				try
				{
					Gauntlet.Log.Verbose("Reporting DeviceUsage event {0} via reporter {1}", ev.ToString(), reporter.GetType().Name);
					reporter.RecordEvent(deviceName, platform, ev, bStarting, state, comment);
				}
				catch (Exception ex)
				{
					Gauntlet.Log.Warning("Failed to report DeviceUsage event {0} via {1}: {2}",
						ev.ToString(), reporter.GetType().Name, ex.Message);
				}
			}
		}

		private static void CommentToAll(string DeviceName, UnrealTargetPlatform platform, EventType ev, string comment)
		{
			foreach (IDeviceUsageReporter reporter in GetEnabledReporters())
			{
				try
				{
					Gauntlet.Log.Verbose("Adding Comment to DeviceUsage event {0} via reporter {1}", ev.ToString(), reporter.GetType().Name);
					reporter.AddCommentToEvent(DeviceName, platform, ev, comment);
				}
				catch (Exception ex)
				{
					Gauntlet.Log.Warning("Failed to add comment to DeviceUsage event {0} via {1}: {2}",
						ev.ToString(), reporter.GetType().Name, ex.Message);
				}
			}
		}

		public abstract void RecordEvent(string deviceName, UnrealTargetPlatform platform, EventType ev, bool bStarting, EventState state = EventState.Success, string comment = "");
		public abstract void AddCommentToEvent(string deviceName, UnrealTargetPlatform platform, EventType ev, string comment);

		public abstract bool IsEnabled();
	}
}