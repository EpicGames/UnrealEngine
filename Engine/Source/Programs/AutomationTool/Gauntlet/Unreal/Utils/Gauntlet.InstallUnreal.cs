// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.IO;
using System.Threading;
using System.Linq;
using AutomationTool;
using UnrealBuildTool;
using System.Threading.Tasks;


namespace Gauntlet
{
	/// <summary>
	/// Base class implementing Installer functionality
	/// </summary>
	public class InstallUnreal
	{
		/// <summary>
		/// A method which is responsible for the core Installer functionality.
		/// Iterates through devices and installs the project to each device.
		/// </summary>
		/// <returns></returns>
		public static bool RunInstall(
			string PlatformParam,
			string BuildPath,
			string ProjectName,
			string CommandLine,
			string DevicesArg,
			int ParallelTasks )
		{
			if (string.IsNullOrEmpty(ProjectName))
			{
				throw new Exception("need -project=<project name>");
			}

			if (string.IsNullOrEmpty(PlatformParam) || string.IsNullOrEmpty(BuildPath))
			{
				throw new Exception("need -platform=<platform> and -path=\"path to build\"");
			}

			if (Directory.Exists(BuildPath) == false)
			{
				throw new AutomationException("Path {0} does not exist", BuildPath);
			}

			UnrealTargetPlatform Platform = UnrealTargetPlatform.Parse(PlatformParam);

			// find all build sources that can be created a folder path
			IEnumerable<IFolderBuildSource> BuildSources = Gauntlet.Utils.InterfaceHelpers.FindImplementations<IFolderBuildSource>();

			if (BuildSources.Count() == 0)
			{
				throw new AutomationException("No BuildSources found for platform {0}", Platform);
			}

			IReadOnlyCollection<IBuild> Builds = BuildSources.Where(S => S.CanSupportPlatform(Platform)).SelectMany(S => S.GetBuildsAtPath(ProjectName, BuildPath)).ToList();

			if (Builds.Count() == 0)
			{
				throw new AutomationException("No builds for {0} found at {1}", Platform, BuildPath);
			}

			IEnumerable<ITargetDevice> DeviceList = null;

			if (string.IsNullOrEmpty(DevicesArg))
			{
				// find all build sources that can be created a folder path
				IEnumerable<IDefaultDeviceSource> DeviceSources = Gauntlet.Utils.InterfaceHelpers.FindImplementations<IDefaultDeviceSource>();

				DeviceList = DeviceSources.Where(S => S.CanSupportPlatform(Platform)).SelectMany(S => S.GetDefaultDevices());
			}
			else
			{
				IEnumerable<string> Devices = DevicesArg.Split(new[] { "," }, StringSplitOptions.RemoveEmptyEntries).Select(S => S.Trim());

				IDeviceFactory Factory = Gauntlet.Utils.InterfaceHelpers.FindImplementations<IDeviceFactory>()
					.Where(F => F.CanSupportPlatform(Platform))
					.FirstOrDefault();

				if (Factory == null)
				{
					throw new AutomationException("No IDeviceFactory implmenetation that supports {0}", Platform);
				}

				DeviceList = Devices.Select(D => Factory.CreateDevice(D, null)).ToArray();
			}

			if (DeviceList.Count() == 0)
			{
				throw new AutomationException("No devices found for {0}", Platform);
			}

			if (ParallelTasks == -1)
			{
				//If a value is not passed in as a paramter, set ParallelTasks equal to the number of devices, MAX 4
				ParallelTasks = (DeviceList.Count() > 4) ? 4 : DeviceList.Count();
			}

			var POptions = new ParallelOptions { MaxDegreeOfParallelism = ParallelTasks };

			// now copy it up to four builds at a time
			Parallel.ForEach(DeviceList, POptions, Device =>
			{
				DateTime StartTime = DateTime.Now;

				UnrealAppConfig Config = new UnrealAppConfig();

				Config.CommandLine = CommandLine;
				Config.ProjectName = ProjectName;
				Config.Name = ProjectName;

				string BuildSandbox = Globals.Params.ParseValue("sandbox", string.Empty);
				Config.Sandbox = string.IsNullOrEmpty(BuildSandbox) ? Config.Sandbox : BuildSandbox;

				// We always (currently..) need to be able to replace the command line
				BuildFlags Flags = BuildFlags.CanReplaceCommandLine;

				if (Globals.Params.ParseParam("dev"))
				{
					Flags |= BuildFlags.CanReplaceExecutable;
				}
				if (Globals.Params.ParseParam("bulk"))
				{
					Flags |= BuildFlags.Bulk;
				}
				if (Globals.Params.ParseParam("notbulk"))
				{
					Flags |= BuildFlags.NotBulk;
				}
				if (Globals.Params.ParseParam("packaged"))
				{
					Flags |= BuildFlags.Packaged;
				}
				if (Globals.Params.ParseParam("staged"))
				{
					Flags |= BuildFlags.Loose;
				}

				IEnumerable<IBuild> FlaggedBuilds = Builds.Where(S => S.Flags.HasFlag(Flags));

				if (FlaggedBuilds.Count() == 0)
				{
					throw new AutomationException("Unable to find build at {0} with build flag(s): {1} ", BuildPath, Flags.ToString());
				}

				string BuildConfiguration = Globals.Params.ParseValue("configuration", String.Empty);

				switch (BuildConfiguration.ToLower())
				{
					case "shipping":
						Config.Build = FlaggedBuilds.Where(S => S.Configuration == UnrealTargetConfiguration.Shipping).FirstOrDefault();
						break;

					case "test":
						Config.Build = FlaggedBuilds.Where(S => S.Configuration == UnrealTargetConfiguration.Test).FirstOrDefault();
						break;

					case "development":
						Config.Build = FlaggedBuilds.Where(S => S.Configuration == UnrealTargetConfiguration.Development).FirstOrDefault();
						break;

					default:
						Log.Info("No build configuration was provided, selecting first available build.");
						Config.Build = FlaggedBuilds.FirstOrDefault();
						break;
				}

				if (!Device.Connect())
				{
					throw new AutomationException("Failed to connect to device: {0}", Device);
				}

				if (Config.Build != null)
				{
					Log.Info("Installing build on device {DeviceName}", Device.Name);
					Device.InstallBuild(Config);
				}
				else
				{
					throw new AutomationException("Unable to find build at {0} with build flag(s)={1} and configuration type={2} for {3} platform type", BuildPath, Flags, BuildConfiguration, Platform);
				}

				TimeSpan Elapsed = (DateTime.Now - StartTime);
				Log.Info("Installed on device {0} in {1:D2}m:{2:D2}s", Device.Name, Elapsed.Minutes, Elapsed.Seconds);
			});

			DeviceList = null;

			// wait for all the adb tasks to start up or UAT will kill them on exit...
			Thread.Sleep(5000);

			return true;
		}
	}
}
