// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Linq;
using System.Text;
using System.Threading.Tasks;

namespace Gauntlet
{
	public class GauntletParamDescription
	{
		/// <summary>
		/// The param name that is passed in on the commandline
		/// In -PARAMNAME or -PARAMNAME=val format
		/// </summary>
		public string ParamName;

		/// <summary>
		/// Is this required for the test to run?
		/// </summary>
		public bool Required;

		/// <summary>		
		/// Very brief desc of what to pass in -
		/// Will show up in -param=<InputFormat> format.
		/// Ex - a value of "Map To Use" would show up as -param=<Map To use>
		/// Leave blank for a param that is just a flag.
		/// </summary>
		public string InputFormat;

		/// <summary>
		/// Helpful description for what this Parameter or flag represents and what can be passed in.
		/// </summary>
		public string ParamDesc;

		/// <summary>
		/// If you would like to provide a sample input for this field, do so here. Will show up as (ex: SampleInput) at the end of the param description
		/// </summary>
		public string SampleInput;

		/// <summary>
		///  If this param has a default value, put it here. Will show ups as (default: DefaultValue)
		/// </summary>
		public string DefaultValue;

		/// <summary>
		/// Whether this is a Test-specific param or a generic gauntlet param.
		/// </summary>
		public bool TestSpecificParam;

		public GauntletParamDescription()
		{
			TestSpecificParam = true;
		}

		public override string ToString()
		{
			string ParamFormat = ParamName;
			if (!string.IsNullOrEmpty(InputFormat))
			{
				ParamFormat += "=" + InputFormat;
			}
			string DefaultFormat = "";
			if (!string.IsNullOrEmpty(DefaultValue))
			{
				DefaultFormat = string.Format("(default: {0}) ", DefaultValue);
			}
			string SampleFormat = "";
			if (!string.IsNullOrEmpty(SampleInput))
			{
				SampleFormat = string.Format(" (ex: {0})", SampleInput);
			}
			ParamFormat = string.Format("{0}:\t\t{1}{2}{3}{4}", ParamFormat, Required ? "*Required* " : "", DefaultFormat, ParamDesc, SampleInput);
			return ParamFormat;
		}
	}
}
