// Copyright Epic Games, Inc. All Rights Reserved.

using AutomationTool;
using EpicGames.Core;
using EpicGames.LogMatchers;
using Microsoft.Extensions.Logging;
using System;
using System.Collections.Generic;
using System.Globalization;
using System.Linq;
using System.Text;
using System.Text.RegularExpressions;
using Logging = Microsoft.Extensions.Logging;

namespace Gauntlet
{
	/// <summary>
	/// Represents a prepared summary of the most relevant info in a log. Generated once by a
	/// LogParser and cached.
	/// </summary>
	public class UnrealLog : IDisposable
	{
		/// <summary>
		/// Represents the level
		/// </summary>
		public enum LogLevel
		{
			Log,
			Display,
			Verbose,
			VeryVerbose,
			Warning,
			Error,
			Fatal
		}

		/// <summary>
		/// Set of log channels that are used to monitor Editor processing
		/// </summary>
		public static string[] EditorBusyChannels = new string[] { "Automation", "FunctionalTest", "Material", "DerivedDataCache", "ShaderCompilers", "Texture", "SkeletalMesh", "StaticMesh", "Python", "AssetRegistry" };

		/// <summary>
		/// Represents an entry in an Unreal logfile with and contails the associated category, level, and message
		/// </summary>
		public class LogEntry
		{
			public string Prefix { get; private set; }

			/// <summary>
			/// Category of the entry. E.g for "LogNet" this will be "Net"
			/// </summary>
			public string Category { get; private set; }

			/// <summary>
			/// Full channel name
			/// </summary>
			public string LongChannelName => Prefix + Category;

			/// <summary>
			/// Represents the level of the entry
			/// </summary>
			public LogLevel Level { get; private set; }

			/// <summary>
			/// The message string from the entry
			/// </summary>
			public string Message { get; private set; }

			/// <summary>
			/// Optional datetime of the entry
			/// </summary>
			public DateTime? Timestamp { get; private set; }

			/// <summary>
			/// Format the entry as it would have appeared in the log (without the timestamp). 
			/// </summary>
			/// <returns></returns>
			public override string ToString()
			{
				// LogFoo: Display: Some Message
				// Match how Unreal does not display the level for 'Log' level messages
				if (Level == LogLevel.Log)
				{
					return string.Format("{0}: {1}", LongChannelName, Message);
				}
				else
				{
					return string.Format("{0}: {1}: {2}", LongChannelName, Level, Message);
				}
			}

			/// <summary>
			/// Format the entry as it would have appeared in the log (with the UTC timestamp).
			/// </summary>
			/// <returns></returns>
			public string ToStringWithUtcTimestamp()
			{
				if (Timestamp.HasValue)
				{
					return string.Format("[{0}]{1}", Timestamp.Value.ToString("yyyy.MM.dd-HH.mm.ss:fff"), ToString());
				}
				else
				{
					return ToString();
				}
			}

			/// <summary>
			/// Constructor that requires all info
			/// </summary>
			/// <param name="InPrefix"></param>
			/// <param name="InCategory"></param>
			/// <param name="InLevel"></param>
			/// <param name="InMessage"></param>
			/// <param name="InTimestamp"></param>
			public LogEntry(string InPrefix, string InCategory, LogLevel InLevel, string InMessage, string InTimestamp)
			{
				Prefix = InPrefix;
				Category = InCategory;
				Level = InLevel;
				Message = InMessage;

				if (DateTime.TryParseExact(InTimestamp, "yyyy.MM.dd-HH.mm.ss:fff", CultureInfo.InvariantCulture, DateTimeStyles.AssumeUniversal, out DateTime ParsedTimestamp))
				{
					Timestamp = ParsedTimestamp;
				}
			}

			/// <summary>
			/// Constructor that requires all info
			/// </summary>
			/// <param name="InPrefix"></param>
			/// <param name="InCategory"></param>
			/// <param name="InLevel"></param>
			/// <param name="InMessage"></param>
			/// <param name="InTimestamp"></param>
			public LogEntry(string InPrefix, string InCategory, LogLevel InLevel, string InMessage, DateTime? InTimestamp)
			{
				Prefix = InPrefix;
				Category = InCategory;
				Level = InLevel;
				Message = InMessage;
				Timestamp = InTimestamp;
			}
		}

		/// <summary>
		/// Compound object that represents a fatal entry 
		/// </summary>
		public class CallstackMessage
		{
			public int Position;
			public string Message;
			public string[] Callstack;
			public bool IsEnsure;
			public bool IsSanReport;

			/// <summary>
			/// Generate a string that represents a CallstackMessage formatted to be inserted into a log file.
			/// </summary>
			/// <returns>Formatted log string version of callstack.</returns>
			public string FormatForLog()
			{
				string FormattedString = string.Format("{0}\n", Message);
				foreach (string StackLine in Callstack)
				{
					FormattedString += string.Format("\t{0}\n", StackLine);
				}
				return FormattedString;
			}
		};

		/// <summary>
		/// Information about the current platform that will exist and be extracted from the log
		/// </summary>
		public class PlatformInfo
		{
			public string OSName;
			public string OSVersion;
			public string CPUName;
			public string GPUName;
		}

		/// <summary>
		/// Information about the current build info
		/// </summary>
		public class BuildInfo
		{
			public string BuildVersion;
			public string BranchName;
			public int Changelist;
		}

		private UnrealLogParser _parser;

		public UnrealLog(UnrealLogParser InParser)
		{
			_parser = InParser;
			_loggedBuildInfo = new(() => _parser.GetBuildInfo());
			_loggedPlatformInfo = new(() => _parser.GetPlatformInfo());
			_logEntries = new(() => _parser.LogEntries);
			_fatalError = new(() => _parser.GetFatalError());
			_ensures = new(() => _parser.GetEnsures());
			_lineCount = new(() => _parser.GetLogReader().GetAvailableLineCount());
			_hasTestExitCode = new(() => _parser.GetTestExitCode(out _testExitCode));
			_engineInitialized = new(() => HasEngineInitialized());
			_hasParseRequestedExitReason = new(() => GetRequestedExitReason());
		}

		#region IDisposable Support
		private bool _disposed = false;
		~UnrealLog()
		{
			Dispose(false);
		}

		public void Dispose()
		{
			Dispose(true);
		}

		private void Dispose(bool bDisposing = true)
		{
			if (!_disposed)
			{
				if (bDisposing)
				{
					_parser?.Dispose(true);
				}
				_disposed = true;
			}
		}
		#endregion

		private bool GetRequestedExitReason()
		{
			// Check request exit and reason
			_parser.MatchAndApplyGroups(@"Engine exit requested \(reason:\s*(.+)\)", (Groups) =>
			{
				_requestedExit = true;
				_requestedExitReason = Groups[1];
			});

			if (!_requestedExit)
			{
				var Completion = _parser.GetAllMatchingLines("F[a-zA-Z0-9]+::RequestExit");
				var ErrorCompletion = _parser.GetAllMatchingLines("StaticShutdownAfterError");

				if (Completion.Any() || ErrorCompletion.Any())
				{
					_requestedExit = true;
					_requestedExitReason = "Unidentified";
				}
			}

			return true;
		}

		private bool HasEngineInitialized()
		{
			// Search for Engine initialized pattern.
			return _parser.GetAllMatches(EngineInitializedPattern).Any();
		}

		/// <summary>
		/// Return the log parser attached to it
		/// </summary>
		/// <returns></returns>
		public UnrealLogParser GetParser() => _parser;

		/// <summary>
		/// Build info from the log
		/// </summary>
		public BuildInfo LoggedBuildInfo => _loggedBuildInfo.Value;
		private Lazy<BuildInfo> _loggedBuildInfo;

		/// <summary>
		/// Platform info from the log
		/// </summary>
		public PlatformInfo LoggedPlatformInfo => _loggedPlatformInfo.Value;
		private Lazy<PlatformInfo> _loggedPlatformInfo;

		/// <summary>
		/// Entries in the log
		/// </summary>
		public IEnumerable<LogEntry> LogEntries { get { return _logEntries.Value; } set { _logEntries = new(value); } }
		private Lazy<IEnumerable<LogEntry>> _logEntries;

		/// <summary>
		/// Warnings for this role
		/// </summary>
		public IEnumerable<LogEntry> Warnings { get { return LogEntries.Where(E => E.Level == LogLevel.Warning); } }

		/// <summary>
		/// Errors for this role
		/// </summary>
		public IEnumerable<LogEntry> Errors { get { return LogEntries.Where(E => E.Level == LogLevel.Error); } }

		/// <summary>
		/// Fatal error instance if one occurred
		/// </summary>
		public CallstackMessage FatalError { get { return _fatalError.Value; } set { _fatalError = new(value); } }
		private Lazy<CallstackMessage> _fatalError;

		/// <summary>
		/// A list of ensures if any occurred
		/// </summary>
		public IEnumerable<CallstackMessage> Ensures { get { return _ensures.Value; } set { _ensures = new(value); } }
		private Lazy<IEnumerable<CallstackMessage>> _ensures;

		/// <summary>
		/// Number of lines in the log
		/// </summary>
		public int LineCount => _lineCount.Value;
		private Lazy<int> _lineCount;

		/// <summary>
		/// True if the engine reached initialization
		/// </summary>
		public bool EngineInitialized { get { return _engineInitialized.Value; } set { _engineInitialized = new(() => value); } }
		private Lazy<bool> _engineInitialized;

		/// <summary>
		/// Regex pattern used to detect if the engine was initialized
		/// </summary>
		public string EngineInitializedPattern = @"LogInit.+Engine is initialized\.";

		/// <summary>
		/// True if the instance requested exit
		/// </summary>
		public bool RequestedExit { get { return _hasParseRequestedExitReason.Value ? _requestedExit : false; } set { _requestedExit = value; _hasParseRequestedExitReason = new(() => true); } }
		private bool _requestedExit;
		public string RequestedExitReason { get { return _hasParseRequestedExitReason.Value ? _requestedExitReason : string.Empty; } set { _requestedExitReason = value; _hasParseRequestedExitReason = new(() => true); } }
		private string _requestedExitReason;
		private Lazy<bool> _hasParseRequestedExitReason;
		public bool HasTestExitCode { get { return _hasTestExitCode.Value; } set { _hasTestExitCode = new(() => value); } }
		private Lazy<bool> _hasTestExitCode;
		public int TestExitCode { get { return HasTestExitCode ? _testExitCode : -1; } set { _testExitCode = value; } }
		protected int _testExitCode;

		// DEPRECATED - it is slow. Get attached parser instead through call of GetParser()
		public string FullLogContent => _parser.GetLogReader().GetContent();

		/// <summary>
		/// Returns true if this log indicates the Unreal instance exited abnormally
		/// </summary>
		public bool HasAbnormalExit
		{
			get
			{
				return FatalError != null
					|| EngineInitialized == false
					|| (RequestedExit == false && HasTestExitCode == false);
			}
		}
	}

	/// <summary>
	/// Parse Unreal log from string chunk and aggregate lines as LogEvents
	/// Support structure logging output and legacy logging style
	/// </summary>
	public class UnrealLogStreamParser
	{
		protected List<UnrealLog.LogEntry> LogEvents { get; private set; }

		private HashSet<string> UnidentifiedLogLevels { get; set; }

		private ILogStreamReader LogReader { get; set; }

		public UnrealLogStreamParser()
		{
			LogEvents = new();
			UnidentifiedLogLevels = new HashSet<string>();
			LogReader = null;
		}

		public UnrealLogStreamParser(ILogStreamReader InLogReader)
			: this()
		{
			LogReader = InLogReader;
		}

		/// <summary>
		/// Set the internal log stream reader
		/// </summary>
		/// <param name="InLogReader"></param>
		public void SetLogReader(ILogStreamReader InLogReader)
		{
			LogReader = InLogReader;
		}

		/// <summary>
		/// Return true if the internal log reader was set.
		/// </summary>
		/// <returns></returns>
		public bool IsAttachedToLogReader()
		{
			return LogReader != null;
		}

		/// <summary>
		/// Clear aggregated log events 
		/// </summary>
		public void Clear()
		{
			LogEvents.Clear();
		}

		/// <summary>
		/// Parse a string as log and aggregate identified unreal log lines using internal Log reader
		/// </summary>
		/// <param name="LineOffset">Line offset to start parsing and aggregate. Default is set to use the internal LogReader cursor.</param>
		/// <param name="bClearAggregatedLines">Whether to clear the previously aggregated lines</param>
		/// <returns>The number of line parsed</returns>
		public int ReadStream(int LineOffset = -1, bool bClearAggregatedLines = true)
		{
			if (!IsAttachedToLogReader())
			{
				throw new AutomationException("Internal Log reader is not set. Use SetLogReader() to set it.");
			}
			return ReadStream(LogReader, LineOffset, bClearAggregatedLines);
		}

		/// <summary>
		/// Parse a string as log and aggregate identified unreal log lines
		/// </summary>
		/// <param name="InContent"></param>
		/// <param name="LineOffset">Line offset to start parsing and aggregate. By passing -1, it will use the internal LogReader cursor.</param>
		/// <param name="bClearAggregatedLines">Whether to clear the previously aggregated lines</param>
		/// <returns>The number of line parsed</returns>
		public int ReadStream(string InContent, int LineOffset = 0, bool bClearAggregatedLines = true)
		{
			return ReadStream(new DynamicStringReader(() => InContent), LineOffset, bClearAggregatedLines);
		}

		/// <summary>
		/// Parse a string as log and aggregate identified unreal log lines
		/// </summary>
		/// <param name="LogReader"></param>
		/// <param name="LineOffset">Line offset to start parsing and aggregate. By passing -1, it will use the internal LogReader cursor.</param>
		/// <param name="bClearAggregatedLines">Whether to clear the previously aggregated lines</param>
		/// <returns>The number of line parsed</returns>
		public int ReadStream(ILogStreamReader LogReader, int LineOffset = 0, bool bClearAggregatedLines = true)
		{
			if (bClearAggregatedLines)
			{
				Clear();
			}

			// Capture of the UE log where the timestamp and frame are optional groups as they are not always available per line
			Regex UELogLinePattern = new Regex(@"(?:\[(?<timestamp>[0-9.\-:]+)\])?(?:\[(?<frame>[\s\d]+)\])?(?<channel>[A-Za-z][\w\d]+):\s(?:(?<level>Display|Verbose|VeryVerbose|Warning|Error|Fatal):\s)?");

			if (LineOffset >= 0)
			{
				LogReader.SetLineIndex(LineOffset);
			}
			foreach(string Line in LogReader.EnumerateNextLines())
			{
				UnrealLog.LogEntry Entry = null;
				// Parse the line as Unreal legacy line
				Match MatchLine = UELogLinePattern.Match(Line);
				if (MatchLine.Success)
				{
					string Channel = MatchLine.Groups["channel"].Value;
					string Message = Line.Substring(MatchLine.Index + MatchLine.Length);
					string Timestamp = MatchLine.Groups["timestamp"].Value;
					ReadOnlySpan<char> LevelSpan = MatchLine.Groups["level"].ValueSpan;

					UnrealLog.LogLevel Level = UnrealLog.LogLevel.Log;
					if (!LevelSpan.IsEmpty)
					{
						if (!Enum.TryParse(LevelSpan, out Level))
						{
							string LevelStr = LevelSpan.ToString();
							// only show a warning once
							if (!UnidentifiedLogLevels.Contains(LevelStr))
							{
								UnidentifiedLogLevels.Add(LevelStr);
								Log.Warning("Failed to match log level {0} to enum!", LevelStr);
							}
						}
					}

					string Prefix = string.Empty;
					if (Channel.StartsWith("log", StringComparison.OrdinalIgnoreCase))
					{
						Prefix = Channel.Substring(0, 3);
						Channel = Channel.Substring(3);
					}
					Entry = new(Prefix, Channel, Level, Message, Timestamp);
				}
				else
				{
					// Not an Unreal Engine log line
					Entry = new(string.Empty, string.Empty, UnrealLog.LogLevel.Log, Line, string.Empty);
				}

				LogEvents.Add(Entry);
			}

			return LogReader.GetLineIndex() - LineOffset;
		}

		/// <summary>
		/// Return All the LogEvent instances
		/// </summary>
		/// <returns></returns>
		public IEnumerable<UnrealLog.LogEntry> GetEvents()
		{
			return LogEvents;
		}

		/// <summary>
		/// Return the LogEvent instances which channel name match
		/// </summary>
		/// <param name="InValues">The channel names to match</param>
		/// <param name="ExactMatch">Whether to use an exact match or partial match</param>
		/// <param name="UseLongName">Whether to use the long channel name to match</param>
		/// <returns></returns>
		private IEnumerable<UnrealLog.LogEntry> InternalGetEventsFromChannels(IEnumerable<string> InValues, bool ExactMatch = false, bool UseLongName = true)
		{
			if (ExactMatch)
			{
				return LogEvents.Where(E => InValues.Contains(UseLongName? E.LongChannelName : E.Category, StringComparer.OrdinalIgnoreCase));
			}
			// partial match
			return LogEvents.Where(E =>
			{
				string Name = UseLongName? E.LongChannelName : E.Category;
				foreach (string Value in InValues)
				{
					if (Name.IndexOf(Value, StringComparison.OrdinalIgnoreCase) >= 0)
					{
						return true;
					}
				}

				return false;
			});
		}

		/// <summary>
		/// Return the LogEvent instances that match the channel names
		/// </summary>
		/// <param name="Channels">The names of the channel to match</param>
		/// <param name="ExactMatch"></param>
		/// <returns></returns>
		public IEnumerable<UnrealLog.LogEntry> GetEventsFromChannels(IEnumerable<string> Channels, bool ExactMatch = true)
		{
			return InternalGetEventsFromChannels(Channels, ExactMatch);
		}

		/// <summary>
		/// Return the LogEvent instances that match the Editor busy channels
		/// </summary>
		/// <returns></returns>
		public IEnumerable<UnrealLog.LogEntry> GetEventsFromEditorBusyChannels()
		{
			return InternalGetEventsFromChannels(UnrealLog.EditorBusyChannels, false);
		}

		/// <summary>
		/// Return the log lines that match the channel names
		/// </summary>
		/// <param name="Channels">The names of the channel to match</param>
		/// <param name="ExactMatch"></param>
		/// <returns></returns>
		public IEnumerable<string> GetLogFromChannels(IEnumerable<string> Channels, bool ExactMatch = true)
		{
			return InternalGetEventsFromChannels(Channels, ExactMatch).Select(E => E.ToString());
		}

		/// <summary>
		/// Return the log lines that match the channel names ignoring the "log" prefix
		/// </summary>
		/// <param name="Channels"></param>
		/// <returns></returns>
		public IEnumerable<string> GetLogFromShortNameChannels(IEnumerable<string> Channels)
		{
			return InternalGetEventsFromChannels(Channels, UseLongName: false).Select(E => E.ToString());
		}

		/// <summary>
		/// Return the log lines that match the Editor busy channels
		/// </summary>
		/// <returns></returns>
		public IEnumerable<string> GetLogFromEditorBusyChannels()
		{
			return GetLogFromChannels(UnrealLog.EditorBusyChannels, false);
		}

		/// <summary>
		/// Return the log lines that match the channel name
		/// </summary>
		/// <param name="Channel">The channel name to match</param>
		/// <param name="ExactMatch"></param>
		/// <returns></returns>
		public IEnumerable<string> GetLogFromChannel(string Channel, bool ExactMatch = true)
		{
			return GetLogFromChannels(new string[] { Channel }, ExactMatch);
		}

		/// <summary>
		/// Return all the aggregated log lines 
		/// </summary>
		/// <returns></returns>
		public IEnumerable<string> GetLogLines()
		{
			return LogEvents.Select(E => E.ToString());
		}

		/// <summary>
		/// Return the log lines that contain a string
		/// </summary>
		/// <param name="Text">The text to match in the line</param>
		/// <returns></returns>
		public IEnumerable<string> GetLogLinesContaining(string Text)
		{
			return GetLogLines().Where(L => L.IndexOf(Text, StringComparison.OrdinalIgnoreCase) >= 0);
		}

		/// <summary>
		/// Return the log lines that match the regex pattern
		/// </summary>
		/// <param name="Pattern">The regex pattern to match in the line</param>
		/// <returns></returns>
		public IEnumerable<string> GetLogLinesMatchingPattern(string Pattern)
		{
			Regex RegexPattern = new Regex(Pattern, RegexOptions.IgnoreCase);
			return GetLogLines().Where(L => RegexPattern.IsMatch(L));
		}
	}

	/// <summary>
	/// Helper class for parsing logs
	/// </summary>
	public class UnrealLogParser : IDisposable
	{
		/// <summary>
		/// DEPRECATED - Use GetLogReader() to read through the log stream efficiently.
		/// </summary>
		public string Content => _logReader.GetContent();

		/// <summary>
		/// Allow reading log line by line with an internal cursor
		/// </summary>
		public ILogStreamReader GetLogReader() => _logReader.Clone();
		private ILogStreamReader _logReader { get; set; }

		/// <summary>
		/// All entries in the log
		/// </summary>
		public IEnumerable<UnrealLog.LogEntry> LogEntries => _logEntries.Value;
		private Lazy<IEnumerable<UnrealLog.LogEntry>> _logEntries;

		/// <summary>
		/// Summary of the log
		/// </summary>
		private Lazy<UnrealLog> _summary;

		// Track log levels we couldn't identify
		protected static HashSet<string> UnidentifiedLogLevels = new HashSet<string>();

		/// <summary>
		/// Constructor that takes a ILogStreamReader instance
		/// </summary>
		/// <param name="InLogReader"></param>
		public UnrealLogParser(ILogStreamReader InLogReader)
		{
			_logReader = InLogReader;
			_logEntries = new(() => ParseEntries());
			_summary = new(() => CreateSummary());
		}

		/// <summary>
		/// Constructor that takes the content to parse
		/// </summary>
		/// <param name="InContent"></param>
		/// <returns></returns>
		public UnrealLogParser(string InContent) : this(new DynamicStringReader(() => InContent))
		{ }

		#region IDisposable Support
		private bool _disposed = false;
		~UnrealLogParser()
		{
			Dispose(false);
		}

		public void Dispose()
		{
			Dispose(true);
		}

		public void Dispose(bool bDisposing = true)
		{
			if (!_disposed)
			{
				if (bDisposing)
				{
					if (_logReader is IDisposable disposable)
					{
						disposable.Dispose();
					}
				}
				
				_disposed = true;
			}
		}
		#endregion

		protected List<UnrealLog.LogEntry> ParseEntries()
		{
			// Search for LogFoo: <Display|Error|etc>: Message
			// Also need to handle 'Log' not always being present, and the category being empty for a level of 'Log'
			// Also handling the optional timestamp and frame groups
			Regex Pattern = new Regex(@"(?:\[(?<timestamp>[0-9.\-:]+)\])?(?:\[(?<frame>[\s\d]+)\])?(?<prefix>Log)?(?<category>[A-Za-z][\w\d]+):\s(?<level>Display|Verbose|VeryVerbose|Warning|Error|Fatal)?(?::\s)?");

			List<UnrealLog.LogEntry> ParsedEntries = new List<UnrealLog.LogEntry>();

			_logReader.SetLineIndex(0);
			foreach (string Line in _logReader.EnumerateNextLines())
			{
				var M = Pattern.Match(Line);
				if (!M.Success) continue;

				string Prefix = M.Groups["prefix"].Value;
				string Category = M.Groups["category"].Value;
				string Message = Line.Substring(M.Index + M.Length);
				string Timestamp = M.Groups["timestamp"].Value;

				ReadOnlySpan<char> LevelSpan = M.Groups["level"].ValueSpan;
				UnrealLog.LogLevel Level = UnrealLog.LogLevel.Log;

				if (!LevelSpan.IsEmpty)
				{
					if (!Enum.TryParse(LevelSpan, out Level))
					{
						string LevelStr = LevelSpan.ToString();
						// only show a warning once
						if (!UnidentifiedLogLevels.Contains(LevelStr))
						{
							UnidentifiedLogLevels.Add(LevelStr);
							Log.Warning("Failed to match log level {0} to enum!", LevelStr);
						}
					}
				}

				ParsedEntries.Add(new UnrealLog.LogEntry(Prefix, Category, Level, Message, Timestamp));
			}

			return ParsedEntries;
		}

		public static string SanitizeLogText(string InContent)
		{
			StringBuilder ContentBuilder = new StringBuilder();

			for (int BaseIdx = 0; BaseIdx < InContent.Length;)
			{
				// Extract the next line
				int EndIdx = InContent.IndexOf('\n', BaseIdx);
				if (EndIdx == -1)
				{
					break;
				}

				// Skip over any windows CR-LF line endings
				int LineEndIdx = EndIdx;
				if (LineEndIdx > BaseIdx && InContent[LineEndIdx - 1] == '\r')
				{
					LineEndIdx--;
				}

				// Render any JSON log events
				string Line = InContent.Substring(BaseIdx, LineEndIdx - BaseIdx);
				try
				{
					Line = SanitizeJsonOutputLine(Line, true);
				}
				catch
				{
					int MinIdx = Math.Max(BaseIdx - 2048, 0);
					int MaxIdx = Math.Min(BaseIdx + 2048, InContent.Length);

					string[] Context = InContent.Substring(MinIdx, MaxIdx - MinIdx).Split('\n');
					for (int idx = 1; idx < Context.Length - 1; idx++)
					{
						EpicGames.Core.Log.Logger.LogDebug("Context {Idx}: {Line}", idx, Context[idx].TrimEnd());
					}
				}

				ContentBuilder.Append(Line);
				ContentBuilder.Append('\n');

				// Move to the next line
				BaseIdx = EndIdx + 1;
			}

			return ContentBuilder.ToString();
		}

		public static string SanitizeJsonOutputLine(string Line, bool ThrowOnFailure = false)
		{
			if (Line.Length > 0 && Line[0] == '{')
			{
				try
				{
					byte[] Buffer = Encoding.UTF8.GetBytes(Line);
					JsonLogEvent JsonEvent = JsonLogEvent.Parse(Buffer);
					Line = JsonEvent.GetLegacyLogLine();
				}
				catch (Exception ex)
				{
					EpicGames.Core.Log.Logger.LogDebug(ex, "Unable to parse log line: {Line}, Exception: {Ex}", Line, ex.ToString());
					if (ThrowOnFailure)
					{
						throw;
					}
				}
			}

			return Line;
		}

		public UnrealLog GetSummary() => _summary.Value;
		protected UnrealLog CreateSummary() => new UnrealLog(this);


		/// <summary>
		/// Returns all lines from the specified content match the specified regex
		/// </summary>
		/// <param name="InLogReader"></param>
		/// <param name="InPattern"></param>
		/// <param name="InOptions"></param>
		/// <returns></returns>
		protected IEnumerable<Match> GetAllMatches(ILogStreamReader InLogReader, string InPattern, RegexOptions InOptions = RegexOptions.None)
		{
			Regex regex = new Regex(InPattern, InOptions);

			InLogReader.SetLineIndex(0);
			foreach (string Line in InLogReader.EnumerateNextLines())
			{
				Match M = regex.Match(Line);
				if (!M.Success) continue;
				yield return M;
			}
		}

		/// <summary>
		/// Returns all lines from the specified content match the specified regex
		/// </summary>
		/// <param name="InLogReader"></param>
		/// <param name="InPattern"></param>
		/// <param name="InOptions"></param>
		/// <returns></returns>
		protected IEnumerable<string> GetAllMatchingLines(ILogStreamReader InLogReader, string InPattern, RegexOptions InOptions = RegexOptions.None)
		{
			return GetAllMatches(InLogReader, InPattern, InOptions).Select(M => M.Value);
		}

		/// <summary>
		/// Returns all lines from the specified content match the specified regex
		/// </summary>
		/// <param name="InContent"></param>
		/// <param name="InPattern"></param>
		/// <param name="InOptions"></param>
		/// <returns></returns>
		protected string[] GetAllMatchingLines(string InContent, string InPattern, RegexOptions InOptions = RegexOptions.None)
		{
			return GetAllMatchingLines(new DynamicStringReader(new(() => InContent)), InPattern, InOptions).ToArray();
		}

		/// <summary>
		/// Returns all lines that match the specified regex
		/// </summary>
		/// <param name="InPattern"></param>
		/// <param name="InOptions"></param>
		/// <returns></returns>
		public string[] GetAllMatchingLines(string InPattern, RegexOptions InOptions = RegexOptions.None)
		{
			return GetAllMatchingLines(_logReader, InPattern, InOptions).ToArray();
		}

		/// <summary>
		/// Returns all Matches that match the specified regex
		/// </summary>
		/// <param name="InPattern"></param>
		/// <param name="InOptions"></param>
		/// <returns></returns>
		public IEnumerable<Match> GetAllMatches(string InPattern, RegexOptions InOptions = RegexOptions.None)
		{
			return GetAllMatches(_logReader, InPattern, InOptions);
		}

		/// <summary>
		/// Returns all lines containing the specified substring
		/// </summary>
		/// <param name="Substring"></param>
		/// <param name="Options"></param>
		/// <returns></returns>
		public IEnumerable<string> GetAllContainingLines(string Substring, StringComparison Options = StringComparison.Ordinal)
		{
			_logReader.SetLineIndex(0);
			foreach (string Line in _logReader.EnumerateNextLines())
			{
				if (!Line.Contains(Substring, Options)) continue;
				yield return Line;
			}
		}

		/// <summary>
		/// Match regex pattern and execute callback with group values passed as argument 
		/// </summary>
		/// <param name="InPattern"></param>
		/// <param name="InFunc"></param>
		/// <param name="InOptions"></param>
		public void MatchAndApplyGroups(string InPattern, Action<string[]> InFunc, RegexOptions InOptions = RegexOptions.None)
		{
			Match M = GetAllMatches(InPattern, InOptions).FirstOrDefault();
			if (M != null && M.Success)
			{
				InFunc(M.Groups.Values.Select(G => G.Value).ToArray());
			}
		}

		/// <summary>
		/// Returns a structure containing platform information extracted from the log
		/// </summary>
		/// <returns></returns>
		public UnrealLog.PlatformInfo GetPlatformInfo()
		{
			var Info = new UnrealLog.PlatformInfo();

			const string InfoRegEx = @"LogInit.+OS:\s*(.+?)\s*(\((.+)\))?,\s*CPU:\s*(.+)\s*,\s*GPU:\s*(.+)";
			MatchAndApplyGroups(InfoRegEx, (Groups) =>
			{
				Info.OSName = Groups[1];
				Info.OSVersion = Groups[3];
				Info.CPUName = Groups[4];
				Info.GPUName = Groups[5];
			});

			return Info;
		}

		/// <summary>
		/// Returns a structure containing build information extracted from the log
		/// </summary>
		/// <returns></returns>
		public UnrealLog.BuildInfo GetBuildInfo()
		{
			var Info = new UnrealLog.BuildInfo();

			// pull from Branch Name: <name>
			Match M = GetAllMatches(@"LogInit.+Name:\s*(.*)", RegexOptions.IgnoreCase).FirstOrDefault();
			if (M != null && M.Success)
			{
				Info.BranchName = M.Groups[1].Value;
				Info.BranchName = Info.BranchName.Replace("+", "/");
			}

			M = GetAllMatches(@"LogInit.+CL-(\d+)", RegexOptions.IgnoreCase).FirstOrDefault();
			if (M != null && M.Success)
			{
				Info.Changelist = Convert.ToInt32(M.Groups[1].Value);
			}

			M = GetAllMatches(@"LogInit.+Build:\s*(\+.*)", RegexOptions.IgnoreCase).FirstOrDefault();
			if (M != null && M.Success)
			{
				Info.BuildVersion = M.Groups[1].Value;
			}

			return Info;
		}

		/// <summary>
		/// Returns all entries from the log that have the specified level
		/// </summary>
		/// <param name="InLevel"></param>
		/// <returns></returns>
		public IEnumerable<UnrealLog.LogEntry> GetEntriesOfLevel(UnrealLog.LogLevel InLevel)
		{
			IEnumerable<UnrealLog.LogEntry> Entries = LogEntries.Where(E => E.Level == InLevel);
			return Entries;
		}

		/// <summary>
		/// Returns all warnings from the log
		/// </summary>
		/// <param name="InCategories"></param>
		/// <param name="ExactMatch"></param>
		/// <returns></returns>
		public IEnumerable<UnrealLog.LogEntry> GetEntriesOfCategories(IEnumerable<string> InCategories, bool ExactMatch = false)
		{
			IEnumerable<UnrealLog.LogEntry> Entries;

			if (ExactMatch)
			{
				Entries = LogEntries.Where(E => InCategories.Contains(E.Category, StringComparer.OrdinalIgnoreCase));
			}
			else
			{
				// check if each channel is a substring of each log entry. E.g. "Shader" should return entries
				// with both ShaderCompiler and ShaderManager
				Entries = LogEntries.Where(E =>
				{
					string LogEntryCategory = E.Category;
					foreach (string Cat in InCategories)
					{
						if (LogEntryCategory.IndexOf(Cat, StringComparison.OrdinalIgnoreCase) >= 0)
						{
							return true;
						}
					}

					return false;
				});
			}
			return Entries;
		}

		/// <summary>
		/// Return all entries for the specified channel. E.g. "OrionGame" will
		/// return all entries starting with LogOrionGame
		/// </summary>
		/// <param name="Channels"></param>
		/// <param name="ExactMatch"></param>
		/// <returns></returns>
		public IEnumerable<string> GetLogChannels(IEnumerable<string> Channels, bool ExactMatch = true)
		{
			return GetEntriesOfCategories(Channels, ExactMatch).Select(E => E.ToString());
		}

		/// <summary>
		/// Returns channels that signify the editor doing stuff
		/// </summary>
		/// <returns></returns>
		public IEnumerable<string> GetEditorBusyChannels()
		{
			return GetLogChannels(UnrealLog.EditorBusyChannels, false);
		}

		/// <summary>
		/// Return all entries for the specified channel. E.g. "OrionGame" will
		/// return all entries starting with LogOrionGame
		/// </summary>
		/// <param name="Channel"></param>
		/// <param name="ExactMatch"></param>
		/// <returns></returns>
		public IEnumerable<string> GetLogChannel(string Channel, bool ExactMatch = true)
		{
			return GetLogChannels(new string[] { Channel }, ExactMatch);
		}


		/// <summary>
		/// Returns all warnings from the log
		/// </summary>
		/// <param name="InChannel">Optional channel to restrict search to</param>
		/// <returns></returns>
		public IEnumerable<string> GetWarnings(string InChannel = null)
		{
			IEnumerable<UnrealLog.LogEntry> Entries = LogEntries.Where(E => E.Level == UnrealLog.LogLevel.Warning);

			if (InChannel != null)
			{
				Entries = Entries.Where(E => E.Category.Equals(InChannel, StringComparison.OrdinalIgnoreCase));
			}

			return Entries.Select(E => E.ToString());
		}

		/// <summary>
		/// Returns all errors from the log
		/// </summary>
		/// <param name="InChannel">Optional channel to restrict search to</param>
		/// <returns></returns>
		public IEnumerable<string> GetErrors(string InChannel = null)
		{
			IEnumerable<UnrealLog.LogEntry> Entries = LogEntries.Where(E => E.Level == UnrealLog.LogLevel.Error);

			if (InChannel != null)
			{
				Entries = Entries.Where(E => E.Category.Equals(InChannel, StringComparison.OrdinalIgnoreCase));
			}

			return Entries.Select(E => E.ToString());
		}

		/// <summary>
		/// Returns all ensures from the log
		/// </summary>
		/// <returns></returns>
		public IEnumerable<UnrealLog.CallstackMessage> GetEnsures()
		{
			IEnumerable<UnrealLog.CallstackMessage> Ensures = ParseTracedErrors(new[] { @"Log.+:\s{0,1}Error:\s{0,1}(Ensure condition failed:.+)" }, 10);

			foreach (UnrealLog.CallstackMessage Error in Ensures)
			{
				Error.IsEnsure = true;
			}

			return Ensures;
		}

		/// <summary>
		/// If the log contains a fatal error return that information
		/// </summary>
		/// <returns></returns>
		public UnrealLog.CallstackMessage GetFatalError()
		{
			string[] ErrorMsgMatches = new string[] { @"(Fatal Error:.+)", @"Critical error: =+\s+(?:[\S\s]+?\s*Error: +)?(.+)", @"(Assertion Failed:.+)", @"(Unhandled Exception:.+)", @"(LowLevelFatalError.+)", @"(Postmortem Cause:.*)" };

			var Traces = ParseTracedErrors(ErrorMsgMatches, 5).Concat(GetASanErrors());

			// If we have a post-mortem error, return that one (on some devices the post-mortem info is way more informative).
			var PostMortemTraces = Traces.Where(T => T.Message.IndexOf("Postmortem Cause:", StringComparison.OrdinalIgnoreCase) > -1);
			if (PostMortemTraces.Any())
			{
				Traces = PostMortemTraces;
			}
			// Keep the one with the most information.
			return Traces.Count() > 0 ? Traces.OrderBy(T => T.Callstack.Length).Last() : null;
		}

		/// <summary>
		/// Parse the log for Address Sanitizer error.
		/// </summary>
		/// <returns></returns>
		public IEnumerable<UnrealLog.CallstackMessage> GetASanErrors()
		{
			// Match:
			// ==5077==ERROR: AddressSanitizer: alloc - dealloc - mismatch(operator new vs free) on 0x602014ab4790
			// Then for gathering the callstack, match
			// ==5077==ABORTING
			// remove anything inside the callstack starting with [2022.12.02-15.22.40:688][618]

			List<UnrealLog.CallstackMessage> ASanReports = new List<UnrealLog.CallstackMessage>();
			Regex InitPattern = SanitizerEventMatcher.ReportLevelPattern;
			Regex EndPattern = SanitizerEventMatcher.ReportEndPattern;
			Regex LineStartsWithTimeStamp = new Regex(@"^[\s\t]*\[[0-9.:-]+\]\[[\s0-9]+\]");

			UnrealLog.CallstackMessage NewTrace = null;
			List<string> Backtrace = null;

			Action AddTraceToList = () =>
			{
				if (Backtrace.Count == 0)
				{
					Backtrace.Add("Unable to parse callstack from log");
				}
				NewTrace.Callstack = Backtrace.ToArray();
				ASanReports.Add(NewTrace);
				NewTrace = null;
				Backtrace = null;
			};

			_logReader.SetLineIndex(0);
			foreach (string Line in _logReader.EnumerateNextLines())
			{
				if (NewTrace == null)
				{
					Match TraceInitMatch = InitPattern.Match(Line);
					if (TraceInitMatch.Success && SanitizerEventMatcher.ConvertReportLevel(TraceInitMatch.Groups["ReportLevel"].Value) == Logging.LogLevel.Error)
					{
						NewTrace = new UnrealLog.CallstackMessage();
						NewTrace.IsSanReport = true;
						NewTrace.Position = _logReader.GetLineIndex() - 1;
						NewTrace.Message = $"{TraceInitMatch.Groups["SanitizerName"].Value}Sanitizer: {TraceInitMatch.Groups["Summary"].Value}";
						Backtrace = new List<string>();
					}

					continue;
				}

				if (EndPattern.IsMatch(Line))
				{
					AddTraceToList();
				}
				else
				{
					// Prune the line with UE log timestamp
					if (!LineStartsWithTimeStamp.IsMatch(Line))
					{
						Backtrace.Add(Line);
					}
				}
			}

			if (NewTrace != null)
			{
				// Happen if end of log is reached before the EndPattern is found
				AddTraceToList();
			}

			return ASanReports;
		}

		/// <summary>
		/// Returns true if the log contains a test complete marker
		/// </summary>
		/// <returns></returns>
		public bool HasTestCompleteMarker() => GetAllMatchingLines(@"\*\*\* TEST COMPLETE.+").Any();

		/// <summary>
		/// Returns true if the log contains a request to exit that was not due to an error
		/// </summary>
		/// <returns></returns>
		public bool HasRequestExit()
		{
			return GetSummary().RequestedExit;
		}

		/// <summary>
		/// Returns a block of lines that start and end with the specified regex patterns
		/// </summary>
		/// <param name="StartPattern">Regex to match the first line</param>
		/// <param name="EndPattern">Regex to match the final line</param>
		/// <param name="PatternOptions">Optional RegexOptions applied to both patterns. IgnoreCase by default.</param>
		/// <returns>Array of strings for each found block of lines. Lines within each string are delimited by newline character.</returns>
		public string[] GetGroupsOfLinesBetween(string StartPattern, string EndPattern, RegexOptions PatternOptions = RegexOptions.IgnoreCase)
		{
			Regex StartRegex = new Regex(StartPattern, PatternOptions);
			Regex EndRegex = new Regex(EndPattern, PatternOptions);
			List<string> Blocks = new List<string>();
			List<string> Block = null;

			_logReader.SetLineIndex(0);
			foreach (string Line in _logReader.EnumerateNextLines())
			{
				if (Block == null)
				{
					if (!StartRegex.IsMatch(Line)) continue;
					Block = new(){ Line };
				}
				else
				{
					Block.Add(Line);
					if (EndRegex.IsMatch(Line))
					{
						Blocks.Add(string.Join('\n', Block));
						Block = null;
					}
				}
			}

			return Blocks.ToArray();
		}

		/// <summary>
		/// Returns a block of lines that start with the specified regex
		/// </summary>
		/// <param name="Pattern">Regex to match the first line</param>
		/// <param name="LineCount">Number of lines in the returned block</param>
		/// <param name="PatternOptions"></param>
		/// <returns>Array of strings for each found block of lines. Lines within each string are delimited by newline character.</returns>
		public string[] GetGroupsOfLinesStartingWith(string Pattern, int LineCount, RegexOptions PatternOptions = RegexOptions.IgnoreCase)
		{
			Regex RegexPattern = new Regex(Pattern, PatternOptions);
			List<string> Blocks = new List<string>();
			List<string> Block = null;

			_logReader.SetLineIndex(0);
			foreach (string Line in _logReader.EnumerateNextLines())
			{
				if (Block == null)
				{
					if (!RegexPattern.IsMatch(Line)) continue;
					Block = new() { Line };
				}
				else
				{
					Block.Add(Line);
				}

				if (Block.Count >= LineCount)
				{
					Blocks.Add(string.Join('\n', Block));
					Block = null;
				}
			}

			return Blocks.ToArray();
		}

		/// <summary>
		/// Finds all callstack-based errors with the specified pattern
		/// </summary>
		/// <param name="Patterns"></param>
		/// <param name="Limit">Limit the number of errors to parse with trace per pattern. Zero means no limit.</param>
		/// <returns></returns>
		protected IEnumerable<UnrealLog.CallstackMessage> ParseTracedErrors(string[] Patterns, int Limit = 0)
		{
			List<UnrealLog.CallstackMessage> Traces = new List<UnrealLog.CallstackMessage>();
			Dictionary<string, (Regex Pattern, int Remaining)> RegexPatterns = new();
			foreach(string Pattern in Patterns)
			{
				RegexPatterns.Add(Pattern, (new Regex(Pattern, RegexOptions.IgnoreCase), Limit));
			};
			_logReader.SetLineIndex(0);
			foreach (string Line in _logReader.EnumerateNextLines())
			{
				if (Limit > 0 && RegexPatterns.Count == 0) break;
				if (string.IsNullOrEmpty(Line)) continue;
				// Try and find an error message
				string SelectedPattern = null;
				foreach (var Entry in RegexPatterns)
				{
					Match TraceMatch = Entry.Value.Pattern.Match(Line);
					if (TraceMatch.Success)
					{
						UnrealLog.CallstackMessage NewTrace = new UnrealLog.CallstackMessage();
						NewTrace.Position = _logReader.GetLineIndex() - 1;
						NewTrace.Message = TraceMatch.Groups[1].Value;
						SelectedPattern = Entry.Key;
						Traces.Add(NewTrace);
						break;
					}
				}
				// Track pattern match limit
				if (Limit > 0 && !string.IsNullOrEmpty(SelectedPattern))
				{
					var SelectedItem = RegexPatterns[SelectedPattern];
					int Remaining = SelectedItem.Remaining - 1;
					if (Remaining <= 0)
					{
						// Limit reached, we remove the pattern from collection
						RegexPatterns.Remove(SelectedPattern);
					}
					else
					{
						// Update count
						RegexPatterns[SelectedPattern] = (SelectedItem.Pattern, Remaining);
					}
				}
			}
			//
			// Handing callstacks-
			//
			// Unreal now uses a canonical format for printing callstacks during errors which is 
			//
			//0xaddress module!func [file]
			// 
			// E.g. 0x045C8D01 OrionClient.self!UEngine::PerformError() [D:\Epic\Orion\Engine\Source\Runtime\Engine\Private\UnrealEngine.cpp:6481]
			//
			// Module may be omitted, everything else should be present, or substituted with a string that conforms to the expected type
			//
			// E.g 0x00000000 UnknownFunction []
			//
			// A callstack as part of an ensure, check, or exception will look something like this -
			// 
			//
			//[2017.08.21-03.28.40:667][313]LogWindows:Error: Assertion failed: false [File:D:\Epic\Orion\Release-Next\Engine\Plugins\NotForLicensees\Gauntlet\Source\Gauntlet\Private\GauntletTestControllerErrorTest.cpp] [Line: 29] 
			//[2017.08.21-03.28.40:667][313]LogWindows:Error: Asserting as requested
			//[2017.08.21-03.28.40:667][313]LogWindows:Error: 
			//[2017.08.21-03.28.40:667][313]LogWindows:Error: 
			//[2017.08.21-03.28.40:667][313]LogWindows:Error: [Callstack] 0x00000000FDC2A06D KERNELBASE.dll!UnknownFunction []
			//[2017.08.21-03.28.40:667][313]LogWindows:Error: [Callstack] 0x00000000418C0119 OrionClient.exe!FOutputDeviceWindowsError::Serialize() [d:\epic\orion\release-next\engine\source\runtime\core\private\windows\windowsplatformoutputdevices.cpp:120]
			//[2017.08.21-03.28.40:667][313]LogWindows:Error: [Callstack] 0x00000000416AC12B OrionClient.exe!FOutputDevice::Logf__VA() [d:\epic\orion\release-next\engine\source\runtime\core\private\misc\outputdevice.cpp:70]
			//[2017.08.21-03.28.40:667][313]LogWindows:Error: [Callstack] 0x00000000418BD124 OrionClient.exe!FDebug::AssertFailed() [d:\epic\orion\release-next\engine\source\runtime\core\private\misc\assertionmacros.cpp:373]
			//[2017.08.21-03.28.40:668][313]LogWindows:Error: [Callstack] 0x000000004604A879 OrionClient.exe!UGauntletTestControllerErrorTest::OnTick() [d:\epic\orion\release-next\engine\plugins\notforlicensees\gauntlet\source\gauntlet\private\gauntlettestcontrollererrortest.cpp:29]
			//[2017.08.21-03.28.40:668][313]LogWindows:Error: [Callstack] 0x0000000046049166 OrionClient.exe!FGauntletModuleImpl::InnerTick() [d:\epic\orion\release-next\engine\plugins\notforlicensees\gauntlet\source\gauntlet\private\gauntletmodule.cpp:315]
			//[2017.08.21-03.28.40:668][313]LogWindows:Error: [Callstack] 0x0000000046048472 OrionClient.exe!TBaseFunctorDelegateInstance<bool __cdecl(float),<lambda_b2e6da8e95d7ed933c391f0ec034aa11> >::Execute() [d:\epic\orion\release-next\engine\source\runtime\core\public\delegates\delegateinstancesimpl.h:1132]
			//[2017.08.21-03.28.40:668][313]LogWindows:Error: [Callstack] 0x00000000415101BE OrionClient.exe!FTicker::Tick() [d:\epic\orion\release-next\engine\source\runtime\core\private\containers\ticker.cpp:82]
			//[2017.08.21-03.28.40:668][313]LogWindows:Error: [Callstack] 0x00000000402887DD OrionClient.exe!FEngineLoop::Tick() [d:\epic\orion\release-next\engine\source\runtime\launch\private\launchengineloop.cpp:3295]
			//[2017.08.21-03.28.40:668][313]LogWindows:Error: [Callstack] 0x00000000402961FC OrionClient.exe!GuardedMain() [d:\epic\orion\release-next\engine\source\runtime\launch\private\launch.cpp:166]
			//[2017.08.21-03.28.40:668][313]LogWindows:Error: [Callstack] 0x000000004029625A OrionClient.exe!GuardedMainWrapper() [d:\epic\orion\release-next\engine\source\runtime\launch\private\windows\launchwindows.cpp:134]
			//[2017.08.21-03.28.40:668][313]LogWindows:Error: [Callstack] 0x00000000402A2D68 OrionClient.exe!WinMain() [d:\epic\orion\release-next\engine\source\runtime\launch\private\windows\launchwindows.cpp:210]
			//[2017.08.21-03.28.40:669][313]LogWindows:Error: [Callstack] 0x0000000046EEC0CB OrionClient.exe!__scrt_common_main_seh() [f:\dd\vctools\crt\vcstartup\src\startup\exe_common.inl:253]
			//[2017.08.21-03.28.40:669][313]LogWindows:Error: [Callstack] 0x0000000077A759CD kernel32.dll!UnknownFunction []
			//[2017.08.21-03.28.40:669][313]LogWindows:Error: [Callstack] 0x0000000077CAA561 ntdll.dll!UnknownFunction []
			//[2017.08.21-03.28.40:669][313]LogWindows:Error: [Callstack] 0x0000000077CAA561 ntdll.dll!UnknownFunction []
			//
			// So the code below starts at the point of the error message, and searches subsequent lines for things that look like a callstack. If we go too many lines without 
			// finding one then we break. Note that it's possible that log messages from another thread may be intermixed, so we can't just break on a change of verbosity or 
			// channel
			// 
			// Must contain 0x00123456 module name [filename]
			// The module name is optional, must start with whitespace, and continues until next white space follow by [
			// filename is optional, must be in [filename]
			// module address is optional, must be in quotes() after address
			// Note - Unreal callstacks are always meant to omit all three with placeholders for missing values, but
			// we'll assume that may not happen...

			Regex CallstackMatch = new Regex(@"(0[xX][0-9A-f]{8,16})(?:\s+\(0[xX][0-9A-f]{8,16}\))?\s+(.+?)\s+\[(.*?)\][^\w]*$");
			Regex ExtraErrorLine = new Regex(@".+:\s*Error:\s*"); 
			Regex TimestampMatch = new Regex(@"\[.+\]\[.+\]Log\w+\:"); 
			foreach (UnrealLog.CallstackMessage NewTrace in Traces)
			{
				List<string> Backtrace = new List<string>();
				int LinesWithoutBacktrace = 0;
				// Move to Trace next line index
				_logReader.SetLineIndex(NewTrace.Position + 1);
				foreach (string Line in _logReader.EnumerateNextLines())
				{
					if (string.IsNullOrEmpty(Line)) continue;
					Match CSMatch = CallstackMatch.Match(Line);
					if (CSMatch.Success)
					{
						// Callstack pattern found
						string Address = CSMatch.Groups[1].Value;
						string Func = CSMatch.Groups[2].Value;
						string File = CSMatch.Groups[3].Value;

						if (string.IsNullOrEmpty(File))
						{
							File = "Unknown File";
						}

						// Remove any exe
						const string StripFrom = ".exe!";
						if (Func.IndexOf(StripFrom) > 0)
						{
							Func = Func.Substring(Func.IndexOf(StripFrom) + StripFrom.Length);
						}

						Backtrace.Add($"{Address} {Func} [{File}]");

						LinesWithoutBacktrace = 0;
					}
					else
					{
						if (Backtrace.Count == 0)
						{
							// Add additional summary error lines before backtrace lines are found
							Match NLMatch = TimestampMatch.Match(Line);
							if (!NLMatch.Success)
							{ //New log line
								NewTrace.Message += "\n" + Line;
							}
							else
							{
								Match MsgMatch = ExtraErrorLine.Match(Line);
								if (MsgMatch.Success)
								{ // Line with error tag
									string MsgString = Line.Substring(MsgMatch.Index + MsgMatch.Length).Trim();
									if (string.IsNullOrEmpty(MsgString)) continue;
									NewTrace.Message += "\n" + MsgString;
								}
							}
						}

						LinesWithoutBacktrace++;
					}

					if (LinesWithoutBacktrace >= 10)
					{
						// No more callstack line found, stop parsing
						break;
					}
				}

				NewTrace.Callstack = Backtrace.Count > 0? Backtrace.Distinct().ToArray() :  new[] { "Unable to parse callstack from log" };
			}

			UnrealLog.CallstackMessage PreviousTrace = null;
			return Traces.Where(Trace =>
			{
				// Because platforms sometimes dump asserts to the log and low-level logging, we need to prune out redundancies.
				// Basic approach: find errors with the same assert message and keep the one with the longest callstack.
				if (PreviousTrace != null && Trace.Message.Equals(PreviousTrace.Message, StringComparison.OrdinalIgnoreCase))
				{
					if (PreviousTrace.Callstack.Length < Trace.Callstack.Length)
					{
						PreviousTrace.Callstack = Trace.Callstack;
					}
					return false;
				}
				PreviousTrace = Trace;
				return true;
			}).ToList(); // Force execution here with ToList() to have the duplicates pruned only once.
		}

		/// <summary>
		/// Attempts to find an exit code for a test
		/// </summary>
		/// <param name="ExitCode"></param>
		/// <returns></returns>
		public bool GetTestExitCode(out int ExitCode)
		{
			Match M = GetAllMatches(@"\*\s+TEST COMPLETE. EXIT CODE:\s*(-?\d?)\s+\*").FirstOrDefault();
			if (M != null && M.Success && M.Groups.Count > 1)
			{
				ExitCode = Convert.ToInt32(M.Groups[1].Value);
				return true;
			}

			M = GetAllMatches(@"RequestExitWithStatus\(\d+,\s*(\d+).*\)").FirstOrDefault();
			if (M != null && M.Success && M.Groups.Count > 1)
			{
				ExitCode = Convert.ToInt32(M.Groups[1].Value);
				return true;
			}

			if (GetAllContainingLines("EnvironmentalPerfTest summary").Any())
			{
				Log.Warning("Found - 'EnvironmentalPerfTest summary', using temp workaround and assuming success (!)");
				ExitCode = 0;
				return true;
			}

			ExitCode = -1;
			return false;
		}
	}
}