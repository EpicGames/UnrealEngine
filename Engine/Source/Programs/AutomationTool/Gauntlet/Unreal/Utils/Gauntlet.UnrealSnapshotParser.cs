// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Linq;
using System.Text;
using System.Threading.Tasks;
using System.Globalization;

namespace Gauntlet
{
	public class UnrealHealthSnapshot
	{
		public string Name;
		public float ProfileLength;

		public float? CpuUsedMemory;
		public float? CpuPeakMemory;
		public float? PhysicalUsedMemory;
		public float? PhysicalPeakMemory;
		public float? StreamingUsedMemory;
		public float? StreamingPeakMemory;
		public float? MVP;
		public float? AvgFps;
		public float? Hitches;
		public float? AvgHitches;
		public float? DynamicRes;
		public float? GTTime;
		public float? RTTime;
		public float? GPUTime;
		public float? FTTime;
		public float? RHIT;
		public int? DrawCalls;
		public int? DrawnPrims;
		public int? UnbuiltHLODs;

		public UnrealHealthSnapshot()
		{
			Name = "Unknown";
			ProfileLength = 0;
		}

		public override string ToString()
		{
			StringBuilder SB = new StringBuilder();

			SB.AppendFormat("Snapshot {0}\n", Name);
			SB.AppendFormat("Duration\t{0}\n", ProfileLength);

			if (CpuUsedMemory.HasValue)
			{
				SB.AppendFormat("CpuUsedMemory:\t\t{0} MB\n", CpuUsedMemory.Value);
			}
			if (CpuPeakMemory.HasValue)
			{
				SB.AppendFormat("CpuPeakMemory:\t\t{0} MB\n", CpuPeakMemory.Value);
			}
			if (PhysicalUsedMemory.HasValue)
			{
				SB.AppendFormat("PhysicalUsedMemory:\t\t{0} MB\n", PhysicalUsedMemory.Value);
			}
			if (PhysicalPeakMemory.HasValue)
			{
				SB.AppendFormat("PhysicalPeakMemory:\t\t{0} MB\n", PhysicalPeakMemory.Value);
			}
			if (StreamingUsedMemory.HasValue)
			{
				SB.AppendFormat("StreamingUsedMemory:\t\t{0} MB\n", StreamingUsedMemory.Value);
			}
			if (StreamingPeakMemory.HasValue)
			{
				SB.AppendFormat("StreamingPeakMemory:\t\t{0} MB\n", StreamingPeakMemory.Value);
			}

			if (ProfileLength > 0)
			{
				if (MVP.HasValue)
				{
					SB.AppendFormat("MVP:\t\t{0:0.00}\n", MVP.Value);
				}
				if (AvgFps.HasValue)
				{
					SB.AppendFormat("AvgFps:\t\t{0:0.00}\n", AvgFps.Value);
				}
				if (Hitches.HasValue)
				{
					SB.AppendFormat("HPM:\t\t{0:0.00}\n", Hitches.Value);
				}
				if (AvgHitches.HasValue)
				{
					SB.AppendFormat("AvgH:\t\t{0:0.00}ms\n", AvgHitches.Value);
				}
				if (DynamicRes.HasValue)
				{
					SB.AppendFormat("DynRes:\t\t{0:0.00}\n", DynamicRes.Value);
				}
				if (GTTime.HasValue)
				{
					SB.AppendFormat("GT:\t\t{0:0.00}ms\n", GTTime.Value);
				}
				if (RTTime.HasValue)
				{
					SB.AppendFormat("RT:\t\t{0:0.00}ms\n", RTTime.Value);
				}
				if (GPUTime.HasValue)
				{
					SB.AppendFormat("GPU:\t\t{0:0.00}ms\n", GPUTime.Value);
				}
				if (FTTime.HasValue)
				{
					SB.AppendFormat("FT:\t\t{0:0.00}ms\n", FTTime.Value);
				}
				if (RHIT.HasValue)
				{
					SB.AppendFormat("RHIT:\t\t{0:0.00}ms\n", RHIT.Value);
				}
				if (DrawCalls.HasValue)
				{
					SB.AppendFormat("DrawCalls:\t{0}\n", DrawCalls.Value);
				}
				if (DrawnPrims.HasValue)
				{
					SB.AppendFormat("DrawnPrims:\t{0}\n", DrawnPrims.Value);
				}
				if (UnbuiltHLODs.HasValue)
				{
					SB.AppendFormat("UnbuiltHLODs:\t{0}\n", UnbuiltHLODs.Value);
				}
			}
			else
			{
				SB.Append("No info\n");
			}
			return SB.ToString();
		}

		public virtual void CreateFromString(string InContent)
		{
			RegexUtil.MatchAndApplyGroups(InContent, @".+Snapshot:\s+(.+?)\s+=", (Groups) =>
			{
				Name = Groups[1];
			});

			RegexUtil.MatchAndApplyGroups(InContent, @"MeasuredPerfTime:\s(\d.+) S", (Groups) =>
			{
				ProfileLength = ParseSingleInvariant(Groups[1]);
			});

			RegexUtil.MatchAndApplyGroups(InContent, @"CPU Memory:[\s\w]+?([\d\.]+)MB,[\s\w]+?([\d\.]+)MB", (Groups) =>
			{
				CpuUsedMemory = ParseSingleInvariant(Groups[1]);
				CpuPeakMemory = ParseSingleInvariant(Groups[2]);
			});

			RegexUtil.MatchAndApplyGroups(InContent, @"Physical Memory:[\s\w]+?([\d\.]+)MB,[\s\w:]+?([\d\.]+)MB", (Groups) =>
			{
				PhysicalUsedMemory = ParseSingleInvariant(Groups[1]);
				PhysicalPeakMemory = ParseSingleInvariant(Groups[2]);
			});

			RegexUtil.MatchAndApplyGroups(InContent, @"Streaming Memory:[\s\w]+?([\d\.]+)MB,[\s\w:]+?([\d\.]+)MB", (Groups) =>
			{
				StreamingUsedMemory = ParseSingleInvariant(Groups[1]);
				StreamingPeakMemory = ParseSingleInvariant(Groups[2]);
			});

			RegexUtil.MatchAndApplyGroups(InContent, @"MVP:\s(\d.+)%,\s.*AvgFPS:(\d.+),\s.*HitchesPerMinute:\s(\d.+),\sAvg\sHitch\s(\d.+)ms", (Groups) =>
			{
				MVP = ParseSingleInvariant(Groups[1]);
				AvgFps = ParseSingleInvariant(Groups[2]);
				Hitches = ParseSingleInvariant(Groups[3]);
				AvgHitches = ParseSingleInvariant(Groups[4]);
			});

			RegexUtil.MatchAndApplyGroups(InContent, @"GT:.* Avg\s(\d.+)ms,", (Groups) =>
			{
				GTTime = ParseSingleInvariant(Groups[1]);
			});

			RegexUtil.MatchAndApplyGroups(InContent, @"RT:.* Avg\s(\d.+)ms,", (Groups) =>
			{
				RTTime = ParseSingleInvariant(Groups[1]);
			});

			RegexUtil.MatchAndApplyGroups(InContent, @"GPU:.* Avg\s(\d.+)ms,", (Groups) =>
			{
				GPUTime = ParseSingleInvariant(Groups[1]);
			});

			RegexUtil.MatchAndApplyGroups(InContent, @"FT:.* Avg:\s([\d\.]+)ms,", (Groups) =>
			{
				FTTime = ParseSingleInvariant(Groups[1]);
			});

			RegexUtil.MatchAndApplyGroups(InContent, @"RHIT:.* Avg\s([\d\.]+)ms,", (Groups) =>
			{
				RHIT = ParseSingleInvariant(Groups[1]);
			});

			RegexUtil.MatchAndApplyGroups(InContent, @"DynRes:.* Avg:\s([\d\.]+)%,", (Groups) =>
			{
				DynamicRes = ParseSingleInvariant(Groups[1]);
			});

			RegexUtil.MatchAndApplyGroups(InContent, @"DrawCalls:[\w\s:]+?([\d\.]+),[\w\s:]+?([\d\.]+),[\w\s:]+?([\d\.]+)", (Groups) =>
			{
				DrawCalls = ParseIntInvariant(Groups[1]);
			});

			RegexUtil.MatchAndApplyGroups(InContent, @"DrawnPrims:[\w\s:]+?([\d\.]+),[\w\s:]+?([\d\.]+),[\w\s:]+?([\d\.]+)", (Groups) =>
			{
				DrawnPrims = ParseIntInvariant(Groups[1]);
			});

			RegexUtil.MatchAndApplyGroups(InContent, @"UnbuiltHLODs:\s(\d.+)", (Groups) =>
			{
				UnbuiltHLODs = ParseIntInvariant(Groups[1]);
			});
		}

		private static float ParseSingleInvariant(string Text)
		{
			return Single.Parse(Text, CultureInfo.InvariantCulture);
		}
		
		private static int ParseIntInvariant(string Text)
		{
			return int.Parse(Text, CultureInfo.InvariantCulture);
		}
	}

	public class UnrealSnapshotSummary<TSnapshotClass>
		where TSnapshotClass : UnrealHealthSnapshot, new()
	{
		public int SampleCount;
		public float SessionTime;
		public float PeakMemory;

		public List<float> MVP;
		public List<float> AvgFps;
		public List<float> Hitches;
		public List<float> AvgHitches;
		public List<float> DynamicRes;
		public List<float> GTTime;
		public List<float> RTTime;
		public List<float> GPUTime;
		public List<float> FTTime;
		public List<float> RHIT;
		public List<int> DrawCalls;
		public List<int> DrawnPrims;
		public List<int> UnbuiltHLODs;

		public IEnumerable<TSnapshotClass> Snapshots;

		public UnrealSnapshotSummary(ILogStreamReader LogReader, string SnapshotTitles="")
		{
			SampleCount = 0;
			SessionTime = 0.0f;
			PeakMemory = 0.0f;
			MVP = new List<float>();
			AvgFps = new List<float>();
			Hitches = new List<float>();
			AvgHitches = new List<float>();
			DynamicRes = new List<float>();
			GTTime = new List<float>();
			RTTime = new List<float>();
			GPUTime = new List<float>();
			FTTime = new List<float>();
			RHIT = new List<float>();
			DrawCalls = new List<int>();
			DrawnPrims = new List<int>();
			UnbuiltHLODs = new List<int>();

			CreateFromLog(LogReader, SnapshotTitles);
		}

		public UnrealSnapshotSummary(string InContent, string SnapshotTitles = "")
			: this(new DynamicStringReader(() => InContent), SnapshotTitles)
		{ }

		protected virtual void CreateFromLog(ILogStreamReader LogReader, string InTitle)
		{
			UnrealLogParser Parser = new UnrealLogParser(LogReader);

			if (string.IsNullOrEmpty(InTitle))
			{
				InTitle = "=== Snapshot: ";
			}

			try
			{
				// Find all end of match reports
				string[] SessionSnapshots = Parser.GetGroupsOfLinesBetween(InTitle, "==============");

				SampleCount = SessionSnapshots.Length;
				SessionTime = 0;

				// convert these blocks into snapshot info
				Snapshots = SessionSnapshots.Select(S =>
				{
					var Snapshot = new TSnapshotClass();
					Snapshot.CreateFromString(S);

					return Snapshot;
				}).ToList();

				// get average MVP, GT, RT, GPU times
				foreach (UnrealHealthSnapshot Snap in Snapshots)
				{
					SessionTime += Snap.ProfileLength;

					if (Snap.MVP.HasValue)
					{
						MVP.Add(Snap.MVP.Value);
					}

					if (Snap.AvgFps.HasValue)
					{
						AvgFps.Add(Snap.AvgFps.Value);
					}

					if (Snap.Hitches.HasValue)
					{
						Hitches.Add(Snap.Hitches.Value);
					}

					if (Snap.AvgHitches.HasValue)
					{
						AvgHitches.Add(Snap.AvgHitches.Value);
					}

					if (Snap.DynamicRes.HasValue)
					{
						DynamicRes.Add(Snap.DynamicRes.Value);
					}

					if (Snap.GTTime.HasValue)
					{
						GTTime.Add(Snap.GTTime.Value);
					}

					if (Snap.RTTime.HasValue)
					{
						RTTime.Add(Snap.RTTime.Value);
					}

					if (Snap.GPUTime.HasValue)
					{
						GPUTime.Add(Snap.GPUTime.Value);
					}

					if (Snap.FTTime.HasValue)
					{
						FTTime.Add(Snap.FTTime.Value);
					}

					if (Snap.RHIT.HasValue)
					{
						RHIT.Add(Snap.RHIT.Value);
					}

					if (Snap.DrawCalls.HasValue)
					{
						DrawCalls.Add(Snap.DrawCalls.Value);
					}

					if (Snap.DrawnPrims.HasValue)
					{
						DrawnPrims.Add(Snap.DrawnPrims.Value);
					}

					if (Snap.UnbuiltHLODs.HasValue)
					{
						UnbuiltHLODs.Add(Snap.UnbuiltHLODs.Value);
					}
				}

				// Now get peak memory from the last health report
				if (Snapshots.Count() > 0)
				{
					var LastSnapshot = Snapshots.Last();

					float? SnapshotPeakMemory = LastSnapshot.PhysicalPeakMemory;

					//if PeakMemory is reporting as 0, use Memory if it's higher than our last
					if (SnapshotPeakMemory.HasValue)
					{
						PeakMemory = SnapshotPeakMemory.Value;
					}
					else
					{
						Log.Info("PeakMemory reported as 0mb");
					}
				}
			}
			catch (Exception Ex)
			{
				Log.Info("Failed parsing PerformanceSummary: " + Ex.ToString());
			}
		}

		public override string ToString()
		{
			StringBuilder SB = new StringBuilder();

			if (SampleCount > 0)
			{
				SB.AppendFormat("Performance report from {0} samples and {1} seconds\n", SampleCount, SessionTime);
				SB.AppendFormat("Peak Memory: {0} MB\n", PeakMemory);

				if (MVP.Count > 0)
				{
					SB.AppendFormat("MVP:\t{0:0.00} (Min: {1:0.00}, Max: {2:0.00})\n", MVP.Average(), MVP.Min(), MVP.Max());
				}

				if (AvgFps.Count > 0)
				{
					SB.AppendFormat("AvgFPS:\t{0:0.00} (Min: {1:0.00}, Max: {2:0.00})\n", AvgFps.Average(), AvgFps.Min(), AvgFps.Max());
				}

				if (Hitches.Count > 0)
				{
					SB.AppendFormat("HPM:\t{0:0.00} (Min: {1:0.00}, Max: {2:0.00})\n", Hitches.Average(), Hitches.Min(), Hitches.Max());
				}

				if (AvgHitches.Count > 0)
				{
					SB.AppendFormat("AvgH:\t{0:0.00}ms (Min: {1:0.00}ms, Max: {2:0.00}ms)\n", AvgHitches.Average(), AvgHitches.Min(), AvgHitches.Max());
				}

				if (DynamicRes.Count > 0)
				{
					SB.AppendFormat("DynRes:\t{0:0.00} (Min: {1:0.00}, Max: {2:0.00})\n", DynamicRes.Average(), DynamicRes.Min(), DynamicRes.Max());
				}

				if (GTTime.Count > 0)
				{
					SB.AppendFormat("GT:\t{0:0.00}ms (Min: {1:0.00}ms, Max: {2:0.00}ms)\n", GTTime.Average(), GTTime.Min(), GTTime.Max());
				}

				if (RTTime.Count > 0)
				{
					SB.AppendFormat("RT:\t{0:0.00}ms (Min: {1:0.00}ms, Max: {2:0.00}ms)\n", RTTime.Average(), RTTime.Min(), RTTime.Max());
				}

				if (GPUTime.Count > 0)
				{
					SB.AppendFormat("GPU:\t{0:0.00}ms (Min: {1:0.00}ms, Max: {2:0.00}ms)\n", GPUTime.Average(), GPUTime.Min(), GPUTime.Max());
				}

				if (FTTime.Count > 0)
				{
					SB.AppendFormat("FT:\t{0:0.00}ms (Min: {1:0.00}ms, Max: {2:0.00}ms)\n", FTTime.Average(), FTTime.Min(), FTTime.Max());
				}

				if (RHIT.Count > 0)
				{
					SB.AppendFormat("RHIT:\t{0:0.00}ms (Min: {1:0.00}ms, Max: {2:0.00}ms)\n", RHIT.Average(), RHIT.Min(), RHIT.Max());
				}

				if (DrawCalls.Count > 0)
				{
					SB.AppendFormat("DrawCalls:\t{0:0} (Min: {1}, Max: {2})\n", DrawCalls.Average(), DrawCalls.Min(), DrawCalls.Max());
				}

				if (DrawnPrims.Count > 0)
				{
					SB.AppendFormat("DrawnPrims:\t{0:0} (Min: {1}, Max: {2})\n", DrawnPrims.Average(), DrawnPrims.Min(), DrawnPrims.Max());
				}
			}
			else
			{
				SB.Append("No session info");
			}
			return SB.ToString();
		}
	}

}
