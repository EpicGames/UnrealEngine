// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.IO;
using System.Data;
using System.Text;
using System.Linq;
using System.Collections.Generic;
using System.Text.RegularExpressions;
using AutomationTool;
using UnrealBuildTool;
using UnrealBuildBase;

namespace Gauntlet
{
	/// <summary>
	/// Utility functions for Unreal Automation Telemetry
	/// </summary>
	static class UnrealAutomationTelemetry
	{
		/// <summary>
		/// Gather csv telemetry outputs generated by UE automated tests and add them to a test report
		/// </summary>
		/// <param name="TelemetryDirectory"></param>
		/// <param name="TelemetryReport"></param>
		/// <param name="Mapping"></param>
		static public void LoadOutputsIntoReport(string TelemetryDirectory, ITelemetryReport TelemetryReport, Dictionary<string, string> Mapping = null)
		{
			// Scan for csv files
			foreach (string File in Directory.GetFiles(TelemetryDirectory))
			{
				if (Path.GetExtension(File) != ".csv")
				{
					continue;
				}

				LoadCSVOutputIntoReport(File, TelemetryReport, Mapping);
			}
		}

		/// <summary>
		/// Gather telemetry data from csv file generated by UE automated tests and add them to a test report
		/// </summary>
		/// <param name="File"></param>
		/// <param name="TelemetryReport"></param>
		/// <param name="Mapping"></param>
		static public void LoadCSVOutputIntoReport(string File, ITelemetryReport TelemetryReport, Dictionary<string, string> Mapping = null)
		{
			List<Dictionary<string, string>> CSVData = null;
			try
			{
				CSVData = CSVParser.Load(File);
			}
			catch (Exception Ex)
			{
				Log.Error("Telemetry - Failed to read CSV file '{0}'. {1}", File, Ex);
			}

			if (CSVData == null)
			{
				return;
			}
			string TestNameKey = CSVColumns.TestName.ToString();
			string DataPointKey = CSVColumns.DataPoint.ToString();
			string MeasurementKey = CSVColumns.Measurement.ToString();
			string ContextKey = CSVColumns.Context.ToString();
			string UnitKey = CSVColumns.Unit.ToString();
			string BaselineKey = CSVColumns.Baseline.ToString();
			if (Mapping != null)
			{
				if (Mapping.ContainsKey(TestNameKey))
					Mapping.TryGetValue(TestNameKey, out TestNameKey);
				if (Mapping.ContainsKey(DataPointKey))
					Mapping.TryGetValue(DataPointKey, out DataPointKey);
				if (Mapping.ContainsKey(MeasurementKey))
					Mapping.TryGetValue(MeasurementKey, out MeasurementKey);
				if (Mapping.ContainsKey(ContextKey))
					Mapping.TryGetValue(ContextKey, out ContextKey);
				if (Mapping.ContainsKey(UnitKey))
					Mapping.TryGetValue(UnitKey, out UnitKey);
				if (Mapping.ContainsKey(BaselineKey))
					Mapping.TryGetValue(BaselineKey, out BaselineKey);
			}
			// Populate telemetry report
			foreach (Dictionary<string, string> Row in CSVData)
			{
				string TestName;
				Row.TryGetValue(TestNameKey, out TestName);
				string DataPoint;
				Row.TryGetValue(DataPointKey, out DataPoint);
				string Measurement;
				Row.TryGetValue(MeasurementKey, out Measurement);
				string Context;
				Row.TryGetValue(ContextKey, out Context);
				string Unit;
				Row.TryGetValue(UnitKey, out Unit);
				string Baseline;
				Row.TryGetValue(BaselineKey, out Baseline);
				double BaselineFloat = 0;
				double.TryParse(Baseline, out BaselineFloat);
				if (string.IsNullOrEmpty(TestName) || string.IsNullOrEmpty(DataPoint) || string.IsNullOrEmpty(Measurement))
				{
					Log.Warning("Telemetry - Missing data in CSV file '{0}':\n TestName='{1}', DataPoint='{2}', Measurement='{3}'", File, TestName, DataPoint, Measurement);
					continue;
				}
				try
				{
					TelemetryReport.AddTelemetry(TestName, DataPoint, double.Parse(Measurement), Context, Unit, BaselineFloat);
				}
				catch (FormatException)
				{
					Log.Error("Telemetry - Failed to parse Measurement value('{0}') in CSV file '{1}'.", Measurement, File);
				}
			}
		}

		public enum CSVColumns
		{
			TestName,
			DataPoint,
			Measurement,
			Context,
			Unit,
			Baseline
		}

		static public void WriteDataTableToCSV(DataTable Data, string OutputFile)
        {
			StringBuilder OutputBuffer = new StringBuilder();

			IEnumerable<string> ColumnNames = Data.Columns.Cast<DataColumn>().Select(C => C.ColumnName);
			OutputBuffer.AppendLine(string.Join(",", ColumnNames));

			foreach (DataRow Row in Data.Rows)
			{
				IEnumerable<string> Fields = Row.ItemArray.Select(F => F.ToString());
				OutputBuffer.AppendLine(string.Join(",", Fields));
			}

			string CSVFolder = Path.GetDirectoryName(OutputFile);
			if (!Directory.Exists(CSVFolder))
			{
				Directory.CreateDirectory(CSVFolder);
			}
			File.WriteAllText(OutputFile, OutputBuffer.ToString());
		}

	}

	public class UnrealTelemetryContext : ITelemetryContext
	{
		public object GetProperty(string Name)
		{
			object Value;
			Properties.TryGetValue(Name, out Value);
			return Value;
		}
		Dictionary<string, object> Properties;

		public UnrealTelemetryContext()
		{
			Properties = new Dictionary<string, object>();
		}
		public void SetProperty(string Name, object Value)
		{
			Properties[Name] = Value;
		}
	}

	/// <summary>
	/// Build command to read Unreal Automation telemetry data saved in csv format and then publish it to a database.
	/// </summary>
	class PublishUnrealAutomationTelemetry : BuildCommand
	{
		[Help("CSVFile", "Path to the csv file to parse.")]
		[Help("CSVDirectory", "Path to a folder containing csv files to parse.")]
		[Help("CSVMapping", "Optional CSV column mapping. Format is: <target key>:<source key>,...")]
		[Help("TelemetryConfig", "Telemetry configuration to use to publish to Database. Default: UETelemetryStaging.")]
		[Help("DatabaseConfigPath", "Path to alternate Database config. Default is TelemetryConfig default.")]
		[Help("Project", "Target Project name.")]
		[Help("Platform", "Target platform name. Default: current environment platform.")]
		[Help("Role", "Target Role name. Default: Editor.")]
		[Help("Branch", "Target Branch name. Default: Unknown.")]
		[Help("Changelist", "Target Changelist number. Default: 0.")]
		[Help("Configuration", "Target Configuration name. Default: Development.")]
		[Help("JobLink", "Http Link to Build Job.")]

		public override ExitCode Execute()
		{
			string CSVFile = ParseParamValue("CSVFile=", "");
			string CSVDirectory = ParseParamValue("CSVDirectory=", "");
			string Config = ParseParamValue("TelemetryConfig=", "UETelemetryStaging");
			string DatabaseConfigPath = ParseParamValue("DatabaseConfigPath=", "");
			List<string> CSVMappingStrings = Globals.Params.ParseValues("CSVMapping", true);
			string ProjectString = ParseParamValue("Project=", "");
			string PlatformString = ParseParamValue("Platform=", "");
			string BranchString = ParseParamValue("Branch=", "Unknown");
			string ChangelistString = ParseParamValue("Changelist=", "0");
			string RoleString = ParseParamValue("Role=", "Editor");
			string ConfigurationString = ParseParamValue("Configuration=", "Development");
			string JobLink = ParseParamValue("JobLink=", "");

			if (string.IsNullOrEmpty(CSVDirectory) && string.IsNullOrEmpty(CSVFile))
			{
				throw new AutomationException("No telemetry files specified. Use -CSVDirectory=<path> or -CSVFile=<path>.csv");
			}

			if (!string.IsNullOrEmpty(CSVDirectory) && !Directory.Exists(CSVDirectory))
			{
				throw new AutomationException(string.Format("CSVDirectory '{0}' is missing.", CSVDirectory));
			}
			if (!string.IsNullOrEmpty(CSVFile) && !File.Exists(CSVFile))
			{
				throw new AutomationException(string.Format("CSVFile '{0}' is missing.", CSVFile));
			}

			// CSV Mapping
			Dictionary<string, string> CSVMapping = null;
			if (CSVMappingStrings.Count > 0)
			{
				CSVMapping = new Dictionary<string, string>();
				foreach (string StringLine in CSVMappingStrings)
				{
					var SplitLine = StringLine.Split(":");
					if (SplitLine.Length != 2)
					{
						throw new AutomationException("CSVMapping entry must be in the form <target key>:<source key>.");
					}
					string TargetKey = SplitLine[0];
					string SourceKey = SplitLine[1];
					object Target;
					if (!Enum.TryParse(typeof(UnrealAutomationTelemetry.CSVColumns), TargetKey, true, out Target))
					{
						string AllKeys = string.Join(", ", Enum.GetNames(typeof(UnrealAutomationTelemetry.CSVColumns)));
						throw new AutomationException(string.Format("Unknown target key '{0}', CSVMapping target key must be one of the values: {1}.", TargetKey, AllKeys));
					}
					TargetKey = ((UnrealAutomationTelemetry.CSVColumns)Target).ToString();
					CSVMapping[TargetKey] = SourceKey;
				}
			}

			// Get Context
			UnrealTelemetryContext Context = new UnrealTelemetryContext();

			if (string.IsNullOrEmpty(ProjectString))
			{
				throw new AutomationException("No project specified. Use -Project=ShooterGame etc");
			}
			Context.SetProperty("ProjectName", ProjectString);

			object Role;
			if (!Enum.TryParse(typeof(UnrealTargetRole), RoleString, true, out Role))
			{
				string AllKeys = string.Join(", ", Enum.GetNames(typeof(UnrealTargetRole)));
				throw new AutomationException(string.Format("Unknown Role '{0}', it must be one of the values: {1}.", RoleString, AllKeys));
			}
			RoleString = ((UnrealTargetRole)Role).ToString();

			object Configuration;
			if (!Enum.TryParse(typeof(UnrealTargetConfiguration), ConfigurationString, true, out Configuration))
			{
				string AllKeys = string.Join(", ", Enum.GetNames(typeof(UnrealTargetConfiguration)));
				throw new AutomationException(string.Format("Unknown Configuration '{0}', it must be one of the values: {1}.", ConfigurationString, AllKeys));
			}
			ConfigurationString = ((UnrealTargetConfiguration)Configuration).ToString();

			Context.SetProperty("Configuration", string.Format("{0} {1}", RoleString, ConfigurationString));

			Context.SetProperty("Platform", string.IsNullOrEmpty(PlatformString) ? BuildHostPlatform.Current.Platform : UnrealTargetPlatform.Parse(PlatformString));

			Context.SetProperty("Branch", BranchString);
			Context.SetProperty("Changelist", ChangelistString);
			Context.SetProperty("ChangelistDateTime", GetChangelistDateTime(int.Parse(ChangelistString)));

			Context.SetProperty("JobLink", JobLink);

			// Create a report
			HordeReport.SimpleTestReport Report = new HordeReport.SimpleTestReport();

			if (Report is ITelemetryReport TelemetryReport)
			{
				// Parse CSV
				if (!string.IsNullOrEmpty(CSVDirectory))
				{
					UnrealAutomationTelemetry.LoadOutputsIntoReport(CSVDirectory, TelemetryReport, CSVMapping);
				}
				else
				{
					UnrealAutomationTelemetry.LoadCSVOutputIntoReport(CSVFile, TelemetryReport, CSVMapping);
				}

				// Publish Telemetry Data
				var DataRows = TelemetryReport.GetAllTelemetryData();

				if (DataRows != null)
				{
					IDatabaseConfig<TelemetryData> DBConfig = DatabaseConfigManager<TelemetryData>.GetConfigByName(Config);
					if (DBConfig != null)
					{
						DBConfig.LoadConfig(DatabaseConfigPath);
						IDatabaseDriver<TelemetryData> DB = DBConfig.GetDriver();
						Log.Info("Submitting telemetry data to {0}", DB.ToString());
						DB.SubmitDataItems(DataRows, Context);
					}
					else
					{
						Log.Warning("Got telemetry data, but database configuration is unknown '{0}'.", Config);
					}
				}
				else
				{
					Log.Error("Failed to retrieve any data from telemetry report.");
				}

			}

			return 0;
		}

		/// <summary>
		/// Get the DateTime of the given changelist
		/// </summary>
		/// <param name="Changelist"></param>
		/// <returns></returns>
		private DateTime GetChangelistDateTime(int Changelist)
		{
			try
			{
				P4Connection.DescribeRecord Record = new P4Connection.DescribeRecord();
				P4.DescribeChangelist(Changelist, out Record, false);

				Regex Regx = new Regex(@" on ([0-9/]+ [0-9:]+)", RegexOptions.Compiled | RegexOptions.IgnoreCase);
				Match DateMatch = Regx.Match(Record.Header);
				return DateTime.Parse(DateMatch.Groups[1].Value);
			}
			catch
			{
				return DateTime.Now;
			}
		}

	}

	/// <summary>
	/// Build command to fetch Unreal Automation telemetry data from a database and save it to csv.
	/// </summary>
	class FetchUnrealAutomationTelemetry : BuildCommand
	{
		[Help("CSVFile", "Path of the csv file to save.")]
		[Help("TelemetryConfig", "Telemetry configuration to use to publish to Database. Default: UETelemetryStaging.")]
		[Help("DatabaseConfigPath", "Path to alternate Database config. Default is TelemetryConfig default.")]
		[Help("Project", "Target Project name.")]
		[Help("Platform", "Target platform name. Default: current environment platform.")]
		[Help("Role", "Target Role name. Default: Editor.")]
		[Help("Branch", "Target Branch name. Default: Unknown.")]
		[Help("Configuration", "Target Configuration name. Default: Development.")]
		[Help("Since", "Filter fetch data from the last 'Since' time. Default: 1month.")]
		[Help("TestName", "Filter fetch by TestName. Support coma separated list.")]
		[Help("DataPoint", "Filter fetch by DataPoint. Support coma separated list.")]
		[Help("Context", "Filter fetch by Context. Support coma separated list.")]

		public override ExitCode Execute()
		{
			string CSVFile = ParseParamValue("CSVFile=", "");
			string Config = ParseParamValue("TelemetryConfig=", "UETelemetryStaging");
			string DatabaseConfigPath = ParseParamValue("DatabaseConfigPath=", "");
			string ProjectString = ParseParamValue("Project=", "");
			string PlatformString = ParseParamValue("Platform=", "");
			string BranchString = ParseParamValue("Branch=", "Unknown");
			string RoleString = ParseParamValue("Role=", "Editor");
			string ConfigurationString = ParseParamValue("Configuration=", "Development");

			string Since = ParseParamValue("Since=", "1month");
			string TestName = ParseParamValue("TestName=", "");
			string DataPoint = ParseParamValue("DataPoint=", "");
			string Context = ParseParamValue("Context=", "");

			if (string.IsNullOrEmpty(CSVFile))
			{
				throw new AutomationException("CSVFile argument is missing.");
			}

			UnrealTelemetryContext Conditions = new UnrealTelemetryContext();
			if (string.IsNullOrEmpty(ProjectString))
			{
				throw new AutomationException("No project specified. Use -Project=ShooterGame etc");
			}
			Conditions.SetProperty("ProjectName", ProjectString);

			object Role;
			if (!Enum.TryParse(typeof(UnrealTargetRole), RoleString, true, out Role))
			{
				string AllKeys = string.Join(", ", Enum.GetNames(typeof(UnrealTargetRole)));
				throw new AutomationException(string.Format("Unknown Role '{0}', it must be one of the values: {1}.", RoleString, AllKeys));
			}
			RoleString = ((UnrealTargetRole)Role).ToString();

			object Configuration;
			if (!Enum.TryParse(typeof(UnrealTargetConfiguration), ConfigurationString, true, out Configuration))
			{
				string AllKeys = string.Join(", ", Enum.GetNames(typeof(UnrealTargetConfiguration)));
				throw new AutomationException(string.Format("Unknown Configuration '{0}', it must be one of the values: {1}.", ConfigurationString, AllKeys));
			}
			ConfigurationString = ((UnrealTargetConfiguration)Configuration).ToString();
			Conditions.SetProperty("Configuration", string.Format("{0} {1}", RoleString, ConfigurationString));

			UnrealTargetPlatform Platform = string.IsNullOrEmpty(PlatformString) ? BuildHostPlatform.Current.Platform : UnrealTargetPlatform.Parse(PlatformString);
			Conditions.SetProperty("Platform", Platform.ToString());

			Conditions.SetProperty("Branch", BranchString);

			if (!string.IsNullOrEmpty(TestName))
			{
				Conditions.SetProperty("TestName", TestName);
			}
			if (!string.IsNullOrEmpty(DataPoint))
			{
				Conditions.SetProperty("DataPoint", DataPoint);
			}
			if (!string.IsNullOrEmpty(Context))
			{
				Conditions.SetProperty("Context", Context);
			}

			Conditions.SetProperty("ChangelistDateTime", SinceStringToDate(Since));

			IDatabaseConfig<TelemetryData> DBConfig = DatabaseConfigManager<TelemetryData>.GetConfigByName(Config);
			if (DBConfig != null)
			{
				DBConfig.LoadConfig(DatabaseConfigPath);
				IDatabaseDriver<TelemetryData> DB = DBConfig.GetDriver();

				var Results = DB.FetchData(Conditions);

				if(Results is null || Results.Tables.Count == 0)
                {
					throw new AutomationException("No result returned from the DB with the requested conditions.");
                }

				Log.Info("Successfully retrived {0} rows from the last {1}.", Results.Tables[0].Rows.Count, Since);

				UnrealAutomationTelemetry.WriteDataTableToCSV(Results.Tables[0], CSVFile);
			}

			return 0;
		}

		private DateTime SinceStringToDate(string SinceString)
		{
			Regex Regx = new Regex(@"([0-9]+)\s*([dwmy])", RegexOptions.Compiled | RegexOptions.IgnoreCase);
			Match DateMatch = Regx.Match(SinceString);

			DateTime SinceDate = DateTime.Now;

			if (DateMatch.Success)
            {
				string NumberString = DateMatch.Groups[1].Value;
				string TimeSpanString = DateMatch.Groups[2].Value.Substring(0, 1).ToLower();

				int Factor = Int32.Parse(NumberString);
				switch(TimeSpanString)
                {
					case "d":
						SinceDate = SinceDate.AddDays(-Factor);
						break;
					case "w":
						SinceDate = SinceDate.AddDays(-Factor * 7);
						break;
					case "m":
						SinceDate = SinceDate.AddMonths(-Factor);
						break;
					case "y":
						SinceDate = SinceDate.AddYears(-Factor);
						break;
				}
			}

			return SinceDate;
		}

	}

}
