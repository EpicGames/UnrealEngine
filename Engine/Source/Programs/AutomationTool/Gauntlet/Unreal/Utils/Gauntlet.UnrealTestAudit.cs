// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Text.RegularExpressions;
using System.Collections.Generic;
using Gauntlet.Report;
using System.Linq;

#nullable enable

namespace Gauntlet
{
	namespace Report
	{
		/// <summary>
		/// Test profile (unit, functional, performance, etc)
		/// </summary>
		public partial class TestProfile
		{
			public static readonly TestProfile Unit = new TestProfile("Unit");
			public static readonly TestProfile Functional = new TestProfile("Functional");
			public static readonly TestProfile Performance = new TestProfile("Performance");
			public static readonly TestProfile Smoke = new TestProfile("Smoke");
		}

		/// <summary>
		/// Test harness
		/// </summary>
		public partial class TestHarness
		{
			public static readonly TestHarness UEFunctionalTest = new TestHarness("UE Functional Test");
			public static readonly TestHarness Gauntlet = new TestHarness("Gauntlet");
			public static readonly TestHarness GauntletRPC = new TestHarness("Gauntlet RPC");
			public static readonly TestHarness LLT = new TestHarness("Low Level Test");
		}

		/// <summary>
		/// Test team
		/// </summary>
		public partial class TestTeam
		{
			public static readonly TestTeam UE = new TestTeam("UE");
		}

		/// <summary>
		/// Test tag
		/// </summary>
		public partial class TestTag
		{			
		}

	}

	/// <summary>
	/// Unreal Test metadata component for Audit
	/// </summary>
	public class UnrealTestMetadata : ITestAuditMetadata
	{
		public UnrealTestMetadata(string Key, string Name, string NodeClass, Version Version, float Timeout, string? CommandLine = null, TestHarness? Harness = null, TestProfile? Profile = null, TestTeam? Team = null, string? Intent = null, TestTag[]? Tags = null, Dictionary<string, object>? Params = null)
		{
			_key = () => Key;
			_name = () => Name;
			_class = () => NodeClass;
			_version =() => Version;
			_timeout = () => Timeout;
			_commandline = () => CommandLine;
			_harness = () => Harness;
			_profile = () => Profile;
			_team = () => Team;
			_intent = () => Intent;
			_tags = () => Tags;
			_params = () => Params;
		}
		public UnrealTestMetadata(Func<string> Key, Func<string> Name, Func<string> NodeClass, Func<Version> Version, Func<float> Timeout, Func<string?>? CommandLine = null, Func<TestHarness?>? Harness = null, Func<TestProfile?>? Profile = null, Func<TestTeam?>? Team = null, Func<string?>? Intent = null, Func<TestTag[]?>? Tags = null, Func<Dictionary<string, object>>? Params = null)
		{
			_key = Key;
			_name = Name;
			_class = NodeClass;
			_version = Version;
			_timeout = Timeout;
			_commandline = CommandLine;
			_harness = Harness;
			_profile = Profile;
			_team = Team;
			_intent = Intent;
			_tags = Tags;
			_params = Params;
		}

		Func<string> _key;
		Func<string> _name;
		Func<string> _class;
		Func<Version> _version;
		Func<float> _timeout;
		Func<string?>? _commandline;
		Func<TestHarness?>? _harness;
		Func<TestProfile?>? _profile;
		Func<TestTeam?>? _team;
		Func<string?>? _intent;
		Func<TestTag[]?>? _tags;
		Func<Dictionary<string, object>?>? _params;

		static Regex AbsoluteFilePathPattern = new Regex(@"^(?:[/\\]+|[a-z]:[/\\]|https?://)", RegexOptions.Compiled | RegexOptions.IgnoreCase);

		/* Unique identifier */
		public string TestKey => _key();

		/* Test name for display */
		public string TestName => _name();

		/* TestNode class */
		public string TestNodeClass => _class();

		/* Test version */
		public Version TestVersion => _version();

		/* Duration in minutes after which the test run has timeout */
		public float TestTimeoutMinutes => _timeout();

		/* Command line used to run the test */
		public string? TestCommandLine => _commandline?.Invoke();

		/* Test harness */
		public TestHarness? TestHarness => _harness?.Invoke();

		/* Test result profile */
		public TestProfile? TestProfile => _profile?.Invoke();

		/* Team to which the test was written for */
		public TestTeam? TestTeam => _team?.Invoke();

		/* The purpose of the test */
		public string? TestIntent => _intent?.Invoke();

		/* Tags associated with the test */
		public TestTag[]? TestTags => _tags?.Invoke();

		/* Params relevant to run this test */
		public Dictionary<string, object>? TestParams => _params?.Invoke()?.Where(P => P.Value is string ? !AbsoluteFilePathPattern.IsMatch((string)P.Value) : true).ToDictionary();
	}
}
