// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.IO;
using System.Collections.Generic;
using System.Text.Json;
using System.Text.Json.Serialization;
using System.Linq;
using AutomationTool;
using EpicGames.Core;
using EpicGames.LogMatchers;
using Microsoft.Extensions.Logging;
using Logging = Microsoft.Extensions.Logging;
using System.Globalization;
using System.Text.RegularExpressions;

namespace Gauntlet
{
	public class UnrealAutomationDevice
	{
		[JsonPropertyName("deviceName")]
		public string DeviceName { get; set; }
		[JsonPropertyName("instance")] 
		public string Instance { get; set; }
		[JsonPropertyName("instanceName")]
		public string InstanceName { get; set; }
		[JsonPropertyName("platform")]
		public string Platform { get; set; }
		[JsonPropertyName("oSversion")]
		public string OSVersion { get; set; }
		[JsonPropertyName("model")]
		public string Model { get; set; }
		[JsonPropertyName("gPU")]
		public string GPU { get; set; }
		[JsonPropertyName("cPUModel")]
		public string CPUModel { get; set; }
		[JsonPropertyName("rAMInGB")]
		public int RAMInGB { get; set; }
		[JsonPropertyName("renderMode")]
		public string RenderMode { get; set; }
		[JsonPropertyName("rHI")]
		public string RHI { get; set; }
		[JsonPropertyName("appInstanceLog")]
		public string AppInstanceLog { get; set; }
	}
	public class UnrealAutomationArtifact
	{
		[JsonPropertyName("id")]
		public string Id { get; set; }
		[JsonPropertyName("name")]
		public string Name { get; set; }
		[JsonPropertyName("type")]
		public string Type { get; set; }
		[JsonPropertyName("files")]
		public Dictionary<string, string> Files { get; set; }
	}
	public class UnrealAutomationEvent
	{
		[JsonPropertyName("type")]
		public EventType Type { get; set; }
		[JsonPropertyName("message")]
		public string Message { get; set; }
		[JsonPropertyName("context")]
		public string Context { get; set; }
		[JsonPropertyName("artifact")]
		public string Artifact { get; set; }

		const string CriticalFailureString = "critical failure";
		const string SanitizerReportString = "Sanitizer: ";

		public UnrealAutomationEvent()
		{ }
		public UnrealAutomationEvent(EventType InType, string InMessage, bool bIsCriticalFailure = false)
		{
			Type = InType;
			Message = InMessage;
			if(bIsCriticalFailure)
			{
				SetAsCriticalFailure();
			}
		}

		[JsonIgnore]
		public bool IsError
		{
			get
			{
				return Type == EventType.Error;
			}

		}

		[JsonIgnore]
		public bool IsWarning
		{
			get
			{
				return Type == EventType.Warning;
			}

		}

		[JsonIgnore]
		public bool IsCriticalFailure
		{
			get
			{
				return IsError && Message.Contains(CriticalFailureString);
			}
		}
		public void SetAsCriticalFailure()
		{
			if (!IsCriticalFailure)
			{
				Type = EventType.Error;
				Message = $"Engine encountered a {CriticalFailureString}. \n" + Message;
			}
		}

		/// <summary>
		/// True if the event contains a Sanitizer report
		/// </summary>
		[JsonIgnore]
		public bool IsSanReport
		{
			get
			{
				return IsError && Message.Contains(SanitizerReportString);
			}
		}

		public string FormatToString()
		{
			return Message;
		}

		protected bool Equals(UnrealAutomationEvent other)
		{
			return string.Equals(Type, other.Type) && string.Equals(Message, other.Message);
		}

		public override bool Equals(object obj)
		{
			if (ReferenceEquals(null, obj)) return false;
			if (ReferenceEquals(this, obj)) return true;
			if (obj.GetType() != this.GetType()) return false;
			return Equals((UnrealAutomationEvent)obj);
		}

		public override int GetHashCode()
		{
			return Type.GetHashCode() ^ Message.GetHashCode();
		}

	}
	public class UnrealAutomationEntry
	{
		[JsonPropertyName("event")]
		public UnrealAutomationEvent Event { get; set; }
		[JsonPropertyName("filename")]
		public string Filename { get; set; }
		[JsonPropertyName("lineNumber")]
		public int LineNumber { get; set; }
		[JsonPropertyName("timestamp")]
		public string Timestamp { get; set; }

		public const string InvalidDateTime = "0001.01.01-00.00.00";
		public const string DateTimeFormat = "yyyy.MM.dd-HH.mm.ss";

		/// <summary>
		/// Regex pattern to match assertion, ensure and other crash report summary from UE
		/// ie: Assertion failed: !This->CurrentlySerializingObject [File:.\\Runtime/CoreUObject/Private/Misc/GCObjectReferencer.cpp] [Line: 54] 
		/// </summary>
		private static Regex SummaryPattern = new Regex(@"\w[\w ]+:(?:\s+.+)?\s+\[[^]]+\.[^]]+\](?:\s*\[[^]]+\])?(?:\s*\n\w+.+)? *", RegexOptions.Multiline | RegexOptions.ExplicitCapture);

		public static DateTime GetTimestampAsDateTime(string StringTime)
		{
			DateTime Time = DateTime.UtcNow;
			if (!string.IsNullOrEmpty(StringTime) && StringTime != InvalidDateTime)
			{
				DateTime.TryParseExact(StringTime, DateTimeFormat, CultureInfo.InvariantCulture, DateTimeStyles.None, out Time);
			}
			return Time;
		}

		/// <summary>
		/// Produce a LogEvent instance from this UnrealAutomationEntry
		/// </summary>
		/// <returns></returns>
		public LogEvent AsLogEvent()
		{
			DateTime Time = GetTimestampAsDateTime(Timestamp);
			Logging.LogLevel Level = Event.IsError ?
										(Event.IsCriticalFailure ?
											Logging.LogLevel.Critical : Logging.LogLevel.Error)
										: (Event.IsWarning ?
											Logging.LogLevel.Warning : Logging.LogLevel.Information);

			EventId EventIdType = KnownLogEvents.Gauntlet_UnrealEngineTestEvent;

			string Message = Event.FormatToString();
			string Format = null;
			Dictionary<string, object> Properties = null;
			if (Event.IsCriticalFailure)
			{
				Properties = new Dictionary<string, object>();
				if (Event.IsSanReport
					&& SanitizerEventMatcher.AddSanitizerSummaryProperties(ref Message, Properties))
				{
					Format = Message;
					EventIdType = SanitizerEventMatcher.ConvertSanitizerNameToEventId(Properties.GetValueOrDefault("SanitizerName")?.ToString());
				}
				else
				{
					Match MatchSummary = SummaryPattern.Match(Message);
					if (MatchSummary.Success)
					{
						Group MatchedGroup = MatchSummary.Groups[0];
						int Index = MatchedGroup.Index;
						Format = $"{MessageTemplate.Escape(Message.Substring(0, Index))}{{Summary}}\n{{Callstack}}";
						Properties.Add("Summary", MatchedGroup.Value);
						Properties.Add("Callstack", Message.Substring(Index + MatchedGroup.Length).TrimStart('\n'));
					}
					else
					{
						Properties.Add("Callstack", Message);
						Format = "{Callstack}";
					}
				}
			}
			else
			{
				if (!string.IsNullOrEmpty(Event.Context))
				{
					Properties = new Dictionary<string, object>() { { "Context", Event.Context } };
					Format = "[{Context}] " + (Format ?? MessageTemplate.Escape(Message));
				}
				if (!string.IsNullOrEmpty(Filename))
				{
					if (Properties == null)
					{
						Properties = new Dictionary<string, object>();
					}
					Properties.Add("SourceFile", Filename);
					Properties.Add("Line", LineNumber.ToString());
					Format = (Format ?? MessageTemplate.Escape(Message)) + " [{SourceFile}({Line})]";
				}
			}

			if (Format != null)
			{
				// Make sure Message is produced from Format
				Message = MessageTemplate.Render(Format, Properties);
			}

			return new LogEvent(Time, Level, EventIdType, Message, Format, Properties, null);
		}

		protected bool Equals(UnrealAutomationEntry other)
		{
			return string.Equals(Event, other.Event);
		}

		public override bool Equals(object obj)
		{
			if (ReferenceEquals(null, obj)) return false;
			if (ReferenceEquals(this, obj)) return true;
			if (obj.GetType() != this.GetType()) return false;
			return Equals((UnrealAutomationEntry)obj);
		}

		public override int GetHashCode()
		{
			return Event.GetHashCode();
		}
	}
	public class UnrealAutomatedTestResult
	{
		[JsonPropertyName("testDisplayName")]
		public string TestDisplayName { get; set; }
		[JsonPropertyName("fullTestPath")]
		public string FullTestPath { get; set; }
		[JsonPropertyName("tags")]
		public List<string> Tags { get; set; }
		[JsonPropertyName("artifactName")]
		public string ArtifactName { get; set; }
		[JsonPropertyName("state")]
		public TestStateType State { get; set; }
		[JsonPropertyName("deviceInstance")]
		public List<string> DeviceInstance { get; set; }
		[JsonPropertyName("duration")]
		public float Duration { get; set; }
		[JsonPropertyName("dateTime")]
		public string DateTime { get; set; }
		[JsonPropertyName("warnings")]
		public int Warnings { get; set; }
		[JsonPropertyName("errors")]
		public int Errors { get; set; }
		[JsonPropertyName("artifacts")]
		public List<UnrealAutomationArtifact> Artifacts { get; set; }
		[JsonPropertyName("entries")]
		public List<UnrealAutomationEntry> Entries { get; set; }

		public UnrealAutomatedTestResult()
		{
			Artifacts = new List<UnrealAutomationArtifact>();
			Entries = new List<UnrealAutomationEntry>();
		}

		public void AddEvent(EventType EventType, string Message, bool bIsCriticalFailure = false)
		{
			var Event = new UnrealAutomationEvent(EventType, Message, bIsCriticalFailure);
			var Entry = new UnrealAutomationEntry();
			Entry.Event = Event;
			Entry.Timestamp = System.DateTime.UtcNow.ToString("yyyy.MM.dd-HH.mm.ss");
			Entries.Add(Entry);

			switch (Event.Type)
			{
				case EventType.Error:
					Errors++;
					break;
				case EventType.Warning:
					Warnings++;
					break;
			}
		}
		public void AddError(string Message, bool bIsCriticalFailure = false)
		{
			AddEvent(EventType.Error, Message, bIsCriticalFailure);
		}
		public void AddWarning(string Message)
		{
			AddEvent(EventType.Warning, Message);
		}
		public void AddInfo(string Message)
		{
			AddEvent(EventType.Info, Message);
		}

		[JsonIgnore]
		public bool IsComplete
		{
			get
			{
				return State != TestStateType.InProcess && State != TestStateType.NotRun;
			}
		}
		[JsonIgnore]
		public bool HasSucceeded
		{
			get
			{
				return State == TestStateType.Success;
			}
		}
		[JsonIgnore]
		public bool HasFailed
		{
			get
			{
				return (State == TestStateType.Fail || State == TestStateType.Interrupted);
			}
		}
		[JsonIgnore]
		public bool WasSkipped
		{
			get
			{
				return State == TestStateType.Skipped;
			}
		}
		[JsonIgnore]
		public bool HasWarnings
		{
			get
			{
				return Warnings > 0;
			}
		}
		[JsonIgnore]
		public IEnumerable<UnrealAutomationEvent> ErrorEvents
		{
			get
			{
				return Entries.Where(E => E.Event.IsError).Select(E => E.Event);
			}
		}
		[JsonIgnore]
		public IEnumerable<UnrealAutomationEvent> WarningEvents
		{
			get
			{
				return Entries.Where(E => E.Event.IsWarning).Select(E => E.Event);
			}
		}
		[JsonIgnore]
		public IEnumerable<UnrealAutomationEvent> WarningAndErrorEvents
		{
			get
			{
				return Entries.Where(E => E.Event.IsError || E.Event.IsWarning).Select(E => E.Event);
			}
		}

		public IEnumerable<UnrealAutomationEntry> GetErrorEntries()
		{
			return Entries.Where(E => E.Event.IsError);
		}

		public IEnumerable<UnrealAutomationEntry> GetWarningEntries()
		{
			return Entries.Where(E => E.Event.IsWarning);
		}

		public IEnumerable<UnrealAutomationEntry> GetWarningAndErrorEntries()
		{
			return Entries.Where(E => E.Event.IsError || E.Event.IsWarning);
		}
	}
	public class UnrealAutomatedTestPassResults
	{
		[JsonPropertyName("devices")]
		public List<UnrealAutomationDevice> Devices { get; set; }
		[JsonPropertyName("reportCreatedOn")]
		public string ReportCreatedOn { get; set; }
		[JsonPropertyName("succeeded")]
		public int Succeeded { get; set; }
		[JsonPropertyName("succeededWithWarnings")]
		public int SucceededWithWarnings { get; set; }
		[JsonPropertyName("failed")]
		public int Failed { get; set; }
		[JsonPropertyName("notRun")]
		public int NotRun { get; set; }
		[JsonPropertyName("inProcess")]
		public int InProcess { get; set; }
		[JsonPropertyName("totalDuration")]
		public float TotalDuration { get; set; }
		[JsonPropertyName("comparisonExported")]
		public bool ComparisonExported { get; set; }
		[JsonPropertyName("comparisonExportDirectory")]
		public string ComparisonExportDirectory { get; set; }
		[JsonPropertyName("tests")]
		public List<UnrealAutomatedTestResult> Tests { get; set; }

		private string FilePath;

		public void SetFilePath(string InFilePath)
		{
			FilePath = InFilePath;
		}

		public UnrealAutomatedTestPassResults()
		{
			Devices = new List<UnrealAutomationDevice>();
			Tests = new List<UnrealAutomatedTestResult>();
		}

		public UnrealAutomatedTestResult AddTest(string DisplayName, string FullName, TestStateType State)
		{
			var Test = new UnrealAutomatedTestResult();
			Test.TestDisplayName = DisplayName;
			Test.FullTestPath = FullName;
			Test.State = State;

			return AddTest(Test);
		}

		public UnrealAutomatedTestResult AddTest(UnrealAutomatedTestResult Test)
		{
			Tests.Add(Test);

			switch (Test.State)
			{
				case TestStateType.Success:
					if (Test.HasWarnings)
					{
						SucceededWithWarnings++;
					}
					else
					{
						Succeeded++;
					}
					break;
				case TestStateType.Fail:
					Failed++;
					break;
				case TestStateType.NotRun:
					NotRun++;
					break;
				case TestStateType.InProcess:
					InProcess++;
					break;
			}

			return Test;
		}

		/// <summary>
		/// Load Unreal Automated Test Results from json report
		/// </summary>
		/// <param name="FilePath"></param>
		public static UnrealAutomatedTestPassResults LoadFromJson(string FilePath)
		{
			JsonSerializerOptions Options = new JsonSerializerOptions
			{
				PropertyNameCaseInsensitive = true
			};
			string JsonString = File.ReadAllText(FilePath);
			UnrealAutomatedTestPassResults JsonTestPassResults = JsonSerializer.Deserialize<UnrealAutomatedTestPassResults>(JsonString, Options);
			JsonTestPassResults.SetFilePath(FilePath);
			return JsonTestPassResults;
		}

		/// <summary>
		/// Write json data into a file
		/// </summary>
		/// <param name="InFilePath"></param>
		public void WriteToJson(string InFilePath = null)
		{
			if (!string.IsNullOrEmpty(InFilePath))
			{
				SetFilePath(InFilePath);
			}
			if (string.IsNullOrEmpty(FilePath))
			{
				throw new AutomationException("Can't Write to Json. FilePath is not specified.");
			}

			string OutputTestDataDir = Path.GetDirectoryName(FilePath);
			if (!Directory.Exists(OutputTestDataDir))
			{
				Directory.CreateDirectory(OutputTestDataDir);
			}

			Log.Verbose("Writing Json to file {0}", FilePath);
			try
			{
				JsonSerializerOptions Options = new JsonSerializerOptions
				{
					WriteIndented = true
				};
				File.WriteAllText(FilePath, JsonSerializer.Serialize(this, Options));
			}
			catch (Exception Ex)
			{
				Log.Error("Failed to save json file. {0}", Ex);
			}
		}
	}
}
