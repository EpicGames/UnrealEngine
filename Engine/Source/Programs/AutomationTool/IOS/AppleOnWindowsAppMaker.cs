// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.IO;
using System.Text;
using AutomationTool;
using EpicGames.Core;
using Ionic.Zip;
using Ionic.Zlib;
using Microsoft.Extensions.Logging;
using UnrealBuildBase;
using UnrealBuildTool;

namespace IOS.Automation
{
	/** Helper class to make Apple apps on Windows.
	 * It contains most of the functionality that used to exist as "Legacy Xcode"
	 */
	public class AppleOnWindowsAppMaker
	{
		public static FileReference GetTargetReceiptFileName(UnrealTargetConfiguration Config, string InExecutablePath, DirectoryReference InEngineDir, DirectoryReference InProjectDirectory, FileReference ProjectFile, bool bIsUEGame)
		{
			Console.WriteLine("target receipt executable path: {0}", InExecutablePath);
			string TargetName = Path.GetFileNameWithoutExtension(InExecutablePath).Split("-".ToCharArray())[0];
			FileReference TargetReceiptFileName;
			UnrealArchitectures Architectures = UnrealArchitectureConfig.ForPlatform(UnrealTargetPlatform.IOS).ActiveArchitectures(ProjectFile, TargetName);
			if (bIsUEGame)
			{
				TargetReceiptFileName = TargetReceipt.GetDefaultPath(InEngineDir, "UnrealGame", UnrealTargetPlatform.IOS, Config, Architectures);
			}
			else
			{
				TargetReceiptFileName = TargetReceipt.GetDefaultPath(ProjectFile.Directory, TargetName, UnrealTargetPlatform.IOS, Config, Architectures);
			}
			return TargetReceiptFileName;
		}
		
		static string MakeIPAFileName(UnrealTargetConfiguration TargetConfiguration, ProjectParams Params, DeploymentContext SC, bool bAllowDistroPrefix, string PlatformName)
		{
			string ExeName = SC.StageExecutables[0];
			if (!SC.IsCodeBasedProject)
			{
				ExeName = ExeName.Replace("UnrealGame", Params.RawProjectPath.GetFileNameWithoutExtension());
			}
			return Path.Combine(Path.GetDirectoryName(Params.RawProjectPath.FullName), "Binaries", PlatformName,
				((bAllowDistroPrefix && Params.Distribution) ? "Distro_" : "") + ExeName + ".ipa");
		}

		public static string GetStagedIPAPath(ProjectParams Params, DeploymentContext SC, string PlatformName)
		{
			UnrealTargetConfiguration TargetConfiguration = SC.StageTargetConfigurations[0];
			string ProjectIPA = MakeIPAFileName(TargetConfiguration, Params, SC, true, PlatformName);
			string StagedIPA = SC.StageDirectory + "\\" + Path.GetFileName(ProjectIPA);

			// verify the .ipa exists
			if (!CommandUtils.FileExists(StagedIPA))
			{
				StagedIPA = ProjectIPA;
				if (!CommandUtils.FileExists(StagedIPA))
				{
					throw new AutomationException("DEPLOY FAILED - {0} was not found", ProjectIPA);
				}
			}

			return StagedIPA;
		}

		public static void GetFilesToArchive(ProjectParams Params, DeploymentContext SC, ILogger Logger, string PlatformName)
		{
			if (SC.StageTargetConfigurations.Count != 1)
			{
				throw new AutomationException("iOS is currently only able to package one target configuration at a time, but StageTargetConfigurations contained {0} configurations", SC.StageTargetConfigurations.Count);
			}
			var TargetConfiguration = SC.StageTargetConfigurations[0];
			var ProjectIPA = MakeIPAFileName(TargetConfiguration, Params, SC, true, PlatformName);

			// verify the .ipa exists
			if (!CommandUtils.FileExists(ProjectIPA))
			{
				throw new AutomationException("ARCHIVE FAILED - {0} was not found", ProjectIPA);
			}

			ConfigHierarchy PlatformGameConfig;
			bool bXCArchive = false;
			if (Params.EngineConfigs.TryGetValue(SC.StageTargetPlatform.PlatformType, out PlatformGameConfig))
			{
				PlatformGameConfig.GetBool("/Script/IOSRuntimeSettings.IOSRuntimeSettings", "bGenerateXCArchive", out bXCArchive);
			}

			if (bXCArchive && !OperatingSystem.IsWindows())
			{
				// Always put the archive in the current user's Library/Developer/Xcode/Archives path if not on the build machine
				string ArchivePath = "/Users/" + Environment.UserName + "/Library/Developer/Xcode/Archives";
				if (CommandUtils.IsBuildMachine)
				{
					ArchivePath = Params.ArchiveDirectoryParam;
				}
				if (!CommandUtils.DirectoryExists(ArchivePath))
				{
					CommandUtils.CreateDirectory(ArchivePath);
				}

				Console.WriteLine("Generating xc archive package in " + ArchivePath);

				string ArchiveName = Path.Combine(ArchivePath, Path.GetFileNameWithoutExtension(ProjectIPA) + ".xcarchive");
				if (!CommandUtils.DirectoryExists(ArchiveName))
				{
					CommandUtils.CreateDirectory(ArchiveName);
				}
				CommandUtils.DeleteDirectoryContents(ArchiveName);

				// create the Products archive folder
				CommandUtils.CreateDirectory(Path.Combine(ArchiveName, "Products", "Applications"));

				// copy in the application
				string AppName = Path.GetFileNameWithoutExtension(ProjectIPA) + ".app";
				if (!File.Exists(ProjectIPA))
				{
					Console.WriteLine("Couldn't find IPA: " + ProjectIPA);
				}
				using (ZipFile Zip = new ZipFile(ProjectIPA))
				{
					Zip.ExtractAll(ArchivePath, ExtractExistingFileAction.OverwriteSilently);

					List<string> Dirs = new List<string>(Directory.EnumerateDirectories(Path.Combine(ArchivePath, "Payload"), "*.app"));
					foreach (string Dir in Dirs)
					{
						if (Dir.Contains(Params.ShortProjectName + ".app"))
						{
							Console.WriteLine("Using Directory: " + Dir);
							AppName = Dir.Substring(Dir.LastIndexOf(UnrealBuildTool.BuildHostPlatform.Current.Platform != UnrealTargetPlatform.Mac ? "\\" : "/") + 1);
						}
					}
					CommandUtils.CopyDirectory_NoExceptions(Path.Combine(ArchivePath, "Payload", AppName), Path.Combine(ArchiveName, "Products", "Applications", AppName));
				}

				// copy in the dSYM if found
				var ProjectExe = MakeIPAFileName(TargetConfiguration, Params, SC, false, PlatformName);
				string dSYMName = (SC.IsCodeBasedProject ? Path.GetFileNameWithoutExtension(ProjectExe) : "UnrealGame") + ".dSYM";
				string dSYMDestName = AppName + ".dSYM";
				string dSYMSrcPath = Path.Combine(SC.ProjectBinariesFolder.FullName, dSYMName);
				string dSYMZipSrcPath = Path.Combine(SC.ProjectBinariesFolder.FullName, dSYMName + ".zip");
				if (File.Exists(dSYMZipSrcPath))
				{
					// unzip the dsym
					using (ZipFile Zip = new ZipFile(dSYMZipSrcPath))
					{
						Zip.ExtractAll(SC.ProjectBinariesFolder.FullName, ExtractExistingFileAction.OverwriteSilently);
					}
				}

				if (CommandUtils.DirectoryExists(dSYMSrcPath))
				{
					// Create the dsyms archive folder
					CommandUtils.CreateDirectory(Path.Combine(ArchiveName, "dSYMs"));
					string dSYMDstPath = Path.Combine(ArchiveName, "dSYMs", dSYMDestName);
					// /Volumes/MacOSDrive1/pfEpicWorkspace/Dev-Platform/Samples/Sandbox/PlatformShowcase/Binaries/IOS/PlatformShowcase.dSYM/Contents/Resources/DWARF/PlatformShowcase
					CommandUtils.CopyFile_NoExceptions(Path.Combine(dSYMSrcPath, "Contents", "Resources", "DWARF", SC.IsCodeBasedProject ? Path.GetFileNameWithoutExtension(ProjectExe) : "UnrealGame"), dSYMDstPath);
				}
				else if (File.Exists(dSYMSrcPath))
				{
					// Create the dsyms archive folder
					CommandUtils.CreateDirectory(Path.Combine(ArchiveName, "dSYMs"));
					string dSYMDstPath = Path.Combine(ArchiveName, "dSYMs", dSYMDestName);
					CommandUtils.CopyFile_NoExceptions(dSYMSrcPath, dSYMDstPath);
				}

				// get the settings from the app plist file
				string AppPlist = Path.Combine(ArchiveName, "Products", "Applications", AppName, "Info.plist");
				string OldPListData = File.Exists(AppPlist) ? File.ReadAllText(AppPlist) : "";

				string BundleIdentifier = "";
				string BundleShortVersion = "";
				string BundleVersion = "";
				if (!string.IsNullOrEmpty(OldPListData))
				{
					// bundle identifier
					int index = OldPListData.IndexOf("CFBundleIdentifier");
					index = OldPListData.IndexOf("<string>", index) + 8;
					int length = OldPListData.IndexOf("</string>", index) - index;
					BundleIdentifier = OldPListData.Substring(index, length);

					// short version
					index = OldPListData.IndexOf("CFBundleShortVersionString");
					index = OldPListData.IndexOf("<string>", index) + 8;
					length = OldPListData.IndexOf("</string>", index) - index;
					BundleShortVersion = OldPListData.Substring(index, length);

					// bundle version
					index = OldPListData.IndexOf("CFBundleVersion");
					index = OldPListData.IndexOf("<string>", index) + 8;
					length = OldPListData.IndexOf("</string>", index) - index;
					BundleVersion = OldPListData.Substring(index, length);
				}
				else
				{
					Console.WriteLine("Could not load Info.plist");
				}

				// date we made this
				const string Iso8601DateTimeFormat = "yyyy-MM-ddTHH:mm:ssZ";
				string TimeStamp = DateTime.UtcNow.ToString(Iso8601DateTimeFormat);

				// create the archive plist
				StringBuilder Text = new StringBuilder();
				Text.AppendLine("<?xml version=\"1.0\" encoding=\"UTF-8\"?>");
				Text.AppendLine("<!DOCTYPE plist PUBLIC \"-//Apple//DTD PLIST 1.0//EN\" \"http://www.apple.com/DTDs/PropertyList-1.0.dtd\">");
				Text.AppendLine("<plist version=\"1.0\">");
				Text.AppendLine("<dict>");
				Text.AppendLine("\t<key>ApplicationProperties</key>");
				Text.AppendLine("\t<dict>");
				Text.AppendLine("\t\t<key>ApplicationPath</key>");
				Text.AppendLine("\t\t<string>Applications/" + AppName + "</string>");
				Text.AppendLine("\t\t<key>CFBundleIdentifier</key>");
				Text.AppendLine(string.Format("\t\t<string>{0}</string>", BundleIdentifier));
				Text.AppendLine("\t\t<key>CFBundleShortVersionString</key>");
				Text.AppendLine(string.Format("\t\t<string>{0}</string>", BundleShortVersion));
				Text.AppendLine("\t\t<key>CFBundleVersion</key>");
				Text.AppendLine(string.Format("\t\t<string>{0}</string>", BundleVersion));
				Text.AppendLine("\t\t<key>SigningIdentity</key>");
				Text.AppendLine(string.Format("\t\t<string>{0}</string>", Params.Certificate));
				Text.AppendLine("\t</dict>");
				Text.AppendLine("\t<key>ArchiveVersion</key>");
				Text.AppendLine("\t<integer>2</integer>");
				Text.AppendLine("\t<key>CreationDate</key>");
				Text.AppendLine(string.Format("\t<date>{0}</date>", TimeStamp));
				Text.AppendLine("\t<key>DefaultToolchainInfo</key>");
				Text.AppendLine("\t<dict>");
				Text.AppendLine("\t\t<key>DisplayName</key>");
				Text.AppendLine("\t\t<string>Xcode 7.3 Default</string>");
				Text.AppendLine("\t\t<key>Identifier</key>");
				Text.AppendLine("\t\t<string>com.apple.dt.toolchain.XcodeDefault</string>");
				Text.AppendLine("\t</dict>");
				Text.AppendLine("\t<key>Name</key>");
				Text.AppendLine(string.Format("\t<string>{0}</string>", SC.ShortProjectName));
				Text.AppendLine("\t<key>SchemeName</key>");
				Text.AppendLine(string.Format("\t<string>{0}</string>", SC.ShortProjectName));
				Text.AppendLine("</dict>");
				Text.AppendLine("</plist>");
				File.WriteAllText(Path.Combine(ArchiveName, "Info.plist"), Text.ToString());
			}
			else if (bXCArchive && OperatingSystem.IsWindows())
			{
				Logger.LogWarning("Can not produce an XCArchive on windows");
			}
			SC.ArchiveFiles(Path.GetDirectoryName(ProjectIPA), Path.GetFileName(ProjectIPA));
		}
		
		private static void StageStandardLaunchScreenStoryboard(ProjectParams Params, DeploymentContext SC)
		{
			string BuildGraphicsDirectory = Path.GetDirectoryName(Params.RawProjectPath.FullName) + "/Build/IOS/Resources/Graphics/";
			if (File.Exists(BuildGraphicsDirectory + "LaunchScreenIOS.png"))
			{
				InternalUtils.SafeCopyFile(BuildGraphicsDirectory + "LaunchScreenIOS.png", SC.StageDirectory + "/LaunchScreenIOS.png");
			}
		}
		
		private static void StageCustomLaunchScreenStoryboard(ProjectParams Params, DeploymentContext SC, ILogger Logger)
		{
			string InterfaceSBDirectory = Path.GetDirectoryName(Params.RawProjectPath.FullName) + "/Build/IOS/Resources/Interface/";
			if (Directory.Exists(InterfaceSBDirectory + "LaunchScreen.storyboardc"))
			{
				string[] StoryboardFilesToStage = Directory.GetFiles(InterfaceSBDirectory + "LaunchScreen.storyboardc", "*", SearchOption.TopDirectoryOnly);

				if (!CommandUtils.DirectoryExists(SC.StageDirectory + "/LaunchScreen.storyboardc"))
				{
					DirectoryInfo createddir = Directory.CreateDirectory(SC.StageDirectory + "/LaunchScreen.storyboardc");
				}

				foreach (string Filename in StoryboardFilesToStage)
				{
					string workingFileName = Filename;
					while (workingFileName.Contains("/"))
					{
						workingFileName = workingFileName.Substring(workingFileName.IndexOf('/') + 1);
					}
					workingFileName = workingFileName.Substring(workingFileName.IndexOf('/') + 1);

					InternalUtils.SafeCopyFile(Filename, SC.StageDirectory + "/" + workingFileName);
				}

				string[] StoryboardAssetsToStage = Directory.GetFiles(InterfaceSBDirectory + "Assets/", "*", SearchOption.TopDirectoryOnly);

				foreach (string Filename in StoryboardAssetsToStage)
				{
					string workingFileName = Filename;
					while (workingFileName.Contains("/"))
					{
						workingFileName = workingFileName.Substring(workingFileName.IndexOf('/') + 1);
					}
					workingFileName = workingFileName.Substring(workingFileName.IndexOf('/') + 1);

					InternalUtils.SafeCopyFile(Filename, SC.StageDirectory + "/" + workingFileName);
				}
			}
			else
			{
				Logger.LogWarning("Use Custom Launch Screen Storyboard is checked but not compiled storyboard could be found. Have you compiled on Mac first ? Falling back to Standard Storyboard");
				StageStandardLaunchScreenStoryboard(Params, SC);
			}
		}

		private static void StageLaunchScreenStoryboard(ProjectParams Params, DeploymentContext SC, ILogger Logger)
		{
			bool bCustomLaunchscreenStoryboard = false;
			ConfigHierarchy PlatformGameConfig;
			if (Params.EngineConfigs.TryGetValue(SC.StageTargetPlatform.PlatformType, out PlatformGameConfig))
			{
				PlatformGameConfig.GetBool("/Script/IOSRuntimeSettings.IOSRuntimeSettings", "bCustomLaunchscreenStoryboard", out bCustomLaunchscreenStoryboard);
			}

			if (bCustomLaunchscreenStoryboard)
			{
				StageCustomLaunchScreenStoryboard(Params, SC, Logger);
			}
			else
			{
				StageStandardLaunchScreenStoryboard(Params, SC);
			}
		}
		
		private static bool IsBuiltAsFramework(ProjectParams Params, DeploymentContext SC, UnrealTargetPlatform TargetPlatformType)
		{
			UnrealTargetConfiguration Config = SC.StageTargetConfigurations[0];
			string InExecutablePath = CommandUtils.CombinePaths(Path.GetDirectoryName(Params.GetProjectExeForPlatform(TargetPlatformType).ToString()), SC.StageExecutables[0]);
			DirectoryReference InEngineDir = DirectoryReference.Combine(SC.LocalRoot, "Engine");
			DirectoryReference InProjectDirectory = Params.RawProjectPath.Directory;
			bool bIsUEGame = !SC.IsCodeBasedProject;

			FileReference ReceiptFileName = GetTargetReceiptFileName(Config, InExecutablePath, InEngineDir, InProjectDirectory, Params.RawProjectPath, bIsUEGame);
			TargetReceipt Receipt;
			bool bIsReadSuccessful = TargetReceipt.TryRead(ReceiptFileName, out Receipt);

			bool bIsBuiltAsFramework = false;
			if (bIsReadSuccessful)
			{
				bIsBuiltAsFramework = Receipt.HasValueForAdditionalProperty("CompileAsDll", "true");
			}

			return bIsBuiltAsFramework;
		}
		
		private static bool ShouldUseMaxIPACompression(ProjectParams Params)
		{
			if (!string.IsNullOrEmpty(Params.AdditionalPackageOptions))
			{
				string[] OptionsArray = Params.AdditionalPackageOptions.Split(' ');
				foreach (string Option in OptionsArray)
				{
					if (Option.Equals("-ForceMaxIPACompression", StringComparison.InvariantCultureIgnoreCase))
					{
						return true;
					}
				}
			}

			return false;
		}
		
		private static void PackageIPA(ProjectParams Params, string ProjectGameExeFilename, DeploymentContext SC, string PlatformName)
		{
			string BaseDirectory = Path.GetDirectoryName(ProjectGameExeFilename);
			string ExeName = Params.IsCodeBasedProject ? SC.StageExecutables[0] : Path.GetFileNameWithoutExtension(ProjectGameExeFilename);
			string GameName = ExeName.Split("-".ToCharArray())[0];
			string ProjectName = Params.ShortProjectName;
			UnrealTargetConfiguration TargetConfig = SC.StageTargetConfigurations[0];

			// create the ipa
			string IPAName = MakeIPAFileName(TargetConfig, Params, SC, true, PlatformName);
			// delete the old one
			if (File.Exists(IPAName))
			{
				File.Delete(IPAName);
			}

			// make the subdirectory if needed
			string DestSubdir = Path.GetDirectoryName(IPAName);
			if (!Directory.Exists(DestSubdir))
			{
				Directory.CreateDirectory(DestSubdir);
			}

			// set up the directories
			string ZipWorkingDir = String.Format("Payload/{0}.app/", GameName);
			string ZipSourceDir = string.Format("{0}/Payload/{1}.app", BaseDirectory, GameName);

			// create the file
			using (ZipFile Zip = new ZipFile())
			{
				// Set encoding to support unicode filenames
				Zip.AlternateEncodingUsage = ZipOption.Always;
				Zip.AlternateEncoding = Encoding.UTF8;
				Zip.UseZip64WhenSaving = Zip64Option.AsNecessary;

				// set the compression level
				bool bUseMaxIPACompression = ShouldUseMaxIPACompression(Params);
				if (Params.Distribution || bUseMaxIPACompression)
				{
					Zip.CompressionLevel = CompressionLevel.BestCompression;
				}

				if (!CommandUtils.DirectoryExists(ZipSourceDir))
				{
					throw new AutomationException("Source dir does not exist | Dir " + ZipSourceDir);
				}

				// add the entire directory
				Zip.AddDirectory(ZipSourceDir, ZipWorkingDir);

				// Update permissions to be UNIX-style
				// Modify the file attributes of any added file to unix format
				foreach (ZipEntry E in Zip.Entries)
				{
					const byte FileAttributePlatform_NTFS = 0x0A;
					const byte FileAttributePlatform_UNIX = 0x03;
					const byte FileAttributePlatform_FAT = 0x00;

					const int UNIX_FILETYPE_NORMAL_FILE = 0x8000;
					//const int UNIX_FILETYPE_SOCKET = 0xC000;
					//const int UNIX_FILETYPE_SYMLINK = 0xA000;
					//const int UNIX_FILETYPE_BLOCKSPECIAL = 0x6000;
					const int UNIX_FILETYPE_DIRECTORY = 0x4000;
					//const int UNIX_FILETYPE_CHARSPECIAL = 0x2000;
					//const int UNIX_FILETYPE_FIFO = 0x1000;

					const int UNIX_EXEC = 1;
					const int UNIX_WRITE = 2;
					const int UNIX_READ = 4;


					int MyPermissions = UNIX_READ | UNIX_WRITE;
					int OtherPermissions = UNIX_READ;

					int PlatformEncodedBy = (E.VersionMadeBy >> 8) & 0xFF;
					int LowerBits = 0;

					// Try to preserve read-only if it was set
					bool bIsDirectory = E.IsDirectory;

					// Check to see if this 
					bool bIsExecutable = false;
					if (Path.GetFileNameWithoutExtension(E.FileName).Equals(ExeName, StringComparison.InvariantCultureIgnoreCase))
					{
						bIsExecutable = true;
					}

					if (bIsExecutable && !bUseMaxIPACompression)
					{
						// The executable will be encrypted in the final distribution IPA and will compress very poorly, so keeping it
						// uncompressed gives a better indicator of IPA size for our distro builds
						E.CompressionLevel = CompressionLevel.None;
					}

					if ((PlatformEncodedBy == FileAttributePlatform_NTFS) || (PlatformEncodedBy == FileAttributePlatform_FAT))
					{
						FileAttributes OldAttributes = E.Attributes;
						//LowerBits = ((int)E.Attributes) & 0xFFFF;

						if ((OldAttributes & FileAttributes.Directory) != 0)
						{
							bIsDirectory = true;
						}

						// Permissions
						if ((OldAttributes & FileAttributes.ReadOnly) != 0)
						{
							MyPermissions &= ~UNIX_WRITE;
							OtherPermissions &= ~UNIX_WRITE;
						}
					}

					if (bIsDirectory || bIsExecutable)
					{
						MyPermissions |= UNIX_EXEC;
						OtherPermissions |= UNIX_EXEC;
					}

					// Re-jigger the external file attributes to UNIX style if they're not already that way
					if (PlatformEncodedBy != FileAttributePlatform_UNIX)
					{
						int NewAttributes = bIsDirectory ? UNIX_FILETYPE_DIRECTORY : UNIX_FILETYPE_NORMAL_FILE;

						NewAttributes |= (MyPermissions << 6);
						NewAttributes |= (OtherPermissions << 3);
						NewAttributes |= (OtherPermissions << 0);

						// Now modify the properties
						E.AdjustExternalFileAttributes(FileAttributePlatform_UNIX, (NewAttributes << 16) | LowerBits);
					}
				}

				// Save it out
				Zip.Save(IPAName);
			}
		}
		
		private static void WriteEntitlements(ProjectParams Params, DeploymentContext SC, UnrealTargetPlatform PlatTargetPlatformType)
		{
			// game name
			string AppName = SC.IsCodeBasedProject ? SC.StageExecutables[0].Split("-".ToCharArray())[0] : "UnrealGame";

			// mobile provisioning file
			DirectoryReference MobileProvisionDir = AppleExports.GetProvisionDirectory();
			FileReference MobileProvisionFile = null;
		
			if(MobileProvisionDir != null && Params.Provision != null)
			{
				MobileProvisionFile = FileReference.Combine(MobileProvisionDir, Params.Provision);
			}

			// distribution build
			bool bForDistribution = Params.Distribution;

			// intermediate directory
			string IntermediateDir = SC.ProjectRoot + "/Intermediate/" + (PlatTargetPlatformType == UnrealTargetPlatform.IOS ? "IOS" : "TVOS");

			//	entitlements file name
			string OutputFilename = Path.Combine(IntermediateDir, AppName + ".entitlements");

			// ios configuration from the ini file
			ConfigHierarchy PlatformGameConfig;
			if (Params.EngineConfigs.TryGetValue(SC.StageTargetPlatform.PlatformType, out PlatformGameConfig))
			{
				IOSExports.WriteEntitlements(PlatTargetPlatformType, PlatformGameConfig, AppName, MobileProvisionFile, bForDistribution, IntermediateDir);
			}
		}
		
		private static string EnsureXcodeProjectExists(FileReference RawProjectPath, string LocalRoot, string ShortProjectName, string ProjectRoot, bool IsCodeBasedProject, out bool bWasGenerated, string PlatformName)
		{
			// first check for the .xcodeproj
			bWasGenerated = false;
			DirectoryReference RawProjectDir = RawProjectPath.Directory;
			DirectoryReference XcodeProj = DirectoryReference.Combine(RawProjectDir, "Intermediate/ProjectFiles", $"{RawProjectPath.GetFileNameWithoutAnyExtensions()}_{PlatformName}.xcworkspace");
			Console.WriteLine("Project: " + XcodeProj.FullName);
			{
				// project.xcodeproj doesn't exist, so generate temp project
				string Arguments = "-project=\"" + RawProjectPath + "\"";
				Arguments += " -platforms=" + PlatformName + " -game -nointellisense -" + PlatformName + "deployonly -ignorejunk -projectfileformat=XCode -includetemptargets -automated";

				// If engine is installed then UBT doesn't need to be built
				if (Unreal.IsEngineInstalled())
				{
					Arguments = "-XcodeProjectFiles " + Arguments;
					CommandUtils.RunUBT(CommandUtils.CmdEnv, UnrealBuild.UnrealBuildToolDll, Arguments);
				}
				else
				{
					string Script = CommandUtils.CombinePaths(CommandUtils.CmdEnv.LocalRoot, "Engine/Build/BatchFiles/Mac/GenerateProjectFiles.sh");
					string CWD = Directory.GetCurrentDirectory();
					Directory.SetCurrentDirectory(Path.GetDirectoryName(Script));
					CommandUtils.Run(Script, Arguments, null, CommandUtils.ERunOptions.Default);
					Directory.SetCurrentDirectory(CWD);
				}

				bWasGenerated = true;

				if (!DirectoryReference.Exists(XcodeProj))
				{
					// something very bad happened
					throw new AutomationException("iOS couldn't find the appropriate Xcode Project " + XcodeProj.FullName);
				}
			}

			return XcodeProj.FullName;
		}
		
		public static void CodeSign(IOSPlatform PlatformContext, string BaseDirectory, string GameName, FileReference RawProjectPath, UnrealTargetConfiguration TargetConfig, string LocalRoot, string ProjectName, string ProjectDirectory, bool IsCode, bool Distribution, string Provision, string Certificate, string Team, bool bAutomaticSigning, string SchemeName, string SchemeConfiguration, ILogger Logger)
		{
			// check for the proper xcodeproject
			bool bWasGenerated = false;
			string XcodeProj = EnsureXcodeProjectExists(RawProjectPath, LocalRoot, ProjectName, ProjectDirectory, IsCode, out bWasGenerated, PlatformContext.PlatformName);

			string Arguments = "UBT_NO_POST_DEPLOY=true";
			Arguments += " /usr/bin/xcrun xcodebuild build -workspace \"" + XcodeProj + "\"";
			Arguments += " -scheme '";
			Arguments += SchemeName != null ? SchemeName : GameName;
			Arguments += "'";
			Arguments += " -configuration \"" + (SchemeConfiguration != null ? SchemeConfiguration : TargetConfig.ToString()) + "\"";
			Arguments += $" -destination generic/platform=\"{AppleExports.GetDestinationPlatform(PlatformContext.TargetPlatformType, new UnrealArchitectures(UnrealArch.Arm64))}\"";
			Arguments += " -sdk " + PlatformContext.SDKName;

			if (bAutomaticSigning)
			{
				Arguments += " CODE_SIGN_IDENTITY=" + (Distribution ? "\"iPhone Distribution\"" : "\"iPhone Developer\"");
				Arguments += " CODE_SIGN_STYLE=\"Automatic\" -allowProvisioningUpdates";
				Arguments += " DEVELOPMENT_TEAM=\"" + Team + "\"";
			}
			else
			{
				if (!string.IsNullOrEmpty(Certificate))
				{
					Arguments += " CODE_SIGN_IDENTITY=\"" + Certificate + "\"";
				}
				else
				{
					Arguments += " CODE_SIGN_IDENTITY=" + (Distribution ? "\"iPhone Distribution\"" : "\"iPhone Developer\"");
				}
				if (!string.IsNullOrEmpty(Provision))
				{
					// read the provision to get the UUID
					DirectoryReference MobileProvisionDir = AppleExports.GetProvisionDirectory();
					FileReference MobileProvisionFile = FileReference.Combine(MobileProvisionDir, Provision);
					if (File.Exists(MobileProvisionFile.FullName))
					{
						string UUID = "";
						string AllText = File.ReadAllText(MobileProvisionFile.FullName);
						int idx = AllText.IndexOf("<key>UUID</key>");
						if (idx > 0)
						{
							idx = AllText.IndexOf("<string>", idx);
							if (idx > 0)
							{
								idx += "<string>".Length;
								UUID = AllText.Substring(idx, AllText.IndexOf("</string>", idx) - idx);
								Arguments += " PROVISIONING_PROFILE_SPECIFIER=" + UUID;

								Logger.LogInformation("Extracted Provision UUID {UUID} from {Provision}", UUID, Provision);
							}
						}
					}
				}
			}
			IProcessResult Result = CommandUtils.Run("/usr/bin/env", Arguments, null, CommandUtils.ERunOptions.Default);
			if (bWasGenerated)
			{
				InternalUtils.SafeDeleteDirectory(XcodeProj, true);
			}
			if (Result.ExitCode != 0)
			{
				throw new AutomationException(ExitCode.Error_FailedToCodeSign, "CodeSign Failed");
			}
		}
		
		static void RunIPP(string IPPArguments, ILogger Logger)
		{
			var IPPExe = CommandUtils.CombinePaths(CommandUtils.CmdEnv.LocalRoot, "Engine/Binaries/DotNET/IOS/IPhonePackager.exe");
			Logger.LogDebug(" Running IPP | IPPExe={IPPExe} | Args {IPPArgs}", IPPExe, IPPArguments);
			CommandUtils.RunAndLog(CommandUtils.CmdEnv, IPPExe, IPPArguments);
		}

		public static void Package(IOSPlatform PlatformContext, ProjectParams Params, DeploymentContext SC, int WorkingCL, ILogger Logger)
		{
			Logger.LogInformation("Package {Arg0}", Params.RawProjectPath);

			bool bIsBuiltAsFramework = IsBuiltAsFramework(Params, SC, PlatformContext.TargetPlatformType);

			// ensure the UnrealGame binary exists, if applicable
	#if !PLATFORM_MAC
			string ProjectGameExeFilename = Params.GetProjectExeForPlatform(PlatformContext.TargetPlatformType).ToString();
			string FullExePath = CommandUtils.CombinePaths(Path.GetDirectoryName(ProjectGameExeFilename), SC.StageExecutables[0] + (UnrealBuildTool.BuildHostPlatform.Current.Platform != UnrealTargetPlatform.Mac ? ".stub" : ""));
			if (!SC.IsCodeBasedProject && !CommandUtils.FileExists_NoExceptions(FullExePath) && !bIsBuiltAsFramework)
			{
				Logger.LogError("{Text}", "Failed to find game binary " + FullExePath);
				throw new AutomationException(ExitCode.Error_MissingExecutable, "Stage Failed. Could not find binary {0}. You may need to build the Unreal Engine project with your target configuration and platform.", FullExePath);
			}
	#endif // PLATFORM_MAC

			if (SC.StageTargetConfigurations.Count != 1)
			{
				throw new AutomationException("iOS is currently only able to package one target configuration at a time, but StageTargetConfigurations contained {0} configurations", SC.StageTargetConfigurations.Count);
			}

			var TargetConfiguration = SC.StageTargetConfigurations[0];

			string MobileProvision;
			string SigningCertificate;
			string TeamUUID;
			bool bAutomaticSigning;
			
			PlatformContext.GetProvisioningData(Params.RawProjectPath, Params.Distribution, out MobileProvision, out SigningCertificate, out TeamUUID, out bAutomaticSigning);

			//@TODO: We should be able to use this code on both platforms, when the following issues are sorted:
			//   - Raw executable is unsigned & unstripped (need to investigate adding stripping to IPP)
			//   - IPP needs to be able to codesign a raw directory
			//   - IPP needs to be able to take a .app directory instead of a Payload directory when doing RepackageFromStage (which would probably be renamed)
			//   - Some discrepancy in the loading screen pngs that are getting packaged, which needs to be investigated
			//   - Code here probably needs to be updated to write 0 byte files as 1 byte (difference with IPP, was required at one point when using Ionic.Zip to prevent issues on device, maybe not needed anymore?)
			if (BuildHostPlatform.Current.Platform == UnrealTargetPlatform.Mac)
			{
				// If we're building as a framework, then we already have everything we need in the .app
				// so simply package it up as an ipa
				if (bIsBuiltAsFramework)
				{
					PackageIPA(Params, ProjectGameExeFilename, SC, PlatformContext.PlatformName);
					return;
				}

				// copy in all of the artwork and plist
				PlatformContext.PrepForUATPackageOrDeploy(TargetConfiguration, Params.RawProjectPath,
					Params.ShortProjectName,
					Params.RawProjectPath.Directory,
					FileReference.Combine(new FileReference(ProjectGameExeFilename).Directory!, SC.StageExecutables[0]),
					DirectoryReference.Combine(SC.LocalRoot, "Engine"),
					Params.Distribution,
					"",
					false,
					false,
					!SC.IsCodeBasedProject);

				// figure out where to pop in the staged files
				string AppDirectory = string.Format("{0}/Payload/{1}.app",
					Path.GetDirectoryName(ProjectGameExeFilename),
					Path.GetFileNameWithoutExtension(ProjectGameExeFilename));

				// delete the old cookeddata
				InternalUtils.SafeDeleteDirectory(AppDirectory + "/cookeddata", true);
				InternalUtils.SafeDeleteFile(AppDirectory + "/uecommandline.txt", true);

				SearchOption searchMethod;
				if (!Params.IterativeDeploy)
				{
					searchMethod = SearchOption.AllDirectories; // copy the Staged files to the AppDirectory
				}
				else
				{
					searchMethod = SearchOption.TopDirectoryOnly; // copy just the root stage directory files
				}

				string[] StagedFiles = Directory.GetFiles(SC.StageDirectory.FullName, "*", searchMethod);
				foreach (string Filename in StagedFiles)
				{
					string DestFilename = Filename.Replace(SC.StageDirectory.FullName, AppDirectory);
					Directory.CreateDirectory(Path.GetDirectoryName(DestFilename));
					InternalUtils.SafeCopyFile(Filename, DestFilename, true);
				}
			}

			StageLaunchScreenStoryboard(Params, SC, Logger);

			IOSExports.GenerateAssetCatalog(Params.RawProjectPath, new FileReference(FullExePath), new DirectoryReference(CommandUtils.CombinePaths(Params.BaseStageDirectory, (PlatformContext.TargetPlatformType == UnrealTargetPlatform.IOS ? "IOS" : "TVOS"))), PlatformContext.TargetPlatformType, Log.Logger);

			PlatformContext.bCreatedIPA = false;
			bool bNeedsIPA = false;
			if (Params.IterativeDeploy)
			{
				if (Params.Devices.Count != 1)
				{
					throw new AutomationException("Can only interatively deploy to a single device, but {0} were specified", Params.Devices.Count);
				}

				String NonUFSManifestPath = SC.GetNonUFSDeploymentDeltaPath(Params.DeviceNames[0]);
				// check to determine if we need to update the IPA
				if (File.Exists(NonUFSManifestPath))
				{
					string NonUFSFiles = File.ReadAllText(NonUFSManifestPath);
					string[] Lines = NonUFSFiles.Split('\n');
					bNeedsIPA = Lines.Length > 0 && !string.IsNullOrWhiteSpace(Lines[0]);
				}
			}

			if (String.IsNullOrEmpty(Params.Provision))
			{
				Params.Provision = MobileProvision;
			}
			if (String.IsNullOrEmpty(Params.Certificate))
			{
				Params.Certificate = SigningCertificate;
			}
			if (String.IsNullOrEmpty(Params.Team))
			{
				Params.Team = TeamUUID;
			}

			Params.AutomaticSigning = bAutomaticSigning;

			// Scheme name and configuration for code signing with Xcode project
			string SchemeName = SC.StageTargets[0].Receipt.TargetName;
			string SchemeConfiguration = TargetConfiguration.ToString();

			WriteEntitlements(Params, SC, PlatformContext.TargetPlatformType);

			if (BuildHostPlatform.Current.Platform != UnrealTargetPlatform.Mac)
			{
				var ProjectIPA = MakeIPAFileName(TargetConfiguration, Params, SC, Params.Distribution, PlatformContext.PlatformName);
				var ProjectStub = Path.GetFullPath(ProjectGameExeFilename);
				var IPPProjectIPA = "";
				if (ProjectStub.Contains("UnrealGame"))
				{
					IPPProjectIPA = Path.Combine(Path.GetDirectoryName(ProjectIPA), Path.GetFileName(ProjectIPA).Replace(Params.RawProjectPath.GetFileNameWithoutExtension(), "UnrealGame"));
				}

				// package a .ipa from the now staged directory
				

				Logger.LogDebug("ProjectName={Arg0}", Params.ShortProjectName);
				Logger.LogDebug("ProjectStub={ProjectStub}", ProjectStub);
				Logger.LogDebug("ProjectIPA={ProjectIPA}", ProjectIPA);
				Logger.LogDebug("IPPProjectIPA={IPPProjectIPA}", IPPProjectIPA);

				bool cookonthefly = Params.CookOnTheFly || Params.SkipCookOnTheFly;

				// if we are incremental check to see if we need to even update the IPA
				if (!Params.IterativeDeploy || !File.Exists(ProjectIPA) || bNeedsIPA)
				{
					// delete the .ipa to make sure it was made
					CommandUtils.DeleteFile(ProjectIPA);
					if (IPPProjectIPA.Length > 0)
					{
						CommandUtils.DeleteFile(IPPProjectIPA);
					}

					PlatformContext.bCreatedIPA = true;

					string IPPArguments = "RepackageFromStage \"" + (Params.IsCodeBasedProject ? Params.RawProjectPath.FullName : "Engine") + "\"";
					IPPArguments += " -config " + TargetConfiguration.ToString();
					IPPArguments += " -schemename " + SchemeName + " -schemeconfig \"" + SchemeConfiguration + "\"";

					// targetname will be eg FooClient for a Client Shipping build.
					IPPArguments += " -targetname " + SC.StageExecutables[0].Split("-".ToCharArray())[0];

					if (TargetConfiguration == UnrealTargetConfiguration.Shipping)
					{
						IPPArguments += " -compress=best";
					}

					// Determine if we should sign
					bool bNeedToSign = PlatformContext.GetCodeSignDesirability(Params);

					if (!String.IsNullOrEmpty(Params.BundleName))
					{
						// Have to sign when a bundle name is specified
						bNeedToSign = true;
						IPPArguments += " -bundlename " + Params.BundleName;
					}

					if (bNeedToSign)
					{
						IPPArguments += " -sign";
						if (Params.Distribution)
						{
							IPPArguments += " -distribution";
						}
						if (Params.IsCodeBasedProject)
						{
							IPPArguments += (" -codebased");
						}
					}

					if (IsBuiltAsFramework(Params, SC, PlatformContext.TargetPlatformType))
					{
						IPPArguments += " -buildasframework";
					}

					IPPArguments += (cookonthefly ? " -cookonthefly" : "");

					string CookPlatformName = PlatformContext.GetCookPlatform(Params.DedicatedServer, Params.Client);
					IPPArguments += " -stagedir \"" + CommandUtils.CombinePaths(Params.BaseStageDirectory, CookPlatformName) + "\"";
					IPPArguments += " -project \"" + Params.RawProjectPath + "\"";
					if (Params.IterativeDeploy)
					{
						IPPArguments += " -iterate";
					}
					if (!string.IsNullOrEmpty(Params.Provision))
					{
						IPPArguments += " -provision \"" + Params.Provision + "\"";
					}
					if (!string.IsNullOrEmpty(Params.Certificate))
					{
						IPPArguments += " -certificate \"" + Params.Certificate + "\"";
					}
					if (PlatformContext.PlatformName == "TVOS")
					{
						IPPArguments += " -tvos";
					}
					
					RunIPP(IPPArguments, Logger);

					if (IPPProjectIPA.Length > 0)
					{
						CommandUtils.CopyFile(IPPProjectIPA, ProjectIPA);
						CommandUtils.DeleteFile(IPPProjectIPA);
					}
				}

				// verify the .ipa exists
				if (!CommandUtils.FileExists(ProjectIPA))
				{
					throw new AutomationException(ExitCode.Error_FailedToCreateIPA, "PACKAGE FAILED - {0} was not created", ProjectIPA);
				}

				if (WorkingCL > 0)
				{
					// Open files for add or edit
					var ExtraFilesToCheckin = new List<string>
					{
						ProjectIPA
					};

					// check in the .ipa along with everything else
					UnrealBuild.AddBuildProductsToChangelist(WorkingCL, ExtraFilesToCheckin);
				}

				//@TODO: This automatically deploys after packaging, useful for testing on PC when iterating on IPP
				//Deploy(Params, SC);
			}
			else
			{
				// create the ipa
				string IPAName = CommandUtils.CombinePaths(Path.GetDirectoryName(Params.RawProjectPath.FullName), "Binaries", PlatformContext.PlatformName, (Params.Distribution ? "Distro_" : "") + Params.ShortProjectName + (SC.StageTargetConfigurations[0] != UnrealTargetConfiguration.Development ? ("-" + PlatformContext.PlatformName + "-" + SC.StageTargetConfigurations[0].ToString()) : "") + ".ipa");

				if (!Params.IterativeDeploy || !File.Exists(IPAName) || bNeedsIPA)
				{
					PlatformContext.bCreatedIPA = true;

					// code sign the app
					CodeSign(PlatformContext, Path.GetDirectoryName(ProjectGameExeFilename), Params.IsCodeBasedProject ? Params.ShortProjectName : Path.GetFileNameWithoutExtension(ProjectGameExeFilename), Params.RawProjectPath, SC.StageTargetConfigurations[0], SC.LocalRoot.FullName, Params.ShortProjectName, Path.GetDirectoryName(Params.RawProjectPath.FullName), SC.IsCodeBasedProject, Params.Distribution, Params.Provision, Params.Certificate, Params.Team, Params.AutomaticSigning, SchemeName, SchemeConfiguration, Logger);

					// now generate the ipa
					PackageIPA(Params, ProjectGameExeFilename, SC, PlatformContext.PlatformName);
				}
			}

			CommandUtils.PrintRunTime();
		}

	}
}

