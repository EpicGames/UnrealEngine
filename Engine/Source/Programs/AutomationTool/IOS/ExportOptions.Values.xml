<?xml version="1.0" encoding="UTF-8"?>
<root>
<methods>
	<type name="development">development</type>
	<type name="adhoc">adhoc</type>
	<type name="store">store</type>
</methods>
<signing>
	<type name="development">iPhone Developer</type>
	<type name="adhoc">iPhone Distribution</type>
	<type name="store">iPhone Distribution</type>
</signing>
<profiles>
	<type name="development">
		<key>com.yourcompany.project</key>
		<string>Project Development</string>
		<key>com.yourcompany.project.StickerPack</key>
		<string>ProjectExtension Development</string>
		<key>com.yourcompany.project.Extension1</key>
		<string>ProjectExtension Development</string>
	</type>
	<type name="adhoc">
		<key>com.yourcompany.project</key>
		<string>Project Adhoc Distribution</string>
		<key>com.yourcompany.project.StickerPack</key>
		<string>ProjectExtension Adhoc Distribution</string>
		<key>com.yourcompany.project.Extension</key>
		<string>ProjectExtension Adhoc Distribution</string>
	</type>
	<type name="store">
		<key>com.yourcompany.project</key>
		<string>Project Store Distribution</string>
		<key>com.yourcompany.project.StickerPack</key>
		<string>ProjectExtension Store Distribution</string>
		<key>com.yourcompany.project.Extension</key>
		<string>ProjectExtension Store Distribution</string>
	</type>
</profiles>
</root>