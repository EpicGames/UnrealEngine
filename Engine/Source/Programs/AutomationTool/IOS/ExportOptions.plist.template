<?xml version="1.0" encoding="UTF-8"?>
<!DOCTYPE plist PUBLIC "-//Apple//DTD PLIST 1.0//EN" "http://www.apple.com/DTDs/PropertyList-1.0.dtd">
<plist version="1.0">
<dict>
    <key>compileBitcode</key>
    <false/>
    <key>method</key>
    <string>{{method}}</string>
    <key>provisioningProfiles</key>
	<dict>
		{{profiles}}
	</dict>
	<key>signingCertificate</key>
	<string>{{signing}}</string>
	<key>signingStyle</key>
	<string>manual</string>
	<key>stripSwiftSymbols</key>
	<true/>
	<key>teamID</key>
	<string>96DBZ92D3Y</string>
</dict>
</plist>