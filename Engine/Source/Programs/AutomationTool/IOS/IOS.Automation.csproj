<Project Sdk="Microsoft.NET.Sdk">
  <Import Project="..\..\Shared\UnrealEngine.csproj.props" />
  
  <PropertyGroup>
    <OutputType>Library</OutputType>
    <GenerateAssemblyInfo>false</GenerateAssemblyInfo>
    <GenerateTargetFrameworkAttribute>false</GenerateTargetFrameworkAttribute>
    <RootNamespace>IOS.Automation</RootNamespace>
    <AssemblyName>IOS.Automation</AssemblyName>
    <WarningsNotAsErrors>612,618</WarningsNotAsErrors>
    <OutputPath Condition="'$(ArtifactsPath)' == ''">..\..\..\..\Binaries\DotNET\AutomationTool\AutomationScripts\Platforms\IOS</OutputPath>
    <TreatWarningsAsErrors>true</TreatWarningsAsErrors>
  </PropertyGroup>
  <ItemGroup>
    <!-- BouncyCastle.Crypto is not actually used but is required so that IPhonePackager has it available after we have copied files to a local folder -->
    <Reference Include="BouncyCastle.Crypto">
      <HintPath>..\..\..\..\Binaries\ThirdParty\IOS\BouncyCastle.Crypto.dll</HintPath>
      <Private>False</Private>
    </Reference>
    <Reference Include="Ionic.Zip.Reduced">
      <HintPath>..\..\..\..\Binaries\DotNET\Ionic.Zip.Reduced.dll</HintPath>
      <Private>True</Private>
    </Reference>
  </ItemGroup>

  <ItemGroup>
    <Compile Include="..\..\Shared\MetaData.cs">
      <Link>Properties\MetaData.cs</Link>
    </Compile>
  </ItemGroup>
  <ItemGroup>
    <ProjectReference Include="..\..\Shared\EpicGames.Core\EpicGames.Core.csproj" PrivateAssets="All"><Private>false</Private></ProjectReference>
    <ProjectReference Include="..\..\UnrealBuildTool\UnrealBuildTool.csproj" PrivateAssets="All"><Private>false</Private></ProjectReference>
    <ProjectReference Include="..\AutomationUtils\AutomationUtils.Automation.csproj" PrivateAssets="All"><Private>false</Private></ProjectReference>
    <ProjectReference Include="..\Apple\Apple.Automation.csproj" />
  </ItemGroup>
  <ItemGroup>
    <PackageReference Include="System.Resources.Extensions" Version="10.0.6" PrivateAssets="all" />
    <PackageReference Include="Portable.BouncyCastle" Version="1.9.0" PrivateAssets="all" />
  </ItemGroup>
</Project>