// Copyright Epic Games, Inc. All Rights Reserved.
using System;
using System.Collections.Generic;
using System.Linq;
using System.Text;
using System.IO;
using AutomationTool;
using UnrealBuildTool;
using System.Text.RegularExpressions;
using Ionic.Zip;
using Ionic.Zlib;
using System.Security.Principal;
using System.Threading;
using System.Diagnostics;
using EpicGames.Core;
using System.Xml;
using IOS.Automation;
using UnrealBuildBase;
using Microsoft.Extensions.Logging;

static class IOSEnvVarNames
{
	// Should we code sign when staging?  (defaults to 1 if not present)
	static public readonly string CodeSignWhenStaging = "uebp_CodeSignWhenStaging";
}

class IOSClientProcess : IProcessResult
{
	private IProcessResult childProcess;
	private Thread consoleLogWorker;
	//private bool			processConsoleLogs;

	public IOSClientProcess(IProcessResult inChildProcess, string inDeviceID)
	{
		childProcess = inChildProcess;

		// Startup another thread that collect device console logs
		//processConsoleLogs = true;
		consoleLogWorker = new Thread(() => ProcessConsoleOutput(inDeviceID));
		consoleLogWorker.Start();
	}

	public void StopProcess(bool KillDescendants = true)
	{
		childProcess.StopProcess(KillDescendants);
		StopConsoleOutput();
	}

	public bool HasExited
	{
		get
		{
			bool result = childProcess.HasExited;

			if (result)
			{
				StopConsoleOutput();
			}

			return result;
		}
	}

	public string GetProcessName()
	{
		return childProcess.GetProcessName();
	}

	public void OnProcessExited()
	{
		childProcess.OnProcessExited();
		StopConsoleOutput();
	}

	public void DisposeProcess()
	{
		childProcess.DisposeProcess();
	}

	public void StdOut(object sender, DataReceivedEventArgs e)
	{
		childProcess.StdOut(sender, e);
	}

	public void StdErr(object sender, DataReceivedEventArgs e)
	{
		childProcess.StdErr(sender, e);
	}

	public int ExitCode
	{
		get { return childProcess.ExitCode; }
		set { childProcess.ExitCode = value; }
	}

	public bool bExitCodeSuccess => ExitCode == 0;

	public string Output
	{
		get { return childProcess.Output; }
	}

	public Process ProcessObject
	{
		get { return childProcess.ProcessObject; }
	}

	public new string ToString()
	{
		return childProcess.ToString();
	}

	public void WaitForExit()
	{
		childProcess.WaitForExit();
	}

	public FileReference WriteOutputToFile(string FileName)
	{
		return childProcess.WriteOutputToFile(FileName);
	}

	private void StopConsoleOutput()
	{
		//processConsoleLogs = false;
		consoleLogWorker.Join();
	}

	public void ProcessConsoleOutput(string inDeviceID)
	{
		// 		MobileDeviceInstance	targetDevice = null;
		// 		foreach(MobileDeviceInstance curDevice in MobileDeviceInstanceManager.GetSnapshotInstanceList())
		// 		{
		// 			if(curDevice.DeviceId == inDeviceID)
		// 			{
		// 				targetDevice = curDevice;
		// 				break;
		// 			}
		// 		}
		// 		
		// 		if(targetDevice == null)
		// 		{
		// 			return;
		// 		}
		// 		
		// 		targetDevice.StartSyslogService();
		// 		
		// 		while(processConsoleLogs)
		// 		{
		// 			string logData = targetDevice.GetSyslogData();
		// 			
		// 			Console.WriteLine("DeviceLog: " + logData);
		// 		}
		// 		
		// 		targetDevice.StopSyslogService();
	}

}

public class IOSPlatform : ApplePlatform
{
	private enum IOSLaunchTool
	{
		Libimobiledevice,
		GoIOS,
		XcrunDevicectl,
		Open
	}

	private static string GoIOSPath => CombinePaths(CommandUtils.CmdEnv.LocalRoot, "Engine/Extras/ThirdPartyNotUE/go-ios/Win64/ios.exe");

	// Intentionally process-lifetime: the Job Object handle must stay open so the OS
	// auto-kills go-ios tunnels on UAT exit
	private static ManagedProcessGroup s_tunnelProcessGroup;

	public bool bCreatedIPA = false;

	public string PlatformName = null;
	public string SDKName = null;
	public IOSPlatform()
		: this(UnrealTargetPlatform.IOS)
	{
	}

	public IOSPlatform(UnrealTargetPlatform TargetPlatform)
		: base(TargetPlatform)
	{
		PlatformName = TargetPlatform.ToString();
		SDKName = (TargetPlatform == UnrealTargetPlatform.TVOS) ? "appletvos" : "iphoneos";
	}

	public override bool GetDeviceUpdateSoftwareCommand(out string Command, out string Params, ref bool bRequiresPrivilegeElevation, ref bool bCreateWindow, ITurnkeyContext TurnkeyContext, DeviceInfo Device)
	{
		if (UnrealBuildTool.BuildHostPlatform.Current.Platform != UnrealTargetPlatform.Mac)
		{
			Command = Params = null;
			return true;
		}

		TurnkeyContext.Log("Installing an offline downloaded .ipsw onto your device using the Apple Configurator application.");

		// cfgtool needs ECID, not UDID, so find it
		string Configurator = Path.Combine(GetConfiguratorLocation().Replace(" ", "\\ "), "Contents/MacOS/cfgutil");

		string CfgUtilParams = string.Format("-c '{0} list | grep {1}'", Configurator, Device.Id);
		string CfgUtilOutput = UnrealBuildTool.Utils.RunLocalProcessAndReturnStdOut("sh", CfgUtilParams);
		bRequiresPrivilegeElevation = false;

		Match Result = Regex.Match(CfgUtilOutput, @"Type: (\S*).*ECID: (\S*)");
		if (!Result.Success)
		{
			TurnkeyContext.ReportError($"Unable to find the given deviceid: {Device} in cfgutil output");
			Command = Params = null;
			return false;
		}

		Command = "sh";
		Params = string.Format("-c '{0} --ecid {1} update --ipsw $(CopyOutputPath)'", Configurator, Result.Groups[2]);

		return true;
	}





	private class VerifyIOSSettings
	{
		public string CodeSigningIdentity = null;
		public string BundleId = null;
		public string Account = null;
		public string Password = null;
		public string Team = null;
		public string Provision = null;

		public string RubyScript = Path.Combine(Unreal.EngineDirectory.FullName, "Build/Turnkey/VerifyIOS.ru");
		public string InstallCertScript = Path.Combine(Unreal.EngineDirectory.FullName, "Build/Turnkey/InstallCert.ru");

		private ITurnkeyContext TurnkeyContext;

		public VerifyIOSSettings(BuildCommand Command, ITurnkeyContext TurnkeyContext)
		{
			this.TurnkeyContext = TurnkeyContext;

			FileReference ProjectPath = Command.ParseProjectParam();
			string ProjectName = ProjectPath == null ? "" : ProjectPath.GetFileNameWithoutAnyExtensions();

			ConfigHierarchy EngineConfig = ConfigCache.ReadHierarchy(ConfigHierarchyType.Engine, ProjectPath == null ? null : ProjectPath.Directory, UnrealTargetPlatform.IOS);

			// first look for settings on the commandline:
			CodeSigningIdentity = Command.ParseParamValue("certificate");
			BundleId = Command.ParseParamValue("bundleid");
			Account = Command.ParseParamValue("devcenterusername");
			Password = Command.ParseParamValue("devcenterpassword");
			Team = Command.ParseParamValue("teamid");
			Provision = Command.ParseParamValue("provision");

			if (string.IsNullOrEmpty(Team)) Team = TurnkeyContext.GetVariable("User_AppleDevCenterTeamID");
			if (string.IsNullOrEmpty(Account)) Account = TurnkeyContext.GetVariable("User_AppleDevCenterUsername");
			if (string.IsNullOrEmpty(Provision)) Provision = TurnkeyContext.GetVariable("User_IOSProvisioningProfile");

			// fall back to ini for anything else
			if (string.IsNullOrEmpty(CodeSigningIdentity)) EngineConfig.GetString("/Script/IOSRuntimeSettings.IOSRuntimeSettings", "DevCodeSigningIdentity", out CodeSigningIdentity);
			if (string.IsNullOrEmpty(BundleId)) EngineConfig.GetString("/Script/IOSRuntimeSettings.IOSRuntimeSettings", "BundleIdentifier", out BundleId);
			if (string.IsNullOrEmpty(Team)) EngineConfig.GetString("/Script/IOSRuntimeSettings.IOSRuntimeSettings", "IOSTeamID", out Team);
			if (string.IsNullOrEmpty(Account)) EngineConfig.GetString("/Script/IOSRuntimeSettings.IOSRuntimeSettings", "DevCenterUsername", out Account);
			if (string.IsNullOrEmpty(Password)) EngineConfig.GetString("/Script/IOSRuntimeSettings.IOSRuntimeSettings", "DevCenterPassword", out Password);
			if (string.IsNullOrEmpty(Provision)) EngineConfig.GetString("/Script/IOSRuntimeSettings.IOSRuntimeSettings", "MobileProvision", out Provision);
			
			BundleId = BundleId.Replace("[PROJECT_NAME]", ProjectName);

			// some are required
			if (string.IsNullOrEmpty(BundleId))
			{
				throw new AutomationException("Turnkey IOS verification requires bundle id (have '{1}', ex: com.company.foo)", CodeSigningIdentity, BundleId);
			}
		}

		public bool RunCommandMaybeInteractive(string Command, string Params, bool bInteractive)
		{
			Console.WriteLine("Running Command '{0} {1}'", Command, Params);

			int ExitCode;
			// if non-interactive, we can just run directly in the current shell
			if (!bInteractive)
			{
				UnrealBuildTool.Utils.RunLocalProcessAndReturnStdOut(Command, Params, Log.Logger, out ExitCode);
			}
			else
			{
				// otherwise, run in a new Terminal window via AppleScript
				string ReturnCodeFilename = Path.GetTempFileName();

				// run potentially interactive scripts in a Terminal window
				Params = string.Format(
						" -e \"tell application \\\"Finder\\\"\"" +
						" -e   \"set desktopBounds to bounds of window of desktop\"" +
						" -e \"end tell\"" +
						" -e \"tell application \\\"Terminal\\\"\"" +
						" -e   \"activate\"" +
						" -e   \"set newTab to do script (\\\"{3}; {0} {1}; echo $? > {2}; {3}; exit\\\")\"" +
						" -e   \"set newWindow to window 1\"" +
						" -e   \"set size of newWindow to {{ item 3 of desktopBounds / 2, item 4 of desktopBounds / 2 }}\"" +
						" -e   \"repeat\"" +
						" -e     \"delay 1\"" +
						" -e     \"if not busy of newTab then exit repeat\"" +
						" -e   \"end repeat\"" +
						" -e   \"set exitCode to item 1 of paragraphs of (read \\\"{2}\\\")\"" +
						" -e   \"if exitCode is equal to \\\"0\\\" then\"" +
						" -e     \"close newWindow\"" +
						" -e   \"end if\"" +
						" -e \"end tell\"",
						Command, Params.Replace("\"", "\\\\\\\""), ReturnCodeFilename, "printf \\\\\\\"\\\\\\n\\\\\\n\\\\\\n\\\\\\n\\\\\\\"");

				Console.WriteLine("\n\n\n{0}\n\n\n", Params);

				UnrealBuildTool.Utils.RunLocalProcessAndReturnStdOut("osascript", Params, Log.Logger, out ExitCode);
				if (ExitCode == 0)
				{
					ExitCode = int.Parse(File.ReadAllText(ReturnCodeFilename));
					File.Delete(ReturnCodeFilename);
				}
				
			}

			if (ExitCode != 0)
			{
				// only ExitCode 3 (needs cert) can be handled. Any other error can't be fixed (Interactive means it can't be fixed)
				if (!bInteractive || ExitCode != 3)
				{
					if (ExitCode == 3)
					{
						TurnkeyContext.ReportError("Signing certificate is required.");
					}
					else
					{
						// @todo turnkey: turn exitcodes into useful messages
						TurnkeyContext.ReportError($"Ruby command exited with code {ExitCode}");
					}

					return false;
				}

				// only here with ExitCode 3
				if (!InstallCert())
				{
					TurnkeyContext.ReportError($"Certificate installation failed.");
					return false;
				}
			}

			return ExitCode == 0;
		}

		public bool RunRubyCommand(bool bVerifyOnly, string DeviceName)
		{
			string Params;

			Params = string.Format("--bundleid {0}", BundleId);

			if (!string.IsNullOrEmpty(CodeSigningIdentity))
			{
				Params += string.Format(" --identity \"{0}\"", CodeSigningIdentity);
			}

			if (!string.IsNullOrEmpty(Account))
			{
				Params += string.Format(" --login {0}", Account);
			}
			if (!string.IsNullOrEmpty(Password))
			{
				Params += string.Format(" --password {0}", Password);
			}
			if (!string.IsNullOrEmpty(Team))
			{
				Params += string.Format(" --team {0}", Team);
			}

			if (!string.IsNullOrEmpty(Provision))
			{
				Params += string.Format(" --provision {0}", Provision);
			}

			if (!string.IsNullOrEmpty(DeviceName))
			{
				Params += string.Format(" --device {0}", DeviceName);
			}

			if (bVerifyOnly)
			{
				Params += string.Format(" --verifyonly");
			}

			return RunCommandMaybeInteractive(RubyScript, Params, !bVerifyOnly);
		}

		private bool InstallCert()
		{
			//		string ProjectName = TurnkeyContext.GetVariable("Project");

			string CertLoc = null;

			if (!string.IsNullOrEmpty(BundleId))
			{
				CertLoc = TurnkeyContext.RetrieveFileSource("DevCert: " + BundleId);
			}
			if (CertLoc == null)
			{
				CertLoc = TurnkeyContext.RetrieveFileSource("DevCert");
			}

			if (CertLoc != null)
			{
				// get the cert password from Studio settings
				string CertPassword = TurnkeyContext.GetVariable("Studio_AppleSigningCertPassword");

				TurnkeyContext.Log($"Will install cert from: '{CertLoc}'");

				// osascript -e 'Tell application "System Events" to display dialog "Enter the network password:" with hidden answer default answer ""' -e 'text returned of result' 2>/dev/null
				string CommandLine = string.Format("'{0}' '{1}'", CertLoc, CertPassword);

				// run ruby script to install cert
				return RunCommandMaybeInteractive(InstallCertScript, CommandLine, true);
			}
			else
			{
				TurnkeyContext.ReportError("Unable to find a tagged source for DevCert");

				return false;
			}

		}
	}



	string GetConfiguratorLocation()
	{
		string FindCommand = "-c 'mdfind \"kMDItemKind == Application\" | grep \"Apple Configurator 2.app\"'";
		return UnrealBuildTool.Utils.RunLocalProcessAndReturnStdOut("sh", FindCommand);
	}

	//Disabling for 5.0 early access as this code was not executing and has not been tested.
	/*
	public override bool UpdateHostPrerequisites(BuildCommand Command, ITurnkeyContext TurnkeyContext, bool bVerifyOnly)
	{
		int ExitCode;

		if (HostPlatform.Current.HostEditorPlatform != UnrealTargetPlatform.Mac)
		{
			return base.UpdateHostPrerequisites(Command, TurnkeyContext, bVerifyOnly);
		}

		// make sure the Configurator is installed
		string ConfiguratorLocation = GetConfiguratorLocation();

		if (ConfiguratorLocation == "")
		{
			if (bVerifyOnly)
			{
				TurnkeyContext.ReportError("Apple Configurator 2 is required.");
				return false;
			}

			TurnkeyContext.PauseForUser("Apple Configurator 2 is required for some automation to work. You should install it from the App Store. Launching...");

			// we need to install Configurator 2, and we will block until it's done
			UnrealBuildTool.Utils.RunLocalProcessAndReturnStdOut("open", "macappstore://apps.apple.com/us/app/apple-configurator-2/id1037126344?mt=12");

			while ((ConfiguratorLocation = GetConfiguratorLocation()) == "")
			{
				Thread.Sleep(1000);
			}
		}

		string IsFastlaneInstalled = UnrealBuildTool.Utils.RunLocalProcessAndReturnStdOut("/usr/bin/gem", "list -ie fastlane");
		if (IsFastlaneInstalled != "true")
		{
			Console.WriteLine("Fastlane is not installed");
			if (bVerifyOnly)
			{
				TurnkeyContext.ReportError("Fastlane is not installed.");
				return false;
			}

			TurnkeyContext.PauseForUser("Installing Fastlane from internet source. You may ignore the error about the bin directory not in your path.");

			// install missing fastlane without needing sudo
			UnrealBuildTool.Utils.RunLocalProcessAndReturnStdOut("/usr/bin/gem", "install fastlane --user-install --no-document", out ExitCode, true);

			if (ExitCode != 0)
			{
				return false;
			}
		}

		VerifyIOSSettings Settings = new VerifyIOSSettings(Command, TurnkeyContext);

		// look if we have a cert that matches it
		return Settings.RunRubyCommand(bVerifyOnly, null);
	}

	public override bool UpdateDevicePrerequisites(DeviceInfo Device, BuildCommand Command, ITurnkeyContext TurnkeyContext, bool bVerifyOnly)
	{
		if (HostPlatform.Current.HostEditorPlatform != UnrealTargetPlatform.Mac)
		{
			return base.UpdateDevicePrerequisites(Device, Command, TurnkeyContext, bVerifyOnly);
		}

		VerifyIOSSettings Settings = new VerifyIOSSettings(Command, TurnkeyContext);

		// @todo turnkey - better to use the device's udid if it's set properly in DeviceInfo
		string DeviceName = Device == null ? null : Device.Name;

		// now look for a provision that can be used with a (maybe newly) instally cert
		return Settings.RunRubyCommand(bVerifyOnly, DeviceName);
	}
*/

	public override DeviceInfo[] GetDevices()
	{

		List<DeviceInfo> Devices = new List<DeviceInfo>();

		var IdeviceIdPath = GetPathToLibiMobileDeviceTool("idevice_id");
		string Output = Utils.RunLocalProcessAndReturnStdOut(IdeviceIdPath, "");
		var ConnectedDevicesUDIDs = Output.Split(new string[] { Environment.NewLine }, StringSplitOptions.None);

		foreach (string UnparsedUDID in ConnectedDevicesUDIDs)
		{
			DeviceInfo CurrentDevice = new DeviceInfo(TargetPlatformType);
			var IdeviceInfoPath = GetPathToLibiMobileDeviceTool("ideviceinfo");
			String ParsedUDID = UnparsedUDID.Split(" ").First();
			String IdeviceInfoArgs = "-u " + ParsedUDID;

			if (UnparsedUDID.Contains("Network"))
			{
				CurrentDevice.PlatformValues["Connection"] = "Network";
				IdeviceInfoArgs = "-n " + IdeviceInfoArgs;
			}
			else
			{
				CurrentDevice.PlatformValues["Connection"] = "USB";
			}

			string OutputInfo = Utils.RunLocalProcessAndReturnStdOut(IdeviceInfoPath, IdeviceInfoArgs);

			foreach (string Line in OutputInfo.Split(Environment.NewLine.ToCharArray()))
			{
				// check we are returning the proper device for this class
				if (Line.StartsWith("DeviceClass:"))
				{
					bool bIsDeviceTVOS = Line.Split(": ").Last().ToLower() == "tvos";
					if (bIsDeviceTVOS != (TargetPlatformType == UnrealTargetPlatform.TVOS))
					{
						Devices.Remove(CurrentDevice);
					}
				}
				else if (Line.StartsWith("DeviceName: "))
				{
					CurrentDevice.Name = Line.Split(": ").Last();
				}
				else if (Line.StartsWith("UniqueDeviceID: "))
				{
					CurrentDevice.Id = Line.Split(": ").Last();
				}
				else if (Line.StartsWith("ProductType: "))
				{
					CurrentDevice.Type = Line.Split(": ").Last();
				}
				else if (Line.StartsWith("ProductVersion: "))
				{
					CurrentDevice.SoftwareVersion = Line.Split(": ").Last();
				}
			}
			Devices.Add(CurrentDevice);
		}
		return Devices.ToArray();
	}

	public override string GetPlatformPakCommandLine(ProjectParams Params, DeploymentContext SC)
	{
		string PakParams = " -patchpaddingalign=0";

		string OodleDllPath = DirectoryReference.Combine(SC.ProjectRoot, "Binaries/ThirdParty/Oodle/Mac/libUnrealPakPlugin.dylib").FullName;
		if (File.Exists(OodleDllPath))
		{
			PakParams += String.Format(" -customcompressor=\"{0}\"", OodleDllPath);
		}

		return PakParams;
	}

	public virtual void GetProvisioningData(FileReference InProject, bool bDistribution, out string MobileProvision, out string SigningCertificate, out string TeamUUID, out bool bAutomaticSigning)
	{
		IOSExports.GetProvisioningData(InProject, bDistribution, out MobileProvision, out SigningCertificate, out TeamUUID, out bAutomaticSigning);
	}

	public virtual bool DeployGeneratePList(FileReference ProjectFile, UnrealTargetConfiguration Config, DirectoryReference ProjectDirectory, bool bIsUEGame, string GameName, bool bIsClient, string ProjectName, DirectoryReference InEngineDir, DirectoryReference AppDirectory, TargetReceipt Receipt)
	{
		return IOSExports.GeneratePList(ProjectFile, Config, ProjectDirectory, bIsUEGame, GameName, bIsClient, ProjectName, InEngineDir, AppDirectory, Receipt, Log.Logger);
	}

	// Determine if we should code sign
	public bool GetCodeSignDesirability(ProjectParams Params)
	{
		//@TODO: Would like to make this true, as it's the common case for everyone else
		bool bDefaultNeedsSign = true;

		bool bNeedsSign = false;
		string EnvVar = InternalUtils.GetEnvironmentVariable(IOSEnvVarNames.CodeSignWhenStaging, bDefaultNeedsSign ? "1" : "0", /*bQuiet=*/ false);
		if (!bool.TryParse(EnvVar, out bNeedsSign))
		{
			int BoolAsInt;
			if (int.TryParse(EnvVar, out BoolAsInt))
			{
				bNeedsSign = BoolAsInt != 0;
			}
			else
			{
				bNeedsSign = bDefaultNeedsSign;
			}
		}

		if (!String.IsNullOrEmpty(Params.BundleName))
		{
			// Have to sign when a bundle name is specified
			bNeedsSign = true;
		}

		return bNeedsSign;
	}

	private void StageCustomLocalizationResources(ProjectParams Params, DeploymentContext SC)
	{
		string RelativeResourcesPath = CombinePaths("Build", "IOS", "Resources", "Localizations");
		DirectoryReference LocalizationDirectory = DirectoryReference.Combine(Params.RawProjectPath.Directory, RelativeResourcesPath);
		if (DirectoryReference.Exists(LocalizationDirectory))
		{
			IEnumerable<DirectoryReference> LocalizationDirsToStage = DirectoryReference.EnumerateDirectories(LocalizationDirectory, "*.lproj", SearchOption.TopDirectoryOnly);
			Logger.LogInformation("There are {0} Localization directories.", LocalizationDirsToStage.Count());

			foreach (DirectoryReference FullLocDirPath in LocalizationDirsToStage)
			{
				StagedDirectoryReference LocInStageDir = new StagedDirectoryReference(FullLocDirPath.GetDirectoryName());
				SC.StageFiles(StagedFileType.SystemNonUFS, FullLocDirPath, StageFilesSearch.TopDirectoryOnly, LocInStageDir);
			}
		}
		else
		{
			Logger.LogInformation("App has no custom Localization resources");
		}
	}

	public override void Package(ProjectParams Params, DeploymentContext SC, int WorkingCL)
	{
		if (AppleExports.CreatingAppOnWindows(Params.RawProjectPath))
		{
			AppleOnWindowsAppMaker.Package(this, Params, SC, WorkingCL, Logger);	
			return;
		}
		
		// use the shared packaging with xcode mode
		base.Package(Params, SC, WorkingCL);
	}

	public virtual bool PrepForUATPackageOrDeploy(UnrealTargetConfiguration Config, FileReference ProjectFile, string InProjectName, DirectoryReference InProjectDirectory, FileReference Executable, DirectoryReference InEngineDir, bool bForDistribution, string CookFlavor, bool bIsDataDeploy, bool bCreateStubIPA, bool bIsUEGame)
	{
		FileReference TargetReceiptFileName = AppleOnWindowsAppMaker.GetTargetReceiptFileName(Config, Executable.FullName, InEngineDir, InProjectDirectory, ProjectFile, bIsUEGame);

		return IOSExports.PrepForUATPackageOrDeploy(Config, ProjectFile, InProjectName, InProjectDirectory, Executable, InEngineDir, bForDistribution, CookFlavor, bIsDataDeploy, bCreateStubIPA, TargetReceiptFileName, Log.Logger);
	}

	public override void GetFilesToDeployOrStage(ProjectParams Params, DeploymentContext SC)
	{
		//		if (UnrealBuildTool.BuildHostPlatform.Current.Platform != UnrealTargetPlatform.Mac)
		{

			// copy any additional framework assets that will be needed at runtime
			{
				DirectoryReference SourcePath = DirectoryReference.Combine((SC.IsCodeBasedProject ? SC.ProjectRoot : SC.EngineRoot), "Intermediate", "IOS", "FrameworkAssets");
				if (DirectoryReference.Exists(SourcePath))
				{
					SC.StageFiles(StagedFileType.SystemNonUFS, SourcePath, StageFilesSearch.AllDirectories, StagedDirectoryReference.Root);
				}
			}


			// copy the plist (only if code signing, as it's protected by the code sign blob in the executable and can't be modified independently)
			if (GetCodeSignDesirability(Params))
			{
				// this would be FooClient when making a client-only build
				string TargetName = SC.StageExecutables[0].Split("-".ToCharArray())[0];
				DirectoryReference SourcePath = DirectoryReference.Combine((SC.IsCodeBasedProject ? SC.ProjectRoot : DirectoryReference.Combine(SC.LocalRoot, "Engine")), "Intermediate", PlatformName);
				FileReference TargetPListFile = FileReference.Combine(SourcePath, (SC.IsCodeBasedProject ? TargetName : "UnrealGame") + "-Info.plist");

				//				if (!File.Exists(TargetPListFile))
				{
					// ensure the plist, entitlements, and provision files are properly copied
					Console.WriteLine("CookPlat {0}, this {1}", GetCookPlatform(false, false), ToString());
					if (!SC.IsCodeBasedProject)
					{
						UnrealBuildTool.PlatformExports.SetRemoteIniPath(SC.ProjectRoot.FullName);
					}

					if (SC.StageTargetConfigurations.Count != 1)
					{
						throw new AutomationException("iOS is currently only able to package one target configuration at a time, but StageTargetConfigurations contained {0} configurations", SC.StageTargetConfigurations.Count);
					}

					var TargetConfiguration = SC.StageTargetConfigurations[0];

					DirectoryReference ProjectRoot = SC.ProjectRoot;
					
					// keep old logic for BP projects with legacy
					if (AppleExports.CreatingAppOnWindows(SC.RawProjectPath) && !SC.IsCodeBasedProject)
					{
						ProjectRoot = DirectoryReference.Combine(SC.LocalRoot, "Engine");
					}

					DeployGeneratePList(
							SC.RawProjectPath,
							TargetConfiguration,
							ProjectRoot,
							!SC.IsCodeBasedProject,
							(SC.IsCodeBasedProject ? TargetName : "UnrealGame"),
							SC.IsCodeBasedProject ? false : Params.Client, // Code based projects will have Client in their executable name already
							SC.ShortProjectName, DirectoryReference.Combine(SC.LocalRoot, "Engine"),
							DirectoryReference.Combine((SC.IsCodeBasedProject ? SC.ProjectRoot : DirectoryReference.Combine(SC.LocalRoot, "Engine")), "Binaries", PlatformName, AppleExports.CreatingAppOnWindows(SC.RawProjectPath) ? "Payload" : "", (SC.IsCodeBasedProject ? SC.ShortProjectName : "UnrealGame") + ".app"),
							SC.StageTargets[0].Receipt);

					if (AppleExports.CreatingAppOnWindows(SC.RawProjectPath))
					{
						// copy the plist to the stage dir
						SC.StageFile(StagedFileType.SystemNonUFS, TargetPListFile, new StagedFileReference("Info.plist"));
					}
				}	

				// copy the udebugsymbols if they exist
				{
					ConfigHierarchy PlatformGameConfig;
					bool bIncludeSymbols = false;
					if (Params.EngineConfigs.TryGetValue(SC.StageTargetPlatform.PlatformType, out PlatformGameConfig))
					{
						PlatformGameConfig.GetBool("/Script/IOSRuntimeSettings.IOSRuntimeSettings", "bGenerateCrashReportSymbols", out bIncludeSymbols);
					}
					if (bIncludeSymbols)
					{
						FileReference SymbolFileName = FileReference.Combine((SC.IsCodeBasedProject ? SC.ProjectRoot : SC.EngineRoot), "Binaries", "IOS", SC.StageExecutables[0] + ".udebugsymbols");
						if (FileReference.Exists(SymbolFileName))
						{
							SC.StageFile(StagedFileType.NonUFS, SymbolFileName, new StagedFileReference((Params.ShortProjectName + ".udebugsymbols").ToLowerInvariant()));
						}
					}
				}
			}
		}

		{
			// Get the final output directory for cooked data
			DirectoryReference CookOutputDir;
			if (!String.IsNullOrEmpty(Params.CookOutputDir))
			{
				CookOutputDir = DirectoryReference.Combine(new DirectoryReference(Params.CookOutputDir), SC.CookPlatform);
			}
			else if (Params.CookInEditor)
			{
				CookOutputDir = DirectoryReference.Combine(SC.ProjectRoot, "Saved", "EditorCooked", SC.CookPlatform);
			}
			else
			{
				CookOutputDir = DirectoryReference.Combine(SC.ProjectRoot, "Saved", "Cooked", SC.CookPlatform);
			}
		}

		{
			// Stage the mute.caf file used by SoundSwitch for mute switch detection
			FileReference MuteCafFile = FileReference.Combine(SC.EngineRoot, "Source", "ThirdParty", "IOS", "SoundSwitch", "SoundSwitch", "SoundSwitch", "mute.caf");
			if (FileReference.Exists(MuteCafFile))
			{
				SC.StageFile(StagedFileType.SystemNonUFS, MuteCafFile, new StagedFileReference("mute.caf"));
			}
		}

		// Copy any project defined iOS specific localization resources into Staging (ie: App Display Name)
		StageCustomLocalizationResources(Params, SC);
	}

	protected void StageMovieFiles(DirectoryReference InputDir, DeploymentContext SC)
	{
		if (DirectoryReference.Exists(InputDir))
		{
			foreach (FileReference InputFile in DirectoryReference.EnumerateFiles(InputDir, "*", SearchOption.AllDirectories))
			{
				if (!InputFile.HasExtension(".uasset") && !InputFile.HasExtension(".umap"))
				{
					SC.StageFile(StagedFileType.NonUFS, InputFile);
				}
			}
		}
	}
	protected void StageMovieFile(DirectoryReference InputDir, string Filename, DeploymentContext SC)
	{
		if (DirectoryReference.Exists(InputDir))
		{
			foreach (FileReference InputFile in DirectoryReference.EnumerateFiles(InputDir, "*", SearchOption.AllDirectories))
			{

				if (!InputFile.HasExtension(".uasset") && !InputFile.HasExtension(".umap") && InputFile.GetFileNameWithoutExtension().Contains(Filename))
				{
					SC.StageFile(StagedFileType.NonUFS, InputFile);
				}
			}
		}
	}

	public override void GetFilesToArchive(ProjectParams Params, DeploymentContext SC)
	{
		if (AppleExports.CreatingAppOnWindows(Params.RawProjectPath))
		{
			AppleOnWindowsAppMaker.GetFilesToArchive(Params, SC, Logger, PlatformName);
			return;
		}
		
		// use the shared archiving with xcode mode
		base.GetFilesToArchive(Params, SC);
	}

	public override bool RetrieveDeployedManifests(ProjectParams Params, DeploymentContext SC, string DeviceName, out List<string> UFSManifests, out List<string> NonUFSManifests)
	{
		if (Params.Devices.Count != 1)
		{
			throw new AutomationException("Can only retrieve deployed manifests from a single device, but {0} were specified", Params.Devices.Count);
		}

		bool Result = true;
		UFSManifests = new List<string>();
		NonUFSManifests = new List<string>();
		try
		{
			string BundleIdentifier = "";
			if (File.Exists(Params.BaseStageDirectory + "/" + PlatformName + "/Info.plist"))
			{
				string Contents = File.ReadAllText(SC.StageDirectory + "/Info.plist");
				int Pos = Contents.IndexOf("CFBundleIdentifier");
				Pos = Contents.IndexOf("<string>", Pos) + 8;
				int EndPos = Contents.IndexOf("</string>", Pos);
				BundleIdentifier = Contents.Substring(Pos, EndPos - Pos);
			}

			string IdeviceInstallerArgs = "--list-apps -u " + Params.DeviceNames[0];
			IdeviceInstallerArgs = GetLibimobileDeviceNetworkedArgument(IdeviceInstallerArgs, Params.DeviceNames[0]);

			var DeviceInstaller = GetPathToLibiMobileDeviceTool("ideviceinstaller");
			Logger.LogInformation("Checking if bundle '{BundleIdentifier}' is installed", BundleIdentifier);

			string Output = "";
			IProcessResult RunResult = CommandUtils.Run(DeviceInstaller, IdeviceInstallerArgs, null, CommandUtils.ERunOptions.NoLoggingOfRunCommand | CommandUtils.ERunOptions.AppMustExist);
			if (RunResult.ExitCode == 0)
			{
				if (!String.IsNullOrEmpty(RunResult.Output))
				{
					Output = RunResult.Output;
				}
			}
			else
			{
				throw new CommandFailedException((ExitCode)RunResult.ExitCode, 
												 String.Format("Command failed (Result:{3}): {0} {1}",
												 DeviceInstaller, IdeviceInstallerArgs, RunResult.ExitCode)){ OutputFormat = AutomationExceptionOutputFormat.Minimal };
			}

			bool bBundleIsInstalled = Output.Contains(string.Format("CFBundleIdentifier -> {0}{1}", BundleIdentifier, Environment.NewLine));
			int ExitCode = 0;

			if (bBundleIsInstalled)
			{
				Logger.LogInformation("Bundle {BundleIdentifier} found, retrieving deployed manifests...", BundleIdentifier);

				var DeviceFS = GetPathToLibiMobileDeviceTool("idevicefs");

				string AllCommandsToPush = " push " + CombinePaths(Params.BaseStageDirectory, PlatformName, SC.GetUFSDeployedManifestFileName(null)) + "\n"
										+ " push " + CombinePaths(Params.BaseStageDirectory, PlatformName, SC.GetNonUFSDeployedManifestFileName(null));		
				System.IO.File.WriteAllText(Directory.GetCurrentDirectory() + "\\CommandsToPush.txt", AllCommandsToPush);

				string IdeviceFSArgs = "-b " + "\"" + BundleIdentifier + " -x " + Directory.GetCurrentDirectory() + "\\CommandsToPush.txt -u " + "\"" + Params.DeviceNames[0];
				IdeviceFSArgs = GetLibimobileDeviceNetworkedArgument(IdeviceFSArgs, Params.DeviceNames[0]);

				Utils.RunLocalProcessAndReturnStdOut(DeviceFS, IdeviceFSArgs, Log.Logger, out ExitCode);
				if (ExitCode != 0)
				{
					throw new AutomationException("Failed to deploy manifest to mobile device.");
				}

				string[] ManifestFiles = Directory.GetFiles(CombinePaths(Params.BaseStageDirectory, PlatformName), "*_Manifest_UFS*.txt");
				UFSManifests.AddRange(ManifestFiles);

				ManifestFiles = Directory.GetFiles(CombinePaths(Params.BaseStageDirectory, PlatformName), "*_Manifest_NonUFS*.txt");
				NonUFSManifests.AddRange(ManifestFiles);
			}
			else
			{
				Logger.LogInformation("Bundle '{BundleIdentifier}' not found, skipping retrieving deployed manifests", BundleIdentifier);
			}
		}
		catch (System.Exception)
		{
			// delete any files that did get copied
			string[] Manifests = Directory.GetFiles(CombinePaths(Params.BaseStageDirectory, PlatformName), "*_Manifest_*.txt");
			foreach (string Manifest in Manifests)
			{
				File.Delete(Manifest);
			}
			Result = false;
		}

		return Result;
	}

	private string GetPathToLibiMobileDeviceTool(string LibimobileExec)
	{
		string ExecWithPath = "";
		if (UnrealBuildTool.BuildHostPlatform.Current.Platform == UnrealTargetPlatform.Win64)
		{
			ExecWithPath = CombinePaths(CmdEnv.LocalRoot, "Engine/Extras/ThirdPartyNotUE/libimobiledevice/x64/" + LibimobileExec + ".exe");
		}
		else if (UnrealBuildTool.BuildHostPlatform.Current.Platform == UnrealTargetPlatform.Mac)
		{
			ExecWithPath = CombinePaths(CmdEnv.LocalRoot, "Engine/Extras/ThirdPartyNotUE/libimobiledevice/Mac/" + LibimobileExec);
		}
		if (!File.Exists(ExecWithPath) || ExecWithPath == "")
		{
			throw new AutomationException("Failed to locate LibiMobileDevice executable.");

		}
		return ExecWithPath;
	}

	private string GetLibimobileDeviceNetworkedArgument(string EntryArguments, string UDID)
	{
		DeviceInfo[] CachedDevices = GetDevices();

		if (CachedDevices.Where(CachedDevice => CachedDevice.Id == UDID && CachedDevice.PlatformValues["Connection"] == "Network").Count() > 0)
		{
			return EntryArguments + " -n";
		}
		return EntryArguments;
	}

	private void DeployManifestContent(string BaseFolder, DeploymentContext SC, ProjectParams Params, ref string Files, ref string BundleIdentifier)
	{
		var DeviceFS = GetPathToLibiMobileDeviceTool("idevicefs");

		string[] FileList = Files.Split('\n');
		string AllCommandsToPush = "";
		foreach (string Filename in FileList)
		{
			if (!string.IsNullOrEmpty(Filename) && !string.IsNullOrWhiteSpace(Filename))
			{
				string Trimmed = Filename.Trim();
				string SourceFilename = BaseFolder + "\\" + Trimmed;
				SourceFilename = SourceFilename.Replace('/', '\\');
				string DestFilename = "/Library/Caches/" + Trimmed.Replace("cookeddata/", "");
				DestFilename = DestFilename.Replace('\\', '/');
				DestFilename = "\"" + DestFilename + "\"";
				SourceFilename = SourceFilename.Replace('\\', Path.DirectorySeparatorChar);
				string CommandToPush = "push -p \"" + SourceFilename + "\" " + DestFilename + "\n";
				AllCommandsToPush += CommandToPush;
			}
		}
		System.IO.File.WriteAllText(Directory.GetCurrentDirectory() + "\\CommandsToPush.txt", AllCommandsToPush);
		int ExitCode = 0;

		string IdeviceFSArgs = "-u " + Params.DeviceNames[0] + " -b " + BundleIdentifier + " -x " + "\"" + Directory.GetCurrentDirectory() + "\\CommandsToPush.txt" + "\"";
		IdeviceFSArgs = GetLibimobileDeviceNetworkedArgument(IdeviceFSArgs, Params.DeviceNames[0]);

		using (Process IDeviceFSProcess = new Process())
		{
			DataReceivedEventHandler StdOutHandler = (E, Args) => { if (Args.Data != null) { Logger.LogInformation("{Text}", Args.Data); } };
			DataReceivedEventHandler StdErrHandler = (E, Args) => { if (Args.Data != null) { Logger.LogError("{Text}", Args.Data); } };

			IDeviceFSProcess.StartInfo.FileName = DeviceFS;
			IDeviceFSProcess.StartInfo.Arguments = IdeviceFSArgs;
			IDeviceFSProcess.OutputDataReceived += StdOutHandler;
			IDeviceFSProcess.ErrorDataReceived += StdErrHandler;

			ExitCode = Utils.RunLocalProcess(IDeviceFSProcess);

			if (ExitCode != 0)
			{
				throw new AutomationException("Failed to push content to mobile device.");
			}
		}

		File.Delete(Directory.GetCurrentDirectory() + "\\CommandsToPush.txt");
	}

	private static bool IsDeviceLockedError(string Output)
	{
		if (string.IsNullOrEmpty(Output))
		{
			return false;
		}
		return Output.Contains("for reason: Locked", StringComparison.OrdinalIgnoreCase) ||
		       Output.Contains("could not be, unlocked", StringComparison.OrdinalIgnoreCase) ||
		       Output.Contains("Password protected", StringComparison.OrdinalIgnoreCase);
	}

	private static bool IsGoIOSDeviceLocked(string UDID)
	{
		IProcessResult Result = Run(GoIOSPath, $"lockdown get PasswordProtected --udid={UDID}", null, ERunOptions.SpewIsVerbose);
		return (Result.Output ?? string.Empty).Contains("true");
	}

	private static bool WaitForDeviceUnlockRetry(Stopwatch Timer, int MaxWaitSeconds)
	{
		const int RetryIntervalSeconds = 3;
		const int RetryIntervalMs = RetryIntervalSeconds * 1000;
		
		if (Timer.Elapsed.TotalSeconds >= MaxWaitSeconds)
		{
			Logger.LogError("[iOS] Timed out after {MaxWaitSeconds}s waiting for device unlock.", MaxWaitSeconds);
			return false;
		}

		Logger.LogInformation("[iOS] Device is locked. Please unlock your device to continue | Retrying in {RetryTimeSeconds}s ...", RetryIntervalSeconds);
		
		Thread.Sleep(RetryIntervalMs);

		Logger.LogInformation("[iOS] Retrying...");
		
		return true;
	}

	public override void Deploy(ProjectParams Params, DeploymentContext SC)
	{
		if (Params.Devices.Count != 1)
		{
			throw new AutomationException("Can only deploy to a single specified device, but {0} were specified", Params.Devices.Count);
		}

		if (SC.StageTargetConfigurations.Count != 1)
		{
			throw new AutomationException("iOS is currently only able to package one target configuration at a time, but StageTargetConfigurations contained {0} configurations", SC.StageTargetConfigurations.Count);
		}
		if (Params.Distribution)
		{
			throw new AutomationException("iOS cannot deploy a package made for distribution.");
		}

		// xcode mode simply deploys the staged .app which has been fully completed
		string AppToDeploy;
		if (AppleExports.CreatingAppOnWindows(Params.RawProjectPath))
		{
			AppToDeploy = AppleOnWindowsAppMaker.GetStagedIPAPath(Params, SC, PlatformName);
		}
		else
		{
			// xcode uses a project-name for the content-only .app, but the StageExecutable is UnrealGame-IOS-Shipping or similar, so
			// fix up the Unreal part
			string AppBaseName = SC.StageExecutables[0];
			if (!SC.IsCodeBasedProject)
			{
				TargetReceipt Target = SC.StageTargets[0].Receipt;
				AppBaseName = AppleExports.MakeBinaryFileName(SC.ShortProjectName, "-", Target.Platform, Target.Configuration, Target.Architectures, UnrealTargetConfiguration.Development, null);
			}
			AppToDeploy = FileReference.Combine(SC.StageDirectory, AppBaseName + ".app").FullName;

			// @todo: deal with iterative deploy - we have the uncombined .app in the Staged dir inin Binaries/IOS
			if (Params.IterativeDeploy)
			{
				throw new AutomationException("Iterative deploy is currently not supported with modern, but it shojld be straightforward");
			}
		}

		if (Params.DeviceNames[0].Contains(AppleToolChainSettings.LocalMacUUID) || Params.bMakeMacNative)
		{
			FileReference AppToDeployFileReference = new(AppToDeploy);
			// Deploy on local mac, not on iOS device
			String MakeMacNativeSHPath = FileReference.Combine(Unreal.EngineDirectory, "Build/BatchFiles/Mac/MakeMacNative.sh").FullName;
			String Arg1 = AppToDeployFileReference.Directory.FullName;														// Path to staged ios folder
			String Arg2 = FileReference.Combine(AppToDeployFileReference.Directory.ParentDirectory, "iOSonMac").FullName;	// Path to destination folder
			String Arg3 = AppToDeployFileReference.GetFileName();															// App name
			Logger.LogInformation("Generating {1}/{2} to be run natively on an Apple Silicon Mac", Arg2, Arg3);
			String Result = Utils.RunLocalProcessAndReturnStdOut("/bin/sh", $"\"{MakeMacNativeSHPath}\" \"{Arg1}\" \"{Arg2}\" \"{Arg3}\"");
			if (Result.Length > 0)
			{
				Logger.LogInformation(Result);
			}

			return;
		}

		// if iterative deploy, determine the file delta
		string BundleIdentifier = "";
		bool bNeedsIPA = true;
		if (Params.IterativeDeploy)
		{
			if (File.Exists(Params.BaseStageDirectory + "/" + PlatformName + "/Info.plist"))
			{
				string Contents = File.ReadAllText(SC.StageDirectory + "/Info.plist");
				int Pos = Contents.IndexOf("CFBundleIdentifier");
				Pos = Contents.IndexOf("<string>", Pos) + 8;
				int EndPos = Contents.IndexOf("</string>", Pos);
				BundleIdentifier = Contents.Substring(Pos, EndPos - Pos);
			}

			// check to determine if we need to update the IPA
			String NonUFSManifestPath = SC.GetNonUFSDeploymentDeltaPath(Params.DeviceNames[0]);
			if (File.Exists(NonUFSManifestPath))
			{
				string NonUFSFiles = File.ReadAllText(NonUFSManifestPath);
				string[] Lines = NonUFSFiles.Split('\n');
				bNeedsIPA = Lines.Length > 0 && !string.IsNullOrWhiteSpace(Lines[0]);
			}
		}

		// deploy the .ipa
		var DeviceInstaller = GetPathToLibiMobileDeviceTool("ideviceinstaller");

		string LibimobileDeviceArguments =  "-i " + "\"" +  Path.GetFullPath(AppToDeploy) + "\"";
		if (Params.DeviceNames[0].Length > 1)
		{
			LibimobileDeviceArguments += " -u " + Params.DeviceNames[0];
		}
		LibimobileDeviceArguments = GetLibimobileDeviceNetworkedArgument(LibimobileDeviceArguments, Params.DeviceNames[0]);

		// If we deploying to a Simulator, use "xcrun simctl" instead
		{
			TargetReceipt Targets = SC.StageTargets[0].Receipt;
			if (Targets.Architectures.SingleArchitecture == UnrealArch.IOSSimulator)
			{
				DeviceInstaller = "xcrun";
				LibimobileDeviceArguments = "simctl install " + Params.DeviceNames[0] + " \"" + Path.GetFullPath(AppToDeploy) + "\"";
			}
		}

		// check for it in the stage directory
		string CurrentDir = Directory.GetCurrentDirectory();
		Directory.SetCurrentDirectory(CombinePaths(CmdEnv.LocalRoot, "Engine/Binaries/DotNET/IOS/"));
		if (!Params.IterativeDeploy || bCreatedIPA || bNeedsIPA)
		{
			if (Params.DeviceLockedTimeout > 0)
			{
				Stopwatch Timer = Stopwatch.StartNew();
				int ExitCode;
				string Output;
				do
				{
					Output = RunAndLog(CmdEnv, DeviceInstaller, LibimobileDeviceArguments, out ExitCode);
					if (ExitCode == 0)
					{
						break;
					}
					string OutputText = Output ?? string.Empty;
					if (!IsDeviceLockedError(OutputText))
					{
						break;
					}
				}
				while (WaitForDeviceUnlockRetry(Timer, Params.DeviceLockedTimeout));

				if (ExitCode != 0)
				{
					throw new AutomationException("Failed to deploy to iOS device: {0}", Output);
				}
			}
			else
			{
				RunAndLog(CmdEnv, DeviceInstaller, LibimobileDeviceArguments);
			}
		}

		// deploy the assets
		if (Params.IterativeDeploy)
		{
			string BaseFolder = Path.GetDirectoryName(SC.GetUFSDeploymentDeltaPath(Params.DeviceNames[0]));
			string FilesString = File.ReadAllText(SC.GetUFSDeploymentDeltaPath(Params.DeviceNames[0]));
			DeployManifestContent(BaseFolder, SC, Params, ref FilesString, ref BundleIdentifier);
			if (bNeedsIPA)
			{
				BaseFolder = Path.GetDirectoryName(SC.GetNonUFSDeploymentDeltaPath(Params.DeviceNames[0]));
				FilesString = File.ReadAllText(SC.GetNonUFSDeploymentDeltaPath(Params.DeviceNames[0]));
				DeployManifestContent(BaseFolder, SC, Params, ref FilesString, ref BundleIdentifier);
			}
			
			Directory.SetCurrentDirectory(CurrentDir);
			PrintRunTime();
		}
	}

	public override string GetCookPlatform(bool bDedicatedServer, bool bIsClientOnly)
	{
		return bIsClientOnly ? "IOSClient" : "IOS";
	}

	public override bool DeployLowerCaseFilenames(StagedFileType FileType)
	{
		// we shouldn't modify the case on files like Info.plist or the icons
		return true;
	}

	public override string LocalPathToTargetPath(string LocalPath, string LocalRoot)
	{
		return LocalPath.Replace("\\", "/").Replace(LocalRoot, "../../..");
	}

	public override bool IsSupported { get { return true; } }

	public override bool LaunchViaUFE { get { return UnrealBuildTool.BuildHostPlatform.Current.Platform != UnrealTargetPlatform.Mac; } }

	public override bool UseAbsLog
	{
		get
		{
			return !LaunchViaUFE;
		}
	}

	public override bool RemapFileType(StagedFileType FileType)
	{
		return (
			FileType == StagedFileType.UFS || 
			FileType == StagedFileType.NonUFS ||
			FileType == StagedFileType.DebugNonUFS);
	}

	public override StagedFileReference Remap(StagedFileReference Dest)
	{
		return new StagedFileReference("cookeddata/" + Dest.Name);
	}
	public override List<string> GetDebugFileExtensions()
	{
		return new List<string> { ".dsym", ".udebugsymbols" };
	}
	public override IProcessResult RunClient(ERunOptions ClientRunFlags, string ClientApp, string ClientCmdLine, ProjectParams Params)
	{
		if (UnrealBuildTool.BuildHostPlatform.Current.Platform == UnrealTargetPlatform.Mac || UnrealBuildTool.BuildHostPlatform.Current.Platform == UnrealTargetPlatform.Win64)
		{
			if (Params.Devices.Count != 1)
			{
				throw new AutomationException("Can only run on a single specified device, but {0} were specified", Params.Devices.Count);
			}

			string PlistFile;
			string BundleIdentifier = "";

			// Params.StageDirectory will be something like /.../Saved/StagedBuilds
			// ClientApp will be something like /.../Saved/StagedBuilds/IOSClient/QAGame/Binaries/IOS/QAGameClient
			// so we want the first directory in the CLientApp after Params.StageDirectory	

			// Currently (2023/09), Remote Mac builds from a Windows System are forced to Legacy mode and uses a different ClientApp value
			if (OperatingSystem.IsMacOS())
			{
				string TargetStageDir = ClientApp.Substring(0, ClientApp.IndexOf('/', Params.BaseStageDirectory.Length + 1));
				if (AppleExports.CreatingAppOnWindows(Params.RawProjectPath))
				{
					PlistFile = Path.Combine(TargetStageDir, "Info.plist");
				}
				else
				{
					// Xcode mode runs from the staged dir.
					// Also, Blueprint only projects have "ClientApp" set to UnrealGame, so use the Project name instead
					string AppName = Path.GetFileNameWithoutExtension(ClientApp);
					if (!Params.IsCodeBasedProject)
					{
						AppName = Params.ShortProjectName;
						switch (Params.ClientConfigsToBuild[0])
						{
							case UnrealTargetConfiguration.Debug:
							case UnrealTargetConfiguration.DebugGame:
							case UnrealTargetConfiguration.Test:
							case UnrealTargetConfiguration.Shipping:
								// build the correct full appname (Development has none). ie: MyProject-IOS-Debug
								AppName += "-" + Params.ClientTargetPlatforms[0] + "-" + Params.ClientConfigsToBuild[0];
								break;
						}
					}
					PlistFile = Path.Combine(TargetStageDir, AppName + ".app", "Info.plist");
				}
			}
			else
			{
				PlistFile = Path.Combine(Params.BaseStageDirectory, PlatformName);
				PlistFile = Path.Combine(PlistFile, "Info.plist");
			}

			Logger.LogWarning("LOOKING IN PLIST {PListfile}", PlistFile);
			Logger.LogWarning("ClientApp is {CLient}", ClientApp);
			Logger.LogWarning("BaseStageDir is is {stage}", Params.BaseStageDirectory);

			if (File.Exists(PlistFile))
			{
				string Contents = File.ReadAllText(PlistFile);
				int Pos = Contents.IndexOf("CFBundleIdentifier");
				Pos = Contents.IndexOf("<string>", Pos) + 8;
				int EndPos = Contents.IndexOf("</string>", Pos);
				BundleIdentifier = Contents.Substring(Pos, EndPos - Pos);
			}

			string Program;
			string Arguments;
			IOSLaunchTool LaunchTool;
			if (Params.DeviceNames[0].Contains(AppleToolChainSettings.LocalMacUUID))
			{
				// Run iOS app natively on Mac, the binary is in StagedBuilds/iOSonMac
				string IOSonMacDir = Path.Combine(Params.BaseStageDirectory, "iOSonMac");
				string AppName = Path.GetFileNameWithoutExtension(ClientApp);
				LaunchTool = IOSLaunchTool.Open;
				Program = "open";
				Arguments = "\"" + IOSonMacDir + "/" + AppName + ".app\" --args";
			}
			else
			{
				VersionNumber DeviceIOSVersion = GetDeviceIOSVersion(Params.DeviceNames[0]);

				if (DeviceIOSVersion >= new VersionNumber(17))
				{
					// As of iOS17, Apple changed the remote debugger protocol and libimobiledevice has stopped working for launching apps.
					if (OperatingSystem.IsMacOS())
					{
						//As of Xcode14(ish), Apple added a new "xcrun devicectl" cli to do pretty much everything libimobiledevice/ios-deploy did.
						LaunchTool = IOSLaunchTool.XcrunDevicectl;
						Program = "xcrun";
						Arguments = "devicectl device process launch";
						Arguments += " --terminate-existing";	// if it's already running on device, kill it first
						Arguments += " --console";	// attach to the console so we can get log output
						Arguments += " --environment-variables '{\\\"APP_DISTRIBUTOR_ID_OVERRIDE\\\":\\\"com.apple.AppStore\\\",\\\"OS_ACTIVITY_DT_MODE\\\":\\\"enable\\\"}'";	// ask os_log to go into stdout/stderr
						Arguments += " --device " + Params.DeviceNames[0];
						Arguments += " \"" + BundleIdentifier + "\" --";
					}
					else if (OperatingSystem.IsWindows())
					{
						// go-ios replaces idevicedebug on Windows, but requires a tunnel for iOS 17+.
						if (!File.Exists(GoIOSPath))
						{
							throw new AutomationException("[iOS] go-ios binary not found at '{0}'. " + "Cannot launch iOS apps on Windows without it.", GoIOSPath);
						}

						string TunnelArgs = GetGoIOSTunnelArgs(DeviceIOSVersion, Params.DeviceNames[0]);
						StartGoIOSTunnel(Params.DeviceNames[0], TunnelArgs);
						LaunchTool = IOSLaunchTool.GoIOS;
						Program = GoIOSPath;
						Arguments = $"launch {BundleIdentifier} --kill-existing --udid={Params.DeviceNames[0]}";
					}
					else
					{
						throw new AutomationException("[iOS] No supported iOS 17+ app launch mechanism for this platform.");
					}
				}
				else
				{
					// iOS < 17: idevicedebug still works on all platforms.
					LaunchTool = IOSLaunchTool.Libimobiledevice;
					Program = GetPathToLibiMobileDeviceTool("idevicedebug");
					Arguments = " -u '" + Params.DeviceNames[0] + "'";
					Arguments += " --detach";
					Arguments = GetLibimobileDeviceNetworkedArgument(Arguments, Params.DeviceNames[0]);
					Arguments += " run '" + BundleIdentifier + "'";
				}
			}

			if (OperatingSystem.IsMacOS())
			{
				// ClientCmdLine is only relevant when running on a Mac
				Arguments += " " + ClientCmdLine;
			}
			
			IProcessResult ClientProcess = null;
			if (Params.DeviceLockedTimeout > 0 && !Params.DeviceNames[0].Contains(AppleToolChainSettings.LocalMacUUID) && !Params.bMakeMacNative)
			{
				Stopwatch Timer = Stopwatch.StartNew();
				do
				{
					// For go-ios, check device lock state before launching to avoid the 5s DTX timeout.
					if (LaunchTool == IOSLaunchTool.GoIOS && IsGoIOSDeviceLocked(Params.DeviceNames[0]))
					{
						if (!WaitForDeviceUnlockRetry(Timer, Params.DeviceLockedTimeout))
						{
							break;
						}
						continue;
					}

					ClientProcess = Run(Program, Arguments, null, ClientRunFlags);
					ClientProcess.WaitForExit();

					if (ClientProcess.ExitCode == 0)
					{
						break;
					}

					string Output = ClientProcess.Output ?? string.Empty;
					if (!IsDeviceLockedError(Output))
					{
						break;
					}
				}
				while (WaitForDeviceUnlockRetry(Timer, Params.DeviceLockedTimeout));

				if (ClientProcess == null || ClientProcess.ExitCode != 0)
				{
					string FinalOutput = ClientProcess?.Output ?? string.Empty;
					throw new AutomationException("Failed to launch app on device: {0}", FinalOutput);
				}
			}
			else
			{
				ClientProcess = Run(Program, Arguments, null, ClientRunFlags);
			}

			if (ClientProcess.ExitCode != 0)
			{
				return HandleLaunchFailure(LaunchTool, ClientProcess, BundleIdentifier, Params);
			}
			return ClientProcess;

		}
		else
		{
			IProcessResult Result = new ProcessResult("DummyApp", null, false);
			Result.ExitCode = 0;
			return Result;
		}
	}

	private IProcessResult HandleLaunchFailure(IOSLaunchTool LaunchTool, IProcessResult ClientProcess, string BundleIdentifier, ProjectParams Params)
	{
		switch (LaunchTool)
		{
			case IOSLaunchTool.Libimobiledevice:
				if (OperatingSystem.IsWindows() && ClientProcess.ExitCode == -1)
				{
					Logger.LogWarning("[iOS] libimobiledevice could not launch {BundleIdentifier} on {DeviceId}. " + "The app is installed but the device may lack developer tools. " +
									"Launch the app manually, or connect the device to a Mac with Xcode to install developer tools.", BundleIdentifier, Params.DeviceNames[0]);
					IProcessResult Result = new ProcessResult("DummyApp", null, false);
					Result.ExitCode = 0;
					return Result;
				}
				break;

			case IOSLaunchTool.GoIOS:
				// go-ios has a hardcoded 5s DTX timeout that fires even when the app launches successfully.
				// Check if the app is actually running before declaring failure.
				IProcessResult ProcessResult = Run(GoIOSPath, $"ps --apps --udid={Params.DeviceNames[0]}", null, ERunOptions.SpewIsVerbose);
				if ((ProcessResult.Output ?? string.Empty).Contains(BundleIdentifier))
				{
					Logger.LogWarning("[iOS] go-ios launch reported failure (exit code {ExitCode}) but {BundleIdentifier} is running on device. Treating as success.", ClientProcess.ExitCode, BundleIdentifier);
					IProcessResult Result = new ProcessResult("DummyApp", null, false);
					Result.ExitCode = 0;
					return Result;
				}
				break;

			case IOSLaunchTool.XcrunDevicectl:
			case IOSLaunchTool.Open:
				break;
		}

		return ClientProcess;
	}

	private static void StartGoIOSTunnel(string UDID, string TunnelArgs)
	{
		Logger.LogInformation("[iOS] Starting go-ios tunnel for device {UDID}...", UDID);

		IProcessResult TunnelResult = Run(GoIOSPath, $"tunnel start {TunnelArgs} --udid={UDID}", null, ERunOptions.NoWaitForExit | ERunOptions.SpewIsVerbose);

		// Tie tunnel lifetime to UAT via Windows Job Object — auto-killed on exit/crash.
		s_tunnelProcessGroup ??= new ManagedProcessGroup();
		s_tunnelProcessGroup.AddProcess(TunnelResult.ProcessObject);

		// Poll "tunnel ls" until the device appears — this is how go-ios exposes readiness.
		Stopwatch Timer = Stopwatch.StartNew();
		int MaxWaitMs = 15000;
		while (Timer.ElapsedMilliseconds < MaxWaitMs)
		{
			Thread.Sleep(1000);

			if (TunnelResult.HasExited)
			{
				throw new AutomationException("[iOS] go-ios tunnel exited unexpectedly (exit code {0}): {1}", TunnelResult.ExitCode, TunnelResult.Output);
			}

			IProcessResult ListResult = Run(GoIOSPath, "tunnel ls", null, ERunOptions.SpewIsVerbose);
			if ((ListResult.Output ?? string.Empty).Contains(UDID))
			{
				Logger.LogInformation("[iOS] Tunnel established for device {UDID} ({ElapsedMs}ms).", UDID, Timer.ElapsedMilliseconds);
				return;
			}
		}

		throw new AutomationException("[iOS] go-ios tunnel did not become ready within {0}ms for device {1}.", MaxWaitMs, UDID);
	}

	private static string GetGoIOSTunnelArgs(VersionNumber IOSVersion, string UDID)
	{
		// Tunnel mode depends on device iOS version:
		//   17.0 - 17.3: Requires kernel tunnel (--enabletun) + wintun.dll + admin privileges.
		//   17.4+:       Userspace tunnel (--userspace), no wintun, no admin.
		if (IOSVersion >= new VersionNumber(17, 4))
		{
			return "--userspace";
		}

		if (IOSVersion >= new VersionNumber(17))
		{
			// iOS 17.0-17.3.x requires kernel tunnel with wintun.dll.
			string GoIOSDir = Path.GetDirectoryName(GoIOSPath);
			bool bWintunFound = File.Exists(Path.Combine(GoIOSDir, "wintun.dll")) || File.Exists(@"C:\Windows\System32\wintun.dll");

			if (!bWintunFound)
			{
				throw new AutomationException(
					"[iOS] Device {0} is running iOS {1} which requires wintun.dll for tunnel support. " +
					"Download it from https://www.wintun.net/ and place in C:\\Windows\\System32\\ " +
					"or alongside ios.exe at {2}. " +
					"Alternatively, update the device to iOS 17.4+ which does not require wintun.",
					UDID, IOSVersion, GoIOSDir);
			}

			Logger.LogWarning("[iOS] Device {UDID} is running iOS {Version} (17.0-17.3.x). " + "Using kernel tunnel mode — this requires administrator privileges.", UDID, IOSVersion);
			return "--enabletun";
		}
		
		Logger.LogWarning("[iOS] Device {UDID} is running iOS {Version}. Should not be using tunnel mode", UDID, IOSVersion);
		return "--userspace";
	}

	private VersionNumber GetDeviceIOSVersion(string UDID)
	{
		DeviceInfo[] Devices = GetDevices();
		DeviceInfo Device = Devices.FirstOrDefault(d => d.Id == UDID);
		if (Device == null || string.IsNullOrEmpty(Device.SoftwareVersion))
		{
			throw new AutomationException("[iOS] Could not determine iOS version for device {0}. Ensure the device is connected and unlocked.", UDID);
		}
		if (!VersionNumber.TryParse(Device.SoftwareVersion, out VersionNumber IOSVersion))
		{
			throw new AutomationException("[iOS] Could not parse iOS version '{0}' for device {1}.", Device.SoftwareVersion, UDID);
		}
		return IOSVersion;
	}

	private static int GetChunkCount(ProjectParams Params, DeploymentContext SC)
	{
		var ChunkListFilename = GetChunkPakManifestListFilename(Params, SC);
		var ChunkArray = ReadAllLines(ChunkListFilename);
		return ChunkArray.Length;
	}

	private static string GetChunkPakManifestListFilename(ProjectParams Params, DeploymentContext SC)
	{
		return CombinePaths(GetChunkManifestPath(Params, SC), "pakchunklist.txt");
	}

	private static string GetChunkManifestPath(ProjectParams Params, DeploymentContext SC)
	{
		return CombinePaths(SC.MetadataDir.FullName, "ChunkManifest");
	}

	private static StringBuilder AppendKeyValue(StringBuilder Text, string Key, object Value, int Level)
	{
		// create indent level
		string Indent = "";
		for (int i = 0; i < Level; ++i)
		{
			Indent += "\t";
		}

		// output key if we have one
		if (Key != null)
		{
			Text.AppendLine(Indent + "<key>" + Key + "</key>");
		}

		// output value
		if (Value is Array)
		{
			Text.AppendLine(Indent + "<array>");
			Array ValArray = Value as Array;
			foreach (var Item in ValArray)
			{
				AppendKeyValue(Text, null, Item, Level + 1);
			}
			Text.AppendLine(Indent + "</array>");
		}
		else if (Value is Dictionary<string, object>)
		{
			Text.AppendLine(Indent + "<dict>");
			Dictionary<string, object> ValDict = Value as Dictionary<string, object>;
			foreach (var Item in ValDict)
			{
				AppendKeyValue(Text, Item.Key, Item.Value, Level + 1);
			}
			Text.AppendLine(Indent + "</dict>");
		}
		else if (Value is string)
		{
			Text.AppendLine(Indent + "<string>" + Value + "</string>");
		}
		else if (Value is bool)
		{
			if ((bool)Value == true)
			{
				Text.AppendLine(Indent + "<true/>");
			}
			else
			{
				Text.AppendLine(Indent + "<false/>");
			}
		}
		else
		{
			Console.WriteLine("PLIST: Unknown array item type");
		}
		return Text;
	}

	private static void GeneratePlist(Dictionary<string, object> KeyValues, string PlistFile)
	{
		// generate the plist file
		StringBuilder Text = new StringBuilder();

		// boiler plate top
		Text.AppendLine("<?xml version=\"1.0\" encoding=\"UTF-8\"?>");
		Text.AppendLine("<!DOCTYPE plist PUBLIC \"-//Apple//DTD PLIST 1.0//EN\" \"http://www.apple.com/DTDs/PropertyList-1.0.dtd\">");
		Text.AppendLine("<plist version=\"1.0\">");
		Text.AppendLine("<dict>");

		foreach (var KeyValue in KeyValues)
		{
			AppendKeyValue(Text, KeyValue.Key, KeyValue.Value, 1);
		}
		Text.AppendLine("</dict>");
		Text.AppendLine("</plist>");

		// write the file out
		if (!Directory.Exists(Path.GetDirectoryName(PlistFile)))
		{
			Directory.CreateDirectory(Path.GetDirectoryName(PlistFile));
		}
		File.WriteAllText(PlistFile, Text.ToString());
	}

	private static void GenerateAssetPlist(string BundleIdentifier, string[] Tags, string AssetDir)
	{
		Dictionary<string, object> KeyValues = new Dictionary<string, object>();
		KeyValues.Add("CFBundleIdentifier", BundleIdentifier);
		KeyValues.Add("Tags", Tags);
		GeneratePlist(KeyValues, CombinePaths(AssetDir, "Info.plist"));
	}

	private static void GenerateAssetPackManifestPlist(KeyValuePair<string, string>[] ChunkData, string AssetDir)
	{
		Dictionary<string, object>[] Resources = new Dictionary<string, object>[ChunkData.Length];
		for (int i = 0; i < ChunkData.Length; ++i)
		{
			Dictionary<string, object> Data = new Dictionary<string, object>();
			Data.Add("URL", CombinePaths("OnDemandResources", ChunkData[i].Value));
			Data.Add("bundleKey", ChunkData[i].Key);
			Data.Add("isStreamable", false);
			Resources[i] = Data;
		}

		Dictionary<string, object> KeyValues = new Dictionary<string, object>();
		KeyValues.Add("resources", Resources);
		GeneratePlist(KeyValues, CombinePaths(AssetDir, "AssetPackManifest.plist"));
	}

	private static void GenerateOnDemandResourcesPlist(KeyValuePair<string, string>[] ChunkData, string AssetDir)
	{
		Dictionary<string, object> RequestTags = new Dictionary<string, object>();
		Dictionary<string, object> AssetPacks = new Dictionary<string, object>();
		Dictionary<string, object> Requests = new Dictionary<string, object>();
		for (int i = 0; i < ChunkData.Length; ++i)
		{
			string ChunkName = "Chunk" + (i + 1).ToString();
			RequestTags.Add(ChunkName, new string[] { ChunkData[i].Key });
			AssetPacks.Add(ChunkData[i].Key, new string[] { ("pak" + ChunkName + "-ios.pak").ToLowerInvariant() });
			Dictionary<string, object> Packs = new Dictionary<string, object>();
			Packs.Add("NSAssetPacks", new string[] { ChunkData[i].Key });
			Requests.Add(ChunkName, Packs);
		}

		Dictionary<string, object> KeyValues = new Dictionary<string, object>();
		KeyValues.Add("NSBundleRequestTags", RequestTags);
		KeyValues.Add("NSBundleResourceRequestAssetPacks", AssetPacks);
		KeyValues.Add("NSBundleResourceRequestTags", Requests);
		GeneratePlist(KeyValues, CombinePaths(AssetDir, "OnDemandResources.plist"));
	}

	public override void PostStagingFileCopy(ProjectParams Params, DeploymentContext SC)
	{
		/*		if (Params.CreateChunkInstall)
				{
					// get the bundle identifier
					string BundleIdentifier = "";
					if (File.Exists(Params.BaseStageDirectory + "/" + PlatformName + "/Info.plist"))
					{
						string Contents = File.ReadAllText(SC.StageDirectory + "/Info.plist");
						int Pos = Contents.IndexOf("CFBundleIdentifier");
						Pos = Contents.IndexOf("<string>", Pos) + 8;
						int EndPos = Contents.IndexOf("</string>", Pos);
						BundleIdentifier = Contents.Substring(Pos, EndPos - Pos);
					}

					// generate the ODR resources
					// create the ODR directory
					string DestSubdir = SC.StageDirectory + "/OnDemandResources";
					if (!Directory.Exists(DestSubdir))
					{
						Directory.CreateDirectory(DestSubdir);
					}

					// read the chunk list and generate the data
					var ChunkCount = GetChunkCount(Params, SC);
					var ChunkData = new KeyValuePair<string, string>[ChunkCount - 1];
					for (int i = 1; i < ChunkCount; ++i)
					{
						// chunk name
						string ChunkName = "Chunk" + i.ToString ();

						// asset name
						string AssetPack = BundleIdentifier + ".Chunk" + i.ToString () + ".assetpack";

						// bundle key
						byte[] bytes = new byte[ChunkName.Length * sizeof(char)];
						System.Buffer.BlockCopy(ChunkName.ToCharArray(), 0, bytes, 0, bytes.Length);
						string BundleKey = BundleIdentifier + ".asset-pack-" + BitConverter.ToString(System.Security.Cryptography.MD5.Create().ComputeHash(bytes)).Replace("-", string.Empty);

						// add to chunk data
						ChunkData[i-1] = new KeyValuePair<string, string>(BundleKey, AssetPack);

						// create the sub directory
						string AssetDir = CombinePaths (DestSubdir, AssetPack);
						if (!Directory.Exists(AssetDir))
						{
							Directory.CreateDirectory(AssetDir);
						}

						// generate the Info.plist for each ODR bundle (each chunk for install past 0)
						GenerateAssetPlist (BundleKey, new string[] { ChunkName }, AssetDir);

						// copy the files to the OnDemandResources directory
						string PakName = "pakchunk" + i.ToString ();
						string FileName =  PakName + "-" + PlatformName.ToLower() + ".pak";
						string P4Change = "UnknownCL";
						string P4Branch = "UnknownBranch";
						if (CommandUtils.P4Enabled)
						{
							P4Change = CommandUtils.P4Env.ChangelistString;
							P4Branch = CommandUtils.P4Env.BuildRootEscaped;
						}
						string ChunkInstallBasePath = CombinePaths(SC.ProjectRoot.FullName, "ChunkInstall", SC.FinalCookPlatform);
						string RawDataPath = CombinePaths(ChunkInstallBasePath, P4Branch + "-CL-" + P4Change, PakName);
						string RawDataPakPath = CombinePaths(RawDataPath, PakName + "-" + SC.FinalCookPlatform + ".pak");
						string DestFile = CombinePaths (AssetDir, FileName);
						CopyFile (RawDataPakPath, DestFile);
					}

					// generate the AssetPackManifest.plist
					GenerateAssetPackManifestPlist (ChunkData, SC.StageDirectory.FullName);

					// generate the OnDemandResources.plist
					GenerateOnDemandResourcesPlist (ChunkData, SC.StageDirectory.FullName);
				}*/

		base.PostStagingFileCopy(Params, SC);
	}

	public override bool RequiresPackageToDeploy(ProjectParams Params)
	{
		return AppleExports.CreatingAppOnWindows(Params.RawProjectPath);
	}

	public override HashSet<StagedFileReference> GetFilesForCRCCheck()
	{
		HashSet<StagedFileReference> FileList = base.GetFilesForCRCCheck();
		FileList.Add(new StagedFileReference("Info.plist"));
		return FileList;
	}
	public override bool SupportsMultiDeviceDeploy
	{
		get
		{
			return true;
		}
	}

	public override void StripSymbols(FileReference SourceFile, FileReference TargetFile)
	{
		IOSExports.StripSymbols(PlatformType, SourceFile, TargetFile, Log.Logger);
	}

	public override DirectoryReference GetProjectRootForStage(DirectoryReference RuntimeRoot, StagedDirectoryReference RelativeProjectRootForStage)
	{
		return DirectoryReference.Combine(RuntimeRoot, "cookeddata/" + RelativeProjectRootForStage.Name);
	}

	public override void SetSecondaryRemoteMac(string ProjectFilePath, string ClientPlatform)
	{
		PrepareForDebugging("", ProjectFilePath, ClientPlatform);
		IOSExports.SetSecondaryRemoteMac(ClientPlatform, new FileReference(ProjectFilePath), Log.Logger);
	}

	public override void PrepareForDebugging(string SourcePackage, string ProjectFilePath, string ClientPlatform)
	{
		Logger.LogInformation("Preparing for Debug ...");
		Logger.LogInformation("SourcePackage : {SourcePackage}", SourcePackage);
		Logger.LogInformation("ProjectFilePath : {SourcePackage}", ProjectFilePath);
		Logger.LogInformation("ClientPlatform : {ClientPlatform}", ClientPlatform);

		if (HostPlatform.Current.HostEditorPlatform == UnrealTargetPlatform.Win64 || HostPlatform.Current.HostEditorPlatform == UnrealTargetPlatform.Mac)
		{
			int StartPos = 0;
			if (HostPlatform.Current.HostEditorPlatform == UnrealTargetPlatform.Win64)
			{
				StartPos = ProjectFilePath.LastIndexOf('/');
			}
			else
			{
				StartPos = ProjectFilePath.LastIndexOf(Path.DirectorySeparatorChar);
			}
			int StringLength = ProjectFilePath.Length - 10; // 9 for .uproject, 1 for the /
    		string PackageName = ProjectFilePath.Substring(StartPos + 1, StringLength - StartPos);
			string PackagePath = Path.Combine(Path.GetDirectoryName(ProjectFilePath), "Binaries", ClientPlatform);
			Logger.LogInformation("PackagePath : {PackagePath}", PackagePath);
			if (string.IsNullOrEmpty(SourcePackage))
    		{
				SourcePackage = Path.Combine(PackagePath, PackageName + ".ipa");
			}

			string[] IPAFiles;
			if (!File.Exists(SourcePackage))
            {
				IPAFiles = Directory.GetFiles(PackagePath, "*.ipa");
				Logger.LogWarning("Source package not found : {SourcePackage}, trying to find another IPA file in the same folder.", SourcePackage);
				if (IPAFiles.Length == 0)
                {
					Logger.LogError("No IPA file found in : {PackagePath}. Aborting.", PackagePath);
					throw new AutomationException(ExitCode.Error_MissingExecutable, "No IPA file found in {0}.", PackagePath);
				}
				else
                {
					if (IPAFiles.Length > 1)
                    {
						Logger.LogWarning("More than one IPA file found. Taking the first one found, {Arg0}", IPAFiles[0]);
					}
					SourcePackage = IPAFiles[0];
				}					

			}
			string ZipFile = Path.ChangeExtension(SourcePackage, "zip");

			string PayloadPath = SourcePackage;
			Logger.LogInformation("PayloadPath : {PayloadPath}", PayloadPath);
			if (HostPlatform.Current.HostEditorPlatform == UnrealTargetPlatform.Win64)
			{
				PayloadPath = PayloadPath.Substring(0, PayloadPath.LastIndexOf('\\'));
			}
			else
			{
				PayloadPath = PayloadPath.Substring(0, PayloadPath.LastIndexOf('/'));
			}
			string CookedDataDirectory = Path.Combine(Path.GetDirectoryName(PayloadPath), ClientPlatform, "Payload", PackageName + ".app", "cookeddata");

			Logger.LogInformation("ClientPlatform : {ClientPlatform}", ClientPlatform);
			Logger.LogInformation("ProjectFilePath : {ProjectFilePath}", ProjectFilePath);
			Logger.LogInformation("Source : {SourcePackage}", SourcePackage);
			Logger.LogInformation("ZipFile {ZipFile}", ZipFile);
			Logger.LogInformation("PackageName {PackageName}", PackageName);
			Logger.LogInformation("PayloadPath {PayloadPath}", PayloadPath);

    		if (File.Exists(ZipFile))
    		{
    			Logger.LogInformation("Deleting previously present ZIP file created from IPA");
    			File.Delete(ZipFile);
    		}

    		File.Copy(SourcePackage, ZipFile);
    		UnzipPackage(ZipFile);

    		if (HostPlatform.Current.HostEditorPlatform == UnrealTargetPlatform.Win64)
    		{
    			IOSExports.PrepareRemoteMacForDebugging(CookedDataDirectory, new FileReference(ProjectFilePath), Log.Logger);
    		}
    		else
        	{
        		string ProjectPath = ProjectFilePath;
        		if (Directory.Exists(Path.Combine(ProjectPath, "Binaries", ClientPlatform, "Payload", PackageName, ".app")))
        		{
        			CopyDirectory_NoExceptions(CookedDataDirectory, Path.Combine(ProjectPath, "Binaries", ClientPlatform, "Payload", PackageName, ".app/cookeddata/"), true);
        		}
        		else
        		{
        			string ProjectRoot = SourcePackage;
        			ProjectRoot = ProjectRoot.Substring(0, ProjectRoot.LastIndexOf('/'));
        			CopyFile(SourcePackage, ProjectRoot + "/Binaries/" + ClientPlatform + "/Payload/" + PackageName + ".ipa", true);
        		}
			}
			//cleanup
			Logger.LogInformation("Deleting temp files ...");
			File.Delete(ZipFile);
			Logger.LogInformation("{ZipFile} deleted", ZipFile);
		}
		else
		{
			Logger.LogInformation("Wrangling data for debug for an iOS/tvOS app for XCode is a Mac and Windows (Remote) only feature. Aborting command.");
			return;
		}
	}

	public void UnzipPackage(string PackageToUnzip)
	{
		Logger.LogInformation("PackageToUnzip : {PackageToUnzip}", PackageToUnzip);
		string UnzipPath = PackageToUnzip;
		if (HostPlatform.Current.HostEditorPlatform == UnrealTargetPlatform.Win64)
		{
			UnzipPath = UnzipPath.Substring(0, UnzipPath.LastIndexOf('\\'));
		}
		else
		{
			UnzipPath = UnzipPath.Substring(0, UnzipPath.LastIndexOf(Path.DirectorySeparatorChar));
		}
		Logger.LogInformation("Unzipping to {UnzipPath}", UnzipPath);

		using (Ionic.Zip.ZipFile Zip = new Ionic.Zip.ZipFile(PackageToUnzip))
		{
			foreach (Ionic.Zip.ZipEntry Entry in Zip.Entries.Where(x => !x.IsDirectory))
			{
				string OutputFileName = Path.Combine(UnzipPath, Entry.FileName);
				Directory.CreateDirectory(Path.GetDirectoryName(OutputFileName));
				using (FileStream OutputStream = new FileStream(OutputFileName, FileMode.Create, FileAccess.Write))
				{
					Entry.Extract(OutputStream);
				}
				Logger.LogInformation("Extracted {OutputFileName}", OutputFileName);
			}
		}
	}
}
