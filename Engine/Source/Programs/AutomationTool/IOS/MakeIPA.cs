// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.IO;
using System.Linq;
using System.Text;
using System.Threading.Tasks;
using System.Xml;
using AutomationTool;
using UnrealBuildBase;
using Microsoft.Extensions.Logging;

[Help(@"Creates an IPA from an xarchive file")]
[Help("method=<method>", @"Purpose of the IPA. (Development, Adhoc, Store)")]
[Help("TemplateFile=<file>", @"Path to plist template that will be filled in based on other arguments. See ExportOptions.plist.template for an example")]
[Help("OptionsFile", @"Path to an XML file of options that we'll select from based on method. See ExportOptions.Values.xml for an example")]
[Help("Project", @"Name of this project (e.g ShooterGame, EngineTest)")]
public class MakeIPA : BuildCommand
{ 
	protected string TemplateFile { get; set; }
	protected string OptionsFile { get; set; }
	protected string Project { get; set; }
	protected string Method { get; set; }

	public MakeIPA()
	{
		Method = "development";
	}

	string SelectPropertyTextOrDie(XmlDocument Doc, string InProperty, string InName)
	{
		string MethodSelect = string.Format("//{0}/type[@name=\"{1}\"]", InProperty, InName);
		XmlNode MethodNode = Doc.SelectSingleNode(MethodSelect);
		
		if (MethodNode == null)
		{
			throw new AutomationException("Failed to find node for property {0} with name {1}", InProperty, InName);
		}
		
		return MethodNode.InnerXml;
	}

	protected void CreateExportPlist(string OutputPath)
	{
		XmlDocument Doc = new XmlDocument();
		Doc.Load(OptionsFile);

		// select all entries via xpath
		string MethodText = SelectPropertyTextOrDie(Doc, "methods", Method);
		string SigningText = SelectPropertyTextOrDie(Doc, "signing", Method);
		string ProfileText = SelectPropertyTextOrDie(Doc, "profiles", Method);

		// replace them in the template
		string TemplateContents = File.ReadAllText(TemplateFile);

		TemplateContents = TemplateContents.Replace("{{method}}", MethodText);
		TemplateContents = TemplateContents.Replace("{{profiles}}", ProfileText);
		TemplateContents = TemplateContents.Replace("{{signing}}", SigningText);

		// write it out
		Directory.CreateDirectory(Path.GetDirectoryName(OutputPath));
		File.WriteAllText(OutputPath, TemplateContents);
	}

	int XcodeBuild(string Arguments)
	{
		string XcodeBuildPath = "/usr/bin/xcodebuild";
		IProcessResult Result = CommandUtils.Run(XcodeBuildPath, Arguments);

		return Result.ExitCode;
	}

	public override ExitCode Execute()
	{

		Project = ParseParamValue("Project", Project);
		OptionsFile = ParseParamValue("OptionsFile", OptionsFile);
		TemplateFile = ParseParamValue("TemplateFile", TemplateFile);
		Method = ParseParamValue("Method", Method);
		
		string Archive = ParseParamValue("archive", null);
		string Output = ParseParamValue("output", null);

		if (string.IsNullOrEmpty(Output))
		{
			Output = Path.Combine(Project, "Binaries", "IOS");
		}

		// xcode won't export to a file just a path, so if a user wants foo.ipa we'll save to
		// that path then rename it
		string OutputFile = Path.GetFileName(Output);
		
		if (!string.IsNullOrEmpty(OutputFile))
		{
			Output = Path.GetDirectoryName(Output);
		}

		if (string.IsNullOrEmpty(Archive))
		{
			throw new AutomationException("No archive specified. Use -archive=<path>");
		}

		if (string.IsNullOrEmpty(OptionsFile))
		{
			throw new AutomationException("No OptionsFile specified. Use -OptionsFile=<path>");
		}

		if (string.IsNullOrEmpty(TemplateFile))
		{
			throw new AutomationException("No TemplateFile specified. Use -TemplateFile=<path>");
		}

		string PlistPath = string.Format(@"{0}/Intermediate/IOS/ExportIPA.plist", Project);

		string ProvisioningArgs = "";

		CreateExportPlist(PlistPath);

		string ExportArgs = string.Format("-exportArchive -archivePath \"{0}\" -exportOptionsPlist {1} -exportPath \"{2}\" {3}", Archive, PlistPath, Output, ProvisioningArgs);

		Logger.LogInformation("Creating IPA at at {Output} with {Method} profile.", Output, Method);

		int Result = XcodeBuild(ExportArgs);

		if (Result != 0)
		{
			Logger.LogError("XcodeBuild failed to export archive. {PlistPath} might help diagose the issue..", PlistPath);
			return ExitCode.Error_FailedToCreateIPA;
		}
		
		if (!string.IsNullOrEmpty(OutputFile))
		{
			string TmpFile = Path.ChangeExtension(Archive, "ipa");
			string FinalFile = Path.Combine(Path.GetDirectoryName(TmpFile), OutputFile);

			if (File.Exists(FinalFile))
			{
				File.Delete(FinalFile);
			}
			
			if (Directory.Exists(FinalFile))
			{
				throw new AutomationException("Output file {0} is a directory!", FinalFile);
			}
			
			
			File.Move(TmpFile, FinalFile);
		}

		return ExitCode.Success;
	}
}

