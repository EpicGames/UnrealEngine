<Project Sdk="Microsoft.NET.Sdk">
  <Import Project="..\..\Shared\UnrealEngine.csproj.props" />
  
  <PropertyGroup>
    <OutputType>Library</OutputType>
    <RootNamespace>Insights.Automation</RootNamespace>
    <AssemblyName>Insights.Automation</AssemblyName>
    <GenerateAssemblyInfo>false</GenerateAssemblyInfo>
    <GenerateTargetFrameworkAttribute>false</GenerateTargetFrameworkAttribute>
    <OutputPath Condition="'$(ArtifactsPath)' == ''">..\..\..\..\Binaries\DotNET\AutomationTool\AutomationScripts\Insights</OutputPath>
    <TreatWarningsAsErrors>true</TreatWarningsAsErrors>
    <SatelliteResourceLanguages>en</SatelliteResourceLanguages>    <!-- remove non english resource languages -->
  </PropertyGroup>
  <ItemGroup>
    <ProjectReference Include="..\..\Shared\EpicGames.Core\EpicGames.Core.csproj" PrivateAssets="All">
      <Private>false</Private>
    </ProjectReference>
    <ProjectReference Include="..\..\UnrealBuildTool\UnrealBuildTool.csproj" PrivateAssets="All">
      <Private>false</Private>
    </ProjectReference>
    <ProjectReference Include="..\AutomationUtils\AutomationUtils.Automation.csproj" PrivateAssets="All">
      <Private>false</Private>
    </ProjectReference>
    <ProjectReference Include="..\Scripts\AutomationScripts.Automation.csproj" PrivateAssets="All">
      <Private>false</Private>
    </ProjectReference>
    <ProjectReference Include="..\Gauntlet\Gauntlet.Automation.csproj"/>
  </ItemGroup>
</Project>
