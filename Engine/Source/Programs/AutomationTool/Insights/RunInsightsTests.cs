// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Linq;
using System.IO;
using AutomationTool;
using UnrealBuildTool;
using UnrealBuildBase;
using Gauntlet;
using AutomationTool.DeviceReservation;
using Microsoft.Extensions.Logging;
using EpicGames.Core;
using System.Net.Sockets;
using System.Net;

namespace InsightsTests
{
	using Log = Gauntlet.Log;
	using LogLevel = Gauntlet.LogLevel;

	public class RunInsightsTests : BuildCommand
	{
		public override ExitCode Execute()
		{
			Log.Level = LogLevel.Verbose;

			Globals.Params = new Params(Params);

			InsightsTestExecutorOptions ContextOptions = new InsightsTestExecutorOptions();
			AutoParam.ApplyParamsAndDefaults(ContextOptions, Globals.Params.AllArguments);

			if (ContextOptions.Mode == InsightsMode.GenerateTraces && ContextOptions.Clients.Count == 0)
			{
				Log.Error("Error: -clients flag is missing or no client specification was set.");
				return ExitCode.Error_Arguments;
			}

			return RunTests(ContextOptions);
		}

		public ExitCode RunTests(InsightsTestExecutorOptions ContextOptions)
		{
			InsightsTestRoleContext RoleContext = new InsightsTestRoleContext();
			InsightsAppBuildSource BuildSource = new InsightsAppBuildSource(ContextOptions.Configuration, ContextOptions.Clients, ContextOptions.OpenTraceFile, ContextOptions.WriteTraceFileOnly);

			SetupDevices(ContextOptions);

			InsightsTestContext TestContext = new InsightsTestContext(BuildSource, RoleContext, ContextOptions);

			ITestNode NewTest = Gauntlet.Utils.TestConstructor.ConstructTest<ITestNode, InsightsTestContext>("", TestContext, new string[] { "InsightsTests" });

			bool TestPassed = ExecuteTest(ContextOptions, NewTest);

			DevicePool.Instance.Dispose();

			return TestPassed ? ExitCode.Success : ExitCode.Error_TestFailure;
		}

		private bool ExecuteTest(InsightsTestExecutorOptions Options, ITestNode InsightsTestNode)
		{
			var Executor = new TestExecutor(ToString());

			try
			{
				bool Result = Executor.ExecuteTests(Options, new List<ITestNode>() { InsightsTestNode });
				return Result;
			}
			catch (Exception ex)
			{
				Log.Error($"{ex.Message}.{Environment.NewLine}{ex.StackTrace}");

				return false;
			}
			finally
			{
				Executor.Dispose();

				if (Options.Devices.Count > 0)
				{
					InsightsDeviceReservation.GetInstance().ReleaseDevices();
				}

				DevicePool.Instance.Dispose();

				if (ParseParam("clean"))
				{
					Logger.LogInformation("Deleting temp dir {Arg0}", Options.TempDir);
					DirectoryInfo TempDirInfo = new DirectoryInfo(Options.TempDir);
					if (TempDirInfo.Exists)
					{
						TempDirInfo.Delete(true);
					}
				}

				GC.Collect();
			}
		}

		protected void SetupDevices(InsightsTestExecutorOptions Options)
		{
			Reservation.ReservationDetails = Options.JobDetails;

			DevicePool.Instance.SetLocalOptions(Options.TempDir, Options.Parallel > 1, Options.DeviceURL);
			DevicePool.Instance.AddLocalDevices(1);
			DevicePool.Instance.AddVirtualDevices(2);

			int TotalDevicesExpected = 0;
			Options.Clients.ForEach(Client => TotalDevicesExpected += Client.Platforms.Count);

			if (Options.Devices.Count > 0 && Options.Devices.Count != TotalDevicesExpected)
			{
				throw new AutomationException($"-devices should contain one device for each client and their listed platform, expected {TotalDevicesExpected} but got {Options.Devices.Count}");
			}

			if (Options.Devices.Count > 0)
			{
				int CurrentDeviceIdx = 0;
				foreach (var Client in Options.Clients)
				{
					foreach (var Platform in Client.Platforms)
					{
						DevicePool.Instance.AddDevices(Platform, Options.Devices[CurrentDeviceIdx++]);
					}
				}
			}
		}
	}

	public class InsightsClientInfo
	{
		public string TargetName { get; set; }
		public List<UnrealTargetPlatform> Platforms { get; set; }
		public FileReference ProjectDir { get; set; }

		public string ProjectName { get; set; }
		public string Test { get; set; }
        public string Traces { get; set; }
	}

	public enum InsightsMode
	{
		GenerateTraces,
		Hub,
		Viewer
	}

	public class InsightsTestExecutorOptions : TestExecutorOptions, IAutoParamNotifiable
	{
		public Params Params { get; protected set; }

		public string TempDir;

		[AutoParam("")]
		public string DeviceURL;

		[AutoParam("")]
		public string JobDetails;

		[AutoParam(0)]
		public int Sleep;

		[AutoParam("")]
		public string LogDir;

		[AutoParam("")]
		public string HostTests;

		[AutoParam("")]
		public string OpenTraceFile;

        [AutoParam(false)]
        public bool WriteTraceFileOnly;

		public int Timeout;

		public InsightsMode Mode = InsightsMode.GenerateTraces;

		public List<InsightsClientInfo> Clients = new List<InsightsClientInfo>();

		public Type BuildSourceType { get; protected set; }

		[AutoParam(UnrealTargetConfiguration.Development)]
		public UnrealTargetConfiguration Configuration;

		public List<string> Devices;

		public InsightsTestExecutorOptions()
		{
			BuildSourceType = typeof(InsightsAppBuildSource);
		}

		public virtual void ParametersWereApplied(string[] InParams)
		{
			Params = new Params(InParams);
			if (string.IsNullOrEmpty(TempDir))
			{
				TempDir = Globals.TempDir;
			}
			else
			{
				Globals.TempDir = TempDir;
			}

			if (string.IsNullOrEmpty(LogDir))
			{
				LogDir = Globals.LogDir;
			}
			else
			{
				Globals.LogDir = LogDir;
			}

			LogDir = Path.GetFullPath(LogDir);
			TempDir = Path.GetFullPath(TempDir);

			string ModeParam = Params.ParseValue("mode=", string.Empty);
			if (!string.IsNullOrEmpty(ModeParam))
			{
				Mode = (InsightsMode)Enum.Parse(typeof(InsightsMode), ModeParam, true);
			}

			Timeout = Params.ParseValue("timeout=", 0);

			Devices = Params.ParseValue("devices=", string.Empty).Split(';').Where(D => !string.IsNullOrEmpty(D)).ToList() ?? new List<string>();

			string ClientsArg = Params.ParseValue("clients=", string.Empty);

			ClientsArg.Split(',').ToList().ForEach(Spec =>
			{
				if (string.IsNullOrEmpty(Spec))
					return;
				string[] ClientSpecParts = Spec.Split(';');
				if (ClientSpecParts.Length != 6)
				{
					throw new AutomationException($"Invalid client spec \"{Spec}\", should be formatted as <TargetName>;<Platform1+Platform2+...>;<ProjectDir>;<ProjectName>;<TestName>;<Traces>.");
				}
				Clients.Add(new InsightsClientInfo() {
					TargetName = ClientSpecParts[0],
					Platforms = ClientSpecParts[1].Split('+')
									.ToList()
									.Select(Item => UnrealTargetPlatform.Parse(Item))
									.ToList(),
					ProjectDir = new FileReference(ClientSpecParts[2]),
					ProjectName = ClientSpecParts[3],
					Test = ClientSpecParts[4],
                    Traces = ClientSpecParts[5].Replace('+', ',')
				});
			});

			string[] CleanArgs = Params.AllArguments
				.Where(Arg => !Arg.StartsWith("test=", StringComparison.OrdinalIgnoreCase)
					&& !Arg.StartsWith("device=", StringComparison.OrdinalIgnoreCase))
				.ToArray();

			Params = new Params(CleanArgs);
		}
	}

	public class InsightsDeviceReservation
	{
		private static UnrealDeviceReservation UnrealDeviceReservation = new UnrealDeviceReservation();
		private static InsightsDeviceReservation Instance;

		private List<ITargetDevice> Devices;
		private List<ITargetDevice> SelectedDevicesPrivate = new List<ITargetDevice>();

		public List<ITargetDevice> ReservedDevices
		{
			get { return Devices ?? new List<ITargetDevice>(); }
		}

		public List<ITargetDevice> SelectedDevices
		{
			get { return SelectedDevicesPrivate; }
		}

		static InsightsDeviceReservation()
		{
			Instance = new InsightsDeviceReservation();
		}

		public static InsightsDeviceReservation GetInstance()
		{
			return Instance;
		}

		public bool Reserve(List<UnrealTargetPlatform> Platforms)
		{
			Dictionary<UnrealDeviceTargetConstraint, int> RequiredDeviceTypes = new Dictionary<UnrealDeviceTargetConstraint, int>();
			foreach (UnrealTargetPlatform Platform in Platforms.Distinct())
			{
				RequiredDeviceTypes.Add(new UnrealDeviceTargetConstraint(Platform), Platforms.Count(P => P == Platform));
			}

			bool ReservedAll = UnrealDeviceReservation.TryReserveDevices(RequiredDeviceTypes, Platforms.Count);
			Devices = UnrealDeviceReservation.ReservedDevices;
			return ReservedAll; 
		}

		public void SelectDevice(ITargetDevice Device)
		{
			SelectedDevicesPrivate.Add(Device);
		}

		public void MarkProblemDevice(ITargetDevice Device, string Message)
		{
			UnrealDeviceReservation.MarkProblemDevice(Device, Message);
		}

		public void ReleaseDevices()
		{
			UnrealDeviceReservation.ReleaseDevices();
		}
	}

    public class InsightsHostSession : IDisposable
    {
        public InsightsAppBuildSource BuildSource { get; protected set; }
        public int Sleep { get; protected set; }
        public bool WriteTraceFileOnly { get;protected set; }
        public InsightsMode Mode { get; protected set; }
        public string HostTests { get; protected set; }
        public IAppInstance Instance { get; protected set; } 

		public InsightsHostSession(InsightsAppBuildSource InBuildSource, InsightsMode InMode, string InHostTests, int InSleep, bool InWriteTraceFileOnly)
		{
			BuildSource = InBuildSource;
			Sleep = InSleep;
			Mode = InMode;
			HostTests = InHostTests;
            WriteTraceFileOnly = InWriteTraceFileOnly;
		}

		public IAppInstance RunInsightsApp()
		{
			ITargetDevice Device = InsightsDeviceReservation.GetInstance().ReservedDevices.Where(D => D.IsConnected && D.Platform == HostPlatform.Platform).First();
			InsightsDeviceReservation.GetInstance().SelectDevice(Device);
			IAppInstall HostInstall = Device.InstallApplication(BuildSource.GetUnrealAppConfig(UnrealTargetRole.Host, Mode, "UnrealInsights", HostPlatform.Platform, Sleep, null, HostTests, null, null, WriteTraceFileOnly));
			Instance = Device.Run(HostInstall);
            return Instance;
		}

		public void Dispose()
		{
            if (Instance != null)
            {
                Instance.Kill();
                InsightsDeviceReservation.GetInstance().ReleaseDevices();
            }
        }
	}

	public class InsightsClientSession : IDisposable
	{
		private static int QUERY_STATE_INTERVAL_SECONDS = 1;

        Dictionary<UnrealTargetPlatform, IAppInstance> ClientApps = new Dictionary<UnrealTargetPlatform, IAppInstance>();

        public InsightsAppBuildSource BuildSource { get; protected set; }
		public InsightsClientInfo ClientInfo { get; protected set; }
		public int Sleep { get; protected set; }
        public bool WriteTraceFileOnly { get;protected set; }

		public InsightsClientSession(InsightsAppBuildSource InBuildSource, int InSleep, bool InWriteTraceFileOnly, InsightsClientInfo InClientInfo)
		{
			BuildSource = InBuildSource;
			Sleep = InSleep;
			ClientInfo = InClientInfo;
            WriteTraceFileOnly = InWriteTraceFileOnly;
		}

		#region IDisposable Support
		public void Dispose()
		{
            foreach (var KVP in ClientApps)
            {
                KVP.Value.Kill();
            }
            InsightsDeviceReservation.GetInstance().ReleaseDevices();
        }
		#endregion

		/// <summary>
		/// Installs one client for each platform.
		/// </summary>
		public Dictionary<UnrealTargetPlatform, IAppInstance> InstallAndRunClientApps()
		{
			UnrealAppConfig AppConfig;

			List<ITargetDevice> DevicesToInstallOn = InsightsDeviceReservation.GetInstance().ReservedDevices.Where(D => D.IsConnected).ToList();

			foreach (var Platform in ClientInfo.Platforms)
			{
				ITargetDevice Device = DevicesToInstallOn.Where(D => D.Platform == Platform && !InsightsDeviceReservation.GetInstance().SelectedDevices.Contains(D)).FirstOrDefault();
                // retry reservation if needed (e.g. when device connection was lost)
                if (Device == null)
                {
                    InsightsDeviceReservation.GetInstance().Reserve(new List<UnrealTargetPlatform>() { Platform });
                    Device = InsightsDeviceReservation.GetInstance().ReservedDevices.Where(D => D.IsConnected && D.Platform == Platform && !InsightsDeviceReservation.GetInstance().SelectedDevices.Contains(D)).FirstOrDefault();
                    if (Device == null)
                    {
                        throw new AutomationException($"Couldn't fetch a connected device for platform {Platform}");
                    }
                }
				InsightsDeviceReservation.GetInstance().SelectDevice(Device);

                string FileTracePath = WriteTraceFileOnly ? Path.Combine(Unreal.EngineDirectory.FullName, "Programs", "AutomationTool", "Saved", "Logs", $"{ClientInfo.TargetName}_{Platform}.utrace") : null;
                AppConfig = BuildSource.GetUnrealAppConfig(UnrealTargetRole.Client, InsightsMode.GenerateTraces, ClientInfo.TargetName, Platform, Sleep, ClientInfo.ProjectDir, ClientInfo.Test, ClientInfo.Traces, FileTracePath);

				if (AppConfig.FullClean)
				{
					Device.FullClean();
				}

				// Install the build onto the device
				if (AppConfig.SkipInstall)
				{
					Log.Info("Skipping install due to SkipInstall");
				}
				else
				{
					DateTimeStopwatch Stopwatch = DateTimeStopwatch.Start();

					try
					{
						Device.InstallBuild(AppConfig);
						Log.Info("Installation completed in {InstallTime}", GetInstallTime(Stopwatch.ElapsedTime));
					}
					catch (Exception Ex)
					{
						if (Ex is DeviceException)
						{
							InsightsDeviceReservation.GetInstance().MarkProblemDevice(Device, $"Failed to install insights tests app onto device: {Ex}");
						}
						else
						{
							Log.Warning("Failed to install insights tests app onto device {DeviceName}: {Exception}", Device.Name, Ex.ToString());
						}
						InsightsDeviceReservation.GetInstance().ReleaseDevices();
					}
				}

                IAppInstall Install = null;
                IAppInstance Instance = null;

				// Create the IAppInstall instance
				try
				{
					Install = Device.CreateAppInstall(AppConfig);
				}
				catch (Exception Ex)
				{
					if (Ex is DeviceException)
					{
						InsightsDeviceReservation.GetInstance().MarkProblemDevice(Device, $"Failed to create IAppInstall for insights tests on device: {Ex}");
					}
					else
					{
						Log.Error("Failed to create IAppInstall for insights tests on device {DeviceName}: {Exception}", Device.Name, Ex.ToString());
					}

					InsightsDeviceReservation.GetInstance().ReleaseDevices();
				}

				// Clean/Copy files to the device
				try
				{
					Device.CleanArtifacts();
					Device.CopyAdditionalFiles(AppConfig.FilesToCopy);
				}
				catch (Exception Ex)
				{
					if (Ex is DeviceException)
					{
						InsightsDeviceReservation.GetInstance().MarkProblemDevice(Device, $"Failed to CleanArtifacts or CopyAdditionalFiles for insights tests on device: {Ex}");
					}
					else
					{
						Log.Info("Failed to CleanArtifacts or CopyAdditionalFiles for insights tests on device {DeviceName}: {Exception}", Device.Name, Ex.ToString());
					}
					InsightsDeviceReservation.GetInstance().ReleaseDevices();
				}

				// Run the application
				try
				{
					if (Device is IRunningStateOptions DeviceWithStateOptions)
					{
						// Don't wait to detect running state and query for running state every second
						DeviceWithStateOptions.WaitForRunningState = false;
						DeviceWithStateOptions.CachedStateRefresh = QUERY_STATE_INTERVAL_SECONDS;
					}

					Instance = Device.Run(Install);
					ClientApps.Add(Platform, Instance);
				}
				catch (DeviceException DeviceEx)
				{
					Log.Warning("Failed to start insights test on {DeviceName}. Marking as problem device. Will not retry.", Device.Name);

					if (Instance != null)
					{
						Instance.Kill();
					}

					InsightsDeviceReservation.GetInstance().MarkProblemDevice(Device, $"Device threw an exception during launch. \nException={DeviceEx.Message}");
					InsightsDeviceReservation.GetInstance().ReleaseDevices();

					throw new AutomationException($"Unable to start insights client app for {ClientInfo.TargetName} platform {Platform}, see warnings for details.");
				}
			}

			return ClientApps;
		}

		private string GetInstallTime(TimeSpan Time)
		{
			string Hours = Time.Hours > 0 ? string.Format("{0} hrs, ", Time.Hours) : string.Empty;
			string Minutes = Time.Minutes > 0 ? string.Format("{0} mins, ", Time.Minutes) : string.Empty;
			string Seconds = string.Format("{0} secs", Time.Seconds);

			return Hours + Minutes + Seconds;
		}
	}

	public class InsightsTestRoleContext : ICloneable
	{
		public UnrealTargetRole Type { get { return UnrealTargetRole.Client; } }
		public UnrealTargetPlatform Platform;
		public UnrealTargetConfiguration Configuration { get { return UnrealTargetConfiguration.Development; } }

		public object Clone()
		{
			return this.MemberwiseClone();
		}

		public override string ToString()
		{
			string Description = string.Format("{0} {1} {2}", Platform, Configuration, Type);
			return Description;
		}
	};

	public class InsightsTestContext : ITestContext, ICloneable
	{
		public InsightsAppBuildSource BuildInfo { get; private set; }

		public string WorkerJobID;

		public InsightsTestExecutorOptions Options { get; set; }

		public Params TestParams { get; set; }

		public InsightsTestRoleContext RoleContext { get; set; }

		public UnrealDeviceTargetConstraint Constraint;

		public int PerTestTimeout { get; private set; }

		public InsightsTestContext(InsightsAppBuildSource InBuildInfo, InsightsTestRoleContext InRoleContext, InsightsTestExecutorOptions InOptions, int InPerTestTimeout = 0)
		{
			BuildInfo = InBuildInfo;
			Options = InOptions;
			TestParams = new Params(new string[0]);
			RoleContext = InRoleContext;
			PerTestTimeout = InPerTestTimeout;
		}

		public object Clone()
		{
			InsightsTestContext Copy = (InsightsTestContext)MemberwiseClone();
			Copy.RoleContext = (InsightsTestRoleContext)RoleContext.Clone();
			return Copy;
		}

		public override string ToString()
		{
			string Description = string.Format("{0}", RoleContext);
			if (WorkerJobID != null)
			{
				Description += " " + WorkerJobID;
			}
			return Description;
		}
	}

	/// <summary>
	/// Discovers builds for multiple clients and platforms
	/// </summary>
	public class InsightsAppBuildSource : IBuildSource
	{
		public string OpenTraceFile { get; protected set; }
        public bool WriteTraceFileOnly { get; protected set; }

        private IFolderBuildSource InsightsClientsBuildFactory;
        public UnrealTargetConfiguration Configuration { get; protected set; }
		public Dictionary<string, List<IBuild>> DiscoveredBuilds { get; protected set; }

		public InsightsAppBuildSource(UnrealTargetConfiguration InConfiguration, List<InsightsClientInfo> InClients, string InOpenTraceFile, bool InWriteTraceFileOnly)
		{
			Configuration = InConfiguration;
			OpenTraceFile = InOpenTraceFile;
            WriteTraceFileOnly = InWriteTraceFileOnly;
			InitBuildSources(InConfiguration, InClients);
		}

		private string GetViewerModeOpenTraceFile()
		{
			return Path.Combine(Unreal.EngineDirectory.FullName, "Source", "Programs", "AutomationTool", "Insights", "Resources", "ViewerMode.utrace");
		}

		protected void InitBuildSources(UnrealTargetConfiguration InConfiguration, List<InsightsClientInfo> InClients)
		{
			DiscoveredBuilds = new Dictionary<string, List<IBuild>>();

			foreach (var Client in InClients)
			{
				foreach(var Platform in Client.Platforms)
				{
					InsightsClientsBuildFactory = Gauntlet.Utils.InterfaceHelpers.FindImplementations<IFolderBuildSource>(true)
						.Where(B => B.CanSupportPlatform(Platform))
						.First();
				
					if (!DiscoveredBuilds.ContainsKey(Client.TargetName))
					{
						DiscoveredBuilds.Add(Client.TargetName, new List<IBuild>());
					}

                    string PlatformFolder = Platform == UnrealTargetPlatform.Win64 ? "Windows" : Platform.ToString();
                    IBuild DiscoveredBuild = InsightsClientsBuildFactory.GetBuildsAtPath(Client.TargetName, Path.Combine(Client.ProjectDir.FullName, "Saved", "StagedBuilds", PlatformFolder)).FirstOrDefault();
					if (DiscoveredBuild == null)
					{
						throw new AutomationException("No builds were discovered.");
					}
					else
					{
						DiscoveredBuilds[Client.TargetName].Add(DiscoveredBuild);
					}
				}

            }

			UnrealTargetPlatform ThisHostPlaform = HostPlatform.Platform;
			string HostName = "UnrealInsights";
			string Extension;
			string BuildPath = Path.Combine(Unreal.EngineDirectory.FullName, "Binaries", ThisHostPlaform.ToString());
			string BuildExecutable;
			if (ThisHostPlaform != UnrealTargetPlatform.Mac)
			{
				Extension = ThisHostPlaform == UnrealTargetPlatform.Linux ? string.Empty : ".exe";
				BuildExecutable = Path.Combine(BuildPath, InConfiguration == UnrealTargetConfiguration.Development ? $"{HostName}{Extension}" : $"{HostName}-{ThisHostPlaform}-{InConfiguration}{Extension}");
			}
			else
			{
				Extension = ".app";
				BuildPath = Path.Combine(Unreal.EngineDirectory.FullName, "Binaries", ThisHostPlaform.ToString(), InConfiguration == UnrealTargetConfiguration.Development ? $"{HostName}{Extension}" : $"{HostName}-{ThisHostPlaform}-{InConfiguration}{Extension}");
				BuildExecutable = Path.Combine(BuildPath, "Contents", "MacOS", InConfiguration == UnrealTargetConfiguration.Development ? $"{HostName}" : $"{HostName}-{ThisHostPlaform}-{InConfiguration}");
			}
			IBuild UnrealInsightsBuild = new StagedBuild(
				ThisHostPlaform,
				InConfiguration,
				UnrealTargetRole.Host,
				BuildPath,
				BuildExecutable);
			DiscoveredBuilds.Add(HostName, new List<IBuild>());
			DiscoveredBuilds[HostName].Add(UnrealInsightsBuild);
		}


        public UnrealAppConfig GetUnrealAppConfig(UnrealTargetRole InRole, InsightsMode InMode, string InAppName, UnrealTargetPlatform InPlatform, int InSleep, FileReference InProjectDir, string InTestsToRun, string InTraces, string InTraceFile = null, bool InWriteTraceFileOnly = false)
		{
			var Config = new UnrealAppConfig();
			Config.Name = BuildName;
			if (InProjectDir != null)
			{
				Config.ProjectName = InProjectDir.FullName;
			}
			Config.ProcessType = InRole;
			Config.Platform = InPlatform;
			Config.Configuration = UnrealTargetConfiguration.Development;
			Config.Build = DiscoveredBuilds[InAppName].Where(Build => Build.Platform == InPlatform).First();
			Config.Sandbox = $"InsightsTests-{InAppName}";
			Config.FilesToCopy = new List<UnrealFileToCopy>();
			Config.CanAlterCommandArgs = !InWriteTraceFileOnly;

			if (InSleep > 0)
			{
				Config.CommandLineParams.AddRawCommandline(String.Format("--sleep={0}", InSleep));
			}
			if (InRole.IsClient())
			{
				// A client can have one or the other command line parameter
				// If tracefile is used it will produce the tracefile directly without connecting to the server
				// If it uses tracehost it will connect to that server
                if (!string.IsNullOrEmpty(InTraceFile))
                {
                    Config.CommandLineParams.Add("tracefile", InTraceFile);
                }
                else
                {
                    Config.CommandLineParams.Add("tracehost", GetLocalHostIP(InPlatform));
                }
				Config.CommandLineParams.Add("trace", InTraces);

				Config.CommandLineParams.AddRawCommandline("-NoWatchdog -stdout -FORCELOGFLUSH -CrashForUAT -nullrhi -unattended -nosplash -FullStdOutLogOutput");
				if (!string.IsNullOrEmpty(InTestsToRun))
				{
					Config.CommandLineParams.AddRawCommandline($"-ExecCmds=\"Automation RunTests {InTestsToRun};Quit\"");
				}
			}
			else if (InRole.IsHost() && !InWriteTraceFileOnly)
			{
				Config.CommandLineParams.Add("abslog", Path.Combine(Unreal.EngineDirectory.FullName, "Programs", "AutomationTool", "Saved", "Logs", "UnrealInsights.log"));

				if (InPlatform == UnrealTargetPlatform.Linux)
				{
					Config.CommandLineParams.Add("RenderOffScreen");
				}

				if (InMode == InsightsMode.Viewer)
				{
					if (string.IsNullOrEmpty(InTestsToRun))
					{
						throw new AutomationException("Please provide tests for view mode.");
					}

					string TraceFile = string.IsNullOrEmpty(OpenTraceFile) ? GetViewerModeOpenTraceFile() : OpenTraceFile;

					Config.CommandLineParams.Add("InsightsTest");
					Config.CommandLineParams.Add("OpenTraceFile", $"\"{TraceFile}\"");
					Config.CommandLineParams.AddRawCommandline($"-ExecOnAnalysisCompleteCmd=\"Automation RunTests {InTestsToRun};Quit\"");
				}
				else if (InMode == InsightsMode.Hub)
				{
					if (string.IsNullOrEmpty(InTestsToRun))
					{
						throw new AutomationException("Please provide tests for hub mode.");
					}
					Config.CommandLineParams.Add("InsightsTest");
					Config.CommandLineParams.AddRawCommandline($"-RunAutomationTests=\"Automation RunTests {InTestsToRun};Quit\"");
				}
			}
			return Config;
		}

		private string GetLocalHostIP(UnrealTargetPlatform Platform)
		{
			if (Platform.IsInGroup(UnrealPlatformGroup.Desktop))
			{
				return "127.0.0.1";
			}
			else
			{
				var Host = Dns.GetHostEntry(Dns.GetHostName());
				foreach (var IPEntry in Host.AddressList)
				{
					if (IPEntry.AddressFamily == AddressFamily.InterNetwork)
					{
						return IPEntry.ToString();
					}
				}
			}
			throw new AutomationException("Couldn't retrieve local IPv4.");
		}

		public bool CanSupportPlatform(UnrealTargetPlatform Platform)
		{
			return true;
		}

		public string BuildName { get { return "Insights Host/Client"; } }
	}
}
