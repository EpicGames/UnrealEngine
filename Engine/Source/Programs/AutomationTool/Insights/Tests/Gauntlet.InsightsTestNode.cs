// Copyright Epic Games, Inc. All Rights Reserved.

using AutomationTool;
using EpicGames.Core;
using Gauntlet;
using System;
using System.Collections.Generic;
using System.IO;
using System.Linq;
using System.Threading;
using UnrealBuildBase;
using UnrealBuildTool;

using Log = Gauntlet.Log;

namespace InsightsTests
{
	public class InsightsTests : BaseTest
	{
		private static readonly string InitializationLogEntry = "Insights slate application initialized successfully";
		private InsightsTestContext Context;

		private List<IAppInstance> ClientInstances = new List<IAppInstance>();
		private IAppInstance HostInstance = null;

		private DateTime SessionStartTime = DateTime.MinValue;

		private TestResult InsightsTestResult;

		public Dictionary<string, InsightsClientSession> InsightsClientSessions { get; private set; }
		public InsightsHostSession HostSession { get; private set; }

		public string DefaultCommandLine;
		private string ArtifactPath;
		public override string Name { get { return "InsightsTest"; } }
		public override float MaxDuration { protected set; get; }

		int TotalClientInstances = 0;

		public InsightsTests(InsightsTestContext InContext)
		{
			Context = InContext;

			MaxDuration = 60 * 30;
			InsightsTestResult = TestResult.Invalid;

			InsightsClientSessions = new Dictionary<string, InsightsClientSession>();
			HostSession = new InsightsHostSession(Context.BuildInfo, Context.Options.Mode, Context.Options.HostTests, Context.Options.Sleep, Context.Options.WriteTraceFileOnly);
		}

		public override bool IsReadyToStart()
		{
			ClearPreviousTraces();

			List<UnrealTargetPlatform> TotalPlatforms = new List<UnrealTargetPlatform>() { HostPlatform.Platform };

			foreach (var Client in Context.Options.Clients)
			{
				if (!InsightsClientSessions.ContainsKey(Client.TargetName))
				{
					InsightsClientSession Session = new InsightsClientSession(Context.BuildInfo, Context.Options.Sleep, Context.Options.WriteTraceFileOnly, Client);
					TotalPlatforms.AddRange(Client.Platforms);
					InsightsClientSessions.Add(Client.TargetName, Session);
				}	
			}
			return InsightsDeviceReservation.GetInstance().Reserve(TotalPlatforms);
		}

		public override string GetTestSummary()
		{
			return "Insights Test";
		}

		public override TestResult GetTestResult()
		{
			return InsightsTestResult;
		}

		public override void SetTestResult(TestResult testResult)
		{
			InsightsTestResult = testResult;
		}

		public override void AddTestEvent(UnrealTestEvent InEvent)
		{
			if (InEvent.Summary.Equals("Insufficient devices found"))
			{
				Log.Error(KnownLogEvents.Gauntlet_TestEvent, "Test didn't run due to insufficient devices.");
			}
		}

		public override bool StartTest(int Pass, int NumPasses)
		{
			if (Context.Options.Mode == InsightsMode.GenerateTraces && InsightsClientSessions.Count == 0)
			{
				throw new AutomationException("Node already has empty InsightsTestsApps, was IsReadyToStart called?");
			}

			if (Context.Options.Mode != InsightsMode.GenerateTraces && Context.Options.WriteTraceFileOnly)
			{
				throw new AutomationException("Viewer and Hub mode can't run with -WriteTraceFileOnly");
			}

			ArtifactPath = Path.Join(Context.Options.LogDir, "");
			Log.Info("InsightsTestNode.StartTest Creating artifacts path at {0}", ArtifactPath);
			Directory.CreateDirectory(ArtifactPath);

			if (!Context.Options.WriteTraceFileOnly)
			{
				HostInstance = HostSession.RunInsightsApp();

				if (HostInstance == null || HostInstance.HasExited || HostInstance.WasKilled)
				{
					throw new AutomationException("Host didn't launch properly, insights test cannot proceed.");
				}

				if (!AwaitInsightsHostSlateInitialization())
				{
					HostInstance.Kill();
					throw new AutomationException("Waited for Insights host initialization signal but didn't receive any, quitting.");
				}
				Log.Info("Started Insights host...");
			}

			foreach (var InsightsTestsApps in InsightsClientSessions.Values)
			{
				ClientInstances.AddRange(InsightsTestsApps.InstallAndRunClientApps().Values);
				
				foreach (var ClientInstance in ClientInstances)
				{
					IDeviceUsageReporter.RecordComment(ClientInstance.Device.Name, (UnrealTargetPlatform)ClientInstance.Device.Platform, IDeviceUsageReporter.EventType.Device, Context.Options.JobDetails);
					IDeviceUsageReporter.RecordComment(ClientInstance.Device.Name, (UnrealTargetPlatform)ClientInstance.Device.Platform, IDeviceUsageReporter.EventType.Test, this.GetType().Name);
				}

				if (SessionStartTime == DateTime.MinValue)
				{
					SessionStartTime = DateTime.Now;
				}
			}

			TotalClientInstances = ClientInstances.Count;
			
			MarkTestStarted();

			return true;
		}

		private bool AwaitInsightsHostSlateInitialization()
		{
			const int MaxRetries = 30;
			for (int i = 0; i < MaxRetries; i++)
			{
				if (HostInstance
					.GetLogBufferReader()
					.EnumerateNextLines()
					.Any(L => L.Contains(InitializationLogEntry)))
				{
					return true;
				}
				Thread.Sleep(2000);
			}
			return false;
		}

		int TotalClientFailures = 0;
		int TotalClientTimeouts = 0;

		public override void TickTest()
		{
			ParseAutomationEvents();

			ClientInstances.ForEach(Client =>
			{
				if (Client.HasExited)
				{
					Log.Info($"Client exited with exit code {Client.ExitCode}");
					if (Client.WasKilled || Client.ExitCode != 0)
					{
						TotalClientFailures++;
					}
				}
				else if (CheckForTimeout())
				{
					Log.Error("One or more clients has timed out.");
					Client.Kill();
					TotalClientTimeouts++;
				}
			});

			ClientInstances.RemoveAll(Client => Client.HasExited);

			if (Context.Options.Mode == InsightsMode.GenerateTraces && ClientInstances.Count == 0)
			{
				if (HostInstance != null)
				{
					Log.Info("All clients exited, closing host...");
					HostInstance.Kill();
				}
				InsightsTestResult = TotalClientFailures > 0 || TotalClientTimeouts > 0 ? TestResult.Failed : TestResult.Passed;
				MarkTestComplete();
			}
			else if (Context.Options.Mode != InsightsMode.GenerateTraces && HostInstance.HasExited)
			{
				InsightsTestResult = HostInstance.ExitCode == 0 ? TestResult.Passed : TestResult.Failed;
				MarkTestComplete();
			}
		}

		public void ClearPreviousTraces()
		{
			Directory.GetFiles(
				Path.Combine(Unreal.EngineDirectory.FullName, "Programs", "AutomationTool", "Saved", "Logs"),
				"*.utrace")
				.ToList()
				.ForEach(T => { File.Delete(T); });
		}

		public void CollectClientTraces()
		{
			List<FileInfo> AllTraces = new List<FileInfo>();

			if (!Context.Options.WriteTraceFileOnly)
			{
				string StorePathRoot = HostPlatform.Platform == UnrealTargetPlatform.Win64
					? Path.Combine(Environment.GetFolderPath(Environment.SpecialFolder.LocalApplicationData), "UnrealEngine", "Common", "UnrealTrace", "Store")
					: Path.Combine(Environment.GetFolderPath(Environment.SpecialFolder.UserProfile), "UnrealEngine", "UnrealTrace", "Store");
				var AllDirs = new DirectoryInfo(StorePathRoot).GetDirectories();
				if (AllDirs.Length > 0)
				{
					var TodayStringFormatted = DateTime.Today.ToString("yyyyMMdd");
					var TomorrowStringFormatted = DateTime.Today.AddDays(1).ToString("yyyyMMdd");
					var StorePath = AllDirs.OrderByDescending(D => D.LastAccessTime).First();
					AllTraces = StorePath.GetFiles("*.utrace")
						.Where(F => F.Name.StartsWith(TodayStringFormatted) || F.Name.StartsWith(TomorrowStringFormatted))
						.OrderByDescending(F => F.LastAccessTime).Take(TotalClientInstances).ToList();
				}
			}

			if (Context.Options.Mode == InsightsMode.GenerateTraces && !Context.Options.WriteTraceFileOnly && AllTraces.Count == 0)
			{
				Log.Error("Could not find any tracefiles generated during this session.");
			}

			string TargetPath = Path.Combine(Unreal.EngineDirectory.FullName, "Programs", "AutomationTool", "Saved", "Logs");
			foreach (FileInfo Trace in AllTraces)
			{
				Log.Info("Copying trace file to Logs folder: " + Trace.FullName);
				File.Copy(Trace.FullName, Path.Combine(TargetPath, Trace.Name));
			}
		}
		

		private int LastStdoutSeekPos;
		private string[] CurrentProcessedLines;
		private void ParseAutomationEvents()
		{
			if (HostInstance != null)
			{
				if (LastStdoutSeekPos < HostInstance.StdOut.Length)
				{
					CurrentProcessedLines = HostInstance.StdOut
						.Substring(LastStdoutSeekPos)
						.Split("\n")
						.Where(Line => !string.IsNullOrWhiteSpace(Line) && Line.Contains("LogAutomationController"))
						.ToArray();
					LastStdoutSeekPos = HostInstance.StdOut.Length - 1;

					foreach (string OutputLine in CurrentProcessedLines)
					{
						Console.WriteLine(OutputLine.TrimEnd(Environment.NewLine.ToCharArray()));
					}
				}
			}
		}

		private bool CheckForTimeout()
		{
			return false;
		}

		public override void StopTest(StopReason InReason)
		{
			try
			{
				base.StopTest(InReason);

				if (ClientInstances.Count > 0 && !ClientInstances.Any(Client => Client.HasExited))
				{
					ClientInstances.Where(Client => Client.HasExited).ToList().ForEach(Client => Client.Kill());
				}

				string ExitReason = "";

				if (ClientInstances.Any(Client => Client.WasKilled))
				{
					if (InReason == StopReason.MaxDuration || InsightsTestResult == TestResult.TimedOut)
					{
						InsightsTestResult = TestResult.TimedOut;
						ExitReason = "Timed Out";
					}
					else
					{
						InsightsTestResult = TestResult.Failed;
						ExitReason = $"Process was killed by Gauntlet with reason {InReason}.";
					}
				}
				else if (ClientInstances.Any(Client => Client.ExitCode != 0))
				{
					InsightsTestResult = TestResult.Failed;
					ExitReason = $"A client exited with non-zero exit code";
				}
				Log.Info(ExitReason);
			}
			catch
			{
				throw;
			}
			finally
			{
				if (HostInstance != null && !HostInstance.HasExited)
				{
					HostInstance.Kill();
				}
				CollectClientTraces();
			}
		}

		public override void CleanupTest()
		{
			foreach (var InsightsTestsApps in InsightsClientSessions.Values)
			{
				InsightsTestsApps.Dispose();
			}
			if (HostInstance != null)
			{
				HostSession.Dispose();
			}
		}
	}
}
