// Copyright Epic Games, Inc. All Rights Reserved.
using System;
using System.Collections.Generic;
using System.Linq;
using System.Text;
using System.Text.RegularExpressions;
using System.IO;
using System.Diagnostics;
using System.Runtime.InteropServices;
using AutomationTool;
using UnrealBuildTool;
using EpicGames.Core;
using UnrealBuildBase;

public abstract class BaseLinuxPlatform : Platform
{
	// Matches strings of the form "DeviceName@IP Address" such as "WindowsServer@10.1.168.74"
	static Regex DeviceRegex = new Regex(@"\w.+@([A-Za-z0-9\.\-]+)[\+]?");

	static string PScpPath  = MakePathSafeToUseWithCommandLine(CombinePaths(Unreal.RootDirectory.FullName, "Engine", "Extras", "ThirdPartyNotUE", "putty", "PSCP.EXE"));
	static string PlinkPath = MakePathSafeToUseWithCommandLine(CombinePaths(Unreal.RootDirectory.FullName, "Engine", "Extras", "ThirdPartyNotUE", "putty", "PLINK.EXE"));
	static string LaunchOnHelperShellScriptName = "LaunchOnHelper.sh";

	public BaseLinuxPlatform(UnrealTargetPlatform P)
		: base(P)
	{
	}

	public override DeviceInfo[] GetDevices()
	{
		List<DeviceInfo> Devices = new List<DeviceInfo>();

		if (HostPlatform.Current.HostEditorPlatform == TargetPlatformType)
		{
			DeviceInfo LocalMachine = new DeviceInfo(TargetPlatformType, Unreal.MachineName, Unreal.MachineName,
				RuntimeInformation.OSDescription, "Computer", true, true);

			Devices.Add(LocalMachine);
		}

		// only add for true Linux, not Arm64, etc
		if (PlatformType == UnrealTargetPlatform.Linux)
		{
			Devices.AddRange(SteamDeckSupport.GetDevices(UnrealTargetPlatform.Linux));
		}

		return Devices.ToArray();
	}

	public override void PersistSdkRootVar()
	{
		string UeSdksRoot = Environment.GetEnvironmentVariable("UE_SDKS_ROOT");
		if (UeSdksRoot != null)
		{
			base.PersistSdkRootVar();
			string AutoSdkFile = Path.Combine(Environment.GetFolderPath(Environment.SpecialFolder.ApplicationData), ".autosdk");
			if (!File.Exists(AutoSdkFile))
			{
				using (StreamWriter wr = new StreamWriter(AutoSdkFile))
				{
					wr.WriteLine(UeSdksRoot);
				}
			}
		}
	}

	public override void GetFilesToDeployOrStage(ProjectParams Params, DeploymentContext SC)
	{
		if (SC.bStageCrashReporter)
		{
			FileReference ReceiptFileName = TargetReceipt.GetDefaultPath(Unreal.EngineDirectory, "CrashReportClient", SC.StageTargetPlatform.PlatformType, UnrealTargetConfiguration.Shipping, null);
			if (FileReference.Exists(ReceiptFileName))
			{
				TargetReceipt Receipt = TargetReceipt.Read(ReceiptFileName);
				SC.StageBuildProductsFromReceipt(Receipt, true, false);
			}
		}

		// Stage all the build products
		Console.WriteLine("Staging all {0} build products", SC.StageTargets.Count);
		int BuildProductIdx = 0;
		foreach (StageTarget Target in SC.StageTargets)
		{
			Console.WriteLine(" Product {0}: {1}", BuildProductIdx, Target.Receipt.TargetName);
			SC.StageBuildProductsFromReceipt(Target.Receipt, Target.RequireFilesExist, Params.bTreatNonShippingBinariesAsDebugFiles);
			++BuildProductIdx;
        }

		FileReference SplashImage = FileReference.Combine(SC.ProjectRoot, "Content", "Splash", "Splash.bmp");
		if(FileReference.Exists(SplashImage))
		{
			SC.StageFile(StagedFileType.NonUFS, SplashImage);
		}

		// Stage the bootstrap executable
		if (!Params.NoBootstrapExe)
		{
			foreach (StageTarget Target in SC.StageTargets)
			{
				BuildProduct Executable = Target.Receipt.BuildProducts.FirstOrDefault(x => x.Type == BuildProductType.Executable);
				if (Executable != null)
				{
					// only create bootstraps for executables
					string FullExecutablePath = Path.GetFullPath(Executable.Path.FullName);
					if (Executable.Path.FullName.Replace("\\", "/").Contains("/" + TargetPlatformType.ToString() + "/"))
					{
						string BootstrapArguments = "";
						if (!ShouldStageCommandLine(Params, SC))
						{
							if (!SC.IsCodeBasedProject)
							{
								BootstrapArguments = String.Format("\\\"../../../{0}/{0}.uproject\\\"", SC.ShortProjectName);
							}
							else
							{
								BootstrapArguments = SC.ShortProjectName;
							}
						}

						string BootstrapExeName;
						if (SC.StageTargetConfigurations.Count > 1)
						{
							BootstrapExeName = Path.GetFileName(Executable.Path.FullName);
						}
						else if (Params.IsCodeBasedProject)
						{
							BootstrapExeName = Target.Receipt.TargetName;
						}
						else
						{
							BootstrapExeName = SC.ShortProjectName;
						}

						string Extension = ".sh";
						if (Target.Receipt.Platform == UnrealTargetPlatform.LinuxArm64)
						{
								Extension = "-Arm64.sh";
						}

						List<StagedFileReference> StagePaths = SC.FilesToStage.NonUFSFiles.Where(x => x.Value == Executable.Path).Select(x => x.Key).ToList();
						foreach (StagedFileReference StagePath in StagePaths)
						{
							StagedFileReference RemappedStagePath = DeploymentContext.ApplyDirectoryRemap(SC, StagePath);
							StageBootstrapExecutable(SC, BootstrapExeName + Extension, FullExecutablePath, RemappedStagePath.Name, BootstrapArguments);
						}
					}
				}
			}
		}
	}

	public override bool ShouldStageCommandLine(ProjectParams Params, DeploymentContext SC)
	{
		return false;
	}

	void StageBootstrapExecutable(DeploymentContext SC, string ExeName, string TargetFile, string StagedRelativeTargetPath, string StagedArguments)
	{
		// create a temp script file location
		DirectoryReference IntermediateDir = DirectoryReference.Combine(SC.ProjectRoot, "Intermediate", "Staging");
		FileReference IntermediateFile = FileReference.Combine(IntermediateDir, ExeName);
		DirectoryReference.CreateDirectory(IntermediateDir);

		// make sure slashes are good
		StagedRelativeTargetPath = StagedRelativeTargetPath.Replace("\\", "/");

		// make contents
		StringBuilder Script = new StringBuilder();
		string EOL = "\n";
		Script.Append("#!/bin/sh" + EOL);

		// If the BootstrapPreamble.sh file exists, prepend it to the regular launch script, so that things like display and required platform environment variables can be set up automatically
		FileReference BootstrapPreambleFile = FileReference.Combine(SC.ProjectRoot, "Config", "BootstrapPreamble.sh");
		if (FileReference.Exists(BootstrapPreambleFile))
		{
			Script.AppendFormat(EOL + "### Added from project Config BootstrapPreamble.sh" + EOL + EOL);
			Script.Append(FileReference.ReadAllText(BootstrapPreambleFile));
			Script.AppendFormat(EOL + "### Added from project Config BootstrapPreamble.sh" + EOL + EOL);			
		}

		// allow running from symlinks
		Script.AppendFormat("UE_TRUE_SCRIPT_NAME=$(echo \\\"$0\\\" | xargs readlink -f)" + EOL);
		Script.AppendFormat("UE_PROJECT_ROOT=$(dirname \"$UE_TRUE_SCRIPT_NAME\")" + EOL);
		Script.AppendFormat("chmod +x \"$UE_PROJECT_ROOT/{0}\"" + EOL, StagedRelativeTargetPath);
		Script.AppendFormat("\"$UE_PROJECT_ROOT/{0}\" {1} \"$@\" " + EOL, StagedRelativeTargetPath, StagedArguments);

		// write out the 
		FileReference.WriteAllText(IntermediateFile, Script.ToString());

		if (!OperatingSystem.IsWindows())
		{
			var Result = CommandUtils.Run("env", string.Format("-- \"chmod\" \"+x\" \"{0}\"", IntermediateFile.ToString().Replace("'", "'\"'\"'")));
			if (Result.ExitCode != 0)
			{
				throw new AutomationException(string.Format("Failed to chmod \"{0}\"", IntermediateFile));
			}
		}

		SC.StageFile(StagedFileType.NonUFS, IntermediateFile, new StagedFileReference(ExeName));
	}

	public override string GetCookPlatform(bool bDedicatedServer, bool bIsClientOnly)
	{
		const string NoEditorCookPlatform = "";
		const string ServerCookPlatform = "Server";
		const string ClientCookPlatform = "Client";
		string PlatformStr = (TargetPlatformType == UnrealTargetPlatform.LinuxArm64) ? "LinuxArm64" : "Linux";

		if (bDedicatedServer)
		{
			return PlatformStr + ServerCookPlatform;
		}
		else if (bIsClientOnly)
		{
			return PlatformStr + ClientCookPlatform;
		}

		return PlatformStr + NoEditorCookPlatform;
	}

	public override string GetEditorCookPlatform()
	{
		if (TargetPlatformType == UnrealTargetPlatform.LinuxArm64)
		{
			return "LinuxArm64Editor";
		}

		return "LinuxEditor";
	}

	/// <summary>
	/// return true if we need to change the case of filenames outside of pak files
	/// </summary>
	/// <returns></returns>
	public override bool DeployLowerCaseFilenames(StagedFileType FileType)
	{
		return false;
	}

	/// <summary>
	/// Deploy the application on the current platform
	/// </summary>
	/// <param name="Params"></param>
	/// <param name="SC"></param>
	public override void Deploy(ProjectParams Params, DeploymentContext SC)
	{
		// We only care about deploying for SteamDeck
		if (Params.Devices.Count == 1 && GetDevices().FirstOrDefault(x => x.Id == Params.DeviceNames[0])?.Type == "SteamDeck")
		{
			SteamDeckSupport.Deploy(UnrealTargetPlatform.Linux, Params, SC);
			return;
		}

		if (BuildHostPlatform.Current.Platform != UnrealTargetPlatform.Linux)
		{
			foreach (string DeviceAddress in Params.DeviceNames)
			{
				string CookPlatformName = GetCookPlatform(Params.DedicatedServer, Params.Client);
				string SourcePath = CombinePaths(Params.BaseStageDirectory, CookPlatformName);
				if (!Directory.Exists(SourcePath))
				{
					throw new AutomationException(string.Format("Source directory \"{0}\" must exist.", SourcePath));
				}

				string DestPath = "./" + Params.ShortProjectName;
				List<FileReference> Exes = GetExecutableNames(SC);
				string BinaryName = "";
				if (Exes.Count > 0)
				{
					// if stage directory does not end with "\\", insert one
					string Separator = "";
					if (Params.BaseStageDirectory.Length > 0 && (Params.BaseStageDirectory.EndsWith("/") || Params.BaseStageDirectory.EndsWith("\\")))
					{
						Separator = "/";
					}
					BinaryName = Exes[0].FullName.Replace(Params.BaseStageDirectory, DestPath + Separator);
					BinaryName = BinaryName.Replace("\\", "/");
				}

				// stage a shell script that makes running easy
				// Starting from 0, finds the first valid X11 connection xset is able to query
				// if it fails to find one it'll default to 0 and fail to connect.
				string Script = String.Format(@"#!/bin/bash
VALID_DISPLAY=0

for i in `seq 0 16`; do
    DISPLAY=:$i xset -q > /dev/null 2>&1
    if [ $? -eq 0 ]
    then
        VALID_DISPLAY=$i
        break
    fi
done

export DISPLAY=:$VALID_DISPLAY
chmod +x {0}
{0} $@
", BinaryName, BinaryName);
				Script = Script.Replace("\r\n", "\n");
				string ScriptFile = Path.Combine(SourcePath, "..", LaunchOnHelperShellScriptName);
				File.WriteAllText(ScriptFile, Script);

				if (!Params.IterativeDeploy)
				{
					// non-null input is essential, since RedirectStandardInput=true is needed for PLINK, see http://stackoverflow.com/questions/1910592/process-waitforexit-on-console-vs-windows-forms
					RunAndLog(CmdEnv, PlinkPath, String.Format("-batch -P 22 -l {0} -pw {1} {2} rm -rf {3} && mkdir -p {3}", Params.DeviceUsername, Params.DevicePassword, Params.DeviceNames[0], DestPath), Input: "");
				}
				else
				{
					// still make sure that the path exists
					RunAndLog(CmdEnv, PlinkPath, String.Format("-batch -P 22 -l {0} -pw {1} {2} mkdir -p {3}", Params.DeviceUsername, Params.DevicePassword, Params.DeviceNames[0], DestPath), Input: "");
				}

				// copy the contents
				RunAndLog(CmdEnv, PScpPath, String.Format("-batch -P 22 -pw {0} -r \"{1}\" {2}", Params.DevicePassword, SourcePath, Params.DeviceUsername + "@" + DeviceAddress + ":" + DestPath));

				// copy the helper script
				RunAndLog(CmdEnv, PScpPath, String.Format("-batch -P 22 -pw {0} -r \"{1}\" {2}", Params.DevicePassword, ScriptFile, Params.DeviceUsername + "@" + DeviceAddress + ":" + DestPath));

				string RemoteScriptFile = DestPath + "/" + LaunchOnHelperShellScriptName;
				// non-null input is essential, since RedirectStandardInput=true is needed for PLINK, see http://stackoverflow.com/questions/1910592/process-waitforexit-on-console-vs-windows-forms
				RunAndLog(CmdEnv, PlinkPath, String.Format(" -batch -P 22 -l {0} -pw {1} {2} chmod +x {3}", Params.DeviceUsername, Params.DevicePassword, Params.DeviceNames[0], RemoteScriptFile), Input: "");
			}
		}
	}

	public override void Package(ProjectParams Params, DeploymentContext SC, int WorkingCL)
	{
		// package up the program
		PrintRunTime();
	}

	public override bool CanHostPlatform(UnrealTargetPlatform Platform)
	{
		if (Platform == UnrealTargetPlatform.Mac || Platform == UnrealTargetPlatform.Win64)
		{
			return false;
		}
		return true;
	}

	public override void GetTargetFile(string RemoteFilePath, string LocalFile, ProjectParams Params)
	{
		var SourceFile = FileReference.Combine(new DirectoryReference(Params.BaseStageDirectory), GetCookPlatform(Params.HasServerCookedTargets, Params.HasClientTargetDetected), RemoteFilePath);
		CommandUtils.CopyFile(SourceFile.FullName, LocalFile);
	}

	/// <summary>
	/// Allow the platform to alter the ProjectParams
	/// </summary>
	/// <param name="ProjParams"></param>
	public override void PlatformSetupParams(ref ProjectParams ProjParams)
	{
		if ((ProjParams.Deploy || ProjParams.Run) && BuildHostPlatform.Current.Platform != UnrealTargetPlatform.Linux)
		{
 			Match ServerDeviceMatch = DeviceRegex.Match(ProjParams.ServerDevice);
			if (ServerDeviceMatch.Success)
			{
				ProjParams.ServerDeviceAddress = ServerDeviceMatch.Groups[1].Value;
			}

			// we don't need username/password if we are only targeting steamdeck devices
			bool bNeedsUsernameAndPassword = false;
			foreach (string DeviceId in ProjParams.DeviceNames)
			{
				if (SteamDeckSupport.GetDeviceInfo(UnrealTargetPlatform.Linux, ProjParams, out _, out _) == false)
				{
					bNeedsUsernameAndPassword = true;
				}
			}

			if (bNeedsUsernameAndPassword)
			{
				// Prompt for username if not already set
				while (String.IsNullOrEmpty(ProjParams.DeviceUsername))
				{
					Console.Write("Username: ");
					ProjParams.DeviceUsername = Console.ReadLine();
				}

				// Prompty for password if not already set
				while (String.IsNullOrEmpty(ProjParams.DevicePassword))
				{
					ProjParams.DevicePassword = String.Empty;
					Console.Write("Password: ");
					ConsoleKeyInfo key;
					do
					{
						key = Console.ReadKey(true);
						if (key.Key != ConsoleKey.Backspace && key.Key != ConsoleKey.Enter)
						{
							ProjParams.DevicePassword += key.KeyChar;
							Console.Write("*");
						}
						else
						{
							if (key.Key == ConsoleKey.Backspace && ProjParams.DevicePassword.Length > 0)
							{
								ProjParams.DevicePassword = ProjParams.DevicePassword.Substring(0, (ProjParams.DevicePassword.Length - 1));
								Console.Write("\b \b");
							}
						}

					} while (key.Key != ConsoleKey.Enter);
					Console.WriteLine();
				}

				// try contacting the device(s) and cache the key(s)
				foreach (string DeviceAddress in ProjParams.DeviceNames)
				{
					RunAndLog(CmdEnv, "cmd.exe", String.Format("/c \'echo y | {0} -P 22 -ssh -t -l {1} -pw {2} {3} echo All Ok\'", PlinkPath, ProjParams.DeviceUsername, ProjParams.DevicePassword, DeviceAddress));
				}
			}
		}
	}
	public override IProcessResult RunClient(ERunOptions ClientRunFlags, string ClientApp, string ClientCmdLine, ProjectParams Params)
	{
		if (Params.Devices.Count == 1 && GetDevices().FirstOrDefault(x => x.Id == Params.DeviceNames[0])?.Type == "SteamDeck")
		{
			return SteamDeckSupport.RunClient(UnrealTargetPlatform.Linux, ClientRunFlags, ClientApp, ClientCmdLine, Params);
		}

		if (BuildHostPlatform.Current.Platform != UnrealTargetPlatform.Linux)
		{
			IProcessResult Result = null;
			foreach (string DeviceAddress in Params.DeviceNames)
			{
				// Use the helper script (should already be +x - see Deploy)
				string RemoteBasePath = "./" + Params.ShortProjectName;
				string RemotePathToBinary = RemoteBasePath + "/" + LaunchOnHelperShellScriptName;
				// non-null input is essential, since RedirectStandardInput=true is needed for PLINK, see http://stackoverflow.com/questions/1910592/process-waitforexit-on-console-vs-windows-forms
				Result = Run(PlinkPath, String.Format("-batch -P 22 -ssh -t -l {0} -pw {1} {2}  {3} {4}", Params.DeviceUsername, Params.DevicePassword, Params.DeviceNames[0], RemotePathToBinary, ClientCmdLine), "");
			}
			return Result;
		}
		else
		{
			return base.RunClient(ClientRunFlags, ClientApp, ClientCmdLine, Params);
		}
	}
	public override List<string> GetDebugFileExtensions()
	{
		return new List<string> { };
	}

	public override bool IsSupported { get { return true; } }

	public override void StripSymbols(FileReference SourceFile, FileReference TargetFile)
	{
		LinuxExports.StripSymbols(SourceFile, TargetFile, Log.Logger);
	}
}


public class GenericLinuxPlatform : BaseLinuxPlatform
{
	public GenericLinuxPlatform()
		: base(UnrealTargetPlatform.Linux)
	{
	}
}

public class GenericLinuxPlatformArm64 : BaseLinuxPlatform
{
	public GenericLinuxPlatformArm64()
		: base(UnrealTargetPlatform.LinuxArm64)
	{
	}
}

public class LinuxPlatformArm64DXT : GenericLinuxPlatformArm64
{
	public LinuxPlatformArm64DXT()
		: base()
	{
	}

	public override TargetPlatformDescriptor GetTargetPlatformDescriptor()
	{
		return new TargetPlatformDescriptor(TargetPlatformType, "DXT");
	}

	public override string GetCookPlatform(bool bDedicatedServer, bool bIsClientOnly)
	{
		return base.GetCookPlatform(bDedicatedServer, bIsClientOnly) + "_DXT";
	}
}

public class LinuxPlatformArm64ASTC : GenericLinuxPlatformArm64
{
	public LinuxPlatformArm64ASTC()
		: base()
	{
	}
	
	public override TargetPlatformDescriptor GetTargetPlatformDescriptor()
	{
		return new TargetPlatformDescriptor(TargetPlatformType, "ASTC");
	}

	public override string GetCookPlatform(bool bDedicatedServer, bool bIsClientOnly)
	{
		return base.GetCookPlatform(bDedicatedServer, bIsClientOnly) + "_ASTC";
	}
}

public class LinuxPlatformArm64ETC2 : GenericLinuxPlatformArm64
{
	public LinuxPlatformArm64ETC2()
		: base()
	{
	}

	public override TargetPlatformDescriptor GetTargetPlatformDescriptor()
	{
		return new TargetPlatformDescriptor(TargetPlatformType, "ETC2");
	}

	public override string GetCookPlatform(bool bDedicatedServer, bool bIsClientOnly)
	{
		return base.GetCookPlatform(bDedicatedServer, bIsClientOnly) + "_ETC2";
	}
}

public class LinuxPlatformArm64Multi : GenericLinuxPlatformArm64
{
	public LinuxPlatformArm64Multi()
		: base()
	{
	}

	public override TargetPlatformDescriptor GetTargetPlatformDescriptor()
	{
		return new TargetPlatformDescriptor(TargetPlatformType, "Multi");
	}

	public override string GetCookPlatform(bool bDedicatedServer, bool bIsClientOnly)
	{
		return base.GetCookPlatform(bDedicatedServer, bIsClientOnly) + "_Multi";
	}
}

