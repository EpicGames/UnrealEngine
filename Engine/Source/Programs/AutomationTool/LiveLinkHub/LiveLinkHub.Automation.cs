// Copyright Epic Games, Inc. All Rights Reserved.

using EpicGames.Core;
using Microsoft.Extensions.Logging;
using System.Collections.Generic;
using System.Linq;



namespace AutomationTool
{
	class MakeLiveLinkHubEditor : MakeCookedEditor
	{
		protected override ProjectParams MakeParams(string DLCName, string BasedOnReleaseVersion)
		{
			ProjectParams Params = new ProjectParams(
				Command: this
				, RawProjectPath: ProjectFile
				, NoBootstrapExe: true
				, DLCName: DLCName
				, BasedOnReleaseVersion: BasedOnReleaseVersion
				, DedicatedServer: bIsCookedCooker
				, SkipBuildClient: true
				, NoClient: bIsCookedCooker
				, OptionalContent: true
				, ClientCookedTargets: new ParamList<string>() // Prevent AutodetectSettings from looking for a game target
				, EditorTargets: new ParamList<string>("LiveLinkHub")
				, UbtArgs: "-SingleModulePlatform"
			);

			return Params;
		}

		protected override void ModifyParams(ProjectParams BuildParams)
		{
			base.ModifyParams(BuildParams);

			// We don't want the SDK dir / CookerSupportFiles
			BuildParams.CookerSupportFilesSubdirectory = null;
		}

		protected override void ModifyDeploymentContext(ProjectParams Params, DeploymentContext SC)
		{
			ModifyStageContext Context = CreateContext(Params, SC);

			DefaultModifyDeploymentContext(Params, SC, Context);

			Context.Apply(SC);

			string PlatName = SC.StageTargetPlatform.PlatformType.ToString();
			string ExeExtension = Platform.GetExeExtension(SC.StageTargetPlatform.PlatformType);

			// Copy Zen binaries
			SC.FilesToStage.NonUFSFiles.Add(
				new StagedFileReference($"Engine/Binaries/{PlatName}/zen{ExeExtension}"),
				new FileReference($"Engine/Binaries/{PlatName}/zen{ExeExtension}"));

			SC.FilesToStage.NonUFSFiles.Add(
				new StagedFileReference($"Engine/Binaries/{PlatName}/zenserver{ExeExtension}"),
				new FileReference($"Engine/Binaries/{PlatName}/zenserver{ExeExtension}"));

			// Copy .target receipt to project and engine bin
			SC.FilesToStage.NonUFSFiles.Add(
				new StagedFileReference($"{Context.ProjectName}/Binaries/{PlatName}/{Context.ProjectName}.target"),
				new FileReference($"Engine/Binaries/{PlatName}/{Context.ProjectName}.target"));

			SC.FilesToStage.NonUFSFiles.Add(
				new StagedFileReference($"Engine/Binaries/{PlatName}/{Context.ProjectName}.target"),
				new FileReference($"Engine/Binaries/{PlatName}/{Context.ProjectName}.target"));

			// Stage TargetInfo and target script. Necessary to avoid "Running incorrect executable for target (...)"
			SC.FilesToStage.NonUFSFiles.Add(
				new StagedFileReference($"{Context.ProjectName}/Source/{Context.ProjectName}.Target.cs"),
				new FileReference($"Engine/Source/Programs/{Context.ProjectName}/Source/{Context.ProjectName}.Target.cs"));

			SC.FilesToStage.NonUFSFiles.Add(
				new StagedFileReference($"{Context.ProjectName}/Intermediate/TargetInfo.json"),
				new FileReference($"Engine/Source/Programs/{Context.ProjectName}/Intermediate/TargetInfo.json"));

			// Copy custom splash screen
			SC.FilesToStage.NonUFSFiles.Add(
				new StagedFileReference($"{Context.ProjectName}/Content/Splash/EdSplash.png"),
				new FileReference($"Engine/Source/Programs/{Context.ProjectName}/Content/Splash/EdSplash.png"));

			SC.FilesToStage.NonUFSFiles.Add(
				new StagedFileReference($"{Context.ProjectName}/Content/Splash/EdSplash.uasset"),
				new FileReference($"Engine/Source/Programs/{Context.ProjectName}/Content/Splash/EdSplash.uasset"));

			// Remove asset registry entry
			SC.FilesToStage.UFSFiles.Remove(new StagedFileReference($"{Context.ProjectName}/EditorClientAssetRegistry.bin"));

			// Move PDBs to debug (FIXME: should this be necessary? otherwise -nodebuginfo doesn't exclude them)
			SC.FilesToStage.NonUFSDebugFiles.Union(SC.FilesToStage.NonUFSFiles.Where(x => x.Key.HasExtension(".pdb")));

			// Remove any files from NonUFS that were also added to Debug
			SC.FilesToStage.NonUFSFiles = SC.FilesToStage.NonUFSFiles.Where(x => !SC.FilesToStage.NonUFSDebugFiles.ContainsKey(x.Key)).ToDictionary(x => x.Key, x => x.Value);
			
			// Make sure there are no restricted folders in the output
			HashSet<StagedFileReference> RestrictedFiles = new HashSet<StagedFileReference>();

			foreach (string RestrictedName in SC.RestrictedFolderNames)
			{
				RestrictedFiles.UnionWith(SC.FilesToStage.NonUFSDebugFiles.Keys.Where(x => x.ContainsName(RestrictedName)));
			}

			foreach (StagedFileReference StagedFileReference in RestrictedFiles)
			{
				SC.FilesToStage.NonUFSDebugFiles.Remove(StagedFileReference);
			}
		}
	}
}
