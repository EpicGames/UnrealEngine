// Copyright Epic Games, Inc. All Rights Reserved.
using EpicGames.Core;
using System;
using System.Collections.Generic;
using System.Linq;
using Microsoft.Extensions.Logging;
using UnrealBuildTool;

using static AutomationTool.CommandUtils;
namespace EpicGames.Localization
{

	public enum LocalizationConfigFileFormat
	{
		LegacyMonolithic,
		Modular,

		// Add new values above this comment and upda te the value of Latest 
		Latest = Modular
	}

	public class LocalizationConfigFile
	{
		private readonly ConfigFile _configFile;
		public string Name { get; set; } = "";
		public LocalizationConfigFile()
		{
			_configFile = new ConfigFile();
		}

		public LocalizationConfigFile(string fileName)
		{
			_configFile = new ConfigFile();
			Name = fileName;
		}

		private LocalizationConfigFile(ConfigFile configFile, string fileName)
		{
			_configFile = (configFile == null) ? new ConfigFile() : configFile;
			Name = fileName;
		}

		public LocalizationConfigFileSection FindOrAddSection(string sectionName)
		{
			ConfigFileSection configSection = _configFile.FindOrAddSection(sectionName);
			return new LocalizationConfigFileSection(configSection);
		}

		public static LocalizationConfigFile Load(string filePath)
		{
			LocalizationConfigFile localizationConfigFile = new();
			if (String.IsNullOrEmpty(filePath))
			{
				return null;
			}
			FileReference filePathReference = new FileReference(filePath);
			if (!filePathReference.ToFileInfo().Exists)
			{
				return null;
			}
			// Parse the passed in config file 
			ConfigFile configFile = new ConfigFile(filePathReference);
			localizationConfigFile = new LocalizationConfigFile(configFile, filePathReference.GetFileName());
			
			return localizationConfigFile;
		}

		private LocalizationConfigFile DeepCopy()
		{
			// All constructors of this class should initialize _configFile in some way. It should never be null and so we don't handle that case
			ConfigFile copyConfigFile = new ConfigFile();
			foreach (string sectionName in _configFile.SectionNames)
			{
				ConfigFileSection originalSection;
				if (_configFile.TryGetSection(sectionName, out originalSection))
				{
					ConfigFileSection copySection = copyConfigFile.FindOrAddSection(originalSection.Name);
					foreach (ConfigLine originalLine in originalSection.Lines)
					{
						// The key and value for entries in localization config files will ever only be strings or bools
						// Strings are immutable and bools are simple value so a direct copy is fine 
						ConfigLine copyLine = new ConfigLine(originalLine.Action, originalLine.Key, originalLine.Value);
						copySection.Lines.Add(copyLine);
					}
				}
			}
			// now create a new localization config file 
			LocalizationConfigFile localizationConfigFile = new LocalizationConfigFile(copyConfigFile, Name);
			return localizationConfigFile;
		}

		public LocalizationConfigFile ReplacePlaceHolders(Dictionary<string, string> replacements)
		{
			LocalizationConfigFile copy = DeepCopy();
			if (replacements.Count == 0)
			{
				return copy;
			}
			// Loop through all of the sections and lines and replace any placeholder values 
			foreach (string sectionName in copy._configFile.SectionNames)
			{
				ConfigFileSection configSection;
				if (copy._configFile.TryGetSection(sectionName, out configSection))
				{
					foreach (ConfigLine configLine in configSection.Lines)
					{
						if (!String.IsNullOrEmpty(configLine.Value))
						{
							foreach (KeyValuePair<string, string> pair in replacements)
							{
								// @TODOLocalization: For now, we only replace placeholders found in the values of the config files, replacing keys is not supported 
								configLine.Value = configLine.Value.Replace(pair.Key, pair.Value);
							}
						}
					}
				}
			}

			return copy;
		}

		public bool IsEmpty()
		{
			return _configFile.SectionNames.Count() == 0;
		}

		// @TODOLocalization: There should be an IsValid function of some kind to ensure the validity of the localization config file. Should also implement validators 

		public void Write(FileReference destinationFilePath)
		{
			// Go through all the config values and replace \ directory separators with / for uniformity across platforms
			foreach (string sectionName in _configFile.SectionNames)
			{
				ConfigFileSection section;
				if (_configFile.TryGetSection(sectionName, out section))
				{
					foreach (ConfigLine line in section.Lines)
					{
						line.Value = line.Value.Replace('\\', '/');
					}
				}
			}
			_configFile.Write(destinationFilePath);
		}
	}

	public class LocalizationConfigFileSection
	{
		private readonly ConfigFileSection _configFileSection;

		public LocalizationConfigFileSection(ConfigFileSection configFileSection)
		{
			_configFileSection = configFileSection;
		}

		public void AddValue(string key, bool value)
		{
			if (!String.IsNullOrEmpty(key))
			{
				_configFileSection.Lines.Add(new ConfigLine(ConfigLineAction.Set, key, value ? "true" : "false"));
			}
		}

		public void AddValue(string key, string value)
		{
			if (!String.IsNullOrEmpty(value) && !String.IsNullOrEmpty(key))
			{
				_configFileSection.Lines.Add(new ConfigLine(ConfigLineAction.Set, key, value));
			}
		}

		public void AddValues(string key, string[] values)
		{
			if (String.IsNullOrEmpty(key) || values.Length == 0)
			{
				return;
			}
			foreach (string value in values)
			{
				if (!String.IsNullOrEmpty(value))
				{
					_configFileSection.Lines.Add(new ConfigLine(ConfigLineAction.Set, key, value));
				}
			}
		}
	}
}
