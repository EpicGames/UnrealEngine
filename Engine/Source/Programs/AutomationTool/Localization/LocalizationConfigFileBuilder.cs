// Copyright Epic Games, Inc. All Rights Reserved.

namespace EpicGames.Localization
{
	public class LocalizationConfigFileBuilder
	{
		private LocalizationConfigFile _configFile;
		private int _gatherTextStepCount;

		public LocalizationConfigFileBuilder(string localizationConfigFileName)
		{
			_configFile = new LocalizationConfigFile(localizationConfigFileName);
			_gatherTextStepCount = 0;
		}

		public void AddCommonSettingsSection(CommonSettingsParams commonSettings)
		{
			LocalizationConfigFileSection configSection = _configFile.FindOrAddSection("CommonSettings");
			configSection.AddValues("ManifestDependencies", commonSettings.manifestDependencies.ToArray());
			configSection.AddValue("SourcePath", commonSettings.SourcePath);
			configSection.AddValue("DestinationPath", commonSettings.DestinationPath);
			configSection.AddValue("ManifestName", commonSettings.ManifestName);
			configSection.AddValue("ArchiveName", commonSettings.ArchiveName);
			configSection.AddValue("PortableObjectName", commonSettings.PortableObjectName);
			configSection.AddValue("ResourceName", commonSettings.ResourceName);
			configSection.AddValue("bSkipSourceCheck", commonSettings.bSkipSourceCheck);
			configSection.AddValue("bValidateFormatPatterns", commonSettings.bValidateFormatPatterns);
			configSection.AddValue("bValidateSafeWhitespace", commonSettings.bValidateSafeWhitespace);
			configSection.AddValue("bValidateRichTextTags", commonSettings.bValidateRichTextTags);
			configSection.AddValue("NativeCulture", commonSettings.NativeCulture);
			configSection.AddValues("CulturesToGenerate", commonSettings.CulturesToGenerate.ToArray());
		}

		public void AddGatherTextFromSourceSection(GatherTextFromSourceParams gatherTextParams)
		{
			LocalizationConfigFileSection configSection = _configFile.FindOrAddSection($"GatherTextStep{_gatherTextStepCount}");
			++_gatherTextStepCount;

			configSection.AddValue("CommandletClass", gatherTextParams.CommandletClass);
			configSection.AddValues("SearchDirectoryPaths", gatherTextParams.SearchDirectoryPaths.ToArray());
			configSection.AddValues("ExcludePathFilters", gatherTextParams.ExcludePathFilters.ToArray());
			configSection.AddValues("FileNameFilters", gatherTextParams.FileNameFilters.ToArray());
			configSection.AddValue("ShouldGatherFromEditorOnlyData", gatherTextParams.bShouldGatherFromEditorOnlyData);
		}

		public void AddGatherTextFromAssetsSection(GatherTextFromAssetsParams gatherTextParams)
		{
			LocalizationConfigFileSection configSection = _configFile.FindOrAddSection($"GatherTextStep{_gatherTextStepCount}");
			++_gatherTextStepCount;

			configSection.AddValue("CommandletClass", gatherTextParams.CommandletClass);
			configSection.AddValues("IncludePathFilters", gatherTextParams.IncludePathFilters.ToArray());
			configSection.AddValues("ExcludePathFilters", gatherTextParams.ExcludePathFilters.ToArray());
			configSection.AddValues("PackageFileNameFilters", gatherTextParams.PackageFileNameFilters.ToArray());
			configSection.AddValues("CollectionFilters", gatherTextParams.CollectionFilters.ToArray());
			configSection.AddValues("WorldCollectionFilters", gatherTextParams.WorldCollectionFilters.ToArray());

			configSection.AddValue("ShouldExcludeDerivedClasses", gatherTextParams.bShouldExcludeDerivedClasses);
			configSection.AddValue("ShouldGatherFromEditorOnlyData", gatherTextParams.bShouldGatherFromEditorOnlyData);
			configSection.AddValue("SkipGatherCache", gatherTextParams.bSkipGatherCache);
		}

		public void AddGatherTextFromMetadataSection(GatherTextFromMetaDataParams gatherTextParams)
		{
			LocalizationConfigFileSection configSection = _configFile.FindOrAddSection($"GatherTextStep{_gatherTextStepCount}");
			++_gatherTextStepCount;

			configSection.AddValue("CommandletClass", gatherTextParams.CommandletClass);

			configSection.AddValues("ModulesToPreload", gatherTextParams.ModulesToPreload.ToArray());
			configSection.AddValues("IncludePathFilters", gatherTextParams.IncludePathFilters.ToArray());
			configSection.AddValues("ExcludePathFilters", gatherTextParams.ExcludePathFilters.ToArray());

			configSection.AddValues("FieldTypesToInclude", gatherTextParams.FieldTypesToInclude.ToArray());
			configSection.AddValues("FieldOwnerTypesToInclude", gatherTextParams.FieldOwnerTypesToInclude.ToArray());
			
			configSection.AddValues("InputKeys", gatherTextParams.InputKeys.ToArray());
			configSection.AddValues("OutputKeys", gatherTextParams.OutputKeys.ToArray());

			configSection.AddValues("InputNamespaces", gatherTextParams.InputNamespaceses.ToArray());
			configSection.AddValues("OutputNamespaces", gatherTextParams.OutputNamespaces.ToArray());

			configSection.AddValue("ShouldGatherFromEditorOnlyData", gatherTextParams.bShouldGatherFromEditorOnlyData);
		}

		public void AddGenerateGatherManifestSection(GenerateGatherManifestParams generateGatherManifestParams)
		{
			LocalizationConfigFileSection configSection = _configFile.FindOrAddSection($"GatherTextStep{_gatherTextStepCount}");
			++_gatherTextStepCount;

			configSection.AddValue("CommandletClass", generateGatherManifestParams.CommandletClass);
		}

		public void AddGenerateGatherArchiveSection(GenerateGatherArchiveParams generateGatherArchiveParams)
		{
			LocalizationConfigFileSection configSection = _configFile.FindOrAddSection($"GatherTextStep{_gatherTextStepCount}");
			++_gatherTextStepCount;

			configSection.AddValue("CommandletClass", generateGatherArchiveParams.CommandletClass);
		}

		public void AddExportPortableObjectFilesSection(ExportPortableObjectFilesParams exportParams)
		{
			LocalizationConfigFileSection configSection = _configFile.FindOrAddSection($"GatherTextStep{_gatherTextStepCount}");
			++_gatherTextStepCount;

			configSection.AddValue("CommandletClass", exportParams.CommandletClass);
			configSection.AddValue("bExportLoc", exportParams.bExportLoc);
			configSection.AddValue("LocalizedTextCollapseMode", exportParams.LocalizedTextCollapseMode);
			configSection.AddValue("POFormat", exportParams.POFormat);
			configSection.AddValue("ShouldPersistCommentsOnExport", exportParams.bShouldPersistCommentsOnExport);
			configSection.AddValue("ShouldAddSourceLocationsAsComments", exportParams.bShouldAddSourceLocationsAsComments);
		}

		public void AddImportPortableObjectFilesSection(ImportPortableObjectFilesParams importParams)
		{
			LocalizationConfigFileSection configSection = _configFile.FindOrAddSection($"GatherTextStep{_gatherTextStepCount}");
			++_gatherTextStepCount;

			configSection.AddValue("CommandletClass", importParams.CommandletClass);
			configSection.AddValue("bImportLoc", importParams.bImportLoc);
			configSection.AddValue("LocalizedTextCollapseMode", importParams.LocalizedTextCollapseMode);
			configSection.AddValue("POFormat", importParams.POFormat);
		}

		public void AddGenerateTextLocalizationResourceSection(GenerateTextLocalizationResourceParams localizationResourceParams)
		{
			LocalizationConfigFileSection configSection = _configFile.FindOrAddSection($"GatherTextStep{_gatherTextStepCount}");
			++_gatherTextStepCount;

			configSection.AddValue("CommandletClass", localizationResourceParams.CommandletClass);
		}

		public void AddGenerateTextLocalizationReportSection(GenerateTextLocalizationReportParams textReportParams)
		{
			LocalizationConfigFileSection configSection = _configFile.FindOrAddSection($"GatherTextStep{_gatherTextStepCount}");
			++_gatherTextStepCount;

			configSection.AddValue("CommandletClass", textReportParams.CommandletClass);
			configSection.AddValue("bWordCountReport", textReportParams.bWordCountReport);
			configSection.AddValue("WordCountReportName", textReportParams.WordCountReportName);
			if (textReportParams.bConflictReport)
			{
				configSection.AddValue("bConflictReport", textReportParams.bConflictReport);
				configSection.AddValue("ConflictReportName", textReportParams.ConflictReportName);
			}
		}

		public LocalizationConfigFile Get()
		{
			return _configFile;
		}
	}
}
