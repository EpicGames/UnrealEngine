// Copyright Epic Games, Inc. All Rights Reserved.
using System;
using System.Collections.Generic;
using System.IO;
using Microsoft.Extensions.Logging;
using UnrealBuildTool;

using static AutomationTool.CommandUtils;

namespace EpicGames.Localization
{
	public class LocalizationConfigFileGeneratorParams
	{
		public LocalizationTargetDescriptorLoadingPolicy LoadingPolicy { get; set; } = LocalizationTargetDescriptorLoadingPolicy.Never;
		public string LocalizationTargetName { get; set; } = "";
		public string LocalizationTargetRootDirectory { get; set; } = "";
	}

	public abstract class LocalizationConfigFileGenerator
	{
		public static LocalizationConfigFileGenerator GetGeneratorForFileFormat(LocalizationConfigFileFormat fileFormat)
		{
			switch (fileFormat)
			{
				case LocalizationConfigFileFormat.Modular:
					return new ModularLocalizationConfigFileGenerator();
				default:
					throw new NotImplementedException("File format is not yet supported.");
			}
		}

		public abstract List<LocalizationConfigFile> GenerateDefaultSettingsConfigFiles(LocalizationConfigFileGeneratorParams generatorParams);
		public abstract List<LocalizationConfigFile> GenerateConfigFilesFromTemplate(LocalizationConfigFileGeneratorParams generatorParams, List<LocalizationConfigFile> templateFiles);
	}

	public class ModularLocalizationConfigFileGenerator : LocalizationConfigFileGenerator
	{
		public override List<LocalizationConfigFile> GenerateDefaultSettingsConfigFiles(LocalizationConfigFileGeneratorParams generatorParams)
		{
			List<LocalizationConfigFile> configFiles = new List<LocalizationConfigFile>();
			configFiles.Add(GenerateLocalizationGatherConfigFile(generatorParams));
			configFiles.Add(GenerateLocalizationExportConfigFile(generatorParams));
			configFiles.Add(GenerateLocalizationImportConfigFile(generatorParams));
			configFiles.Add(GenerateLocalizationCompileConfigFile(generatorParams));
			configFiles.Add(GenerateLocalizationGenerateReportsConfigFile(generatorParams));
			return configFiles;
		}

		public override List<LocalizationConfigFile> GenerateConfigFilesFromTemplate(LocalizationConfigFileGeneratorParams generatorParams, List<LocalizationConfigFile> templateFiles)
		{
			List<LocalizationConfigFile> generatedFiles = new List<LocalizationConfigFile>();
			foreach (LocalizationConfigFile templateFile in templateFiles)
			{
				// @TODOLocalization: Should have additional validity checks 
				if (templateFile.IsEmpty())
				{
					Logger.LogError($"Provided template localization config file '{templateFile.Name}' is invalid. The file will not be used to generate a localization config file for '{generatorParams.LocalizationTargetName}'.");
					continue;
				}

				// @TODOLocalization: Should also include %LOCENGINEROOT%, %LOCPROJECTROOT% placeholders as well 
				Dictionary<string, string> replacements = new Dictionary<string, string>
				{
					{"%LOCTARGETNAME%", generatorParams.LocalizationTargetName},
					{"%LOCTARGETROOT%", generatorParams.LocalizationTargetRootDirectory}
				};
				LocalizationConfigFile copyFile = templateFile.ReplacePlaceHolders(replacements);
				copyFile.Name = $"{generatorParams.LocalizationTargetName}{LocalizationUtilities.GetModularLocalizationConfigSuffix(templateFile.Name)}{LocalizationFileExtensions.ModularLocalizationConfigFileExtension}";

				generatedFiles.Add(copyFile);
			}
			// Somehow we skipped using a template file to use for generation. Most likely because the template file is invalid 
			// Warn/log here 
			if (generatedFiles.Count != templateFiles.Count)
			{
				Logger.LogError($"{templateFiles.Count} template files provided but {generatedFiles.Count} files generated for localization target '{generatorParams.LocalizationTargetName}'. Verify that all provided template localization config files are valid.");
			}
			return generatedFiles;
		}

		private LocalizationConfigFile GenerateLocalizationGatherConfigFile(LocalizationConfigFileGeneratorParams generatorParam)
		{
			CommonSettingsParams commonSettings = CommonSettingsParams.CreateDefault(generatorParam.LocalizationTargetName, generatorParam.LocalizationTargetRootDirectory);
			LocalizationConfigFileBuilder builder = new LocalizationConfigFileBuilder(generatorParam.LocalizationTargetName + ModularLocalizationConfigFileSuffixes.GatherSuffix + LocalizationFileExtensions.ModularLocalizationConfigFileExtension);
			builder.AddCommonSettingsSection(commonSettings);

			if (generatorParam.LoadingPolicy != LocalizationTargetDescriptorLoadingPolicy.ToolTips &&
				generatorParam.LoadingPolicy != LocalizationTargetDescriptorLoadingPolicy.PropertyNames)
			{
				GatherTextFromSourceParams gatherTextFromSourceParams = GatherTextFromSourceParams.CreateDefault(generatorParam.LocalizationTargetName, generatorParam.LocalizationTargetRootDirectory);
				builder.AddGatherTextFromSourceSection(gatherTextFromSourceParams);

				GatherTextFromAssetsParams gatherTextFromAssetsParams = GatherTextFromAssetsParams.CreateDefault(generatorParam.LocalizationTargetName, generatorParam.LocalizationTargetRootDirectory);
				builder.AddGatherTextFromAssetsSection(gatherTextFromAssetsParams);
			}
			else
			{
				GatherTextFromMetaDataParams gatherTextFromMetadata = GatherTextFromMetaDataParams.CreateDefault(generatorParam.LocalizationTargetRootDirectory, generatorParam.LoadingPolicy);
				builder.AddGatherTextFromMetadataSection(gatherTextFromMetadata);
			}

			GenerateGatherManifestParams generateGatherManifestParams = GenerateGatherManifestParams.CreateDefault(generatorParam.LocalizationTargetName, generatorParam.LocalizationTargetRootDirectory);
			builder.AddGenerateGatherManifestSection(generateGatherManifestParams);

			GenerateGatherArchiveParams generateGatherArchiveParams = GenerateGatherArchiveParams.CreateDefault(generatorParam.LocalizationTargetName, generatorParam.LocalizationTargetRootDirectory);
			builder.AddGenerateGatherArchiveSection(	generateGatherArchiveParams);

			GenerateTextLocalizationReportParams generateTextLocalizationReportParams = GenerateTextLocalizationReportParams.CreateDefault(generatorParam.LocalizationTargetName, generatorParam.LocalizationTargetRootDirectory);
			builder.AddGenerateTextLocalizationReportSection(generateTextLocalizationReportParams);

			return builder.Get();
		}

		private LocalizationConfigFile GenerateLocalizationExportConfigFile(LocalizationConfigFileGeneratorParams generatorParams)
		{
			CommonSettingsParams commonSettings = CommonSettingsParams.CreateDefault(generatorParams.LocalizationTargetName, generatorParams.LocalizationTargetRootDirectory);
			LocalizationConfigFileBuilder builder = new LocalizationConfigFileBuilder(generatorParams.LocalizationTargetName + ModularLocalizationConfigFileSuffixes.ExportSuffix + LocalizationFileExtensions.ModularLocalizationConfigFileExtension);
			builder.AddCommonSettingsSection(commonSettings);

			ExportPortableObjectFilesParams exportParams = ExportPortableObjectFilesParams.CreateDefault(generatorParams.LocalizationTargetName, generatorParams.LocalizationTargetRootDirectory);
			builder.AddExportPortableObjectFilesSection(exportParams);
			return builder.Get();
		}

		private LocalizationConfigFile GenerateLocalizationImportConfigFile(LocalizationConfigFileGeneratorParams generatorParams)
		{
			CommonSettingsParams commonSettings = CommonSettingsParams.CreateDefault(generatorParams.LocalizationTargetName, generatorParams.LocalizationTargetRootDirectory);
			LocalizationConfigFileBuilder builder = new LocalizationConfigFileBuilder(generatorParams.LocalizationTargetName + ModularLocalizationConfigFileSuffixes.ImportSuffix + LocalizationFileExtensions.ModularLocalizationConfigFileExtension);
			builder.AddCommonSettingsSection(commonSettings);

			ImportPortableObjectFilesParams importParams = ImportPortableObjectFilesParams.CreateDefault(generatorParams.LocalizationTargetName, generatorParams.LocalizationTargetRootDirectory);
			builder.AddImportPortableObjectFilesSection(importParams);
			return builder.Get();
		}

		private LocalizationConfigFile GenerateLocalizationCompileConfigFile(LocalizationConfigFileGeneratorParams generatorParams)
		{
			CommonSettingsParams commonSettings = CommonSettingsParams.CreateDefault(generatorParams.LocalizationTargetName, generatorParams.LocalizationTargetRootDirectory);
			LocalizationConfigFileBuilder builder = new LocalizationConfigFileBuilder(generatorParams.LocalizationTargetName + ModularLocalizationConfigFileSuffixes.CompileSuffix + LocalizationFileExtensions.ModularLocalizationConfigFileExtension);
			builder.AddCommonSettingsSection(commonSettings);

			GenerateTextLocalizationResourceParams generateTextLocalizationResourceParams = GenerateTextLocalizationResourceParams.CreateDefault(generatorParams.LocalizationTargetName, generatorParams.LocalizationTargetRootDirectory);
			builder.AddGenerateTextLocalizationResourceSection(generateTextLocalizationResourceParams);
			return builder.Get();
		}

		private LocalizationConfigFile GenerateLocalizationGenerateReportsConfigFile(LocalizationConfigFileGeneratorParams generatorParams)
		{
			CommonSettingsParams commonSettings = CommonSettingsParams.CreateDefault(generatorParams.LocalizationTargetName, generatorParams.LocalizationTargetRootDirectory);
			LocalizationConfigFileBuilder builder = new LocalizationConfigFileBuilder(generatorParams.LocalizationTargetName + ModularLocalizationConfigFileSuffixes.GenerateReportsSuffix + LocalizationFileExtensions.ModularLocalizationConfigFileExtension);
			builder.AddCommonSettingsSection(commonSettings);

			GenerateTextLocalizationReportParams reportParams = GenerateTextLocalizationReportParams.CreateDefault(generatorParams.LocalizationTargetName, generatorParams.LocalizationTargetRootDirectory);
			// We don't need the conflict report 
			reportParams.bConflictReport = false;
			reportParams.ConflictReportName = "";
			builder.AddGenerateTextLocalizationReportSection(reportParams);
			return builder.Get();
		}
	}
}

