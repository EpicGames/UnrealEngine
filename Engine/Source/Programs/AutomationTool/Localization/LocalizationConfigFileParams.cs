// Copyright Epic Games, Inc. All Rights Reserved.
using System.Collections.Generic;
using System.IO;
using UnrealBuildTool;

namespace EpicGames.Localization
{

	public class CommonSettingsParams
	{
		public List<string> manifestDependencies { get; set; } = new List<string>();
		public string SourcePath { get; set; } = "";
		public string DestinationPath { get; set; } = "";
		public string ManifestName { get; set; } = "";
		public string ArchiveName { get; set; } = "";
		public string ResourceName { get; set; } = "";
		public string PortableObjectName { get; set; } = "";
		public bool bSkipSourceCheck { get; set; } = false;
		public bool bValidateFormatPatterns { get; set; } = true;
		public bool bValidateSafeWhitespace { get; set; } = false;
		public bool bValidateRichTextTags { get; set; } = false;
		public string NativeCulture { get; set; } = "";
		public List<string> CulturesToGenerate { get; set; } = new List<string>();

		public static CommonSettingsParams CreateDefault(string localizationTargetName, string localizationTargetRootDirectory)
		{
			CommonSettingsParams commonSettings = new CommonSettingsParams();
			commonSettings.manifestDependencies.Add(Path.Combine("..", "Engine", "Content", "Localization", "Engine", $"Engine{LocalizationFileExtensions.ManifestFileExtension}"));
			commonSettings.manifestDependencies.Add(Path.Combine("..", "Engine", "Content", "Localization", "Editor", $"Editor{LocalizationFileExtensions.ManifestFileExtension}"));

			commonSettings.SourcePath = Path.Combine(localizationTargetRootDirectory, "Content", "Localization", localizationTargetName);
			commonSettings.DestinationPath = Path.Combine(localizationTargetRootDirectory, "Content", "Localization", localizationTargetName);
			commonSettings.ManifestName = localizationTargetName + LocalizationFileExtensions.ManifestFileExtension;
			commonSettings.ArchiveName = localizationTargetName + LocalizationFileExtensions.ArchiveFileExtension;
			commonSettings.PortableObjectName = localizationTargetName + LocalizationFileExtensions.PortableObjectFileExtension;
			commonSettings.ResourceName = localizationTargetName + LocalizationFileExtensions.LocalizationResourceFileExtension;

			commonSettings.NativeCulture = "en";
			commonSettings.CulturesToGenerate = new List<string>() { "en", "fr", "de", "pl", "es-419", "es", "pt-BR", "it", "ru", "ko", "tr", "ar", "ja", "zh-Hans"};
			commonSettings.CulturesToGenerate.Sort();
			commonSettings.bSkipSourceCheck = false;
			commonSettings.bValidateFormatPatterns = true;
			commonSettings.bValidateSafeWhitespace = true;
			commonSettings.bValidateRichTextTags = true;
			return commonSettings;
		}
	}

	public class GatherTextFromSourceParams
	{
		public string CommandletClass { get; } = "GatherTextFromSource";
		public List<string> SearchDirectoryPaths { get; set; } = new List<string>();
		public List<string> ExcludePathFilters { get; set; } = new List<string>();
		public List<string> FileNameFilters { get; set; } = new List<string>();
		public bool bShouldGatherFromEditorOnlyData { get; set; } = false;

		public static GatherTextFromSourceParams CreateDefault(string localizationTargetName, string localizationTargetRootDirectory)
		{
			GatherTextFromSourceParams gatherTextFromSource = new GatherTextFromSourceParams();
			gatherTextFromSource.SearchDirectoryPaths.Add(Path.Combine(localizationTargetRootDirectory, "Source"));
			gatherTextFromSource.SearchDirectoryPaths.Add(Path.Combine(localizationTargetRootDirectory, "Config"));

			gatherTextFromSource.ExcludePathFilters.Add(Path.Combine(localizationTargetRootDirectory, "Config", "Localization", "*"));
			gatherTextFromSource.FileNameFilters = new List<string>() { "*.h", "*.cpp", "*.inl", "*.ini" };
			gatherTextFromSource.bShouldGatherFromEditorOnlyData = false;
			return gatherTextFromSource;
		}
	}

	public class GatherTextFromAssetsParams
	{
		public string CommandletClass { get; } = "GatherTextFromAssets";
		public List<string> IncludePathFilters { get; set; } = new List<string>();
		public List<string> ExcludePathFilters { get; set; } = new List<string>();
		public List<string> PackageFileNameFilters { get; set; } = new List<string>();
		public List<string> CollectionFilters { get; set; } = new List<string>();
		public List<string> WorldCollectionFilters { get; set; } = new List<string>();
		public bool bShouldExcludeDerivedClasses { get; set; } = false;
		public bool bShouldGatherFromEditorOnlyData { get; set; } = false;
		public bool bSkipGatherCache { get; set; } = false;

		public static GatherTextFromAssetsParams CreateDefault(string localizationTargetName, string localizationTargetRootDirectory)
		{
			GatherTextFromAssetsParams gatherTextFromAssets = new GatherTextFromAssetsParams();
			gatherTextFromAssets.IncludePathFilters.Add(Path.Combine(localizationTargetRootDirectory, "Content", "*"));

			gatherTextFromAssets.ExcludePathFilters.Add(Path.Combine(localizationTargetRootDirectory, "Content", "Localization", "*"));
			gatherTextFromAssets.ExcludePathFilters.Add(Path.Combine(localizationTargetRootDirectory, "Content", "L10N", "*"));
			gatherTextFromAssets.ExcludePathFilters.Add(Path.Combine(localizationTargetRootDirectory, "Content", "Developers", "*"));
			gatherTextFromAssets.ExcludePathFilters.Add(Path.Combine(localizationTargetRootDirectory, "Content", "*Test", "*"));
			
			gatherTextFromAssets.PackageFileNameFilters= new List<string>() { "*.umap", "*.uasset" };
			// @TODOLocalization: Decide whether or not to relax the errors when gathering assets as right now allowing this as a collection will cause
			// Engine plugins that rely on default auto plugin localization to fail 
			//gatherTextFromAssets.WorldCollectionFilters = new List<string>() { "Audit_InCook" };

			gatherTextFromAssets.bShouldExcludeDerivedClasses = false;
			gatherTextFromAssets.bShouldGatherFromEditorOnlyData = false;
			gatherTextFromAssets.bSkipGatherCache = false;
			return gatherTextFromAssets;
		}
	}

	public class GatherTextFromMetaDataParams
	{
		public string CommandletClass { get; } = "GatherTextFromMetaData";
		public List<string> ModulesToPreload { get; set; } = new List<string>();
		public List<string> IncludePathFilters { get; set; } = new List<string>();
		public List<string> ExcludePathFilters { get; set; } = new List<string>();
		public List<string> FieldTypesToInclude { get; set; } = new List<string>();
		public List<string> FieldOwnerTypesToInclude { get; set; } = new List<string>();
		public List<string> InputKeys { get; set; } = new List<string>();
		public List<string> OutputKeys { get; set; } = new List<string>();
		public List<string> InputNamespaceses { get; set; } = new List<string>();
		public List<string> OutputNamespaces { get; set; } = new List<string>();
		public bool bShouldGatherFromEditorOnlyData { get; set; } = false;

		public static GatherTextFromMetaDataParams CreateDefault(string localizationTargetRootDirectory, LocalizationTargetDescriptorLoadingPolicy localizationPolicy)
		{
			GatherTextFromMetaDataParams gatherTextFromMetaData = new GatherTextFromMetaDataParams();

			gatherTextFromMetaData.IncludePathFilters.Add(Path.Combine(localizationTargetRootDirectory, "Source",  "*"));

			if (localizationPolicy == LocalizationTargetDescriptorLoadingPolicy.PropertyNames)
			{
				gatherTextFromMetaData.InputKeys.Add("DisplayName");
				gatherTextFromMetaData.OutputKeys.Add("\"{FieldPath}\"");
				gatherTextFromMetaData.OutputNamespaces.Add("UObjectDisplayNames");
			}

			if (localizationPolicy == LocalizationTargetDescriptorLoadingPolicy.ToolTips)
			{
				gatherTextFromMetaData.InputKeys.Add("ToolTip");
				gatherTextFromMetaData.InputKeys.Add("ShortTooltip");
				gatherTextFromMetaData.OutputKeys.Add("\"{FieldPath}\"");
				gatherTextFromMetaData.OutputKeys.Add("\"{FieldPath}\"");
				gatherTextFromMetaData.OutputNamespaces.Add("UObjectToolTips");
				gatherTextFromMetaData.OutputNamespaces.Add("UObjectShortToolTips");
			}

			gatherTextFromMetaData.bShouldGatherFromEditorOnlyData = true;
			return gatherTextFromMetaData;
		}
	}

	public class GenerateGatherManifestParams
	{
		public string CommandletClass { get; } = "GenerateGatherManifest";
		public static GenerateGatherManifestParams CreateDefault(string localizationTargetName, string localizationTargetRootDirectory)
		{
			GenerateGatherManifestParams generateGatherManifestParams = new GenerateGatherManifestParams();

			return generateGatherManifestParams;
		}
	}

	public class GenerateGatherArchiveParams
	{
		public string CommandletClass { get; } = "GenerateGatherArchive";
		public bool bPurgeOldEmptyEntries { get; set; } = false;

		public static GenerateGatherArchiveParams CreateDefault(string localizationTargetName, string localizationTargetRootDirectory)
		{
			GenerateGatherArchiveParams generateGatherArchiveParams = new GenerateGatherArchiveParams();
			generateGatherArchiveParams.bPurgeOldEmptyEntries = false;

			return generateGatherArchiveParams;
		}
	}

	public class ImportPortableObjectFilesParams
	{
		public string CommandletClass { get; } = "InternationalizationExport";
		public bool bImportLoc { get; }  = true;
		public string LocalizedTextCollapseMode { get; set; } = "";
		public string POFormat { get; set; } = "";

		public static ImportPortableObjectFilesParams CreateDefault(string localizationTargetName, string localizationTargetRootDirectory)
		{
			ImportPortableObjectFilesParams importPOFilesParams = new ImportPortableObjectFilesParams();
			// @TODOLocalization: These should be enums not hard coded strings. 
			importPOFilesParams.LocalizedTextCollapseMode = "ELocalizedTextCollapseMode::IdenticalTextIdAndSource";
			importPOFilesParams.POFormat = "EPortableObjectFormat::Unreal";
			return importPOFilesParams;
		}
	}

	public class GenerateTextLocalizationResourceParams
	{
		public string CommandletClass { get; } = "GenerateTextLocalizationResource";
		public static GenerateTextLocalizationResourceParams CreateDefault(string localizationTargetName, string localizationTargetRootDirectory)
		{
			return new GenerateTextLocalizationResourceParams();
		}
	}

	public class ExportPortableObjectFilesParams
	{
		public string CommandletClass { get; } = "InternationalizationExport";
		public bool bExportLoc { get; } = true;
		public string LocalizedTextCollapseMode { get; set; } = "";
		public string POFormat { get; set; } = "";
		public bool bShouldPersistCommentsOnExport { get; set; } = true;
		public bool bShouldAddSourceLocationsAsComments { get; set; } = true;

		public static ExportPortableObjectFilesParams CreateDefault(string localizationTargetName, string localizationTargetRootDirectory)
		{
			ExportPortableObjectFilesParams exportPOFilesParams = new ExportPortableObjectFilesParams();
			exportPOFilesParams.LocalizedTextCollapseMode = "ELocalizedTextCollapseMode::IdenticalTextIdAndSource";
			exportPOFilesParams.POFormat = "EPortableObjectFormat::Unreal";
			exportPOFilesParams.bShouldPersistCommentsOnExport = true;
			exportPOFilesParams.bShouldAddSourceLocationsAsComments = true;
			return exportPOFilesParams;
		}
	}

	public class GenerateTextLocalizationReportParams
	{
		public string CommandletClass { get; } = "GenerateTextLocalizationReport";
		public string DestinationPath { get; set; } = "";
		public bool bWordCountReport { get; set; } = false;
		public string WordCountReportName { get; set; } = "";
		public bool bConflictReport { get; set; } = false;
		public string ConflictReportName { get; set; } = "";

		public static GenerateTextLocalizationReportParams CreateDefault(string localizationTargetName, string localizationTargetRootDirectory)
		{
			GenerateTextLocalizationReportParams generateTextLocalizationReportParams = new GenerateTextLocalizationReportParams();
			generateTextLocalizationReportParams.bWordCountReport = true;
			generateTextLocalizationReportParams.WordCountReportName = $"{localizationTargetName}.csv";
			generateTextLocalizationReportParams.bConflictReport = true;
			generateTextLocalizationReportParams.ConflictReportName = $"{localizationTargetName}_Conflicts.csv";
			return generateTextLocalizationReportParams;
		}
	}
}
