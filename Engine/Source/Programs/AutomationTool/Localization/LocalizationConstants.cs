// Copyright Epic Games, Inc. All Rights Reserved.

namespace EpicGames.Localization
{
	public static class LocalizationFileExtensions
	{
		public static string LegacyMonolithicLocalizationConfigFileExtension{ get; } = ".ini";
		public static string ModularLocalizationConfigFileExtension { get; } = ".ini";
		public static string ArchiveFileExtension { get; } = ".archive";
		public static string ManifestFileExtension { get; } = ".manifest";
		public static string PortableObjectFileExtension { get; } = ".po";
		public static string LocalizationResourceFileExtension { get; } = ".locres";
	}

	public static class ModularLocalizationConfigFileSuffixes
	{
		public static string GatherSuffix { get; } = "_Gather";
		public static string ExportSuffix { get; } = "_Export";
		public static string ImportSuffix { get; } = "_Import";
		public static string CompileSuffix { get; } = "_Compile";
		public static string GenerateReportsSuffix { get; } = "_GenerateReports";
	}

}