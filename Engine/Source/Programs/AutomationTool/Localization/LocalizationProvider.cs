// Copyright Epic Games, Inc. All Rights Reserved.
using System;
using System.Collections.Generic;
using System.IO;
using AutomationTool;
using UnrealBuildTool;
using EpicGames.Core;
using System.Threading.Tasks;
using System.Reflection;
using Microsoft.Extensions.Logging;

using static AutomationTool.CommandUtils;

namespace EpicGames.Localization
{
	public class ProjectImportExportInfo
	{
		public ProjectImportExportInfo(string InDestinationPath, string InManifestName, string InArchiveName, string InPortableObjectName, string InNativeCulture, IReadOnlyList<string> InCulturesToGenerate, bool InUseCultureDirectory)
		{
			DestinationPath = InDestinationPath;
			ManifestName = InManifestName;
			ArchiveName = InArchiveName;
			PortableObjectName = InPortableObjectName;
			NativeCulture = InNativeCulture;
			CulturesToGenerate = InCulturesToGenerate;
			bUseCultureDirectory = InUseCultureDirectory;
		}

		/** The destination path for the files containing the gathered data for this project (extracted from its config file - relative to the root working directory for the commandlet) */
		public string DestinationPath { get; private set; }

		/** The name to use for this projects manifest file (extracted from its config file) */
		public string ManifestName { get; private set; }

		/** The name to use for this projects archive file (extracted from its config file) */
		public string ArchiveName { get; private set; }

		/** The name to use for this projects portable object file (extracted from its config file) */
		public string PortableObjectName { get; private set; }

		/** The native culture for this project (extracted from its config file) */
		public string NativeCulture { get; private set; }

		/** The cultures to generate for this project (extracted from its config file) */
		public IReadOnlyList<string> CulturesToGenerate { get; private set; }

		/** True if we should use a per-culture directly when importing/exporting */
		public bool bUseCultureDirectory { get; private set; }

		/** The platform names that we have split into sub-folders */
		public IReadOnlyList<string> SplitPlatformNames { get; private set; }

		/** The platforms sub-folder */
		public const string PlatformLocalizationFolderName = "Platforms";

		public void CalculateSplitPlatformNames(string RootWorkingDirectory)
		{
			// Is this isn't a single culture import/export, then also check for split platform sub-folders
			List<string> NewSplitPlatformNames = new List<string>();
			if (bUseCultureDirectory)
			{
				DirectoryReference PlatformSourceDirectory = new DirectoryReference(CommandUtils.CombinePaths(RootWorkingDirectory, DestinationPath, PlatformLocalizationFolderName));
				if (DirectoryReference.Exists(PlatformSourceDirectory))
				{
					foreach (DirectoryReference FoundDirectory in DirectoryReference.EnumerateDirectories(PlatformSourceDirectory, "*", SearchOption.TopDirectoryOnly))
					{
						string SplitPlatformName = CommandUtils.GetLastDirectoryName(FoundDirectory.FullName);
						NewSplitPlatformNames.Add(SplitPlatformName);
					}
				}
			}
			SplitPlatformNames = NewSplitPlatformNames;
		}
	};

	public class ProjectStepInfo
	{
		public ProjectStepInfo(string InName, string InLocalizationConfigFile)
		{
			Name = InName;
			LocalizationConfigFile = InLocalizationConfigFile;
		}

		/** The name of this localization step */
		public string Name { get; private set; }

		/** Absolute path to this steps localization config file */
		public string LocalizationConfigFile { get; private set; }
	};

	public class ProjectInfo
	{
		public ProjectInfo(string InProjectName, IReadOnlyList<ProjectStepInfo> InLocalizationSteps, ProjectImportExportInfo InImportInfo, ProjectImportExportInfo InExportInfo)
		{
			ProjectName = InProjectName;
			LocalizationSteps = InLocalizationSteps;
			ImportInfo = InImportInfo;
			ExportInfo = InExportInfo;
		}

		public List<string> GetConfigFilesToRun(List<string> RequestedLocalizationStepNames)
		{
			List<string> ConfigFiles = new();
			foreach (ProjectStepInfo LocalizationStep in LocalizationSteps)
			{
				if (RequestedLocalizationStepNames.Contains(LocalizationStep.Name))
				{
					ConfigFiles.Add(LocalizationStep.LocalizationConfigFile);
				}
			}
			return ConfigFiles;
		}

		/** The name of this project */
		public string ProjectName { get; private set; }

		/** Path to this projects localization step data - ordered so that iterating them runs in the correct order */
		public IReadOnlyList<ProjectStepInfo> LocalizationSteps { get; private set; }

		/** Config data used by the PO file import process */
		public ProjectImportExportInfo ImportInfo { get; private set; }

		/** Config data used by the PO file export process */
		public ProjectImportExportInfo ExportInfo { get; private set; }
	};

	public abstract class LocalizationProvider
	{
		public struct LocalizationProviderArgs
		{
			public string RootWorkingDirectory;
			public string RootLocalizationTargetDirectory;
			public string RemoteFilenamePrefix;
			public BuildCommand Command;
			public int PendingChangeList;
		};

		public LocalizationProvider(LocalizationProviderArgs InArgs)
		{
			RootWorkingDirectory = InArgs.RootWorkingDirectory;
			RemoteFilenamePrefix = InArgs.RemoteFilenamePrefix;
			Command = InArgs.Command;
			PendingChangeList = InArgs.PendingChangeList;

			LocalizationBranchName = Command.ParseParamValue("LocalizationBranch");
			bUploadAllCultures = Command.ParseParam("UploadAllCultures");
		}

		public virtual string GetLocalizationProviderId()
		{
			throw new AutomationException("Unimplemented GetLocalizationProviderId.");
		}

		public async virtual Task InitializeProjectWithLocalizationProvider(string ProjectName, ProjectImportExportInfo ProjectImportInfo)
		{
			await Task.CompletedTask;
		}

		public async virtual Task DownloadProjectFromLocalizationProvider(string ProjectName, ProjectImportExportInfo ProjectImportInfo)
		{
			await Task.FromException(new NotImplementedException());
		}

		public async virtual Task UploadProjectToLocalizationProvider(string ProjectName, ProjectImportExportInfo ProjectExportInfo)
		{
			await Task.FromException(new NotImplementedException());
		}

		public static LocalizationProvider GetLocalizationProvider(string InLocalizationProviderId, LocalizationProvider.LocalizationProviderArgs InLocalizationProviderArgs)
		{
			if (String.IsNullOrEmpty(InLocalizationProviderId))
			{
				return null;
			}

			if (CachedLocalizationProviderTypes == null)
			{
				// Find all types that derive from LocalizationProvider in any of our DLLs
				CachedLocalizationProviderTypes = new Dictionary<string, Type>();
				foreach (Assembly Dll in ScriptManager.AllScriptAssemblies)
				{
					Type[] AllTypes = Dll.GetTypes();
					foreach (Type PotentialLocalizationNodeType in AllTypes)
					{
						if (PotentialLocalizationNodeType != typeof(LocalizationProvider) && !PotentialLocalizationNodeType.IsAbstract && !PotentialLocalizationNodeType.IsInterface && typeof(LocalizationProvider).IsAssignableFrom(PotentialLocalizationNodeType))
						{
							// Types should implement a static StaticGetLocalizationProviderId method
							MethodInfo Method = PotentialLocalizationNodeType.GetMethod("StaticGetLocalizationProviderId");
							if (Method != null)
							{
								string LocalizationProviderId = null;

								try
								{
									LocalizationProviderId = Method.Invoke(null, null) as string;
								}
								catch(Exception ex)
								{
									Logger.LogWarning("Type '{Name}' threw when calling its StaticGetLocalizationProviderId method. Exception: {Message}. (Assembly={Assembly}, AssemblyQualifiedName={AssemblyQualifiedName})", PotentialLocalizationNodeType.FullName, ex.Message, PotentialLocalizationNodeType.Assembly.FullName, PotentialLocalizationNodeType.AssemblyQualifiedName);
								}

								if (!String.IsNullOrEmpty(LocalizationProviderId))
								{
									if(CachedLocalizationProviderTypes.TryAdd(LocalizationProviderId, PotentialLocalizationNodeType))
									{
										Logger.LogInformation("Registered '{Name}' in the localization provider cache using id '{Id}'. (Assembly={Assembly}, AssemblyQualifiedName={AssemblyQualifiedName})", PotentialLocalizationNodeType.FullName, LocalizationProviderId, PotentialLocalizationNodeType.Assembly.FullName, PotentialLocalizationNodeType.AssemblyQualifiedName);
									}
									else
									{
										Logger.LogWarning("Could not add '{Name}' in the localization provider cache using id '{Id}'. (Assembly={Assembly}, AssemblyQualifiedName={AssemblyQualifiedName})", PotentialLocalizationNodeType.FullName, LocalizationProviderId, PotentialLocalizationNodeType.Assembly.FullName, PotentialLocalizationNodeType.AssemblyQualifiedName);
									}
								}
								else
								{
									Logger.LogWarning("{Name}.StaticGetLocalizationProviderId method returned an empty ID. (Assembly={Assembly}, AssemblyQualifiedName={AssemblyQualifiedName})", PotentialLocalizationNodeType.FullName, PotentialLocalizationNodeType.Assembly.FullName, PotentialLocalizationNodeType.AssemblyQualifiedName);
								}
							}
							else
							{
								Logger.LogWarning("Type '{Name}' derives from LocalizationProvider but is missing its StaticGetLocalizationProviderId method. (Assembly={Assembly}, AssemblyQualifiedName={AssemblyQualifiedName})", PotentialLocalizationNodeType.FullName, PotentialLocalizationNodeType.Assembly.FullName, PotentialLocalizationNodeType.AssemblyQualifiedName);
							}
						}
					}
				}
			}

			Type LocalizationNodeType;
			CachedLocalizationProviderTypes.TryGetValue(InLocalizationProviderId, out LocalizationNodeType);
			if (LocalizationNodeType != null)
			{
				try
				{
					return Activator.CreateInstance(LocalizationNodeType, new object[] { InLocalizationProviderArgs }) as LocalizationProvider;
				}
				catch (Exception e)
				{
					Logger.LogWarning(e, "Unable to create an instance of the type '{Type}'. {Message}", LocalizationNodeType.FullName, e.ToString());
				}
			}
			else
			{
				Logger.LogWarning("Could not find a localization provider for '{Id}'", InLocalizationProviderId);
			}

			return null;
		}

		protected string RootWorkingDirectory;
		protected string LocalizationBranchName;
		protected string RemoteFilenamePrefix;
		protected bool bUploadAllCultures;
		protected BuildCommand Command;
		protected int PendingChangeList;

		private static Dictionary<string, Type> CachedLocalizationProviderTypes;
	};
}
