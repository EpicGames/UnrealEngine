// Copyright Epic Games, Inc. All Rights Reserved.
using AutomationTool;
using Microsoft.Extensions.Logging;
using System;
using System.Collections.Generic;
using System.Linq;
using System.Reflection;
using System.Text;

using static AutomationTool.CommandUtils;

namespace EpicGames.Localization
{
	public abstract class LocalizationTargetCommand
	{
		protected static BuildCommand _commandLineHelper;
		private static Dictionary<string, LocalizationTargetCommand> _nameToLocalizationTargetCommandMap = new Dictionary<string, LocalizationTargetCommand>();

		public string Name { get; protected set; } = "";
		public string DisplayName { get; protected set; } = "";
		public bool bIsExecutingInPreview { get; private set; } = false;

		static LocalizationTargetCommand()
		{
			// look for all subclasses, and cache by their ProviderToken
			foreach (Assembly assembly in ScriptManager.AllScriptAssemblies)
			{
				Type[] allTypesInAssembly = assembly.GetTypes();
				foreach (Type typeInAssembly in allTypesInAssembly)
				{
					// we also guard against abstract classes as we can have abstract classes as children of LocalizationTargetEditorCommand e.g PluginLocalizationTargetCommand
					if (typeof(LocalizationTargetCommand).IsAssignableFrom(typeInAssembly) && typeInAssembly != typeof(LocalizationTargetCommand) && !typeInAssembly.IsAbstract)
					{
						LocalizationTargetCommand provider = (LocalizationTargetCommand)Activator.CreateInstance(typeInAssembly);
						_nameToLocalizationTargetCommandMap[provider.Name] = provider;
					}
				}
			}
		}

		public static void Initialize(BuildCommand commandLineHelper)
		{
			_commandLineHelper = commandLineHelper;
		}

		public static LocalizationTargetCommand GetLocalizationTargetCommandFromName(string commandName)
		{
			LocalizationTargetCommand command = null;
			_nameToLocalizationTargetCommandMap.TryGetValue(commandName, out command);
			return command;
		}

		protected virtual bool ParseCommandLine()
		{
			bIsExecutingInPreview = _commandLineHelper.ParseParam("Preview");
			return true;
		}

		public abstract bool Execute();

		public void PrintHelpText()
		{
			Logger.LogInformation("Help:");
			Logger.LogInformation($"{GetHelpText()}");
		}

		public virtual string GetHelpText()
		{
			StringBuilder helpTextStringBuilder = new StringBuilder();
			BuildHelpDescription(helpTextStringBuilder);
			BuildHelpArguments(helpTextStringBuilder);
			BuildHelpUsage(helpTextStringBuilder);
			return helpTextStringBuilder.ToString();
		}

		/// <summary>
		/// Buils the description portion of the help text. Concrete child classes of LocalizationTargetCommand should override this function 
		/// </summary> and call the base implementation to build any descriptions from parent classes.
		/// <param name="builder"></param>
		protected virtual void BuildHelpDescription(StringBuilder builder)
		{
			builder.AppendLine("Description:");
		}

		/// <summary>
		/// Builds the arguments portion of the help text explaining what each parameter of the command does. Child classes should override this function
		/// </summary> to call the base implementation as well as provide descriptions for the child arguments.
		/// <param name="builder"></param>
		protected virtual void BuildHelpArguments(StringBuilder builder)
		{
			builder.AppendLine("Arguments:");
			builder.AppendLine("Preview - An optional flag that will execute this command in preview mode. No files will be created or edited. No folders will be created. No files will be added or checked out of perforce.");
		}

		/// <summary>
		/// Builds the usage portion of the help text. Useres should override this function and call teh base implementation 
		/// and provide usage scenarios and the associated usage syntax for each scenario.
		/// </summary>
		/// <param name="builder"></param>
		protected virtual void BuildHelpUsage(StringBuilder builder)
		{
			builder.AppendLine("Usage:");
		}

		public static List<LocalizationTargetCommand> GetAllCommands()
		{
			return _nameToLocalizationTargetCommandMap.Values.ToList();
		}
	}
}
