// Copyright Epic Games, Inc. All Rights Reserved.
using System;
using System.Collections.Generic;
using System.Linq;
using System.Text;
using System.IO;
using UnrealBuildTool;
using EpicGames.Core;
using Microsoft.Extensions.Logging;

using static AutomationTool.CommandUtils;

namespace EpicGames.Localization
{
	public class NewPluginLocalizationTargetCommand : PluginLocalizationTargetCommand
	{
		private LocalizationTargetDescriptorLoadingPolicy _loadingPolicy;
		private LocalizationConfigGenerationPolicy _configGenerationPolicy;
		private string _localizationTargetNameSuffix = "";
		private LocalizationConfigFileFormat _fileFormat = LocalizationConfigFileFormat.Latest;
		private List<string> _filesToAdd = new List<string>();
		private List<string> _filesToEdit = new List<string>();

		public NewPluginLocalizationTargetCommand()
		{
			Name = "NewPluginLocalizationTarget";
			DisplayName = "New Plugin Localization Target";
		}

		protected override void BuildHelpDescription(StringBuilder builder)
		{
			base.BuildHelpDescription(builder);
			builder.AppendLine("Creates new localization targets for plugins. This command generates the necessary localization config files for a plugin and updates the plugin descriptor with new localization target data.");
			builder.AppendLine("Passing the -p4 flag to UAT will create a change list with all of the added and edited files.");
			builder.AppendLine("If the localization config files already exist for the plugin or if the plugin descriptor already contains the localization target, this command will not edit any of the existing files.");
		}

		protected override void BuildHelpArguments(StringBuilder builder)
		{
			base.BuildHelpArguments(builder);
			builder.AppendLine("LocalizationTargetLoadingPolicy - A required string that determines when the plugin should load the localization data for the localization target. Valid values are Game, Always, Editor, PropertyNames, ToolTips. See LocalizationTargetDescriptorLoadingPolicy");
			builder.AppendLine("LocalizationConfigGenerationPolicy- An optional string that specifies how localization config files should be generated during the localization gather process for the plugin. Acceptable values are Never, Auto, User. If not specified, defaults to Auto.");
			builder.AppendLine("Never means no localization config files will be generated or used during the localization process. No localization data will be generated for the plugin druing a gather. Auto means temporary, default localization config files will be generated druing localization gather and used to generate localization data. User means there are user provided localization config files in the plugins's Config/Localization folder that will be used during localization gathers to generate localization data.");
			builder.AppendLine("LocalizationTargetNameSuffix - An optional suffix to give to the plugin localization target. By default, the name of the plugin localization target would be the name of the plugin. This allows multiple localization targets to be created for plugins.");
		}

		protected override void BuildHelpUsage(StringBuilder builder)
		{
			base.BuildHelpUsage(builder);
			builder.AppendLine("Creates a localization target for MyPlugin with a loading policy of Game and localization config generation policy of Auto and and checks out the necessary files in perforce.");
			builder.AppendLine("RunUAT.bat LocalizationTargetEditor -Command=NewPluginLocalizationTarget -UEProjectDirectory=MyGame -IncludePlugins=MyPlugin -LocalizationTargetLoadingPolicy=Game -p4");
			builder.AppendLine("Creates a new localization target for all plugins under MyDirectory with loading policy Game and config generaiton policy of User so that localization config files will be generated in each plugin's Config/Localization folder. The relevant files will be checked out of perforce.");
			builder.AppendLine("RunUAT.bat LocalizationTargetEditor -Command=NewPluginLocalizationTarget -UEProjectDirectory=MyGame -IncludePluginsDirectory=Plugins/MyDirectory -LocalizationTargetLoadingPolicy=Game -LocalizationConfigGenerationPolicy=User -p4");
		}

		protected override bool ParseCommandLine()
		{
			if (!base.ParseCommandLine())
			{
				return false;
			}
			_localizationTargetNameSuffix = _commandLineHelper.ParseParamValue("LocalizationTargetNameSuffix");
			// @TODOLocalization: Support an override for the localization config file format 
			_fileFormat = LocalizationConfigFileFormat.Latest;
			_loadingPolicy = _commandLineHelper.ParseRequiredEnumParamEnum<LocalizationTargetDescriptorLoadingPolicy>("LocalizationTargetLoadingPolicy");
			LocalizationConfigGenerationPolicy? nullableGenerationPolicy = _commandLineHelper.ParseOptionalEnumParam<LocalizationConfigGenerationPolicy>("LocalizationConfigGenerationPolicy");
			if (nullableGenerationPolicy.HasValue)
			{
				_configGenerationPolicy = nullableGenerationPolicy.Value;
			}
			else
			{
				_configGenerationPolicy = LocalizationConfigGenerationPolicy.Auto;
				Logger.LogInformation("LocalizationConfigGenerationPolicy is defaulting to Auto. All plugin localization targets created will have default, temporary localization config files generated during the localization pipeline. To change this behavior, please see the helpt text for the LocalizationConfigGenerationPolicy parameter.");
			}
			
				return true;
		}

		public override bool Execute()
		{
			if (!ParseCommandLine())
			{
				return false;
			}

			string projectPluginDirectory = Path.Combine(UERootDirectory, UEProjectDirectory, "Plugins");

			List<string> pluginsToGenerateFor = IncludePlugins.Except(ExcludePlugins).ToList();
			if (pluginsToGenerateFor.Count == 0)
			{
				Logger.LogInformation("Based on provided plugins to include and exclude, no plugins require a localization target to be generated.");
				return true;
			}
			DirectoryReference ueProjectDirectoryReference = new DirectoryReference(Path.Combine(UERootDirectory, UEProjectDirectory));
			// Load all available plugins so we can find them 
			PluginType pluginType = String.IsNullOrEmpty(UEProjectName) ? PluginType.Engine : PluginType.Project;
			_ = pluginType == PluginType.Engine ? Plugins.ReadEnginePlugins(ueProjectDirectoryReference) : Plugins.ReadProjectPlugins(ueProjectDirectoryReference);

			// loop through all plugins and generate the files 
			foreach (string pluginName in pluginsToGenerateFor)
			{
				PluginInfo pluginInfo = Plugins.GetPlugin(pluginName);
				if (pluginInfo is null)
				{
					Logger.LogWarning($"Cannot find {pluginName}. Skipping generation of localization config files for this plugin.");
					continue;
				}
				string pluginLocalizationTargetName = pluginInfo.Name;
				// we add on the suffix if a user provides it 
				if (!String.IsNullOrEmpty(_localizationTargetNameSuffix))
				{
					pluginLocalizationTargetName += "_" + _localizationTargetNameSuffix;
				}

				// We  only generate the config files if it's set to user. Auto will autogen the config files and never implies we don't need localization config files at all.
				if (_configGenerationPolicy == LocalizationConfigGenerationPolicy.User)
				{
					GeneratePluginLocalizationConfigFiles(pluginInfo, pluginLocalizationTargetName, pluginInfo.Directory.MakeRelativeTo(ueProjectDirectoryReference));
				}
				else
				{
					Logger.LogInformation($"Localization target generation policy is {_configGenerationPolicy.ToString()}. No localization config files will be generated for {pluginLocalizationTargetName}.	");
				}
				// Update the .uplugin file with updated localization target descriptors 
				UpdatePluginLocalizationTargetDescriptor(pluginInfo, pluginLocalizationTargetName);
			}

			// Create changelist with all edited/added files 
			if (!bIsExecutingInPreview)
			{
				CreatePerforceChangeList();
			}

			return true;
		}

		private void GeneratePluginLocalizationConfigFiles(PluginInfo pluginInfo, string localizationTargetName, string localizationTargetRootDirectory)
		{
			LocalizationConfigFileGenerator generator = LocalizationConfigFileGenerator.GetGeneratorForFileFormat(_fileFormat);
			// Create the necessary directories if they dont' exist for the plugin 
			string pluginConfigLocalizationDirectory = Path.Combine(pluginInfo.Directory.FullName, "Config", "Localization");
			if (!Directory.Exists(pluginConfigLocalizationDirectory) && !bIsExecutingInPreview)
			{
				Logger.LogInformation($"Plugin '{pluginInfo.Name}' does not have a localization configuration folder. Creating '{pluginConfigLocalizationDirectory}'");
				Directory.CreateDirectory(pluginConfigLocalizationDirectory);
			}

			// Generate the config files 
			LocalizationConfigFileGeneratorParams generatorParams = new LocalizationConfigFileGeneratorParams();
			generatorParams.LocalizationTargetName = localizationTargetName;
			generatorParams.LocalizationTargetRootDirectory = localizationTargetRootDirectory;
			// @TODOLocalization: Create an option to generate the files based on a template localization config file
			List<LocalizationConfigFile> generatedConfigFiles = generator.GenerateDefaultSettingsConfigFiles(generatorParams);
			foreach (LocalizationConfigFile generatedConfigFile in generatedConfigFiles)
			{
				string saveFilePath = Path.Combine(pluginConfigLocalizationDirectory, generatedConfigFile.Name);
				// We skip writing the file if it already exists. We only create new files 
				if (File.Exists(saveFilePath)) 
				{
					Logger.LogInformation($"Localization config file {saveFilePath} already exists. File contents will not be overwritten. If you would like to edit the contents of the file, use the UpdatePluginLocalizationTarget command instead.");
					continue;
				}
				_filesToAdd.Add(saveFilePath);
				if (!bIsExecutingInPreview)
				{
					Logger.LogInformation($"Writing '{saveFilePath}'");
					generatedConfigFile.Write(new FileReference(saveFilePath));
				}
			}
		}

		private void UpdatePluginLocalizationTargetDescriptor(PluginInfo pluginInfo, string localizationTargetName)
		{
			PluginDescriptor descriptor = pluginInfo.Descriptor;
			if (descriptor.LocalizationTargets is not null)
			{
				foreach (LocalizationTargetDescriptor locTargetDescriptor in descriptor.LocalizationTargets)
				{
					// the localization target with the same name already exists. Early out and don't update anything 
					if (locTargetDescriptor.Name == localizationTargetName)
					{
						Logger.LogWarning($"Plugin {pluginInfo.Name} already contains localization target descriptor with name {locTargetDescriptor.Name}. No updates to the plugin descriptor will be made.");
						return;
					}
				}
			}
			LocalizationTargetDescriptor locTargetDescriptorToAdd = new LocalizationTargetDescriptor(localizationTargetName, _loadingPolicy, _configGenerationPolicy);
			if (descriptor.LocalizationTargets is null)
			{
				descriptor.LocalizationTargets = new LocalizationTargetDescriptor[] { locTargetDescriptorToAdd };
			}
			else
			{
				int numLocalizationTargets = descriptor.LocalizationTargets.Length;
				LocalizationTargetDescriptor[] newLocalizationTargets = new LocalizationTargetDescriptor[numLocalizationTargets + 1];
				Array.Copy(descriptor.LocalizationTargets, newLocalizationTargets, numLocalizationTargets);
				newLocalizationTargets[numLocalizationTargets] = locTargetDescriptorToAdd;
				descriptor.LocalizationTargets = newLocalizationTargets;
			}

			Logger.LogInformation($"Updating localization target information for {pluginInfo.Name} plugin. Adding localization target {localizationTargetName} with loading policy {_loadingPolicy.ToString()} and ConfigGenerationPolicy {_configGenerationPolicy}.");

			// Add descriptor to processing list 
			_filesToEdit.Add(pluginInfo.File.FullName);
			if (!bIsExecutingInPreview)
			{
				Logger.LogInformation($"Updating {pluginInfo.Name} plugin descriptor file '{pluginInfo.File.FullName}'");
				// make the file writable and save the data
				pluginInfo.File.ToFileInfo().IsReadOnly = false;
				descriptor.Save2(pluginInfo.File.FullName);
			}
		}

		private void CreatePerforceChangeList()
		{
			if (!P4Enabled)
			{
				Logger.LogInformation("Perfordce is not enabled. No files will be added to a change list.");
				return;
			}
			StringBuilder clDescriptionBuilder= new StringBuilder();
			clDescriptionBuilder.AppendLine("Generated localization config files with the NewPluginLocalizationTarget command of the LocalizationTargetEditor UAT command.");
			clDescriptionBuilder.AppendLine("#rnx");
			clDescriptionBuilder.AppendLine("#jira: none");
			clDescriptionBuilder.AppendLine("#rb:");
			clDescriptionBuilder.AppendLine("#preflight:");

			int changeListNumber = P4.CreateChange(P4Env.Client, clDescriptionBuilder.ToString());

			if (P4Enabled && changeListNumber != -1)
			{
				if (_filesToAdd.Count > 0)
				{
					P4.Add(changeListNumber, _filesToAdd);
				}

				if (_filesToEdit.Count > 0)
				{
					P4.Edit(changeListNumber, _filesToEdit);
				}
			}
			else
			{
				// p4 is not enabled somehow. We list all the files that should be added or edited.
				Logger.LogWarning("Failed to create a perforce changelist for files to be added/edited.");
				Logger.LogInformation("Files to add:");
				foreach (string fileToAdd in _filesToAdd)
				{
					Logger.LogInformation(fileToAdd);
				}

				Logger.LogInformation("Files to edit:");
				foreach (string fileToEdit in _filesToEdit)
				{
					Logger.LogInformation(fileToEdit);
				}
			}
		}
	}
}
