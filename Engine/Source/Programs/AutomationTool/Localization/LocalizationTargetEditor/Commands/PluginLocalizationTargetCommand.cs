// Copyright Epic Games, Inc. All Rights Reserved.
using AutomationTool;
using Microsoft.Extensions.Logging;
using System;
using System.Collections.Generic;
using System.IO;
using System.Text;
using UnrealBuildTool;

using static AutomationTool.CommandUtils;

namespace EpicGames.Localization
{

	public abstract class PluginLocalizationTargetCommand : LocalizationTargetCommand
	{
		public string UERootDirectory { get; protected set; } = "";
		public string UEProjectDirectory { get; protected set; } = "";
		public string UEProjectName { get; protected set; } = ""; 
		
		public HashSet<string> IncludePlugins { get; protected set; } = new HashSet<string>();
		public HashSet<string> ExcludePlugins { get; protected set; } = new HashSet<string>();

		protected override void BuildHelpArguments(StringBuilder builder)
		{
			base.BuildHelpArguments(builder);
			builder.AppendLine("UERootDirectory - Optional root directory for the Unreal Engine. This is usually one level above your project directory. Defaults to CmdEnv.LocalRoot");
			builder.AppendLine("UEProjectDirectory - Required relative path from UERootDirectory to the project. This should be  your project directory that contains the Plugins directory.");
			builder.AppendLine("UEProjectName - An optional name of the project the plugin is under. If blank, this implies the plugin is under Engine.");

			// Include plugins 
			builder.AppendLine("IncludePlugins - An optional comma separated list of plugins to operate on. E.g PluginA,PluginB,PluginC");
			builder.AppendLine("IncludePluginsDirectory - An optional relative directory to UEProjectDirectory. All plugins under this directory will be operated on if they are not excluded. E.g Plugins/PluginFolderA.");
			builder.AppendLine("ExcludePlugins - A comma separated list of plugins to exclude from being operated on. E.g PluginA,BpluginB,PluginC");
			builder.AppendLine("ExcludePluginsDirectory - An optional relative directory from UEProjectDirectory. All plugins under this directory will be excluded from being operated on. E.g Plugin/DirectoryToExclude");
		}

		protected override bool ParseCommandLine()
		{
			if (!base.ParseCommandLine())
			{
				return false;
			}
			UERootDirectory = _commandLineHelper.ParseParamValue("UERootDirectory");
			if (String.IsNullOrEmpty(UERootDirectory))
			{
				UERootDirectory = CommandUtils.CmdEnv.LocalRoot;
			}
			UEProjectName = _commandLineHelper.ParseParamValue("UEProject");
			UEProjectDirectory = _commandLineHelper.ParseRequiredStringParam("UEProjectDirectory");
			
			// getting the plugins 
			string includePluginsString = _commandLineHelper.ParseParamValue("IncludePlugins");
			if (!String.IsNullOrEmpty(includePluginsString))
			{
				foreach (string plugin in includePluginsString.Split(","))
				{
					IncludePlugins.Add(plugin.Trim());
				}
			}

			string excludePluginsString = _commandLineHelper.ParseParamValue("ExcludePlugins");
			if (!String.IsNullOrEmpty(excludePluginsString))
			{
				foreach (string plugin in excludePluginsString.Split(","))
				{
					ExcludePlugins.Add(plugin.Trim());
				}
			}

			string projectPluginDirectory = Path.Combine(UERootDirectory, UEProjectDirectory, "Plugins");
			PluginType pluginType = String.IsNullOrEmpty(UEProjectName) ? PluginType.Engine : PluginType.Project;
			// The provided path should be relative to the project directory 
			string includePluginsDirectoryString = _commandLineHelper.ParseParamValue("IncludePluginsDirectory");
			if (!String.IsNullOrEmpty(includePluginsDirectoryString))
			{
				// @TODOLocalization: May need to manipulate passed in string to replace directory separators on various platforms 
				string IncludePluginsDirectoryAbsolutePath = Path.Combine(UERootDirectory, UEProjectDirectory, includePluginsDirectoryString);
				Logger.LogInformation($"Including plugins under '{IncludePluginsDirectoryAbsolutePath}' directory.");
				IncludePlugins.UnionWith(LocalizationUtilities.GetPluginNamesUnderDirectory(IncludePluginsDirectoryAbsolutePath, projectPluginDirectory, pluginType));
			}

			string excludePluginsDirectoryString = _commandLineHelper.ParseParamValue("ExcludePluginsDirectory");
			if (!String.IsNullOrEmpty(excludePluginsDirectoryString))
			{
				string excludePluginsDirectoryAbsolutePath = Path.Combine(UERootDirectory, UEProjectDirectory, excludePluginsDirectoryString);
				Logger.LogInformation($"Excluding plugins under '{excludePluginsDirectoryAbsolutePath}' directory.");
				ExcludePlugins.UnionWith(LocalizationUtilities.GetPluginNamesUnderDirectory(excludePluginsDirectoryAbsolutePath, projectPluginDirectory, pluginType));
			}
			
			return true;
		}
	}
}
