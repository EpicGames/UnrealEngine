// Copyright Epic Games, Inc. All Rights Reserved.
using System;
using System.Text;
using AutomationTool;
using Microsoft.Extensions.Logging;

namespace EpicGames.Localization
{
	[Help("Allows users to create, update, delete or query localization targets for the Engine or projects. This command has sub-commands and more info for each command can be retrieved with the -Info flag.")]
	[Help("Command", "The sub-command you want to run. For a full list of valid command, use LocalizationTargetEditor -Info")]
	[Help("Info", "An optional flag that can be passed to any sub-command. It will print out help text for the command and its associated parameters.")]
	class LocalizationTargetEditor : BuildCommand
	{
		public override void ExecuteBuild()
		{
			LocalizationTargetCommand.Initialize(this);
			string commandName = ParseParamValue("Command");
			if (String.IsNullOrEmpty(commandName))
			{
				if (ParseParam("Info"))
				{
					StringBuilder helpTextBuilder = new StringBuilder();
					helpTextBuilder.AppendLine("The Localization Target Editor is a means for you to create new localization targets. Use this command to create, edit, delete or query localization targets for projects, plugisn and platforms.");
					helpTextBuilder.AppendLine("Below are all available commands under the Localization Target Editor. Add the -Info flag after any of these commands for more information.");
					foreach (LocalizationTargetCommand helpCommand in LocalizationTargetCommand.GetAllCommands())
					{
						helpTextBuilder.AppendLine($"{helpCommand.Name}");
					}
					Logger.LogInformation($"{helpTextBuilder.ToString()}");
					return;
				}
			}
			LocalizationTargetCommand command = LocalizationTargetCommand.GetLocalizationTargetCommandFromName(commandName);

			if (command is null)
			{
				Logger.LogError($"{commandName} is not a supported localization target command.");
				return;
			}
			// If there's the -Info flag, we just display the help text and exit
			// We have to use -Info as -Help is handled globally via UAT 
			if (ParseParam("Info"))
			{
				command.PrintHelpText();
				return;
			}
			Logger.LogInformation($"Starting execution of '{command.Name}' localization target editor command.");
			if (!command.Execute())
			{
				Logger.LogError($"Localization Target Editor '{command.Name}' failed.");
			}
		}
	}
}
