// Copyright Epic Games, Inc. All Rights Reserved.
using System.Collections.Generic;
using System.IO;
using UnrealBuildTool;
using EpicGames.Core;
using System;

namespace EpicGames.Localization
{
	public static class LocalizationUtilities
	{
		public static HashSet<string> GetPluginNamesUnderDirectory(string PluginDirectoryPath, string PluginRoot, PluginType Type)
		{
			HashSet<string> ReturnList = new HashSet<string>();
			if (!Directory.Exists(PluginDirectoryPath))
			{
				return ReturnList;
			}
			if (!Directory.Exists(PluginRoot))
			{
				return ReturnList;
			}

			DirectoryReference PluginDirectoryReference = new DirectoryReference(PluginDirectoryPath);
			DirectoryReference PluginRootDirectoryReference = new DirectoryReference(PluginRoot);
			bool bIsSubdirectory = PluginDirectoryReference.IsUnderDirectory(PluginRootDirectoryReference);
			if (bIsSubdirectory)
			{
				string RelativePath = PluginDirectoryReference.MakeRelativeTo(PluginRootDirectoryReference);
				IReadOnlyList<PluginInfo> PluginsUnderDirectory = Plugins.ReadPluginsFromDirectory(PluginRootDirectoryReference, RelativePath, Type);
				foreach (PluginInfo Plugin in PluginsUnderDirectory)
				{
					ReturnList.Add(Plugin.Name);
				}
			}

			return ReturnList;
		}

		public static string[] GetModularConfigSuffixes()
		{
			string[] FileSuffixes = {
				"_Gather",
				"_Import",
				"_Export",
				"_Compile",
				"_GenerateReports",
			};

			return FileSuffixes;
		}

		public static bool IsModularLocalizationConfigFile(String file)
		{
			if (String.IsNullOrEmpty(file))
			{
				return false;
			}

			String fileName = Path.GetFileNameWithoutExtension(file);
			String[] suffixes = LocalizationUtilities.GetModularConfigSuffixes();
			foreach (String suffix in suffixes)
			{
				if (fileName.EndsWith(suffix))
				{
					return true;
				}
			}
			return false;
		}

		public static string GetModularLocalizationConfigSuffix(String file)
		{
			if (String.IsNullOrEmpty(file))
			{
				return "";
			}

			String fileName = Path.GetFileNameWithoutExtension(file);
			String[] suffixes = LocalizationUtilities.GetModularConfigSuffixes();
			foreach (String suffix in suffixes)
			{
				if (fileName.EndsWith(suffix))
				{
					return suffix;
				}
			}
			return "";
		}
	}
}
