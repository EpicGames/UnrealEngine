<Project Sdk="Microsoft.NET.Sdk">
  <Import Project="..\..\Shared\UnrealEngine.csproj.props" />
  
  <PropertyGroup>
    <OutputType>Library</OutputType>
    <RootNamespace>LowLevelTests.Automation</RootNamespace>
    <AssemblyName>LowLevelTests.Automation</AssemblyName>
    <GenerateAssemblyInfo>false</GenerateAssemblyInfo>
    <GenerateTargetFrameworkAttribute>false</GenerateTargetFrameworkAttribute>
    <OutputPath Condition="'$(ArtifactsPath)' == ''">..\..\..\..\Binaries\DotNET\AutomationTool\AutomationScripts\LowLevelTests</OutputPath>
    <TreatWarningsAsErrors>true</TreatWarningsAsErrors>
    <SatelliteResourceLanguages>en</SatelliteResourceLanguages>    <!-- remove non english resource languages -->
  </PropertyGroup>
  <ItemGroup>
    <ProjectReference Include="..\..\Shared\EpicGames.Core\EpicGames.Core.csproj" PrivateAssets="All">
      <Private>false</Private>
    </ProjectReference>
    <ProjectReference Include="..\..\UnrealBuildTool\UnrealBuildTool.csproj" PrivateAssets="All">
      <Private>false</Private>
    </ProjectReference>
    <ProjectReference Include="..\..\AutomationTool\AutomationUtils\AutomationUtils.Automation.csproj" PrivateAssets="All">
      <Private>false</Private>
    </ProjectReference>
    <ProjectReference Include="..\..\AutomationTool\Scripts\AutomationScripts.Automation.csproj" PrivateAssets="All">
      <Private>false</Private>
    </ProjectReference>
    <ProjectReference Include="..\..\AutomationTool\Gauntlet\Gauntlet.Automation.csproj" PrivateAssets="All">
      <Private>false</Private>
    </ProjectReference>
  </ItemGroup>
  <ItemGroup>
    <Compile Include="../../../../Platforms/*/Source/Programs/AutomationTool/LowLevelTests/**/*.cs">
      <Link>Platform/$([System.Text.RegularExpressions.Regex]::Replace(%(Compile.RecursiveDir), [\\/].+$, ``))/%(Compile.FileName).cs</Link>
    </Compile>
  </ItemGroup>
  <ItemGroup>
    <PackageReference Include="MySql.Data" Version="6.10.9" PrivateAssets="all" />
  </ItemGroup>
  <ItemGroup>
    <Folder Include="Utility\" />
  </ItemGroup>
</Project>
