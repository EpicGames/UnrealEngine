// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Linq;
using System.IO;
using System.Text;
using AutomationTool;
using UnrealBuildTool;
using UnrealBuildBase;
using Gauntlet;
using AutomationTool.DeviceReservation;
using Microsoft.Extensions.Logging;

namespace LowLevelTests
{
	using LogLevel = Gauntlet.LogLevel;

	public class RunLowLevelTests : BuildCommand
	{
		public override ExitCode Execute()
		{
			Log.Level = LogLevel.VeryVerbose;

			Globals.Params = new Params(Params);

			LowLevelTestExecutorOptions ContextOptions = new LowLevelTestExecutorOptions();
			AutoParam.ApplyParamsAndDefaults(ContextOptions, Globals.Params.AllArguments);

			if (ContextOptions.TestApp == string.Empty)
			{
				Log.Error("Error: -testapp flag is missing on the command line. Expected test project that extends LowLevelTests module.");
				return ExitCode.Error_Arguments;
			}

			if (string.IsNullOrEmpty(ContextOptions.Build))
			{
				Log.Error("No build path specified. Set -build= to test executable and resources directory.");
				return ExitCode.Error_Arguments;
			}

			if (!Path.IsPathRooted(ContextOptions.Build))
			{
				ContextOptions.Build = Path.Combine(Globals.UnrealRootDir, ContextOptions.Build);
			}

			var Result = RunTests(ContextOptions);

			Gauntlet.Horde.GenerateStepSummary();
			Gauntlet.Horde.GenerateJobSummary();

			return Result;
		}

		public ExitCode RunTests(LowLevelTestExecutorOptions ContextOptions)
		{
			UnrealTargetPlatform TestPlatform = ContextOptions.Platform;

			LowLevelTestRoleContext RoleContext = new LowLevelTestRoleContext();
			RoleContext.Platform = TestPlatform;

			LowLevelTestsBuildSource BuildSource = new LowLevelTestsBuildSource(
				ContextOptions.TestApp,
				ContextOptions.Build,
				ContextOptions.Platform,
				ContextOptions.Configuration,
				ContextOptions.SkipStage);

			SetupDevices(TestPlatform, ContextOptions);

			LowLevelTestContext TestContext = new LowLevelTestContext(BuildSource, RoleContext, ContextOptions);

			ITestNode NewTest = Gauntlet.Utils.TestConstructor.ConstructTest<ITestNode, LowLevelTestContext>(ContextOptions.TestApp, TestContext, new string[] { "LowLevelTests" });

			if (!(NewTest is LowLevelTests))
			{
				throw new AutomationException("Expected ITestNode type of LowLevelTests");
			}

			bool TestPassed = ExecuteTest(ContextOptions, NewTest);

			DevicePool.Instance.Dispose();

			DoCleanup(TestPlatform);

			return TestPassed ? ExitCode.Success : ExitCode.Error_TestFailure;
		}

		void DoCleanup(UnrealTargetPlatform Platform)
		{
			// TODO: Platform specific cleanup
		}

		private bool ExecuteTest(LowLevelTestExecutorOptions Options, ITestNode LowLevelTestNode)
		{
			var Executor = new TestExecutor(ToString());

			try
			{
				bool Result = Executor.ExecuteTests(Options, new List<ITestNode>() { LowLevelTestNode });
				return Result;
			}
			catch (Exception ex)
			{
				Log.Info("");
				Log.Error("{0}.\r\n\r\n{1}", ex.Message, ex.StackTrace);

				return false;
			}
			finally
			{
				Executor.Dispose();

				if (!string.IsNullOrEmpty(Options.Device))
				{
					(LowLevelTestNode as LowLevelTests)
						.LowLevelTestsApp?
						.UnrealDeviceReservation?
						.ReleaseDevices();
				}

				DevicePool.Instance.Dispose();

				if (ParseParam("clean"))
				{
					Logger.LogInformation("Deleting temp dir {Arg0}", Options.TempDir);
					DirectoryInfo TempDirInfo = new DirectoryInfo(Options.TempDir);
					if (TempDirInfo.Exists)
					{
						TempDirInfo.Delete(true);
					}
				}

				GC.Collect();
			}
		}

		protected void SetupDevices(UnrealTargetPlatform TestPlatform, LowLevelTestExecutorOptions Options)
		{
			Reservation.ReservationDetails = Options.JobDetails;

			DevicePool.Instance.SetLocalOptions(Options.TempDir, Options.Parallel > 1, Options.DeviceURL);
			DevicePool.Instance.AddLocalDevices(1);
			DevicePool.Instance.AddVirtualDevices(2);

			if (!string.IsNullOrEmpty(Options.Device))
			{
				DevicePool.Instance.AddDevices(TestPlatform, Options.Device);
			}
		}
	}

	public class LowLevelTestExecutorOptions : TestExecutorOptions, IAutoParamNotifiable
	{
		public Params Params { get; protected set; }

		public string TempDir;

		[AutoParam("")]
		public string DeviceURL;

		[AutoParam("")]
		public string JobDetails;

		public string TestApp;

		public string Tags;

		public List<string> TestNames { get; set; } = new List<string>();

		[AutoParam(0)]
		public int Sleep;

		public bool AttachToDebugger;

		public bool VerifyLogin;

		// Android specific
		public bool NoPlayProtect;

		[AutoParam(false)]
		public bool SkipStage;

		public string Build;

		[AutoParam("")]
		public string TestExtraArgs;

		[AutoParam("")]
		public string LogDir;

		public string ReportType;

		public int Timeout;

		public bool LogReportContents;

		public bool CaptureOutput;

		public Type BuildSourceType { get; protected set; }

		[AutoParam]
		public string HordeTestDataKey = "";

		[AutoParam(UnrealTargetConfiguration.Development)]
		public UnrealTargetConfiguration Configuration;

		public UnrealTargetPlatform Platform;
		public string Device;

		public bool Containerized;

		public LowLevelTestExecutorOptions()
		{
			BuildSourceType = typeof(LowLevelTestsBuildSource);
		}

		public virtual void ParametersWereApplied(string[] InParams)
		{
			Params = new Params(InParams);
			if (string.IsNullOrEmpty(TempDir))
			{
				TempDir = Globals.TempDir;
			}
			else
			{
				Globals.TempDir = TempDir;
			}

			if (string.IsNullOrEmpty(LogDir))
			{
				LogDir = Globals.LogDir;
			}
			else
			{
				Globals.LogDir = LogDir;
			}

			LogDir = Path.GetFullPath(LogDir);
			TempDir = Path.GetFullPath(TempDir);

			Timeout = Params.ParseValue("timeout=", 0);

			LogReportContents = Params.ParseParam("printreport");

			CaptureOutput = Params.ParseParam("captureoutput");

			var buildValue = Params.ParseValue("build=", null);

			if (buildValue != null)
			{
				Build = buildValue.Replace('/', Path.DirectorySeparatorChar).Replace('\\', Path.DirectorySeparatorChar);
			}
			else
			{
				Build = null;
			}

			TestApp = Globals.Params.ParseValue("testapp=", "");

			Tags = Params.ParseValue("tags=", null);

			// Parse all -testnames= parameters (supports multiple test names)
			TestNames = new List<string>();
			foreach (string Arg in Params.AllArguments)
			{
				if (Arg.StartsWith("testnames=", StringComparison.OrdinalIgnoreCase))
				{
					string TestName = Arg.Substring("testnames=".Length);
					if (!string.IsNullOrEmpty(TestName))
					{
						TestNames.Add(TestName);
					}
				}
			}

			AttachToDebugger = Params.ParseParam("attachtodebugger");
			VerifyLogin = Params.ParseParam("verifylogin");
			NoPlayProtect = Params.ParseParam("no-play-protect");

			TestExtraArgs = Params.ParseValue("extra-args=", null);

			SkipStage = Params.ParseParam("skipstage");

			ReportType = Params.ParseValue("reporttype=", null);

			string PlatformArgString = Params.ParseValue("platform=", null);
			Platform = string.IsNullOrEmpty(PlatformArgString) ? BuildHostPlatform.Current.Platform : UnrealTargetPlatform.Parse(PlatformArgString);

			string DeviceArgString = Params.ParseValue("device=", null);
			Device = string.IsNullOrEmpty(PlatformArgString) ? "default" : DeviceArgString;

			Containerized = Params.ParseParam("containerized");

			string[] CleanArgs = Params.AllArguments
				.Where(Arg => !Arg.StartsWith("test=", StringComparison.OrdinalIgnoreCase)
					&& !Arg.StartsWith("platform=", StringComparison.OrdinalIgnoreCase)
					&& !Arg.StartsWith("device=", StringComparison.OrdinalIgnoreCase))
				.ToArray();

			Params = new Params(CleanArgs);
		}
	}

	public class LowLevelTestsSession : IDisposable
	{
		private static int QUERY_STATE_INTERVAL = 1;

		public IAppInstall Install { get; protected set; }
		public IAppInstance Instance { get; protected set; }
		private LowLevelTestsBuildSource BuildSource { get; set; }
		private string Tags { get; set; }
		private List<string> TestNames { get; set; }
		private int Sleep { get; set; }
		private bool AttachToDebugger { get; set; }
		private bool VerifyLogin { get; set; }
		private string ReportType { get; set; }
		private int PerTestTimeout { get; set; }
		private string TestExtraArgs { get; set; }

		private bool Containerized { get;set; }

		public UnrealDeviceReservation UnrealDeviceReservation { get; private set; }

		public LowLevelTestsSession(LowLevelTestsBuildSource InBuildSource, LowLevelTestExecutorOptions InOptions)
		{
			BuildSource = InBuildSource;
			Tags = InOptions.Tags;
			TestNames = new List<string>(InOptions.TestNames);

			Sleep = InOptions.Sleep;
			AttachToDebugger = InOptions.AttachToDebugger;
			VerifyLogin = InOptions.VerifyLogin;
			ReportType = InOptions.ReportType;
			PerTestTimeout = InOptions.Timeout;
			UnrealDeviceReservation = new UnrealDeviceReservation();
			TestExtraArgs = InOptions.TestExtraArgs;
			Containerized = InOptions.Containerized;
		}

		#region IDisposable Support
		public void Dispose()
		{
			if (Instance != null)
			{
				Instance.Kill();
				IDeviceUsageReporter.RecordEnd(Instance.Device.Name, (UnrealTargetPlatform)Instance.Device.Platform, IDeviceUsageReporter.EventType.Test, IDeviceUsageReporter.EventState.Success);

				if (Instance.HasExited)
				{
					IDeviceUsageReporter.RecordEnd(Instance.Device.Name, (UnrealTargetPlatform)Instance.Device.Platform, IDeviceUsageReporter.EventType.Test, IDeviceUsageReporter.EventState.Failure);
				}

				UnrealDeviceReservation?.ReleaseDevices();
			}
		}
		#endregion

		public bool TryReserveDevices()
		{
			// Low level tests require exactly one device of the build source's platform.
			Dictionary<UnrealDeviceTargetConstraint, int> RequiredDeviceTypes = new Dictionary<UnrealDeviceTargetConstraint, int>
			{
				{ new UnrealDeviceTargetConstraint(BuildSource.Platform), 1 }
			};

			if (Containerized)
			{
				// Linux platforms, including arm64, can be run through Docker on Windows machines.
				// Add a Linux device on Windows hosts when running locally.
				// Add a local LinuxArm64 device when running on the build system: the device service can't provision devices for this platform.
				string IsBuildMachineEnvVar = Environment.GetEnvironmentVariable("IsBuildMachine") ?? string.Empty;
				bool IsBuildMachine = (IsBuildMachineEnvVar != null && IsBuildMachineEnvVar == "1");
				if ((!IsBuildMachine && BuildSource.Platform.IsInGroup(UnrealPlatformGroup.Linux) && BuildHostPlatform.Current.Platform == UnrealTargetPlatform.Win64) ||
					(IsBuildMachine && BuildSource.Platform == UnrealTargetPlatform.LinuxArm64))
				{
					DevicePool.Instance.AddLocalDevices(1, BuildSource.Platform);
				}
			}

			bool Reserved = UnrealDeviceReservation.TryReserveDevices(RequiredDeviceTypes, 1);
			if (!Reserved)
			{
				Gauntlet.Horde.UpdateStepMessage(Gauntlet.Horde.GauntletHordeStepMessage.NotEnoughDevices);
			}
			return Reserved;
		}

		/// <summary>
		/// Copies build folder on device and launches app natively.
		/// Does not retry.
		/// No packaging required.
		/// </summary>
		public IAppInstance InstallAndRunNativeTestApp()
		{
			if(!Globals.UseExperimentalFeatures)
			{
				return Legacy_InstallAndRunNativeTestApp();
			}

			// TargetDevice<Platform> classes have a hard dependency on UnrealAppConfig instead of IAppConfig.
			// More refactoring needed to support non-packaged applications that can be run natively from a path on the device.
			UnrealAppConfig AppConfig = BuildSource.GetUnrealAppConfig(Tags, Sleep, AttachToDebugger, ReportType, PerTestTimeout, TestExtraArgs, Containerized, TestNames);

			IEnumerable<ITargetDevice> DevicesToInstallOn = UnrealDeviceReservation.ReservedDevices.ToArray();
			ITargetDevice Device = DevicesToInstallOn.Where(D => D.IsConnected && D.Platform == BuildSource.Platform).First();

			if (AppConfig.FullClean)
			{
				Device.FullClean();
			}

			// Install the build onto the device
			if (AppConfig.SkipInstall)
			{
				Log.Info("Skipping install due to SkipInstall");
			}
			else
			{
				// Telemetry
				DateTimeStopwatch Stopwatch = DateTimeStopwatch.Start();
				Log.Info("Installing {BuildName} of type {BuildType} to {DeviceName}...", BuildSource.BuildName, AppConfig.Build.GetType().Name, Device.Name);
				IDeviceUsageReporter.RecordStart(Device.Name, Device.Platform, IDeviceUsageReporter.EventType.Install, IDeviceUsageReporter.EventState.Success, BuildSource.BuildName);

				try
				{
					Device.InstallBuild(AppConfig);
					IDeviceUsageReporter.RecordEnd(Device.Name, Device.Platform, IDeviceUsageReporter.EventType.Install, IDeviceUsageReporter.EventState.Success);
					Log.Info("Installation completed in {InstallTime}", GetInstallTime(Stopwatch.ElapsedTime));
				}
				catch (Exception Ex)
				{
					IDeviceUsageReporter.RecordEnd(Device.Name, Device.Platform, IDeviceUsageReporter.EventType.Install, IDeviceUsageReporter.EventState.Failure);

					if (Ex is DeviceException)
					{
						UnrealDeviceReservation.MarkProblemDevice(Device, $"Failed to install low level tests app onto device: {Ex}");
					}
					else
					{
						Log.Info("Failed to install low level tests app onto device {DeviceName}: {Exception}", Device.Name, Ex.ToString());
					}

					Gauntlet.Horde.ReportInvalidatingIssue(Gauntlet.Horde.GauntletHordeStepMessage.InstallationFailed, "Unable to install application for session.");


					UnrealDeviceReservation.ReleaseDevices();
				}
			}

			// Create the IAppInstall instance
			try
			{
				Install = Device.CreateAppInstall(AppConfig);
			}
			catch (Exception Ex)
			{
				if (Ex is DeviceException)
				{
					UnrealDeviceReservation.MarkProblemDevice(Device, $"Failed to create IAppInstall for low level tests on device: {Ex}");
				}
				else
				{
					Log.Info("Failed to create IAppInstall for low level tests on device {DeviceName}: {Exception}", Device.Name, Ex.ToString());
				}

				UnrealDeviceReservation.ReleaseDevices();
			}

			// Clean/Copy files to the device
			try
			{
				Device.CleanArtifacts();
				Device.CopyAdditionalFiles(AppConfig.FilesToCopy);
			}
			catch (Exception Ex)
			{
				if (Ex is DeviceException)
				{
					UnrealDeviceReservation.MarkProblemDevice(Device, $"Failed to CleanArtifacts or CopyAdditionalFiles for low level tests on device: {Ex}");
				}
				else
				{
					Log.Info("Failed to CleanArtifacts or CopyAdditionalFiles for low level tests on device {DeviceName}: {Exception}", Device.Name, Ex.ToString());
				}
				UnrealDeviceReservation.ReleaseDevices();
			}

			// Run the application
			try
			{
				if (Device is IRunningStateOptions DeviceWithStateOptions)
				{
					// Don't wait to detect running state and query for running state every second
					DeviceWithStateOptions.WaitForRunningState = false;
					DeviceWithStateOptions.CachedStateRefresh = QUERY_STATE_INTERVAL;
				}

				Instance = Device.Run(Install);
				IDeviceUsageReporter.RecordStart(Instance.Device.Name, Instance.Device.Platform, IDeviceUsageReporter.EventType.Test);
			}
			catch (DeviceException DeviceEx)
			{
				Log.Warning("Failed to start low level test on {DeviceName}. Marking as problem device. Will not retry.", Device.Name);

				if (Instance != null)
				{
					Instance.Kill();
				}

				UnrealDeviceReservation.MarkProblemDevice(Device, $"Device threw an exception during launch. \nException={DeviceEx.Message}");
				UnrealDeviceReservation.ReleaseDevices();

				throw new AutomationException("Unable to start low level tests app, see warnings for details.");
			}

			return Instance;
		}

		private IAppInstance Legacy_InstallAndRunNativeTestApp()
		{
			bool InstallSuccess = false;
			bool RunSuccess = false;

			// TargetDevice<Platform> classes have a hard dependency on UnrealAppConfig instead of IAppConfig.
			// More refactoring needed to support non-packaged applications that can be run natively from a path on the device.
			UnrealAppConfig AppConfig = BuildSource.GetUnrealAppConfig(Tags, Sleep, AttachToDebugger, ReportType, PerTestTimeout, TestExtraArgs, Containerized, TestNames);

			IEnumerable<ITargetDevice> DevicesToInstallOn = UnrealDeviceReservation.ReservedDevices.ToArray();
			ITargetDevice Device = DevicesToInstallOn.Where(D => D.IsConnected && D.Platform == BuildSource.Platform).First();

			IDeviceUsageReporter.RecordStart(Device.Name, (UnrealTargetPlatform)Device.Platform, IDeviceUsageReporter.EventType.Device, IDeviceUsageReporter.EventState.Success);
			IDeviceUsageReporter.RecordStart(Device.Name, (UnrealTargetPlatform)Device.Platform, IDeviceUsageReporter.EventType.Install, IDeviceUsageReporter.EventState.Success, BuildSource.BuildName);
			try
			{
				if (VerifyLogin && Device is IOnlineServiceLogin)
				{
					Log.Info("\nVerifying device login...");
					if (!(Device as IOnlineServiceLogin).VerifyLogin())
					{
						throw new AutomationException("Unable to secure login to an online platform account!");
					}
					Log.Info("Success! User signed-in.\n");
				}

				Install = Device.InstallApplication(AppConfig);
				InstallSuccess = true;
				IDeviceUsageReporter.RecordEnd(Device.Name, (UnrealTargetPlatform)Device.Platform, IDeviceUsageReporter.EventType.Install, IDeviceUsageReporter.EventState.Success);
			}
			catch (Exception Ex)
			{
				InstallSuccess = false;

				if (Ex is DeviceException)
				{
					UnrealDeviceReservation.MarkProblemDevice(Device, $"Failed to install low level tests app onto device: {Ex}");
				}
				else
				{
					Log.Info("Failed to install low level tests app onto device {DeviceName}: {Exception}", Device.Name, Ex.ToString());
				}
				IDeviceUsageReporter.RecordEnd(Device.Name, (UnrealTargetPlatform)Device.Platform, IDeviceUsageReporter.EventType.Install, IDeviceUsageReporter.EventState.Failure);
			}

			if (!InstallSuccess)
			{
				// release all devices
				UnrealDeviceReservation.ReleaseDevices();
				Log.Info("\nUnable to install low level tests app.\n");
			}
			else
			{
				try
				{
					if (Device is IRunningStateOptions)
					{
						// Don't wait to detect running state and query for running state every second
						IRunningStateOptions DeviceWithStateOptions = (IRunningStateOptions)Device;
						DeviceWithStateOptions.WaitForRunningState = false;
						DeviceWithStateOptions.CachedStateRefresh = QUERY_STATE_INTERVAL;
					}

					Instance = Device.Run(Install);
					IDeviceUsageReporter.RecordStart(Instance.Device.Name, (UnrealTargetPlatform)Instance.Device.Platform, IDeviceUsageReporter.EventType.Test);
					RunSuccess = true;
				}
				catch (DeviceException DeviceEx)
				{
					UnrealDeviceReservation.MarkProblemDevice(Device, $"Device threw an exception during launch. \nException={DeviceEx.Message}");
					RunSuccess = false;
				}

				if (RunSuccess == false)
				{
					Log.Warning("Failed to start low level test on {DeviceName}. Marking as problem device. Will not retry.", Device.Name);

					if (Instance != null)
					{
						Instance.Kill();
					}

					UnrealDeviceReservation.ReleaseDevices();

					throw new AutomationException("Unable to start low level tests app, see warnings for details.");
				}
			}

			return Instance;
		}

		private string GetInstallTime(TimeSpan Time)
		{
			string Hours = Time.Hours > 0 ? string.Format("{0} hrs, ", Time.Hours) : string.Empty;
			string Minutes = Time.Minutes > 0 ? string.Format("{0} mins, ", Time.Minutes) : string.Empty;
			string Seconds = string.Format("{0} secs", Time.Seconds);

			return Hours + Minutes + Seconds;
		}
	}

	public class LowLevelTestRoleContext : ICloneable
	{
		public UnrealTargetRole Type { get { return UnrealTargetRole.Client; } }
		public UnrealTargetPlatform Platform;
		public UnrealTargetConfiguration Configuration { get { return UnrealTargetConfiguration.Development; } }

		public object Clone()
		{
			return this.MemberwiseClone();
		}

		public override string ToString()
		{
			string Description = string.Format("{0} {1} {2}", Platform, Configuration, Type);
			return Description;
		}
	};

	public class LowLevelTestContext : ITestContext, ICloneable
	{
		public LowLevelTestsBuildSource BuildInfo { get; private set; }

		public string WorkerJobID;

		public LowLevelTestExecutorOptions Options { get; set; }

		public Params TestParams { get; set; }

		public LowLevelTestRoleContext RoleContext { get; set; }

		public UnrealDeviceTargetConstraint Constraint;

		public int PerTestTimeout { get; private set; }

		public LowLevelTestContext(LowLevelTestsBuildSource InBuildInfo, LowLevelTestRoleContext InRoleContext, LowLevelTestExecutorOptions InOptions, int InPerTestTimeout = 0)
		{
			BuildInfo = InBuildInfo;
			Options = InOptions;
			TestParams = new Params(new string[0]);
			RoleContext = InRoleContext;
			PerTestTimeout = InPerTestTimeout;
		}

		public object Clone()
		{
			LowLevelTestContext Copy = (LowLevelTestContext)MemberwiseClone();
			Copy.RoleContext = (LowLevelTestRoleContext)RoleContext.Clone();
			return Copy;
		}

		public override string ToString()
		{
			string Description = string.Format("{0}", RoleContext);
			if (WorkerJobID != null)
			{
				Description += " " + WorkerJobID;
			}
			return Description;
		}
	}

	/// <summary>
	/// Platform and test specific extension that provides extra command line arguments or logic.
	/// </summary>
	public interface ILowLevelTestsExtension
	{
		/// <summary>
		/// Use this to implement platform and test specific specializations of ILowLevelTestsExtension.
		/// </summary>
		bool IsSupported(UnrealTargetPlatform InPlatform, string InTestApp);

		/// <summary>
		/// Return extra command line arguments specific to a platform and/or test.
		/// </summary>
		string ExtraCommandLine(UnrealTargetPlatform InPlatform, string InTestApp, string InBuildPath);

		/// <summary>
		/// Run extra logic before running tests
		/// </summary>
		void PreRunTests();

		/// <summary>
		/// Run extra logic after finishing tests
		/// </summary>
		void PostRunTests();
	}

	/// <summary>
	/// Platform-specific reporting utility for defining Catch2 report path and means to copy it to a saved storage either from its local directory or from a target device.
	/// </summary>
	public interface ILowLevelTestsReporting
	{
		/// <summary>
		/// Use this to implement platform-specific specializations of ILowLevelTestsReporting.
		/// </summary>
		bool CanSupportPlatform(UnrealTargetPlatform InPlatform);

		/// <summary>
		/// Specify Catch2 target report path for this platform.
		/// </summary>
		string GetTargetReportPath(UnrealTargetPlatform InPlatform, string InTestApp, string InBuildPath);

		/// <summary>
		/// Copy generated report  and return copied path
		/// </summary>
		string CopyDeviceReportTo(IAppInstall InAppInstall, UnrealTargetPlatform InPlatform, string InTestApp, string InBuildPath, string InTargetDirectory);
	}

	public interface ILowLevelTestsBuildFactory
	{
		bool CanSupportPlatform(UnrealTargetPlatform InPlatform);

		IBuild CreateBuild(UnrealTargetPlatform InPlatform, UnrealTargetConfiguration InConfiguration, string InTestApp, string InBuildPath, bool bSkipStage);
		protected static string GetExecutable(UnrealTargetPlatform InPlatform, UnrealTargetConfiguration InConfiguration, string InTestApp, string InBuildPath, string FileRegEx)
		{
			IEnumerable<string> Executables = null;
			if (InPlatform == UnrealTargetPlatform.IOS)
			{
				Executables = DirectoryUtils.FindMatchingDirectories(InBuildPath, FileRegEx, -1).Select(FileInfo => FileInfo.FullName);
			}
			else
			{
				Executables = DirectoryUtils.FindMatchingFiles(InBuildPath, FileRegEx, -1).Select(FileInfo => FileInfo.FullName);
			}
			string ParentDirPath;
			string BuildExecutableName;
			foreach (string Executable in Executables)
			{
				Log.VeryVerbose("Found path: {0}", Executable);
				ParentDirPath = Directory.GetParent(Executable).FullName;
				BuildExecutableName = Path.GetFileNameWithoutExtension(Executable);

				bool ContainsPlatformName = ParentDirPath.Contains(InPlatform.ToString(), StringComparison.OrdinalIgnoreCase);
				bool ContainsWindowsName = (InPlatform == UnrealTargetPlatform.Win64 && ParentDirPath.Contains("Windows", StringComparison.OrdinalIgnoreCase));
				if ( !(ContainsPlatformName || ContainsWindowsName) )
				{
					Log.Error("ParentPath did not have platform ({1}) : {0}", ParentDirPath, InPlatform);
					continue;
				}

				if (!ParentDirPath.Contains(InTestApp.ToString(), StringComparison.OrdinalIgnoreCase))
				{
					Log.Error("ParentPath did not have app ({1}) : {0}", ParentDirPath, InTestApp);
					continue;
				}

				if ((InPlatform == UnrealTargetPlatform.Mac
					|| InPlatform == UnrealTargetPlatform.Linux
					|| InPlatform == UnrealTargetPlatform.LinuxArm64)
					&& !string.IsNullOrEmpty(Path.GetExtension(Executable)))
				{
					// Mac & Linux executable candidates should have no extension
					continue;
				}

				if (InPlatform == UnrealTargetPlatform.Android && BuildExecutableName.StartsWith(InTestApp))
				{
					Log.VeryVerbose("Output Executable for Android: {0}", Path.GetRelativePath(InBuildPath, Executable));
					return Path.GetRelativePath(InBuildPath, Executable);
				}

				// Development executable does not contain configuration or platform name
				// Note: The executable name may legitimately contain a platform name as part of the test app name
				// (e.g., OnlineServicesPS4Tests). The Equals check is sufficient to identify Development builds.
				Log.Verbose("Config type: {0}", InConfiguration);
				if (InConfiguration == UnrealTargetConfiguration.Development)
				{
					if (BuildExecutableName.Equals(InTestApp, StringComparison.OrdinalIgnoreCase))
					{
						Log.VeryVerbose("Output Development Executable: {0}", Path.GetRelativePath(InBuildPath, Executable));
						return Path.GetRelativePath(InBuildPath, Executable);
					}
					else // move to the next executable until we find the Development one
					{
						continue;
					}
				}

				// All executables other than Development contain the configuration in their name
				if (!BuildExecutableName.Contains(InConfiguration.ToString()))
				{
					continue;
				}

				if (!(BuildExecutableName.Contains(InTestApp, StringComparison.OrdinalIgnoreCase) 
					&& BuildExecutableName.Contains(InPlatform.ToString(), StringComparison.OrdinalIgnoreCase)))
				{
					Log.Error("BuildExecutableName did not have expected name ({0}) or is missing platform in name ({1}): {2}", InTestApp, InPlatform.ToString(), BuildExecutableName);
					continue;
				}

				Log.VeryVerbose("Output Executable: {0}", Path.GetRelativePath(InBuildPath, Executable));
				return Path.GetRelativePath(InBuildPath, Executable);
			}
			Gauntlet.Horde.ReportInvalidatingIssue(Gauntlet.Horde.GauntletHordeStepMessage.BuildNotFound, "The test was unable to find the necessary executables.");
			throw new AutomationException("Cannot find low level test executable for {0} in build path {1} for {2} using regex \"{3}\"", InPlatform, InBuildPath, InTestApp, FileRegEx);
		}

		protected static void CleanupUnusedFiles(UnrealTargetPlatform InPlatform, UnrealTargetConfiguration InConfiguration, string InBuildPath)
		{
			try
			{
				string[] BuildFiles = Directory.GetFiles(InBuildPath);
				foreach (string BuildFile in BuildFiles)
				{
					if (new FileInfo(BuildFile).Extension == ".pdb" && InConfiguration != UnrealTargetConfiguration.Debug)
					{
						File.Delete(BuildFile);
					}
				}
			}
			catch (Exception cleanupEx)
			{
				Log.Error("Could not cleanup files for {0} build: {1}.", InPlatform.ToString(), cleanupEx);
			}
		}
	}

	public class DesktopLowLevelTestsBuildFactory : ILowLevelTestsBuildFactory
	{
		public bool CanSupportPlatform(UnrealTargetPlatform InPlatform)
		{
			return InPlatform.IsInGroup(UnrealPlatformGroup.Desktop);
		}

		public IBuild CreateBuild(UnrealTargetPlatform InPlatform, UnrealTargetConfiguration InConfiguration, string InTestApp, string InBuildPath, bool bSkipStage)
		{
			string ExecutablePath = ILowLevelTestsBuildFactory.GetExecutable(InPlatform, InConfiguration, InTestApp, InBuildPath, GetExecutableRegex(InPlatform));
			return new LowLevelTestsBuild(InPlatform, InConfiguration, InBuildPath, ExecutablePath);
		}

		public string GetExecutableRegex(UnrealTargetPlatform InPlatform)
		{
			if (InPlatform.IsInGroup(UnrealPlatformGroup.Windows))
			{
				return @"[A-Za-z0-9_]+(Tests)?(?:-[A-Za-z0-9_]+)?(?:-[A-Za-z0-9_]+)?.exe$";
			}
			else if (InPlatform == UnrealTargetPlatform.Linux ||
					 InPlatform == UnrealTargetPlatform.LinuxArm64 ||
					 InPlatform == UnrealTargetPlatform.Mac)
			{
				return @"[A-Za-z0-9_]+(Tests)?(?:-[A-Za-z0-9_]+)?(?:-[A-Za-z0-9_]+)?$";
			}
			else
			{
				throw new AutomationException("Cannot create build for non-desktop platform " + InPlatform);
			}
		}
	}

	public class AndroidLowLevelTestsBuildFactory : ILowLevelTestsBuildFactory
	{
		public bool CanSupportPlatform(UnrealTargetPlatform InPlatform)
		{
			return InPlatform.IsInGroup(UnrealPlatformGroup.Android);
		}

		public IBuild CreateBuild(UnrealTargetPlatform InPlatform, UnrealTargetConfiguration InConfiguration, string InTestApp, string InBuildPath, bool bSkipStage)
		{
			string ExecutablePath = ILowLevelTestsBuildFactory.GetExecutable(InPlatform, InConfiguration, InTestApp, InBuildPath, GetExecutableRegex(InPlatform));
			return new AndroidBuild(InConfiguration, $"com.epicgames.{InTestApp}", Path.Combine(InBuildPath, ExecutablePath), new Dictionary<string, string>(), BuildFlags.Packaged | BuildFlags.CanReplaceCommandLine, false, true, false, true);
		}

		public string GetExecutableRegex(UnrealTargetPlatform InPlatform)
		{
			return @"[A-Za-z0-9_]+(Tests)?(?:-[A-Za-z0-9_]+)?(?:-[A-Za-z0-9_]+)?.apk";
		}
	}

	public class AndroidLowLevelTestsReporting : ILowLevelTestsReporting
	{
		public bool CanSupportPlatform(UnrealTargetPlatform InPlatform)
		{
			return InPlatform.IsInGroup(UnrealPlatformGroup.Android);
		}

		public string GetTargetReportPath(UnrealTargetPlatform InPlatform, string InTestApp, string InBuildPath)
		{
			return string.Format("{0}LLTResults.out", InPlatform.ToString());
		}

		public string CopyDeviceReportTo(IAppInstall InAppInstall, UnrealTargetPlatform InPlatform, string InTestApp, string InBuildPath, string InTargetDirectory)
		{
			throw new NotSupportedException("Reports currently not supported for Android."); 
		}
	}

	public class IOSLowLevelTestsBuildFactory : ILowLevelTestsBuildFactory
	{
		public bool CanSupportPlatform(UnrealTargetPlatform InPlatform)
		{
			return InPlatform.IsInGroup(UnrealPlatformGroup.IOS);
		}

		public IBuild CreateBuild(UnrealTargetPlatform InPlatform, UnrealTargetConfiguration InConfiguration, string InTestApp, string InBuildPath, bool bSkipStage)
		{
			string ExecutablePath = ILowLevelTestsBuildFactory.GetExecutable(InPlatform, InConfiguration, InTestApp, InBuildPath, GetExecutableRegex(InPlatform));
			return new IOSBuild(InConfiguration, InTestApp, Path.Combine(InBuildPath, ExecutablePath), BuildFlags.Packaged | BuildFlags.CanReplaceCommandLine);
		}

		public string GetExecutableRegex(UnrealTargetPlatform InPlatform)
		{
			return @"[A-Za-z0-9_]+(Tests)?(?:-[A-Za-z0-9_]+)?(?:-[A-Za-z0-9_]+)?.app";
		}
	}

	public class IOSLowLevelTestsReporting : ILowLevelTestsReporting
	{
		public bool CanSupportPlatform(UnrealTargetPlatform InPlatform)
		{
			return InPlatform.IsInGroup(UnrealPlatformGroup.IOS);
		}

		public string GetTargetReportPath(UnrealTargetPlatform InPlatform, string InTestApp, string InBuildPath)
		{
			return string.Format("{0}LLTResults.out", InPlatform.ToString());
		}

		public string CopyDeviceReportTo(IAppInstall InAppInstall, UnrealTargetPlatform InPlatform, string InTestApp, string InBuildPath, string InTargetDirectory)
		{
			throw new NotSupportedException("Reports currently not supported for IOS.");
		}
	}

	public class DesktopLowLevelTestsReporting : ILowLevelTestsReporting
	{
		public bool CanSupportPlatform(UnrealTargetPlatform InPlatform)
		{
			return InPlatform.IsInGroup(UnrealPlatformGroup.Desktop);
		}

		public string GetTargetReportPath(UnrealTargetPlatform InPlatform, string InTestApp, string InBuildPath)
		{
			return Path.Combine(InBuildPath, string.Format("{0}LLTResults.out", InPlatform.ToString()));
		}

		public string CopyDeviceReportTo(IAppInstall InAppInstall, UnrealTargetPlatform InPlatform, string InTestApp, string InBuildPath, string InTargetDirectory)
		{
			string ReportRelativePath = Path.GetFileName(GetTargetReportPath(InPlatform, InTestApp, InBuildPath));
			string ReportPath = Path.Combine(InBuildPath, ReportRelativePath);
			string ExpectedLocalPath = Path.Combine(InTargetDirectory, ReportRelativePath);
			if (!ExpectedLocalPath.Equals(ReportPath))
			{
				if (InAppInstall is IContainerized)
				{
					ContainerInfo Container = ((IContainerized)InAppInstall).ContainerInfo;
					string ReportPathInContainer = Container.WorkingDir + "/" + Path.GetRelativePath(Globals.UnrealRootDir, ReportPath).Replace("\\", "/");
					CommandUtils.Run("docker", $"cp {Container.ContainerName}:{ReportPathInContainer} {ExpectedLocalPath}");
				}
				else
				{
					File.Copy(ReportPath, ExpectedLocalPath, true);
				}
			}
			return ExpectedLocalPath;
		}
	}

	public class LowLevelTestsBuildSource : IBuildSource
	{
		private string TestApp;
		private string BuildPath;
		private UnrealAppConfig CachedConfig = null;

		private ILowLevelTestsBuildFactory LowLevelTestsBuildFactory;
		private ILowLevelTestsReporting LowLevelTestsReporting;
		public ILowLevelTestsExtension[] LowLevelTestsExtensions { get; protected set; }

		public UnrealTargetPlatform Platform { get; protected set; }
		public UnrealTargetConfiguration Configuration { get; protected set; }
		public IBuild DiscoveredBuild { get; protected set; }

		public LowLevelTestsBuildSource(string InTestApp, string InBuildPath, UnrealTargetPlatform InTargetPlatform, UnrealTargetConfiguration InConfiguration, bool InSkipStage)
		{
			TestApp = InTestApp;
			Platform = InTargetPlatform;
			BuildPath = InBuildPath;
			Configuration = InConfiguration;
			InitBuildSource(InTestApp, InBuildPath, InTargetPlatform, InConfiguration, InSkipStage);
		}

		protected void InitBuildSource(string InTestApp, string InBuildPath, UnrealTargetPlatform InTargetPlatform, UnrealTargetConfiguration InConfiguration, bool bSkipStage)
		{
			LowLevelTestsBuildFactory = Gauntlet.Utils.InterfaceHelpers.FindImplementations<ILowLevelTestsBuildFactory>(true)
				.Where(B => B.CanSupportPlatform(InTargetPlatform))
				.First();
			
			DiscoveredBuild = LowLevelTestsBuildFactory.CreateBuild(InTargetPlatform, InConfiguration, InTestApp, InBuildPath, bSkipStage);

			if (DiscoveredBuild == null)
			{
				throw new AutomationException("No builds were discovered at path {0} matching test app name {1} and target platform {2}", InBuildPath, InTestApp, InTargetPlatform);
			}

			LowLevelTestsReporting = Gauntlet.Utils.InterfaceHelpers.FindImplementations<ILowLevelTestsReporting>(true)
				.Where(B => B.CanSupportPlatform(InTargetPlatform))
				.First();

			LowLevelTestsExtensions = Gauntlet.Utils.InterfaceHelpers.FindImplementations<ILowLevelTestsExtension>(true)
				.Where(B => B.IsSupported(InTargetPlatform, InTestApp))
				.ToArray();
		}

		public UnrealAppConfig GetUnrealAppConfig(string InTags, int InSleep, bool InAttachToDebugger, string InReportType, int InPerTestTimeout = 0, string InTestExtraArgs = null, bool InContainerized = false, List<string> InTestNames = null)
		{
			if (CachedConfig == null)
			{
				CachedConfig = new UnrealAppConfig();
				CachedConfig.Name = BuildName;
				CachedConfig.ProjectName = TestApp;
				CachedConfig.ProcessType = UnrealTargetRole.Client;
				CachedConfig.Platform = Platform;
				CachedConfig.Configuration = UnrealTargetConfiguration.Development;
				CachedConfig.Build = DiscoveredBuild;
				CachedConfig.Sandbox = $"LowLevelTests-{TestApp}";
				CachedConfig.FilesToCopy = new List<UnrealFileToCopy>();
				if (InContainerized)
				{
					CachedConfig.ContainerInfo = new ContainerInfo();
					CachedConfig.ContainerInfo.ImageName = $"{TestApp}-{Platform}-Image".ToLower();
					CachedConfig.ContainerInfo.ContainerName = $"{TestApp}-{Platform}-Container".ToLower();
					// LinuxArm64 runs through emulator in container
					if (Platform == UnrealTargetPlatform.LinuxArm64)
					{
						CachedConfig.ContainerInfo.RunCommandPrepend = "qemu-aarch64 -L /usr/aarch64-linux-gnu";
						CachedConfig.ContainerInfo.WorkingDir = "/app";
					}
				}

				// Both test names and tags can be provided as positional arguments to Catch2
				// Multiple test names are passed as quoted strings without spaces between them: "Test1","Test2"
				// This allows Catch2 to treat them as comma-separated filters (OR logic)
				// Commas within test names are escaped with backslash (\,) for Catch2
				if (InTestNames != null && InTestNames.Count > 0)
				{
					// Build comma-separated test names as adjacent quoted strings
					StringBuilder TestNamesBuilder = new StringBuilder();
					for (int i = 0; i < InTestNames.Count; ++i)
					{
						// Escape commas in the test name for Catch2 parsing
						string EscapedTestName = InTestNames[i].Replace(",", "\\,");
						TestNamesBuilder.AppendFormat("\"{0}\"", EscapedTestName);

						// Add comma between test names (no space, so shell concatenates them)
						if (i < InTestNames.Count - 1)
						{
							TestNamesBuilder.Append(",");
						}
					}
					// Pass as single argument (e.g., "Test1","Test2" or "Test, with comma\, in name","Test2")
					CachedConfig.CommandLineParams.Add(TestNamesBuilder.ToString(), null, true);
				}
				if (!string.IsNullOrEmpty(InTags))
				{
					// Tags/filters as a positional argument
					// Catch2 applies AND logic when both test names and tags are provided
					CachedConfig.CommandLineParams.Add(InTags, null, true);
				}

				// Set reporting options, filters etc
				CachedConfig.CommandLineParams.AddRawCommandline("--durations=no");
				CachedConfig.CommandLineParams.AddRawCommandline("--order=decl");
				if (CachedConfig.Platform == UnrealTargetPlatform.Android)
				{
					CachedConfig.CommandLineParams.AddRawCommandline("--reporter=console");
				}
				else if (!string.IsNullOrEmpty(InReportType))
				{
					CachedConfig.CommandLineParams.AddRawCommandline(string.Format("--reporter={0}", InReportType));
					string ReportPath = LowLevelTestsReporting.GetTargetReportPath(Platform, TestApp, BuildPath);
					if (InContainerized)
					{
						ReportPath = CachedConfig.ContainerInfo.WorkingDir + "/" + Path.GetRelativePath(Globals.UnrealRootDir, ReportPath).Replace("\\", "/");
					}
					CachedConfig.CommandLineParams.AddRawCommandline(string.Format("--out={0}", ReportPath));
				}
				CachedConfig.CommandLineParams.AddRawCommandline("--filenames-as-tags");
				if (InSleep > 0)
				{
					CachedConfig.CommandLineParams.AddRawCommandline(String.Format("--sleep={0}", InSleep));
				}
				if (InPerTestTimeout > 0)
				{
					CachedConfig.CommandLineParams.AddRawCommandline(String.Format("--timeout={0}", InPerTestTimeout));
				}
				CachedConfig.CommandLineParams.AddRawCommandline("--debug");
				CachedConfig.CommandLineParams.AddRawCommandline("--log");
				if (InAttachToDebugger)
				{
					CachedConfig.CommandLineParams.AddRawCommandline("--attach-to-debugger");
				}
				if (Environment.GetEnvironmentVariable("IsBuildMachine") == "1")
				{
					CachedConfig.CommandLineParams.AddRawCommandline("--buildmachine");
				}

				string ExtraCmd = "";

				foreach (ILowLevelTestsExtension LowLevelTestsExtension in LowLevelTestsExtensions)
				{
					string ExtensionExtraCmd = LowLevelTestsExtension.ExtraCommandLine(Platform, TestApp, BuildPath);
					if (!string.IsNullOrEmpty(ExtensionExtraCmd))
					{
						ExtraCmd += " ";
						ExtraCmd += ExtensionExtraCmd;
					}
				}

				if (InTestExtraArgs != null)
				{
					ExtraCmd += string.Format(" {0}", InTestExtraArgs);
				}

				if (!string.IsNullOrEmpty(ExtraCmd))
				{
					CachedConfig.CommandLineParams.AddRawCommandline("--extra-args" + ExtraCmd);
				}				


				CachedConfig.CanAlterCommandArgs = false; // No further changes by IAppInstall instances etc.
			}
			return CachedConfig;
		}

		public bool CanSupportPlatform(UnrealTargetPlatform Platform)
		{
			return true;
		}

		public string BuildName { get { return TestApp; } }
	}

	public class LowLevelTestsBuild : NativeStagedBuild
	{
		public LowLevelTestsBuild(UnrealTargetPlatform InPlatform, UnrealTargetConfiguration InConfiguration, string InBuildPath, string InExecutablePath)
			: base(InPlatform, InConfiguration, UnrealTargetRole.Client, InBuildPath, InExecutablePath)
		{
			Flags = BuildFlags.CanReplaceExecutable | BuildFlags.CanReplaceCommandLine;
		}
	}
}
