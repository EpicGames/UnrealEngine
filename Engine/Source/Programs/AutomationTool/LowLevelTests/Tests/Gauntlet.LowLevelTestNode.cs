// Copyright Epic Games, Inc. All Rights Reserved.

using AutomationTool;
using EpicGames.Core;
using Gauntlet;
using System;
using System.IO;
using System.Linq;
using System.Text.RegularExpressions;
using UnrealBuildBase;
using UnrealBuildTool;

using Log = Gauntlet.Log;

namespace LowLevelTests
{
	public class LowLevelTests : BaseTest
	{
		private LowLevelTestContext Context;

		private IAppInstance TestInstance;

		private DateTime SessionStartTime = DateTime.MinValue;

		private TestResult LowLevelTestResult;

		public LowLevelTestsSession LowLevelTestsApp { get; private set; }

		private ILogStreamReader LogReader = null;
		private string[] CurrentProcessedLines;

		public LowLevelTests(LowLevelTestContext InContext)
		{
			Context = InContext;

			MaxDuration = 60 * 30;
			LowLevelTestResult = TestResult.Invalid;
		}

		public string DefaultCommandLine;
		private string ArtifactPath;

		public override string Name { get { return Context != null ? Context.Options.TestApp : "LowLevelTest"; } }

		public override float MaxDuration { protected set; get; }

		private DateTime InactivityStart = DateTime.MinValue;
		private TimeSpan InactivityPeriod = TimeSpan.Zero;

		public override bool IsReadyToStart()
		{
			if (LowLevelTestsApp == null)
			{
				LowLevelTestsApp = new LowLevelTestsSession(Context.BuildInfo, Context.Options);
			}

			return LowLevelTestsApp.TryReserveDevices();
		}

		public override string GetTestSummary()
		{
			return "Low Level Test";
		}

		public override TestResult GetTestResult()
		{
			return LowLevelTestResult;
		}

		public override void SetTestResult(TestResult testResult)
		{
			LowLevelTestResult = testResult;
		}

		public override void AddTestEvent(UnrealTestEvent InEvent)
		{
			if (InEvent.Summary.Equals("Insufficient devices found"))
			{
				Log.Error(KnownLogEvents.Gauntlet_TestEvent, "Test didn't run due to insufficient devices.");
			}
		}

		public override bool StartTest(int Pass, int NumPasses)
		{
			if (LowLevelTestsApp == null)
			{
				throw new AutomationException("Node already has a null LowLevelTestsApp, was IsReadyToStart called?");
			}

			ArtifactPath = Path.Join(Context.Options.LogDir, Context.Options.TestApp);
			Log.Info("LowLevelTestNode.StartTest Creating artifacts path at {0}", ArtifactPath);
			Directory.CreateDirectory(ArtifactPath);

			foreach (ILowLevelTestsExtension LowLevelTestsExtension in Context.BuildInfo.LowLevelTestsExtensions)
			{
				LowLevelTestsExtension.PreRunTests();
			}

			TestInstance = LowLevelTestsApp.InstallAndRunNativeTestApp();
			if (TestInstance != null)
			{
				IDeviceUsageReporter.RecordComment(TestInstance.Device.Name, (UnrealTargetPlatform)TestInstance.Device.Platform, IDeviceUsageReporter.EventType.Device, Context.Options.JobDetails);
				IDeviceUsageReporter.RecordComment(TestInstance.Device.Name, (UnrealTargetPlatform)TestInstance.Device.Platform, IDeviceUsageReporter.EventType.Test, this.GetType().Name);
			}

			if (SessionStartTime == DateTime.MinValue)
			{
				SessionStartTime = DateTime.Now;
			}

			if (TestInstance != null)
			{
				MarkTestStarted();
				LogReader = TestInstance.GetLogBufferReader();
			}

			return TestInstance != null;
		}

		public override void TickTest()
		{
			if (TestInstance != null)
			{
				if (TestInstance.HasExited)
				{
					if (TestInstance.WasKilled)
					{
						LowLevelTestResult = TestResult.Failed;
					}
					MarkTestComplete();
				}
				else
				{
					ParseLowLevelTestsLog();

					// Print stdout when -captureoutput, certain platforms don't always redirect stdout
					PrintLogIfCaptureOutput();

					if (CheckForTimeout())
					{
						Log.Error("Timeout detected from application logged events, stopping.");
						MarkTestComplete();
						LowLevelTestResult = TestResult.TimedOut;
					}
					else if (CurrentProcessedLines != null && CurrentProcessedLines.Length > 0)
					{
						InactivityStart = DateTime.MinValue;
					}
					else if ((CurrentProcessedLines == null || CurrentProcessedLines.Length == 0) && InactivityStart == DateTime.MinValue)
					{
						InactivityStart = DateTime.Now;
					}
					else if (InactivityStart != DateTime.MinValue)
					{
						InactivityPeriod = DateTime.Now - InactivityStart;
					}

					if (Context.Options.Timeout != 0 && InactivityPeriod.TotalMinutes > Context.Options.Timeout + 0.5)
					{
						Log.Error($"Test application didn't log any test events after timeout period of {Context.Options.Timeout} minutes, stopping.");
						MarkTestComplete();
						LowLevelTestResult = TestResult.TimedOut;
					}

					CurrentProcessedLines = null;
				}
			}
		}

		public override void StopTest(StopReason InReason)
		{
			try
			{
				base.StopTest(InReason);

				ParseLowLevelTestsLog();
				PrintLogIfCaptureOutput();

				if (TestInstance != null && !TestInstance.HasExited)
				{
					TestInstance.Kill();
				}

				// Save log artifact
				const string ClientLogFile = "ClientOutput.log";
				string ClientOutputLog = Path.Combine(ArtifactPath, ClientLogFile);
				string LogDir = Path.Combine(Unreal.EngineDirectory.FullName, "Programs", "AutomationTool", "Saved", "Logs");
				if (!TestInstance.WriteOutputToFile(ClientOutputLog))
				{
					Log.Warning("No StdOut returned from low level test app.");
				}
				else
				{
					// Copy to UAT artifacts
					string DestClientLogFile = Path.Combine(LogDir, ClientLogFile);
					TestInstance.WriteOutputToFile(DestClientLogFile);
				}

				bool? ReportCopied = null;
				string ReportPath = null;

				int? ExitCodeOverride = null;

				// No reports from Android tests yet. Since adb shell doesn't forward exit code, we look for it in the log output.
				if (Context.Options.Platform == UnrealTargetPlatform.Android)
				{
					ILogStreamReader AndroidLogReader = TestInstance.GetLogReader();
					string ExitCodeLine = AndroidLogReader.EnumerateNextLines().Where(Line => Line.Contains("Tests finished with exit code")).FirstOrDefault();
					if (!string.IsNullOrEmpty(ExitCodeLine))
					{
						ExitCodeOverride = int.Parse(Regex.Match(ExitCodeLine, @"\d+").Value);
					}
					else
					{
						ExitCodeOverride = -1;
						AndroidLogReader.SetLineIndex(0); // Reset reader
						string CrashLine = AndroidLogReader.EnumerateNextLines().Where(Line => Line.Contains("beginning of crash")).FirstOrDefault();
						if (!string.IsNullOrEmpty(CrashLine))
						{
							Log.Info("Crash occurred during test.");
						}
						else
						{
							Log.Error("Could not find exit code in Android log, assuming failure.");
						}
					}
				}
				else if (!string.IsNullOrEmpty(Context.Options.ReportType))
				{
					ILowLevelTestsReporting LowLevelTestsReporting = Gauntlet.Utils.InterfaceHelpers.FindImplementations<ILowLevelTestsReporting>(true)
						.Where(B => B.CanSupportPlatform(Context.Options.Platform))
						.First();

					try
					{
						ReportPath = LowLevelTestsReporting.CopyDeviceReportTo(LowLevelTestsApp.Install, Context.Options.Platform, Context.Options.TestApp, Context.Options.Build, LogDir);
						ReportCopied = true;
					}
					catch (Exception ex)
					{
						ReportCopied = false;
						Log.Error("Failed to copy report: {0}", ex.ToString());
					}
				}

				string ExitReason = "";
				int ExitCode = ExitCodeOverride.HasValue ? ExitCodeOverride.Value : TestInstance.ExitCode;
				if (TestInstance.WasKilled)
				{
					if (InReason == StopReason.MaxDuration || LowLevelTestResult == TestResult.TimedOut)
					{
						LowLevelTestResult = TestResult.TimedOut;
						ExitReason = "Timed Out";
					}
					else
					{
						LowLevelTestResult = TestResult.Failed;
						ExitReason = $"Process was killed by Gauntlet with reason {InReason.ToString()}.";
					}
				}
				else if (ExitCode != 0)
				{
					LowLevelTestResult = TestResult.Failed;
					ExitReason = $"Process exited with exit code {ExitCode}";
				}
				
				if (ReportCopied.HasValue && !ReportCopied.Value)
				{
					LowLevelTestResult = TestResult.Failed;
					ExitReason = "Unable to read test report";
				}
				else if (ReportPath != null)
				{
					string ReportContents = File.ReadAllText(ReportPath);
					if (Context.Options.LogReportContents) // Some tests prefer to log report contents
					{
						Log.Info(ReportContents);
					}
					string ReportType = Context.Options.ReportType.ToLower();
					if (ReportType == "console")
					{
						LowLevelTestsLogParser LowLevelTestsLogParser = new LowLevelTestsLogParser(ReportContents);
						if (LowLevelTestsLogParser.GetCatchTestResults().Passed)
						{
							LowLevelTestResult = TestResult.Passed;
							ExitReason = "Tests passed";
						}
						else
						{
							LowLevelTestResult = TestResult.Failed;
							ExitReason = "Tests failed according to console report";
						}
					}
					else if (ReportType == "xml")
					{
						LowLevelTestsReportParser LowLevelTestsReportParser = new LowLevelTestsReportParser(ReportContents);
						if (LowLevelTestsReportParser.HasPassed())
						{
							LowLevelTestResult = TestResult.Passed;
							ExitReason = "Tests passed";
						}
						else
						{
							LowLevelTestResult = TestResult.Failed;
							ExitReason = $"Tests failed according to {ReportType} report";
						}
					}
					else if (ReportType == "testdata")
					{
						LowLevelTestsJsonTestDataParser LowLevelTestsJsonTestDataParser = new LowLevelTestsJsonTestDataParser(ReportContents);
						if (LowLevelTestsJsonTestDataParser.HasPassed())
						{
							LowLevelTestResult = TestResult.Passed;
							ExitReason = "Tests passed";
						}
						else
						{
							LowLevelTestResult = TestResult.Failed;
							ExitReason = $"Tests failed according to {ReportType} report";
						}

						MinimalHordeReportGenerator ReportGenerator = new MinimalHordeReportGenerator(
							new RoleReportData()
							{
								Configuration = Context.RoleContext.Configuration,
								Platform = Context.RoleContext.Platform,
								RoleType = "Client",
								SessionStartTime = SessionStartTime,
								TargetDevice = TestInstance.Device,
								InitializationEvents = new System.Collections.Generic.List<UnrealTestEvent>(),
								ClosureEvents = new System.Collections.Generic.List<UnrealTestEvent>()
							});
						ITestReport Report = ReportGenerator.GenerateReport(Context.Options.HordeTestDataKey, this, ReportPath, null);
						ReportGenerator.SubmitToHorde(Report, Name, string.Empty, Context.Options.HordeTestDataKey);
					}
					else
					{
						LowLevelTestResult = TestResult.Failed;
						ExitReason = $"Unknown report type {Context.Options.ReportType}";
						Log.Error(ExitReason);
					}
				}
				else // ReportPath == null
				{
					if (ExitCode != 0)
					{
						LowLevelTestResult = TestResult.Failed;
						ExitReason = "Tests failed (no report to parse)";
					}
					else
					{
						LowLevelTestResult = TestResult.Passed;
						ExitReason = "Tests passed (no report to parse)";
					}
				}
				string ExitMessage = $"Low level test exited with code {ExitCode} and reason: {ExitReason}";
				if (LowLevelTestResult != TestResult.Passed)
				{
					Log.Error(ExitMessage);
				}
				else
				{
					Log.Info(ExitMessage);
				}
			}
			finally 
			{ 
				foreach (ILowLevelTestsExtension LowLevelTestsExtension in Context.BuildInfo.LowLevelTestsExtensions)
				{
					LowLevelTestsExtension.PostRunTests();
				}
			}
		}

		public override void CleanupTest()
		{
			if (LowLevelTestsApp != null)
			{
				LowLevelTestsApp.Dispose();
				LowLevelTestsApp = null;
			}
		}

		private void ParseLowLevelTestsLog()
		{
			// Parse new lines from log, if any
			CurrentProcessedLines = LogReader?.EnumerateNextLines().Where(Line => !string.IsNullOrWhiteSpace(Line)).ToArray();
		}

		private void PrintLogIfCaptureOutput()
		{
			if (CurrentProcessedLines != null && Context.Options.CaptureOutput)
			{
				foreach (string OutputLine in CurrentProcessedLines)
				{
					Console.WriteLine(OutputLine);
				}
			}
		}

		private bool CheckForTimeout()
		{
			if (CurrentProcessedLines == null || CurrentProcessedLines.Length == 0)
			{
				return false;
			}
			foreach (string Line in CurrentProcessedLines)
			{
				if (Line.Contains("Timeout detected"))
				{
					return true;
				}
			}
			return false;
		}
	}
}
