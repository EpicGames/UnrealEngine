// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Diagnostics;
using System.IO;
using EpicGames.Core;
using UnrealBuildTool;
using Gauntlet;
using UnrealBuildBase;
using System.Net.Sockets;

namespace LowLevelTests
{
	public class WebTestsLowLevelTestsExtension : ILowLevelTestsExtension
	{
		// Set to true to use the C++ server (WebTestsServerCpp), false to use the Python server (WebTestsServerPy).
		private static readonly bool bUseCppServer = false;

		private Process ServerProcess;

		public bool IsSupported(UnrealTargetPlatform InPlatform, string InTestApp)
		{
			return InTestApp == "WebTests";
		}

		public string ExtraCommandLine(UnrealTargetPlatform InPlatform, string InTestApp, string InBuildPath)
		{
			string HostIp = UnrealHelpers.GetHostIpAddress();
			Console.WriteLine($"[WebTests] ExtraCommandLine: web_server_ip={HostIp} platform={InPlatform} bUseCppServer={bUseCppServer}");

			// Also disable LogInit which is spamming for each test case, we are only interested in LogHttp channel
			return string.Format("--web_server_ip={0} -LogCmds=\"LogInit off\"", HostIp);
		}

		public void PreRunTests()
		{
			if (bUseCppServer)
			{
				BuildWebTestsServerCpp();
				AsyncLaunchCppServerProcess();
			}
			else
			{
				InstallPythonServer();
				AsyncLaunchPythonServerProcess();
			}
			// If start WebTests right after launching web server process without waiting, it could get refused to connect, especially on Linux
			WaitUntilWebServerPortOpen();
		}

		private string PlatformName()
		{
			return OperatingSystem.IsWindows() ? "Win64" : (OperatingSystem.IsMacOS() ? "Mac" : "Linux");
		}

		// -----------------------------------------------------------------------
		// C++ server
		// -----------------------------------------------------------------------

		private string WebTestsServerCppBinary()
		{
			string ExeName = OperatingSystem.IsWindows() ? "WebTestsServerCpp.exe" : "WebTestsServerCpp";
			return Path.Combine(Unreal.EngineDirectory.FullName, "Binaries", PlatformName(), ExeName);
		}

		private void BuildWebTestsServerCpp()
		{
			string BuildScript = OperatingSystem.IsWindows()
				? Path.Combine(Unreal.EngineDirectory.FullName, "Build", "BatchFiles", "Build.bat")
				: Path.Combine(Unreal.EngineDirectory.FullName, "Build", "BatchFiles", "RunUBT.sh");

			Console.WriteLine($"Building WebTestsServerCpp: {BuildScript} WebTestsServerCpp {PlatformName()} Development");

			ProcessStartInfo StartInfo = new ProcessStartInfo();
			StartInfo.FileName = OperatingSystem.IsWindows() ? BuildScript : "/bin/sh";
			StartInfo.Arguments = OperatingSystem.IsWindows()
				? $"WebTestsServerCpp {PlatformName()} Development"
				: $"{BuildScript} WebTestsServerCpp {PlatformName()} Development";
			StartInfo.UseShellExecute = false;
			StartInfo.CreateNoWindow = true;
			StartInfo.RedirectStandardOutput = true;
			StartInfo.RedirectStandardError = true;

			Process BuildProcess = new Process();
			BuildProcess.StartInfo = StartInfo;
			BuildProcess.Start();

			string Output = BuildProcess.StandardOutput.ReadToEnd();
			string Error = BuildProcess.StandardError.ReadToEnd();
			BuildProcess.WaitForExit();

			if (!string.IsNullOrWhiteSpace(Output)) Console.WriteLine(Output);
			if (!string.IsNullOrWhiteSpace(Error))  Console.WriteLine(Error);

			if (BuildProcess.ExitCode != 0)
			{
				throw new Exception($"Failed to build WebTestsServerCpp (exit code {BuildProcess.ExitCode}).");
			}

			Console.WriteLine($"WebTestsServerCpp built successfully. Binary: {WebTestsServerCppBinary()}");
		}

		private void AsyncLaunchCppServerProcess()
		{
			string Binary = WebTestsServerCppBinary();
			if (!File.Exists(Binary))
			{
				throw new FileNotFoundException($"WebTestsServerCpp binary not found at: {Binary}");
			}

			Console.WriteLine($"Launching web server: {Binary} (platform: {PlatformName()}, exists: {File.Exists(Binary)})");

			ProcessStartInfo StartInfo = new ProcessStartInfo();
			StartInfo.FileName = Binary;
			StartInfo.Arguments = "";
			StartInfo.UseShellExecute = false;
			StartInfo.CreateNoWindow = true;

			ServerProcess = new Process();
			ServerProcess.StartInfo = StartInfo;
			ServerProcess.Start();

			Console.WriteLine($"Web server process started (PID {ServerProcess.Id}).");
		}

		// -----------------------------------------------------------------------
		// Python server
		// -----------------------------------------------------------------------

		private string WebTestsServerPyDir()
		{
			return Path.Combine(Unreal.EngineDirectory.FullName, "Source", "Programs", "WebTestsServerPy");
		}

		private void InstallPythonServer()
		{
			ProcessStartInfo StartInfo = new ProcessStartInfo();
			StartInfo.WorkingDirectory = WebTestsServerPyDir();
			StartInfo.FileName = OperatingSystem.IsWindows() ? "cmd.exe" : "/bin/sh";
			StartInfo.Arguments = OperatingSystem.IsWindows() ? "/c createenv.bat" : "-c './createenv.sh'";
			StartInfo.UseShellExecute = false;
			StartInfo.CreateNoWindow = true;

			Process InstallProcess = new Process();
			InstallProcess.StartInfo = StartInfo;
			InstallProcess.Start();
			InstallProcess.WaitForExit();

			Console.WriteLine("Requirements installed.");
		}

		private void AsyncLaunchPythonServerProcess()
		{
			string WorkingDir = WebTestsServerPyDir();
			string PythonFile = Path.Combine(WorkingDir, "env", OperatingSystem.IsWindows() ? "Scripts" : "bin", OperatingSystem.IsWindows() ? "python.exe" : "python");

			ProcessStartInfo StartInfo = new ProcessStartInfo();
			StartInfo.WorkingDirectory = WorkingDir;
			StartInfo.FileName = PythonFile;
			StartInfo.WindowStyle = ProcessWindowStyle.Normal;
			StartInfo.Arguments = "-m daphne -e tcp:port=8000:interface=0.0.0.0 -e tcp:port=8001:interface=0.0.0.0 webtests.asgi:application";
			StartInfo.UseShellExecute = true;
			StartInfo.CreateNoWindow = false;

			ServerProcess = new Process();
			ServerProcess.StartInfo = StartInfo;
			ServerProcess.Start();

			Console.WriteLine("Web server process is now running.");
		}

		// -----------------------------------------------------------------------
		// Shared
		// -----------------------------------------------------------------------

		private void WaitUntilWebServerPortOpen()
		{
			// Use localhost for the port check — the server binds to 0.0.0.0/127.0.0.1 locally.
			// GetHostIpAddress() returns the LAN IP used by remote test devices, not for local checks.
			string IpAddress = "127.0.0.1";
			string HostIp = UnrealHelpers.GetHostIpAddress();
			Console.WriteLine($"Waiting for web server ports 8000 and 8001 on {IpAddress} (host LAN IP: {HostIp})...");

			Stopwatch sw = new Stopwatch();
			sw.Start();

			bool Port8000Open = false;
			bool Port8001Open = false;

			while (!Port8000Open || !Port8001Open)
			{
				if (bUseCppServer && ServerProcess.HasExited)
				{
					throw new Exception($"Web server process exited unexpectedly (exit code {ServerProcess.ExitCode}) before ports opened.");
				}

				if (sw.ElapsedMilliseconds > 60000)
				{
					sw.Stop();
					throw new TimeoutException($"Server ports did not open within the specified time. Port 8000 open: {Port8000Open}, Port 8001 open: {Port8001Open}.");
				}

				bool Was8000 = Port8000Open;
				bool Was8001 = Port8001Open;
				Port8000Open = Port8000Open || IsServerPortOpen(IpAddress, 8000);
				Port8001Open = Port8001Open || IsServerPortOpen(IpAddress, 8001);
				if (!Was8000 && Port8000Open) Console.WriteLine($"Port 8000 is now open ({sw.ElapsedMilliseconds}ms).");
				if (!Was8001 && Port8001Open) Console.WriteLine($"Port 8001 is now open ({sw.ElapsedMilliseconds}ms).");

				System.Threading.Thread.Sleep(1000);
			}

			sw.Stop();

			Console.WriteLine($"Web server ports 8000 and 8001 are now open (took {sw.ElapsedMilliseconds}ms).");
		}

		private bool IsServerPortOpen(string ipAddress, int port)
		{
			using (TcpClient client = new TcpClient())
			{
				try
				{
					client.Connect(ipAddress, port);
					return true;
				}
				catch
				{
					return false;
				}
			}
		}

		public void PostRunTests()
		{
			CloseWebServer();
		}

		private void CloseWebServer()
		{
			if (ServerProcess != null)
			{
				if (bUseCppServer)
				{
					ServerProcess.Kill();
					ServerProcess.WaitForExit();
				}
				else
				{
					ServerProcess.CloseMainWindow();
				}
				ServerProcess = null;

				Console.WriteLine("Web server killed.");
			}
		}
	}
}
