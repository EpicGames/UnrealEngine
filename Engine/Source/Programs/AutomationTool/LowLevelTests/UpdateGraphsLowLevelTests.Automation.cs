// Copyright Epic Games, Inc. All Rights Reserved.

using System.Collections.Generic;
using AutomationTool;

using UnrealBuildBase;
using Gauntlet;
using Microsoft.Extensions.FileSystemGlobbing.Abstractions;
using Microsoft.Extensions.FileSystemGlobbing;
using System.IO;
using EpicGames.Core;
using Microsoft.Extensions.Logging;
using System.Linq;

namespace LowLevelTests
{
	using Log = Gauntlet.Log;
	using LogLevel = Gauntlet.LogLevel;

	[RequireP4]
	[Help("Perforce helper to checkout all related LowLevelTest Build.xml files into perforce.")]
	[Help("EditCl=<CL>","(optional) the CL we wish to append graph changes to. Defaults to 'default'")]
	public class UpdateGraphsLowLevelTests : BuildCommand
	{
		public override ExitCode Execute()
		{
			Log.Level = LogLevel.VeryVerbose;
			Globals.Params = new Params(Params);

			int EditCL = ParseParamNullableInt("EditCL") ?? 0;

			List<string> BuildPaths = new List<string>() { "Build/LowLevelTests/*.xml"};
			Matcher matcher = new();
			matcher.AddIncludePatterns(BuildPaths);


			List<string> LowLevelTestXml = new List<string>();
			foreach(DirectoryReference EngineBase in GetAllEngineDirectories())
			{
				foreach (DirectoryReference ExtendedEngineDir in Unreal.GetExtensionDirs(EngineBase))
				{
					Logger.LogInformation(ExtendedEngineDir.FullName);
					IEnumerable<string> matchingFiles = matcher.GetResultsInFullPath(ExtendedEngineDir.FullName);
					foreach (string v in matchingFiles)
					{
						Logger.LogInformation(v);
						LowLevelTestXml.Add(v);
					}
				}
			}

			if(EditCL == 0)
			{
				P4.BatchedCommand("edit", LowLevelTestXml);
			}
			else
			{
				P4.BatchedCommand($"edit -c {EditCL}", LowLevelTestXml);
			}

			//TODO(SMA): Enable bUpdateBuildGraphPropertiesFile in config and run UpdateBuildGraphPropertiesFile on all relevant .Build.cs files.
			//Note: The metadata files are now generated/updated using RunUBT.bat -Mode=Test -GenerateMetadata

			return ExitCode.Success;
		}
	}
}
