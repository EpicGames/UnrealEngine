// Copyright Epic Games, Inc. All Rights Reserved.

using Gauntlet;
using System;
using System.Text.Json;

namespace LowLevelTests
{
	public class LowLevelTestsJsonTestDataParser
	{
		private JsonDocument ReportDoc;
		public bool IsValid { get; protected set; }

		public LowLevelTestsJsonTestDataParser(string InContents)
		{
			try
			{
				ReportDoc = JsonDocument.Parse(InContents);
				IsValid = true;
			}
			catch (Exception ParseEx)
			{
				Log.Warning("Encountered error while parsing report {0}", ParseEx.ToString());
				IsValid = false;
			}
		}

		public bool HasPassed()
		{
			if (!IsValid)
			{
				return false;
			}

			try
			{
				JsonElement root = ReportDoc.RootElement;

				// Read the summary statistics from the JSON report
				int succeeded = root.GetProperty("Succeeded").GetInt32();
				int failed = root.GetProperty("Failed").GetInt32();

				// Test passes if there are no failures and at least one success
				return failed == 0 && succeeded > 0;
			}
			catch (Exception Ex)
			{
				Log.Warning("Encountered error while reading test results from report: {0}", Ex.ToString());
				return false;
			}
		}
	}
}
