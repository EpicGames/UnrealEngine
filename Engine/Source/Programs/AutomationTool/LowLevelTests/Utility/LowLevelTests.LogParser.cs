// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Linq;
using System.Text.RegularExpressions;

namespace LowLevelTests
{
	public class LowLevelTestsLogParser
	{
		public class CatchTestResults
		{
			public bool Passed = false;
			public CatchAssertionResult Assertions;
			public CatchTestCaseResult TestCases;

			public CatchTestResults()
			{
				Assertions = new CatchAssertionResult();
				TestCases = new CatchTestCaseResult();
			}
		}

		public class CatchTestCaseResult
		{
			public string TotalCases;
			public string Passed;
			public string Failed;
			public string Skipped;

			public CatchTestCaseResult()
			{
				TotalCases = "";
				Passed = "";
				Failed = "";
				Skipped = "";
			}
		}
		public class CatchAssertionResult
		{
			public string TotalAssertions;
			public string Passed;
			public string Failed;

			public CatchAssertionResult()
			{
				TotalAssertions = "";
				Passed = "";
				Failed = "";
			}
		}

		internal class ParsePatterns
		{
			internal const string AllCatchTestPassed = @"All tests passed\s\((?<assertions>[\w\d]+)\sassertion[s]?\sin\s(?<testcases>[\w\d]+)\stest\scase[s]?.*\)";

			internal const string CatchTestCaseResults = @"test\scase[s]?\:\s+(?<totalcases>[\d]+)\s+";
			internal const string CatchPassedCases = @"(?<passed>[\d]+)\s+passed";
			internal const string CatchFailedCases = @"(?<failed>[\d]+)\s+failed";
			internal const string CatchSkippedCases = @"(?<skipped>[\d]+)\s+skipped";


			internal const string CatchTestAssertionResults = @"assertion[s]?\:\s+(?<totalcases>[\d]+)\s+\|\s+(?<passed>[\d]+)\s+passed\s+\|\s+(?<failed>[\d]+)\s+failed";

			internal const string CatchThrowException = "Catch will terminate because it needed to throw an exception.";
		}

		private string Content;

		public LowLevelTestsLogParser(string InContent)
		{
			Content = InContent;
		}

		public CatchTestResults GetCatchTestResults()
		{
			var TestResults = new CatchTestResults { };

			if (Regex.Matches(Content, ParsePatterns.CatchThrowException).Count > 0)
			{
				TestResults.Passed = false;
			}

			var AllCatchTestPassesPattern = ParsePatterns.AllCatchTestPassed;

			MatchCollection mCollection = Regex.Matches(Content, AllCatchTestPassesPattern);

			//If all the tests pass, report it and return.
			foreach (Match aMatch in mCollection)
			{
				if (aMatch.Success)
				{
					TestResults.Passed = true;
					TestResults.Assertions = new CatchAssertionResult
					{
						TotalAssertions = aMatch.Groups["assertions"].ToString()
					};
					TestResults.TestCases = new CatchTestCaseResult
					{
						TotalCases = aMatch.Groups["testcases"].ToString()
					};
					return TestResults;
				}
			}

			// If we did not find any matching test passed string from catch, check to see test results of pass/fail/skip were logged.
			Match testResultsMatch = Regex.Matches(Content, ParsePatterns.CatchTestCaseResults).FirstOrDefault();

			if (testResultsMatch != null && testResultsMatch.Success)
			{
				CatchTestCaseResult TestCaseResult = new CatchTestCaseResult()
				{
					TotalCases = testResultsMatch.Groups["totalcases"].ToString(),
				};
				
				Match testPassedMatch = Regex.Matches(Content, ParsePatterns.CatchPassedCases).FirstOrDefault();
				if (testPassedMatch != null && testPassedMatch.Success)
				{
					TestCaseResult.Passed = testPassedMatch.Groups["passed"].ToString();
				}

				Match testFailedMatch = Regex.Matches(Content, ParsePatterns.CatchFailedCases).FirstOrDefault();
				if (testFailedMatch != null && testFailedMatch.Success)
				{
					TestCaseResult.Failed = testFailedMatch.Groups["failed"].ToString();
				}

				Match testSkippedMatch = Regex.Matches(Content, ParsePatterns.CatchSkippedCases).FirstOrDefault();
				if (testSkippedMatch != null && testSkippedMatch.Success)
				{
					TestCaseResult.Skipped = testSkippedMatch.Groups["skipped"].ToString();
				}

				TestResults.Passed = string.IsNullOrEmpty(TestCaseResult.Failed) || Convert.ToInt32(TestCaseResult.Failed) == 0;
				TestResults.TestCases = TestCaseResult;
			}

			MatchCollection aCollection = Regex.Matches(Content, ParsePatterns.CatchTestAssertionResults);

			foreach (Match aMatch in aCollection)
			{
				if (aMatch.Success)
				{
					TestResults.Passed = false;
					TestResults.Assertions = new CatchAssertionResult()
					{
						TotalAssertions = aMatch.Groups["totalcases"].ToString(),
						Passed = aMatch.Groups["passed"].ToString(),
						Failed = aMatch.Groups["failed"].ToString()
					};
				}
			}

			return TestResults;
		}
	}
}
