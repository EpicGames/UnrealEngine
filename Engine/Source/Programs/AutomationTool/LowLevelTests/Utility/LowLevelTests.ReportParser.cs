// Copyright Epic Games, Inc. All Rights Reserved.

using Gauntlet;
using System;
using System.Linq;
using System.Xml.Linq;

namespace LowLevelTests
{
	public class LowLevelTestsReportParser
	{
		private XDocument ReportDoc;
		public bool IsValid { get; protected set; }

		public LowLevelTestsReportParser(string InContents)
		{
			try
			{
				ReportDoc = XDocument.Parse(InContents);
				IsValid = true;
			}
			catch (Exception ParseEx)
			{
				Log.Warning("Encountered error while parsing report {0}", ParseEx.ToString());
				IsValid = false;
			}
		}

		public bool HasPassed()
		{
			if (!IsValid)
			{
				return false;
			}

			int NrOverallResultsFailures = -1;
			int NrOverallResultsSuccesses = -1;
			int NrOverallResultsCasesFailures = -1;
			int NrOverallResultsCasesSuccesses = -1;
			// OverallResults also exists in each SECTION of every test case, we need to use the last one in the end of the xml report
			XElement OverallResults = ReportDoc.Descendants("OverallResults").LastOrDefault();
			if (OverallResults != null)
			{
				NrOverallResultsFailures = int.Parse(OverallResults.Attribute("failures").Value);
				NrOverallResultsSuccesses = int.Parse(OverallResults.Attribute("successes").Value);
			}
			XElement OverallResultsCases = ReportDoc.Descendants("OverallResultsCases").LastOrDefault();
			if (OverallResultsCases != null)
			{
				NrOverallResultsCasesFailures = int.Parse(OverallResultsCases.Attribute("failures").Value);
				NrOverallResultsCasesSuccesses = int.Parse(OverallResultsCases.Attribute("successes").Value);
			}
			
			return NrOverallResultsFailures == 0 && NrOverallResultsCasesFailures == 0 && NrOverallResultsSuccesses > 0 && NrOverallResultsCasesSuccesses > 0;
		}
	}
}
