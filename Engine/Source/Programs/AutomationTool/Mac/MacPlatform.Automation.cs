// Copyright Epic Games, Inc. All Rights Reserved.
using System;
using System.Collections.Generic;
using System.Linq;
using System.Text;
using System.Text.RegularExpressions;
using System.IO;
using AutomationTool;
using UnrealBuildTool;
using UnrealBuildBase;
using EpicGames.Core;
using Microsoft.Extensions.Logging;
using System.Security.Cryptography;

public class MacPlatform : ApplePlatform
{
	/// <summary>
	/// Default architecture to build projects for. Defaults to Intel
	/// </summary>
	protected UnrealArchitectures ProjectTargetArchitectures = new(UnrealArch.X64);

	public MacPlatform()
		: base(UnrealTargetPlatform.Mac)
	{

	}

	public override void PlatformSetupParams(ref ProjectParams ProjParams)
	{
		base.PlatformSetupParams(ref ProjParams);

		string ConfigTargetArchicture = "";
		ConfigHierarchy PlatformEngineConfig = null;
		if (ProjParams.EngineConfigs.TryGetValue(PlatformType, out PlatformEngineConfig))
		{
			PlatformEngineConfig.GetString("/Script/MacTargetPlatform.MacTargetSettings", "TargetArchitecture", out ConfigTargetArchicture);

			if (ConfigTargetArchicture.ToLower().Contains("intel"))
			{
				ProjectTargetArchitectures = new UnrealArchitectures(UnrealArch.X64);
			}
			else if (ConfigTargetArchicture.ToLower().Contains("apple"))
			{
				ProjectTargetArchitectures = new UnrealArchitectures(UnrealArch.Arm64);
			}
			else if (ConfigTargetArchicture.ToLower().Contains("universal"))
			{
				ProjectTargetArchitectures = new UnrealArchitectures(new[] { UnrealArch.X64, UnrealArch.Arm64 });
			}
		}		
	}

	public override void PersistSdkRootVar()
	{
		string UeSdksRoot = Environment.GetEnvironmentVariable("UE_SDKS_ROOT");
		if (UeSdksRoot != null)
		{
			base.PersistSdkRootVar();
			string AutoSdkFile = Path.Combine(Environment.GetFolderPath(Environment.SpecialFolder.ApplicationData), ".autosdk");
			if (!File.Exists(AutoSdkFile))
			{
				using (StreamWriter wr = new StreamWriter(AutoSdkFile))
				{
					wr.WriteLine(UeSdksRoot);
				}
			}
		}
	}

	public override DeviceInfo[] GetDevices()
	{
		List<DeviceInfo> Devices = new List<DeviceInfo>();

		if (HostPlatform.Current.HostEditorPlatform == TargetPlatformType)
		{
			DeviceInfo LocalMachine = new DeviceInfo(TargetPlatformType, Unreal.MachineName, Unreal.MachineName,
				Environment.OSVersion.Version.ToString(), "Computer", true, true);

			Devices.Add(LocalMachine);
		}

		return Devices.ToArray();
	}

	public override string GetCookPlatform(bool bDedicatedServer, bool bIsClientOnly)
	{
		const string NoEditorCookPlatform = "Mac";
		const string ServerCookPlatform = "MacServer";
		const string ClientCookPlatform = "MacClient";

		if (bDedicatedServer)
		{
			return ServerCookPlatform;
		}
		else if (bIsClientOnly)
		{
			return ClientCookPlatform;
		}
		else
		{
			return NoEditorCookPlatform;
		}
	}

	public override string GetEditorCookPlatform()
	{
		return "MacEditor";
	}

	/// <summary>
	/// Override PreBuildAgenda so we can control the architecture that targets are built for based on
	/// project settings and the current user environment
	/// </summary>
	/// <param name="Build"></param>
	/// <param name="Agenda"></param>
	/// <param name="Params"></param>
	public override void PreBuildAgenda(UnrealBuild Build, UnrealBuild.BuildAgenda Agenda, ProjectParams Params)
	{
		base.PreBuildAgenda(Build, Agenda, Params);

		// Go through the agenda for all targets and set the architecture if needed
		foreach (UnrealBuild.BuildTarget Target in Agenda.Targets)
		{
			// if building for Distribution, and no arch is already specified, then get the distro architectures and use that for this build
			// editors aren't usually distributed, so if we are doing distribution, it's probably not for the editor target, and we don't want to make universal 
			// editors just to make a distribution client
			if (Params.Distribution && !Target.TargetName.Contains("Editor") && !Target.UBTArgs.ToLower().Contains("-architecture="))
			{
				UnrealArchitectures DistroArches = UnrealArchitectureConfig.ForPlatform(UnrealTargetPlatform.Mac).DistributionArchitectures(Params.RawProjectPath, Target.TargetName);
				Target.UBTArgs += " -architecture=" + DistroArches.ToString();
			}
		}
	}

	private void StageAppBundle(DeploymentContext SC, DirectoryReference InPath, StagedDirectoryReference NewName)
	{
		// Files with DebugFileExtensions should always be DebugNonUFS
		List<string> DebugExtensions = GetDebugFileExtensions();
		if(DirectoryExists(InPath.FullName))
		{
			foreach (FileReference InputFile in DirectoryReference.EnumerateFiles(InPath, "*", SearchOption.AllDirectories))
			{
				StagedFileReference OutputFile = StagedFileReference.Combine(NewName, InputFile.MakeRelativeTo(InPath));
				StagedFileType FileType = DebugExtensions.Any(x => InputFile.HasExtension(x)) ? StagedFileType.DebugNonUFS : StagedFileType.NonUFS;
				SC.StageFile(FileType, InputFile, OutputFile);
			}
		}
	}

	public override void GetFilesToDeployOrStage(ProjectParams Params, DeploymentContext SC)
	{
		// Stage all the build products
		foreach (StageTarget Target in SC.StageTargets)
		{
			SC.StageBuildProductsFromReceipt(Target.Receipt, Target.RequireFilesExist, Params.bTreatNonShippingBinariesAsDebugFiles);
		}

		if (SC.bStageCrashReporter)
		{
			StagedDirectoryReference CrashReportClientPath = StagedDirectoryReference.Combine("Engine/Binaries", SC.PlatformDir, "CrashReportClient.app");
			StageAppBundle(SC, DirectoryReference.Combine(SC.LocalRoot, "Engine/Binaries", SC.PlatformDir, "CrashReportClient.app"), CrashReportClientPath);
		}

		// Find the app bundle path
		List<FileReference> Exes = GetExecutableNames(SC);
		foreach (var Exe in Exes)
		{
			StagedDirectoryReference AppBundlePath = null;
			if (Exe.IsUnderDirectory(DirectoryReference.Combine(SC.RuntimeProjectRootDir, "Binaries", SC.PlatformDir)))
			{
				AppBundlePath = StagedDirectoryReference.Combine(SC.ShortProjectName, "Binaries", SC.PlatformDir, Path.GetFileNameWithoutExtension(Exe.FullName) + ".app");
			}
			else if (Exe.IsUnderDirectory(DirectoryReference.Combine(SC.RuntimeRootDir, "Engine/Binaries", SC.PlatformDir)))
			{
				AppBundlePath = StagedDirectoryReference.Combine("Engine/Binaries", SC.PlatformDir, Path.GetFileNameWithoutExtension(Exe.FullName) + ".app");
			}

			// Copy the custom icon and Steam dylib, if needed
			if (AppBundlePath != null)
			{
				FileReference AppIconsFile = FileReference.Combine(SC.ProjectRoot, "Build", "Mac", "Application.icns");
				if(FileReference.Exists(AppIconsFile))
				{
					SC.StageFile(StagedFileType.NonUFS, AppIconsFile, StagedFileReference.Combine(AppBundlePath, "Contents", "Resources", "Application.icns"));
				}
			}
		}

		// Copy the splash screen, Mac specific
		FileReference SplashImage = FileReference.Combine(SC.ProjectRoot, "Content", "Splash", "Splash.bmp");
		if(FileReference.Exists(SplashImage))
		{
			SC.StageFile(StagedFileType.NonUFS, SplashImage);
		}

		// Copy the ShaderCache files, if they exist
		FileReference DrawCacheFile = FileReference.Combine(SC.ProjectRoot, "Content", "DrawCache.ushadercache");
		if(FileReference.Exists(DrawCacheFile))
		{
			SC.StageFile(StagedFileType.UFS, DrawCacheFile);
		}

		FileReference ByteCodeCacheFile = FileReference.Combine(SC.ProjectRoot, "Content", "ByteCodeCache.ushadercode");
		if(FileReference.Exists(ByteCodeCacheFile))
		{
			SC.StageFile(StagedFileType.UFS, ByteCodeCacheFile);
		}

		{
			// Get the final output directory for cooked data
			DirectoryReference CookOutputDir;
			if(!String.IsNullOrEmpty(Params.CookOutputDir))
			{
				CookOutputDir = DirectoryReference.Combine(new DirectoryReference(Params.CookOutputDir), SC.CookPlatform);
			}
			else if(Params.CookInEditor)
			{
				CookOutputDir = DirectoryReference.Combine(SC.ProjectRoot, "Saved", "EditorCooked", SC.CookPlatform);
			}
			else
			{
				CookOutputDir = DirectoryReference.Combine(SC.ProjectRoot, "Saved", "Cooked", SC.CookPlatform);
			}
		}
	}

	string GetValueFromInfoPlist(string InfoPlist, string Key, string DefaultValue = "")
	{
		string Value = DefaultValue;
		string KeyString = "<key>" + Key + "</key>";
		int KeyIndex = InfoPlist.IndexOf(KeyString);
		if (KeyIndex > 0)
		{
			int ValueStartIndex = InfoPlist.IndexOf("<string>", KeyIndex + KeyString.Length) + "<string>".Length;
			int ValueEndIndex = InfoPlist.IndexOf("</string>", ValueStartIndex);
			if (ValueStartIndex > 0 && ValueEndIndex > ValueStartIndex)
			{
				Value = InfoPlist.Substring(ValueStartIndex, ValueEndIndex - ValueStartIndex);
			}
		}
		return Value;
	}

	void StageBootstrapExecutable(DeploymentContext SC, string ExeName, string TargetFile, string StagedRelativeTargetPath, string StagedArguments)
	{
		DirectoryReference InputApp = DirectoryReference.Combine(SC.LocalRoot, "Engine", "Binaries", SC.PlatformDir, "BootstrapPackagedGame.app");
		if (InternalUtils.SafeDirectoryExists(InputApp.FullName))
		{
			// Create the new bootstrap program
			DirectoryReference IntermediateDir = DirectoryReference.Combine(SC.ProjectRoot, "Intermediate", "Staging");
			InternalUtils.SafeCreateDirectory(IntermediateDir.FullName);

			DirectoryReference IntermediateApp = DirectoryReference.Combine(IntermediateDir, ExeName);
			if (DirectoryReference.Exists(IntermediateApp))
			{
				DirectoryReference.Delete(IntermediateApp, true);
			}
			CloneDirectory(InputApp.FullName, IntermediateApp.FullName);

			// Rename the executable
			string GameName = Path.GetFileNameWithoutExtension(ExeName);
			FileReference.Move(FileReference.Combine(IntermediateApp, "Contents", "MacOS", "BootstrapPackagedGame"), FileReference.Combine(IntermediateApp, "Contents", "MacOS", GameName));

			// Copy the icon
			string SrcInfoPlistPath = CombinePaths(TargetFile, "Contents", "Info.plist");
			string SrcInfoPlist = File.ReadAllText(SrcInfoPlistPath);

			string IconName = GetValueFromInfoPlist(SrcInfoPlist, "CFBundleIconFile");
			if (!string.IsNullOrEmpty(IconName))
			{
				string IconPath = CombinePaths(TargetFile, "Contents", "Resources", IconName + ".icns");
				InternalUtils.SafeCreateDirectory(CombinePaths(IntermediateApp.FullName, "Contents", "Resources"));
				File.Copy(IconPath, CombinePaths(IntermediateApp.FullName, "Contents", "Resources", IconName + ".icns"));
			}

			// Update Info.plist contents
			string DestInfoPlistPath = CombinePaths(IntermediateApp.FullName, "Contents", "Info.plist");
			string DestInfoPlist = File.ReadAllText(DestInfoPlistPath);

			string AppIdentifier = GetValueFromInfoPlist(SrcInfoPlist, "CFBundleIdentifier");
			if (AppIdentifier == "com.epicgames.UnrealGame")
			{
				AppIdentifier = "";
			}

			string Copyright = GetValueFromInfoPlist(SrcInfoPlist, "NSHumanReadableCopyright");
			string BundleVersion = GetValueFromInfoPlist(SrcInfoPlist, "CFBundleVersion", "1");
			string ShortVersion = GetValueFromInfoPlist(SrcInfoPlist, "CFBundleShortVersionString", "1.0");

			DestInfoPlist = DestInfoPlist.Replace("com.epicgames.BootstrapPackagedGame", string.IsNullOrEmpty(AppIdentifier) ? "com.epicgames." + GameName + "_bootstrap" : AppIdentifier + "_bootstrap");
			DestInfoPlist = DestInfoPlist.Replace("BootstrapPackagedGame", GameName);
			DestInfoPlist = DestInfoPlist.Replace("__UE4_ICON_FILE__", IconName);
			DestInfoPlist = DestInfoPlist.Replace("__UE4_APP_TO_LAUNCH__", StagedRelativeTargetPath);
			DestInfoPlist = DestInfoPlist.Replace("__UE4_COMMANDLINE__", StagedArguments);
			DestInfoPlist = DestInfoPlist.Replace("__UE4_COPYRIGHT__", Copyright);
			DestInfoPlist = DestInfoPlist.Replace("__UE4_BUNDLE_VERSION__", BundleVersion);
			DestInfoPlist = DestInfoPlist.Replace("__UE4_SHORT_VERSION__", ShortVersion);

			File.WriteAllText(DestInfoPlistPath, DestInfoPlist);

			StageAppBundle(SC, IntermediateApp, new StagedDirectoryReference(ExeName));
		}
	}

	private void RemoveExtraRPaths(ProjectParams Params, DeploymentContext SC)
	{
		// When we link the executable we add RPATH entries for all possible places where dylibs can be loaded from, so that the same executable can be used from Binaries/Mac
		// as well as in a packaged, self-contained application. In recent versions of macOS, Gatekeeper doesn't allow RPATHs pointing to folders that don't exist,
		// so we remove these based on the type of packaging (Params.CreateAppBundle).
		List<FileReference> Exes = GetExecutableNames(SC);
		foreach (var ExePath in Exes)
		{
			IProcessResult CommandResult = Run("otool", "-l \"" + ExePath + "\"", null, ERunOptions.None);
			if (CommandResult.ExitCode == 0)
			{
				StringReader Reader = new StringReader(CommandResult.Output);
				Regex RPathPattern = new Regex(@"^\s+path (?<rpath>.+)\s\(offset");
				string ToRemovePattern = Params.CreateAppBundle ? "/../../../" : "@loader_path/../UE4/";

				string OutputLine;
				while ((OutputLine = Reader.ReadLine()) != null)
				{
					if (OutputLine.EndsWith("cmd LC_RPATH"))
					{
						OutputLine = Reader.ReadLine();
						OutputLine = Reader.ReadLine();
						Match RPathMatch = RPathPattern.Match(OutputLine);
						if (RPathMatch.Success)
						{
							string RPath = RPathMatch.Groups["rpath"].Value;
							if (RPath.Contains(ToRemovePattern))
							{
								Run("xcrun", "install_name_tool -delete_rpath \"" + RPath + "\" \"" + ExePath + "\"", null, ERunOptions.NoStdOutCapture);
							}
						}
					}
				}
			}
		}
	}

	private void FixupFrameworks(string TargetPath)
	{
		DirectoryReference TargetCEFDir = DirectoryReference.Combine(new DirectoryReference(TargetPath), "Engine/Binaries/ThirdParty/CEF3/Mac");
		DirectoryReference X86Framework = DirectoryReference.Combine(TargetCEFDir, "Chromium Embedded Framework x86.framework");
		DirectoryReference X86Versions = DirectoryReference.Combine(X86Framework, "Versions");
		DirectoryReference Arm64Framework = DirectoryReference.Combine(TargetCEFDir, "Chromium Embedded Framework arm64.framework");
		DirectoryReference Arm64Versions = DirectoryReference.Combine(Arm64Framework, "Versions");

		DirectoryReference EngineCEFDir = DirectoryReference.Combine(Unreal.EngineDirectory, "Binaries/ThirdParty/CEF3/Mac");
		FileReference X86Zip = FileReference.Combine(EngineCEFDir, "Chromium Embedded Framework x86.framework.zip");
		FileReference Arm64Zip = FileReference.Combine(EngineCEFDir, "Chromium Embedded Framework arm64.framework.zip");

		// if the archive has a framework without Versions directory, it won't be allowed for App Store submission, so replace it with the zipped version
		// that has the proper symlinks 
		if (DirectoryReference.Exists(X86Framework) && !DirectoryReference.Exists(X86Versions))
		{
			Logger.LogInformation($"Replacing {X86Framework} with {X86Zip}...");

			DirectoryReference.Delete(X86Framework, true);
			Utils.RunLocalProcessAndLogOutput("/usr/bin/unzip", $"-q -o \"{X86Zip}\" -d \"{TargetCEFDir}\" -x \"__MACOSX/*\" \"*.DS_Store\"", Logger);
		}
		if (DirectoryReference.Exists(Arm64Framework) && !DirectoryReference.Exists(Arm64Versions))
		{
			Logger.LogInformation($"Replacing {Arm64Framework} with {Arm64Zip}...");

			DirectoryReference.Delete(Arm64Framework, true);
			Utils.RunLocalProcessAndLogOutput("/usr/bin/unzip", $"-q -o \"{Arm64Zip}\" -d \"{TargetCEFDir}\" -x \"__MACOSX/*\" \"*.DS_Store\"", Logger);
		}
	}

	public override void ProcessArchivedProject(ProjectParams Params, DeploymentContext SC)
	{
		// nothing to do. This is handled by xcode
	}

	public override IProcessResult RunClient(ERunOptions ClientRunFlags, string ClientApp, string ClientCmdLine, ProjectParams Params)
	{
		// xcode creates a full .app in the root of the Staged dir, but ClientApp as passed in is in Staged/Project/Binaries/Mac, which exists, but is not a full app
		string ExeName = Path.GetFileNameWithoutExtension(ClientApp);
		Int32 BaseDirLen = Params.BaseStageDirectory.Length;

		string StageSubDir = ClientApp.Substring(BaseDirLen, ClientApp.IndexOf("/", BaseDirLen + 1) - BaseDirLen);
		ClientApp = CombinePaths(Params.BaseStageDirectory, StageSubDir, $"{ExeName}.app/Contents/MacOS/{ExeName}");
		if (!File.Exists(ClientApp))
		{
			// Could be blueprint only projects which ClientApp would be pointing at non-existing UnrealGame/UnrealClient
			ExeName = Params.RawProjectPath.GetFileNameWithoutAnyExtensions();
			ClientApp = CombinePaths(Params.BaseStageDirectory, StageSubDir, $"{ExeName}.app/Contents/MacOS/{ExeName}");
		}

		PushDir(Path.GetDirectoryName(ClientApp));
		// Always start client process and don't wait for exit.
		IProcessResult ClientProcess = Run(ClientApp, ClientCmdLine, null, ClientRunFlags | ERunOptions.NoWaitForExit);
		PopDir();

		return ClientProcess;
	}

	public override bool IsSupported { get { return true; } }

	public override List<string> GetDebugFileExtensions()
	{
		return new List<string> { ".dSYM" };
	}
	public override bool CanHostPlatform(UnrealTargetPlatform Platform)
	{
		if (Platform == UnrealTargetPlatform.IOS || Platform == UnrealTargetPlatform.Mac || Platform == UnrealTargetPlatform.TVOS)
		{
			return true;
		}
		return false;
	}

	public override bool ShouldStageCommandLine(ProjectParams Params, DeploymentContext SC)
	{
		// xcode doesn't use the Bootstrap wrapper app, so we always insert the commandline file into the .app so double-clicking the .app works
		return true;
	}

	public override bool SignExecutables(DeploymentContext SC, ProjectParams Params)
	{
		if (UnrealBuildTool.BuildHostPlatform.Current.Platform == UnrealTargetPlatform.Mac)
		{
			if (Params.Archive)
			{
				// Remove extra RPATHs if we will be archiving the project
				Logger.LogInformation("Removing extraneous rpath entries");
				RemoveExtraRPaths(Params, SC);
			}
		}
		return true;
	}

	public override void StripSymbols(FileReference SourceFile, FileReference TargetFile)
	{
		MacExports.StripSymbols(SourceFile, TargetFile, Log.Logger);
	}
}
