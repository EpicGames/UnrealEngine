// Copyright Epic Games, Inc. All Rights Reserved.
using System;
using System.Collections.Generic;
using System.Linq;
using System.Text;
using System.IO;
using AutomationTool;
using UnrealBuildTool;
using UnrealBuildTool.GDK;
using EpicGames.Core;
using System.Xml.Linq;
using Microsoft.Extensions.Logging;

namespace AutomationUtils.GDK
{
	public class GDKAutomationUtils
	{
		public static void GenerateGameConfigForStaging(ProjectParams Params, DeploymentContext SC, string IniSection_TargetSettings, ILogger Logger)
		{
			DirectoryReference ManifestTargetDirectory = DirectoryReference.Combine(SC.ProjectRoot, "Saved", SC.StageTargetPlatform.PlatformType.ToString(), "Manifest");
			InternalUtils.SafeDeleteDirectory(ManifestTargetDirectory.FullName);

			List<AppXManifestExecutable> Executables = GetGameConfigExecutables(Params, SC, GameConfigPurpose.ForStaging, IniSection_TargetSettings);
			GDKGameConfigGenerator.Create(SC.StageTargetPlatform.PlatformType, Logger, ManifestTargetDirectory, SC.ShortProjectName, Params.RawProjectPath, Executables, DLCFile: Params.DLCFile, CustomConfig: SC.CustomConfig, NoBootstrapExe: Params.NoBootstrapExe);

			if (Params.ApplyIoStoreOnDemand)
			{
				ApplyIoStoreOnDemandSettings(Params, ManifestTargetDirectory, Logger);
			}

			SC.StageFiles(StagedFileType.SystemNonUFS, ManifestTargetDirectory, "*", StageFilesSearch.AllDirectories, StagedDirectoryReference.Root);
		}

		public enum GameConfigPurpose
		{
			ForStaging,
			ForPackaging,
		}
		public static List<AppXManifestExecutable> GetGameConfigExecutables(ProjectParams Params, DeploymentContext SC, GameConfigPurpose Purpose, string IniSection_TargetSettings, UnrealTargetConfiguration? OnlyConfig = null)
		{
			bool bForStaging = (Purpose == GameConfigPurpose.ForStaging);

			// build filtered stage target list
			List<StageTarget> FilteredStageTargets = SC.StageTargets.Where(X => OnlyConfig == null || (X.Receipt.Configuration == OnlyConfig!.Value)).ToList();
			if (OnlyConfig != null && !FilteredStageTargets.Any())
			{
				throw new AutomationException($"Could not find receipt for config {OnlyConfig!.Value} in currently stage targets - has the game been built & staged for this configuration?");
			}

			// special case for the Windows bootstrapper - this is in the game root and will run the most appropriate x64/arm64 executable automatically
			// there is one bootstrapper per configuration (per build target)
			if (!Params.NoBootstrapExe && SC.StageTargetPlatform.PlatformType.IsInGroup(UnrealPlatformGroup.Windows))
			{
				Dictionary<UnrealTargetConfiguration, List<string>> PerConfigExecutables = GetPerConfigBootstrapExecutables(Params, SC);
					
				bool bAllExist = PerConfigExecutables.Any();
				List<AppXManifestExecutable> BootstrapExecutables = [];
				foreach (KeyValuePair<UnrealTargetConfiguration, List<string>> ConfigExecutables in PerConfigExecutables)
				{
					if (OnlyConfig != null && OnlyConfig!.Value != ConfigExecutables.Key)
					{
						continue;
					}

					foreach (string Executable in ConfigExecutables.Value)
					{
						BootstrapExecutables.Add( new(ConfigExecutables.Key, Executable));

						bAllExist &= bForStaging ? SC.FilesToStage.NonUFSSystemFiles.ContainsKey(new StagedFileReference(Executable)) : FileReference.Exists(FileReference.Combine(SC.StageDirectory, Executable));
					}
				}

				// only use the bootstrap executables if they exist (to allow for development iteration scenarios). Bootstrap must exist for distribution
				if (Params.Distribution || bAllExist)
				{
					return BootstrapExecutables;
				}
			}


			// build the list of executables, applying config remapping as necessary
			List<AppXManifestExecutable> Executables = [];
			ConfigHierarchy EngineIni = ConfigCache.ReadHierarchy(ConfigHierarchyType.Engine, DirectoryReference.FromFile(SC.RawProjectPath), SC.StageTargetPlatform.PlatformType, SC.CustomConfig);
			foreach ( StageTarget Target in FilteredStageTargets)
			{
				UnrealTargetConfiguration Config = Target.Receipt.Configuration;
				UnrealArch Arch = Target.Receipt.Architectures.SingleArchitecture;
				string Executable = SC.GetStagedFileLocation(Target.Receipt.Launch!).ToString();

				string ConfigEntry = $"OverrideLaunchExe_{Config}_{Arch}";
				if (EngineIni.GetString(IniSection_TargetSettings, ConfigEntry, out string OverrideExectuablePath) && !string.IsNullOrEmpty(OverrideExectuablePath))
				{
					Executable = OverrideExectuablePath;
				}

				Executables.Add( new(Config, Executable, Arch));
			}

			return Executables;
		}

		public static Dictionary<UnrealTargetConfiguration, List<string>> GetPerConfigBootstrapExecutables(ProjectParams Params, DeploymentContext SC)
		{
			Dictionary<UnrealTargetConfiguration, List<string>> PerConfigExecutables = [];

			if (!Params.NoBootstrapExe && SC.StageTargetPlatform.PlatformType.IsInGroup(UnrealPlatformGroup.Windows))
			{
				// bootstrap logic from WinPlatform.Automation.cs -> GetFilesToDeployOrStage
				// @todo: unify this logic

				// one bootstrap per configuration, per target
				foreach (UnrealTargetConfiguration Configuration in SC.StageTargetConfigurations)
				{
					List<string> BootstrapExecutables = [];

					foreach (string TargetName in SC.StageTargets.Select( T => T.Receipt.TargetName ).Distinct())
					{
						BuildProduct PrimaryExecutable = null;
						StageTarget? PrimaryTarget = null;

						// collected the staged executable files for each staged architecture
						foreach (StageTarget Target in SC.StageTargets.Where( T => T.Receipt.Configuration == Configuration && T.Receipt.TargetName == TargetName))
						{
							BuildProduct Executable = Target.Receipt.BuildProducts.FirstOrDefault(x => x.Type == BuildProductType.Executable);
							if (Executable != null)
							{
								UnrealArch Architecture = Target.Receipt.Architectures.SingleArchitecture;
								if (PrimaryExecutable == null || Architecture.bIsX64) // prefer x64 for the BootstrapExeName because it is undecorated
								{
									PrimaryExecutable = Executable;
									PrimaryTarget = Target;
								}
							}
						}
					
						string BootstrapExeName;
						if(SC.StageTargetConfigurations.Count > 1)
						{
							BootstrapExeName = PrimaryExecutable.Path.GetFileName();
						}
						else if(Params.IsCodeBasedProject)
						{
							BootstrapExeName = PrimaryTarget.Value.Receipt.TargetName + ".exe";
						}
						else
						{
							BootstrapExeName = SC.ShortProjectName + ".exe";
						}

						BootstrapExecutables.Add(BootstrapExeName);
					}

					PerConfigExecutables.Add(Configuration, BootstrapExecutables);
				}
			}

			return PerConfigExecutables;
		}

		/// <summary>
		/// Makes sure that the project config file is setup to allow enough persistent local storage
		/// for IoStoreOnDemand caching. If the project is not set up for this or has not requested enough
		/// space then the config file will be modified to satisfy our requirements.
		/// </summary>
		/// <param name="Params">Parameters for the current project</param>
		/// <param name="ManifestTargetDirectory">Path to the directory containing the manifests</param>
		/// <param name="Logger">Logger interface so we can track progress</param>
		private static void ApplyIoStoreOnDemandSettings(ProjectParams Params, DirectoryReference ManifestTargetDirectory, ILogger Logger)
		{
			try
			{
				bool UpdateDoc = false;

				string ConfigFilePath = ManifestTargetDirectory + "\\MicrosoftGame.config";
				XDocument Document = XDocument.Load(ConfigFilePath);

				XElement Node;
				if ((Node = Document.Root.Element("PersistentLocalStorage")?.Element("SizeMB")) != null)
				{
					int SizeMB = 0;
					int.TryParse(Node.Value, out SizeMB);
					if (SizeMB < 256)
					{
						Node.Value = "256";
						UpdateDoc = true;
					}
				}
				else if ((Node = Document.Root.Element("PersistentLocalStorage")) != null)
				{
					Node.Add(new XElement("SizeMB", "256"));
					UpdateDoc = true;
				}
				else
				{
					Document.Root.Add(new XElement("PersistentLocalStorage", new XElement("SizeMB", "256")));
					UpdateDoc = true;
				}

				if (UpdateDoc)
				{
					Document.Save(ConfigFilePath);

					Logger.LogInformation("IoStoreOnDemand applied changes to persistent local storage settings to the project");
				}
			}
			catch (Exception Ex)
			{
				Logger.LogWarning("Error encountered while trying to apply persistent download data settings, IoStoreOnDemand may not work! [{Msg}]", Ex.Message);
			}
		}
	}
}
