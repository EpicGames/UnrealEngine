// Copyright Epic Games, Inc. All Rights Reserved.
using System;
using System.Collections.Generic;
using System.Linq;
using System.Text;
using System.IO;
using System.Text.Json;
using System.Text.RegularExpressions;
using System.Xml.Linq;
using System.ComponentModel;
using Microsoft.Extensions.Logging;
using AutomationTool;
using AutomationUtils.Automation;
using UnrealBuildTool;
using UnrealBuildTool.GDK;
using EpicGames.Core;

using static AutomationTool.CommandUtils;
using System.Threading.Tasks;
using ILogger = Microsoft.Extensions.Logging.ILogger;

namespace AutomationUtils.GDK
{
	public abstract class GDKPackageGenerator
	{
		protected readonly string IniSection_TargetSettings;
		protected readonly UnrealTargetPlatform PlatformType;
		protected readonly ConfigHierarchy EngineIni;

		/// Whether to generate separate packages for each configuration
		public bool GenerateSinglePackages { get; protected set; } = true;

		/// Whether to generate a single package containing all configurations combined
		public bool GenerateCombinedPackage { get; protected set; } = false;

		/// Whether to create metadata during staging that can be used to make the package. This ensures that only the explicitly staged files will be included in the package. The alternative is that everything in the staging directory is scraped and packaged
		public bool PrecachePackageLayout { get; protected set; } = true;

		/// Whether to parallelize calls to makepkg when generating multiple configuration packages. This requires PrecachePackageLayout
		public bool ParallelizePackageGeneration { get; protected set; } = true;

		/// When true, specifying -NoDebugInfo will cause /skipsymbolbundling to be passed to the package generator for non-submission packages. This will mean the symbols zip file for uploading to Partner Center will not be created. Potentially useful for iteration.
		public bool NoDebugInfoSkipsSymbolBundling { get; protected set; } = false;


		protected static int GDKEdition => GDKExports.GetGDKVersionNumber() ?? 0;
		protected static string GDKRootDir => Utils.CleanDirectorySeparators(Path.Combine(GDKExports.GetGSDKRoot()));
		protected static string GDKBinariesDir => Path.Combine( GDKRootDir, "bin");


		public GDKPackageGenerator(FileReference InProjectPath, UnrealTargetPlatform InPlatformType, string InIniSection_TargetSettings, string CustomConfig)
		{
			PlatformType = InPlatformType;
			IniSection_TargetSettings = InIniSection_TargetSettings;
			EngineIni = ConfigCache.ReadHierarchy(ConfigHierarchyType.Engine, DirectoryReference.FromFile(InProjectPath), PlatformType, CustomConfig);

			if (EngineIni.GetBool(IniSection_TargetSettings, "bGenerateSinglePackages", out bool bGenerateSinglePackages))
			{
				GenerateSinglePackages = bGenerateSinglePackages;
			}
			if (EngineIni.GetBool(IniSection_TargetSettings, "bGenerateCombinedPackage", out bool bGenerateCombinedPackage))
			{
				GenerateCombinedPackage = bGenerateCombinedPackage;
			}
			if (EngineIni.GetBool(IniSection_TargetSettings, "bPrecachePackageLayout", out bool bPrecachePackageLayout))
			{
				PrecachePackageLayout = bPrecachePackageLayout;
			}
			if (EngineIni.GetBool(IniSection_TargetSettings, "bParallelizePackageGeneration", out bool bParallelizePackageGeneration))
			{
				ParallelizePackageGeneration = bParallelizePackageGeneration;
			}
			if (EngineIni.GetBool(IniSection_TargetSettings, "bNoDebugInfoSkipsSymbolBundling", out bool bNoDebugInfoSkipsSymbolBundling))
			{
				NoDebugInfoSkipsSymbolBundling = bNoDebugInfoSkipsSymbolBundling;
			}

			// makepkg validation only honors layout.xml from March 2024 GDK. (prior to this it would ensure the executables were relative to the MicrosoftGame.config) 
			if (GDKEdition < 240300)
			{
				PrecachePackageLayout = false;
			}

			if (!PrecachePackageLayout)
			{
				ParallelizePackageGeneration = false;
			}

			VerifyWindowsToolsEnvironment();
		}


		public abstract string GetPlatformPackageFileExtension();

		protected abstract string GetPlatformSpecificPackageCommandlineOptions(ProjectParams Params, DeploymentContext SC, string DLCPackage);

		protected virtual void GetPlatformSpecificPackageSymbolPaths(HashSet<string> SymbolPaths, ProjectParams Params, DeploymentContext SC)
		{
		}

		protected string GetPackagingEncryptionOptions(ProjectParams Params)
		{
			string EncryptionOptions = "/lt";
			if (Params.Distribution)
			{
				if (String.IsNullOrWhiteSpace(Params.PackageEncryptionKeyFile))
				{
					EncryptionOptions = "/l";
				}
				else
				{
					EncryptionOptions = string.Format("/lk {0}", Params.PackageEncryptionKeyFile);
				}
			}

			return EncryptionOptions;
		}

		protected string GetPackageEncryptionOptions(string PackagePath)
		{
			string PackageUtilPath = Path.Combine(GDKBinariesDir, "PackageUtil.exe");
			string PackageUtilCommandLine = String.Format("license {0}", PackagePath);

			string PackageEncryptionOptions = "";
			IProcessResult RunResult = CommandUtils.Run(PackageUtilPath, PackageUtilCommandLine);
			if (RunResult.ExitCode == 0)
			{
				PackageEncryptionOptions = RunResult.Output.Trim();
			}

			return PackageEncryptionOptions;
		}

		protected string FindPackageFile(DirectoryReference PackagePath)
		{
			string ResultFilePath = null;

			if (Directory.Exists(PackagePath.ToString()))
			{
				string PackageFilenameFilter = String.Format("*{0}", GetPlatformPackageFileExtension());
				DirectoryInfo LatestPatchDI = new DirectoryInfo(PackagePath.ToString());
				var FileInLatestPatchDirectory = LatestPatchDI.GetFiles(PackageFilenameFilter, SearchOption.AllDirectories).Select(file => file.FullName);

				if (FileInLatestPatchDirectory.Any())
				{
					ResultFilePath = FileInLatestPatchDirectory.First();
					Logger.LogDebug("{Text}", String.Format("Found package file {0}", ResultFilePath));
				}
			}

			return ResultFilePath;
		}

		string GetOptionalConfigGuidOption(string KeyName, string Parameter)
		{
			string Value;
			if (EngineIni.GetString(IniSection_TargetSettings, KeyName, out Value) && !String.IsNullOrEmpty(Value))
			{
				Guid DummyGuid;
				if (Guid.TryParse(Value, out DummyGuid))
				{
					return " " + Parameter + " " + Value;
				}
				else
				{
					throw new AutomationException("{0} must be a valid GUID: {0}. Check the value set in {1}", KeyName, Value, IniSection_TargetSettings);
				}
			}

			return "";
		}

		public void GetPackageFilesToArchive(ProjectParams Params, DeploymentContext SC, string PackageOutputPathFragment = "")
		{
			List<PackageRequest> PackageRequests = MakePackageRequests(Params, SC);

			foreach (PackageRequest Request in PackageRequests)
			{
				string PackageConfig = (Request.DLCPackage == null) ? (Request.Config?.ToString() ?? "Combined") : Request.DLCPackage;

				DirectoryReference PackagePath = DirectoryReference.Combine(SC.ProjectRoot, "Saved", "Packages", SC.CookPlatform, PackageOutputPathFragment, PackageConfig);
				if (Params.HasCreateReleaseVersion)
				{
					PackagePath = DirectoryReference.FromString(Path.Combine(Params.GetCreateReleaseVersionPath(SC, Params.Client), PackageOutputPathFragment, PackageConfig));
				}

				SC.ArchiveFiles(PackagePath.FullName, NewPath: Path.Combine(PackageOutputPathFragment, PackageConfig));
			}
		}



		public void GeneratePackages(ProjectParams Params, DeploymentContext SC, DirectoryReference PackageMetadataBasePath = null, string PackageOutputPathFragment = "")
		{
			List<PackageRequest> PackageRequests = MakePackageRequests(Params, SC);

			// generate the packages
			if (ParallelizePackageGeneration && PackageRequests.Count > 1 && !Params.PreStaged)
			{
				Logger.LogInformation("Generating {Num} packages in parallel...", PackageRequests.Count);

				var LockObject = new object();
				Parallel.ForEach(PackageRequests, (PackageRequest Request) =>
				{
					CaptureLogger CaptureLogger = new();
					try
					{
						GenerateSinglePackage(Params, SC, Request.DLCPackage, PackageMetadataBasePath, Request.Config, CaptureLogger, PackageOutputPathFragment);
					}
					catch (AutomationException e)
					{
						CaptureLogger.LogError("{Ex}", e.Message);
						throw new AutomationException("Packaging failed");
					}
					finally
					{
						// lock ensures each package's output is not interleaved
						lock (LockObject)
						{
							CaptureLogger.RenderTo(Logger);
						}
					}
				});
			}
			else
			{
				foreach (PackageRequest Request in PackageRequests)
				{
					GenerateSinglePackage(Params, SC, Request.DLCPackage, PackageMetadataBasePath, Request.Config, null, PackageOutputPathFragment);
				}
			}
		}

		public void GeneratePackageMetadata(ProjectParams Params, DeploymentContext SC, DirectoryReference PackageMetadataBasePath)
		{
			List<PackageRequest> PackageRequests = MakePackageRequests(Params, SC);
			foreach (PackageRequest Request in PackageRequests)
			{
				GenerateSinglePackageMetadata(Params, SC, PackageMetadataBasePath, Request.DLCPackage, Request.Config);
			}
		}



		struct PackageRequest
		{
			public PackageRequest()
			{
			}

			public PackageRequest(UnrealTargetConfiguration? InConfig)
			{
				Config = InConfig;
			}
			public PackageRequest(string InDLCPackage)
			{
				DLCPackage = InDLCPackage;
			}

			public readonly UnrealTargetConfiguration? Config = null; // null means "no specific single config - package all configurations"
			public readonly string DLCPackage = null;
		};

		private List<PackageRequest> MakePackageRequests(ProjectParams Params, DeploymentContext SC)
		{
			List<PackageRequest> PackageRequests = [];

			// prepare the list of configurations to package
			if (GenerateSinglePackages || Params.HasCreateReleaseVersion)
			{
				PackageRequests.AddRange(SC.StageTargetConfigurations.Select(X => new PackageRequest(X)));
			}
			if (GenerateCombinedPackage && !Params.HasCreateReleaseVersion)
			{
				PackageRequests.Add(new PackageRequest());
			}

			// add any DLC packages we require
			if (EngineIni.GetBool(IniSection_TargetSettings, "bAutoCreateDLCPackages", out bool bAutoCreateDLCPackages) && bAutoCreateDLCPackages)
			{
				foreach (string DLCPackage in DLCChunks.GetFromConfig(Params, SC, IniSection_TargetSettings).Keys)
				{
					PackageRequests.Add(new PackageRequest(DLCPackage));
				}
			}

			return PackageRequests;
		}

		public void GenerateSinglePackage(ProjectParams Params, DeploymentContext SC, string DLCPackage, DirectoryReference PackageMetadataBasePath = null, UnrealTargetConfiguration? OnlyConfig = null, ILogger CustomLogger = null, string PackageOutputPathFragment = "")
		{
			string PackageConfig = (DLCPackage == null) ? (OnlyConfig?.ToString() ?? "Combined") : DLCPackage;

			// default to global logger unless one has been specified & output banner
			CustomLogger ??= Logger;
			CustomLogger.LogInformation("");
			CustomLogger.LogInformation("*** {PackageConfig} {Platform} Package ***", PackageConfig, PlatformType);

			// set package output directory
			DirectoryReference PackageOutputDir = DirectoryReference.Combine(SC.ProjectRoot, "Saved", "Packages", SC.CookPlatform, PackageOutputPathFragment, PackageConfig);
			if (Params.HasCreateReleaseVersion)
			{
				PackageOutputDir = DirectoryReference.FromString(Path.Combine(Params.GetCreateReleaseVersionPath(SC, Params.Client), PackageOutputPathFragment, PackageConfig));
			}

			// fall back to legacy packaging if there is no metadata
			if (!PrecachePackageLayout || Params.PreStaged)
			{
				PackageImmediate(Params, SC, PackageOutputDir, CustomLogger, DLCPackage, OnlyConfig, PackageOutputPathFragment);
				return;
			}

			// make sure there is valid metadata
			PackageMetadataBasePath ??= DirectoryReference.Combine(SC.StageDirectory, ".pkgsrc");
			DirectoryReference PackageMetadataDir = DirectoryReference.Combine(PackageMetadataBasePath, PackageConfig);
			FileReference LayoutXmlFile = FileReference.Combine(PackageMetadataDir, "layout.xml");
			if (!FileReference.Exists(LayoutXmlFile))
			{
				throw new AutomationException($"{LayoutXmlFile} not found - was the game staged?");
			}

			// attempt to package
			PackageInternal(Params, SC, PackageOutputDir, LayoutXmlFile, PackageMetadataDir, CustomLogger, DLCPackage, OnlyConfig, PackageOutputPathFragment);
		}




		protected virtual void PackageInternal(ProjectParams Params, DeploymentContext SC, DirectoryReference PackagePath, FileReference LayoutPath, DirectoryReference SourceFolder, ILogger Logger, string DLCPackage, UnrealTargetConfiguration? OnlyConfig = null, string PackageOutputPathFragment = "")
		{
			// sanity check
			if (SC.StageDirectory == null)
			{
				throw new AutomationException("Cannot package: No staging directory. Please specify -stage, -skipstage or -stagingdirectory");
			}
			else if (!DirectoryReference.Exists(SC.StageDirectory))
			{
				throw new AutomationException($"Cannot package: Staging directory {SC.StageDirectory} does not exist");
			}

			// locate submission validator DLL
			string AutoSdkSubValRoot = Path.GetFullPath(@"..\..\SubmissionValidator_Latest\", GDKRootDir); // this will be %UE_SDKS_ROOT%\HostWin64\GDK\SubmissionValidator_Latest\ folder when using AutoSDK

			string MakePkgValidationPath;
			string SubmissionValidator = Path.Combine(AutoSdkSubValRoot, "SubmissionValidator.dll");
			if (File.Exists(SubmissionValidator))
			{
				// use shared AutoSDK version of SubmissionValidator.dll			
				MakePkgValidationPath = Path.TrimEndingDirectorySeparator(AutoSdkSubValRoot);
			}
			else
			{
				// try default location, alongside makepkg itself
				SubmissionValidator = Path.Combine(GDKBinariesDir, "SubmissionValidator.dll");
				MakePkgValidationPath = null;
			}

			// add explicit warning if submission validator DLL cannot be found. The actual error from makepkg just says it is out of date which isn't helpful.			
			bool bIncludesShipping = (OnlyConfig != null) ? (OnlyConfig == UnrealTargetConfiguration.Shipping) : SC.StageTargetConfigurations.Contains(UnrealTargetConfiguration.Shipping);
			if (bIncludesShipping && Params.Distribution && !File.Exists(SubmissionValidator))
			{
				Log.TraceWarningOnce($"SubmissionValidator DLL is not present in {GDKBinariesDir} or %UE_SDKS_ROOT%/HostWin64/GDK/SubmissionValidator_Latest/ - It is a separate download from the main GDK and is used when making a submission package for uploading to Partner Center. Please see the GDK documentation for details.");
			}

			// prepare paths
			string MakePkgPath = Path.Combine(GDKBinariesDir, "makepkg.exe");
			if (DirectoryReference.Exists(PackagePath))
			{
				CommandUtils.DeleteDirectory(PackagePath);
			}
			DirectoryReference.CreateDirectory(PackagePath);

			// prepare the makepkg command line
			string AdditionalOptions = GetPlatformSpecificPackageCommandlineOptions(Params, SC, DLCPackage) + " " + GetPackagingEncryptionOptions(Params);
			string PackageCommandline = string.Format("pack /f \"{0}\" {1} /d \"{2}\" /pd \"{3}\" /loggable", LayoutPath, AdditionalOptions, SourceFolder, PackagePath);

			// add path to shared submission validator, if necessary
			if (MakePkgValidationPath != null)
			{
				PackageCommandline += $" /validationpath \"{MakePkgValidationPath}\"";
			}

			// read store Id: we only include the legacy ProductId if the store Id is not specified
			if (Params.DLCFile == null && DLCPackage == null)
			{
				bool bHasStoreId = (EngineIni.GetString(IniSection_TargetSettings, "StoreId", out string StoreId) && !String.IsNullOrEmpty(StoreId));
				if (!bHasStoreId)
				{
					PackageCommandline += GetOptionalConfigGuidOption("ProductId", "/productid");
				}
				PackageCommandline += GetOptionalConfigGuidOption("ContentId", "/contentid");
			}

			// do not generate a symbols zip if they don't want debug info
			if (NoDebugInfoSkipsSymbolBundling && Params.NoDebugInfo && !Params.Distribution)
			{
				PackageCommandline += " /skipsymbolbundling";
			}
			else
			{
				// append all paths containing PDBs, for SubmissionValidator.dll
				HashSet<string> SymbolPaths = new HashSet<string>();
				GetPlatformSpecificPackageSymbolPaths(SymbolPaths, Params, SC);
				SymbolPaths.Add(Params.GetProjectBinariesPathForPlatform(PlatformType).ToString());
				foreach (StageTarget Target in SC.StageTargets.Where(X => OnlyConfig == null || OnlyConfig == X.Receipt.Configuration))
				{
					foreach (RuntimeDependency RuntimeDependency in Target.Receipt.RuntimeDependencies.Where(x => x.Path.HasExtension("pdb") || x.Path.HasExtension("dll")))
					{
						SymbolPaths.Add(RuntimeDependency.Path.Directory.ToString());
					}
					foreach (BuildProduct BuildProduct in Target.Receipt.BuildProducts.Where(x => x.Path.HasExtension("pdb") || x.Path.HasExtension("dll")))
					{
						SymbolPaths.Add(BuildProduct.Path.Directory.ToString());
					}
				}
				PackageCommandline += " /symbolpaths ";
				foreach (string SymbolPath in SymbolPaths)
				{
					PackageCommandline += "\"" + SymbolPath + "\";";
				}
			}


			// Look for a release package to use as base package for patch generation
			string ReleasePackagePath = null;
			if (Params.HasBasedOnReleaseVersion && Params.DLCFile == null)
			{
				DirectoryReference ReleaseVersionBasePath = new DirectoryReference(Params.GetBasedOnReleaseVersionPath(SC, Params.Client));
				if (!string.IsNullOrEmpty(PackageOutputPathFragment))
				{
					ReleaseVersionBasePath = DirectoryReference.Combine(ReleaseVersionBasePath, PackageOutputPathFragment);
				}
				if (DLCPackage != null)
				{
					ReleaseVersionBasePath = DirectoryReference.Combine(ReleaseVersionBasePath, DLCPackage);
				}

				DirectoryReference[] ReleaseVersionPaths = new DirectoryReference[]
				{
					DirectoryReference.Combine(ReleaseVersionBasePath, "LatestPatch"), // legacy support. check this first as the package search is recursive.
					ReleaseVersionBasePath, // for consistency with other platforms
				};

				foreach (DirectoryReference ReleaseVersionPath in ReleaseVersionPaths)
				{
					ReleasePackagePath = FindPackageFile(ReleaseVersionPath);
					if (!String.IsNullOrEmpty(ReleasePackagePath))
					{
						Logger.LogDebug("{Text}", String.Format("Found base package for delta patching {0}", ReleasePackagePath));
						PackageCommandline += String.Format(" /priorpackage \"{0}\"", ReleasePackagePath);
						break;
					}
				}

				if (String.IsNullOrEmpty(ReleasePackagePath))
				{
					Logger.LogWarning("Attempting to create a new release package but no previous release package could be found in {ReleaseVersionBasePath}. This may negatively affect patch download sizes.", ReleaseVersionBasePath);
				}
			}

			string AdditionalMakePkgArguments;
			if (EngineIni.GetString(IniSection_TargetSettings, "AdditionalMakePkgArguments", out AdditionalMakePkgArguments) && !String.IsNullOrEmpty(AdditionalMakePkgArguments))
			{
				PackageCommandline += String.Format(" {0}", AdditionalMakePkgArguments);
			}

			// run makepkg & apply message filtering
			SpewFilterHelper SpewFilter = new SpewFilterHelper();
			PrepareMessageFilterForMakePkg(SpewFilter, EngineIni, Params, SC);

			// Get makepkg to write its temp files to a controlled folder by overriding TMP environment variable to 
			DirectoryReference MakePKGTempPath = DirectoryReference.Combine(SC.ProjectRoot, "Intermediate", PlatformType.ToString(), "makepkgtmp");
			DirectoryReference.CreateDirectory(MakePKGTempPath);
			Dictionary<string, string> ExtraEnvVars = new();
			ExtraEnvVars["TMP"] = MakePKGTempPath.FullName;
            ExtraEnvVars["TEMP"] = MakePKGTempPath.FullName;

			Logger.LogInformation("Running \"{MakePkgPath}\" {PackageCommandline}", MakePkgPath, PackageCommandline);
			IProcessResult Result = CommandUtils.Run(MakePkgPath, PackageCommandline, Options: ERunOptions.AppMustExist|ERunOptions.NoLoggingOfRunCommand, Env: ExtraEnvVars);
			PrettyLogProcessOutput(Logger, Result.Output.EnumerateLines().Select(X => SpewFilter.FilterMessage(X)));

			// check the result from makepkg
			if (Result.ExitCode != 0)
			{
				throw new AutomationException("Packaging failed - \"{0}\" {1} returned {2}", MakePkgPath, PackageCommandline, Result.ExitCode);
			}

			// estimate size of delta patch if we are generating a patch.  Packages must have compatible encryption keys.
			if (!String.IsNullOrEmpty(ReleasePackagePath) && !Params.HasDLCName)
			{
				string PatchPackagePath = FindPackageFile(PackagePath);
				if (!String.IsNullOrEmpty(PatchPackagePath))
				{
					GDKGameConfigInfo ReleasePackageInfo = new(Path.GetFileNameWithoutExtension(ReleasePackagePath));
					GDKGameConfigInfo PatchPackageInfo = new(Path.GetFileNameWithoutExtension(PatchPackagePath));
					if (Version.Parse(ReleasePackageInfo.Version) <= Version.Parse(PatchPackageInfo.Version))
					{
						string ReleasePackageEncryptionOptions = GetPackageEncryptionOptions(ReleasePackagePath);
						string PatchPackageEncryptionOptions = GetPackageEncryptionOptions(PatchPackagePath);
						if (GetPackagingEncryptionOptions(Params) != "/l" && ReleasePackageEncryptionOptions == PatchPackageEncryptionOptions)
						{
							Logger.LogDebug("Found patch package for patch size estimate {releasepath}", ReleasePackagePath);

							string PackageUtilPath = Path.Combine(GDKBinariesDir, "PackageUtil.exe");
							string PackageUtilCommandLine = String.Format("compare {0} {1}", ReleasePackagePath, PatchPackagePath);

							Logger.LogInformation("Running \"{PackageUtilPath}\" {PackageUtilCommandLine}", PackageUtilPath, PackageUtilCommandLine);
							Result = CommandUtils.Run(PackageUtilPath, PackageUtilCommandLine, Options: ERunOptions.AppMustExist | ERunOptions.NoLoggingOfRunCommand);
							PrettyLogProcessOutput(Logger, Result.Output.EnumerateLines());
							if (Result.ExitCode != 0)
							{
								Logger.LogWarning("Patch size estimation failed - {packageutil} {params} returned {errcode}", PackageUtilPath, PackageUtilCommandLine, Result.ExitCode);
							}
						}
						else
						{
							Logger.LogDebug("Base and patch package encryption options are not compatible for patch size estimation.  Base = {baseopt}  Patch = {patchopt}", ReleasePackageEncryptionOptions, PatchPackageEncryptionOptions);
						}
					}
					else
					{
						Logger.LogWarning("Release package is newer than the patch: skipping patch comparison. Base = {basever}  Patch = {patchver}", ReleasePackageInfo.Version, PatchPackageInfo.Version);
					}
				}
			}

			// write out the output, suppressing needless empty lines. this matches the output when the gdk tool is run directly from the command line, outside of UAT.
			static void PrettyLogProcessOutput(ILogger Logger, IEnumerable<string> OutputLines)
			{
				bool bWriteBlank = false;
				foreach (string OutputLine in OutputLines)
				{
					bool bIsBlank = (OutputLine.Length == 0);
					if (!bIsBlank || (bIsBlank == bWriteBlank))
					{
						// Log HRESULT as warning
						if (Regex.Match(OutputLine, @"\(0x8.{7}\)").Success)
						{
							Logger.LogWarning("{OutputLine}", OutputLine);
						}
						else
						{
							Logger.LogInformation("{OutputLine}", OutputLine);
						}
					}
					bWriteBlank = bIsBlank && !bWriteBlank;
				}
			}
		}


		private List<FileReference> MoveFiles(List<FileReference> Files, DirectoryReference SrcDir, DirectoryReference DstDir)
		{
			DirectoryReference.CreateDirectory(DstDir);

			List<FileReference> DstFiles = new List<FileReference>();
			foreach (FileReference SrcFile in Files)
			{
				FileReference DstFile = FileReference.Combine(DstDir, SrcFile.MakeRelativeTo(SrcDir));
				DirectoryReference.CreateDirectory(DstFile.Directory);
				FileReference.Move(SrcFile, DstFile);

				DstFiles.Add(DstFile);
			}

			return DstFiles;
		}

		private void DeleteFiles(List<FileReference> Files)
		{
			foreach (FileReference File in Files)
			{
				FileReference.Delete(File);
			}
		}



		protected void PackageImmediate(ProjectParams Params, DeploymentContext SC, DirectoryReference PackagePath, ILogger Logger, string DLCPackage, UnrealTargetConfiguration? OnlyConfig = null, string PackageOutputPathFragment = "")
		{
			// prepare intermediate packaging directory
			DirectoryReference IntermediatePath = DirectoryReference.Combine(SC.ProjectRoot, "Intermediate", PlatformType.ToString(), "Packaging");
			InternalUtils.SafeDeleteDirectory(IntermediatePath.FullName);
			DirectoryReference.CreateDirectory(IntermediatePath);

			// generate a filtered list of the files that should be included in the package
			IEnumerable<StagedFileReference> FilesToInclude = FindStagedFilesToPackage(Params, SC, OnlyConfig).Where(X=>
				ShouldPackageFile(X.Value, Params, SC, OnlyConfig))
				.Select(f => f.Key);

			// exclude any staged manifest resources as we're generating new ones
			IEnumerable<StagedFileReference> ManifestFiles = GetGDKManifestFilesInDirectory(SC.StageDirectory)
					.Select(File => StagedFileReference.Combine(File.MakeRelativeTo(SC.StageDirectory)));
			FilesToInclude = FilesToInclude.Except(ManifestFiles);

			// generate a GDK manifest that just contains the selected configurations
			List<AppXManifestExecutable> Executables = GDKAutomationUtils.GetGameConfigExecutables(Params, SC, GDKAutomationUtils.GameConfigPurpose.ForPackaging, IniSection_TargetSettings, OnlyConfig);
			GDKGameConfigGenerator.Create(PlatformType, Logger, IntermediatePath, SC.ShortProjectName, Params.RawProjectPath, Executables, bDeleteIntermediates: true, DLCFile: Params.DLCFile, DLCPackage: DLCPackage, CustomConfig: SC.CustomConfig, NoBootstrapExe: Params.NoBootstrapExe);
			IEnumerable<StagedFileReference> GeneratedFilesToInclude =  GetGDKManifestFilesInDirectory(IntermediatePath)
				.Select( File => StagedFileReference.Combine(File.MakeRelativeTo(IntermediatePath)));

			// layout.xml & package.manifest generation
			GeneratePackageMetadataInternal(Params, SC, FilesToInclude, GeneratedFilesToInclude, IntermediatePath, Logger, DLCPackage);

			// attempt to package
			FileReference LayoutXmlFile = FileReference.Combine(IntermediatePath, "layout.xml");
			PackageInternal(Params, SC, PackagePath, LayoutXmlFile, IntermediatePath, Logger, DLCPackage, OnlyConfig, PackageOutputPathFragment);
		}




		private List<FileReference> GetGDKManifestFilesInDirectory(DirectoryReference Directory)
		{
			List<FileReference> Result = new List<FileReference>();

			FileReference MSGameConfig = FileReference.Combine(Directory, "MicrosoftGame.config");
			if (FileReference.Exists(MSGameConfig))
			{
				Result.Add(MSGameConfig);
			}

			Result.AddRange(DirectoryReference.EnumerateFiles(Directory, "*.pri", SearchOption.TopDirectoryOnly));

			DirectoryReference ResourcesDir = DirectoryReference.Combine(Directory, "Resources");
			if (DirectoryReference.Exists(ResourcesDir))
			{
				Result.AddRange(DirectoryReference.EnumerateFiles(ResourcesDir, "*.*", SearchOption.AllDirectories));
			}

			return Result;
		}



		protected void VerifyWindowsToolsEnvironment()
		{
			if (System.Environment.OSVersion.Version.Major < 10) //https://docs.microsoft.com/windows/win32/sysinfo/operating-system-version
			{
				Logger.LogWarning("Your Windows version {Arg0}.{Arg1}.{Arg2} may be too low to run GDK tools - it should be at least Windows 10 or Windows Server 2016", System.Environment.OSVersion.Version.Major, System.Environment.OSVersion.Version.Minor, System.Environment.OSVersion.Version.Build);
			}
		}


		#region Packaging error/warning filter

		#region Helper class
		protected class SpewFilterHelper
		{
			private List<string> IgnoreList = new List<string>();
			private Dictionary<string, string> ReplaceDict = new Dictionary<string, string>();
			private Dictionary<string, string> RegExReplaceDict = new Dictionary<string, string>();

			public void ReplaceMessages(Dictionary<string, string> Replacements)
			{
				foreach (var Replacement in Replacements)
				{
					ReplaceDict.Add(Replacement.Key, Replacement.Value);
				}
			}

			public void RegExReplaceMessages(Dictionary<string, string> Replacements)
			{
				foreach (var Replacement in Replacements)
				{
					RegExReplaceDict.Add(Replacement.Key, Replacement.Value);
				}
			}

			public void IgnoreMessages(List<string> Ignore)
			{
				IgnoreList.AddRange(Ignore);
			}

			public string FilterMessage(string Message)
			{
				foreach (string IgnoreMessage in IgnoreList)
				{
					Message = Message.Replace(IgnoreMessage, "", StringComparison.InvariantCultureIgnoreCase);
				}

				foreach (var ReplaceMessage in ReplaceDict)
				{
					if (Message.Contains(ReplaceMessage.Key))
					{
						Message = Message.Replace(ReplaceMessage.Key, ReplaceMessage.Value, StringComparison.InvariantCultureIgnoreCase);
					}
				}

				foreach (var RegExReplacee in RegExReplaceDict)
				{
					Message = Regex.Replace(Message, RegExReplacee.Key, RegExReplacee.Value, RegexOptions.IgnoreCase);
				}

				return Message;
			}
		}
		#endregion

		private static readonly Dictionary<string, string> SubmissionValidatorErrorsToDowngrade = new Dictionary<string, string>()
		{
			{ "Error: A required update to Submission Validator is available", "Warning: A required update to Submission Validator is available (this will be an error in Shipping for Distribution)" },
		};

		private static readonly Dictionary<string, string> StoreAssociationMessageToAlter = new Dictionary<string, string>()
		{
			{ "Please use the MicrosoftGame.config Editor \"Associate with Store\" feature", "Please use the \"Update Partner Center Association\" button in Unreal Editor's Project Settings" },
		};

		protected virtual void PrepareMessageFilterForMakePkg(SpewFilterHelper Filter, ConfigHierarchy EngineIni, ProjectParams Params, DeploymentContext SC)
		{
			Filter.ReplaceMessages(StoreAssociationMessageToAlter);

			bool bSubmissionValidatorExpirationError = false;
			if (!EngineIni.GetBool(IniSection_TargetSettings, "bSubmissionValidatorExpirationError", out bSubmissionValidatorExpirationError) || !bSubmissionValidatorExpirationError)
			{
				if (!Params.Distribution || !Params.ClientConfigsToBuild.Contains(UnrealTargetConfiguration.Shipping)) //only downgrade the error if we're not building a shipping package for distribution
				{
					Filter.ReplaceMessages(SubmissionValidatorErrorsToDowngrade);
				}
			}

			List<string> AllowedMissingSymbolFiles;
			if (EngineIni.GetArray(IniSection_TargetSettings, "AllowedMissingSymbolFiles", out AllowedMissingSymbolFiles) && AllowedMissingSymbolFiles != null)
			{
				foreach (string BinaryName in AllowedMissingSymbolFiles)
				{
					string Key = string.Format("Warning:.*(?<msg>Symbol file not found for.*{0}\\.).*", Regex.Escape(BinaryName));
					string Value = "${msg} (Ignoring warning due to the file being listed in AllowedMissingSymbolFiles config entry.)";
					Filter.RegExReplaceMessages(new Dictionary<string, string>() { { Key, Value } });
				}
			}
		}

		#endregion

		#region Package chunk layout generation

		protected virtual bool ShouldPackageFile(FileReference SrcFile, ProjectParams Params, DeploymentContext SC, UnrealTargetConfiguration? OnlyConfig = null)
		{
			// Exclude all PDBs
			if (SrcFile.HasExtension(".pdb"))
			{
				return false;
			}

			// Exclude specific files
			string[] ExcludedFiles = { "package.manifest", $"Manifest_DebugFiles_{PlatformType}.txt", $"Manifest_NonUFSFiles_{PlatformType}.txt", $"Manifest_UFSFiles_{PlatformType}.txt", $"Manifest_ObsoleteNonUFSFiles.txt", $"Manifest_ObsoleteUFSFiles.txt", $"Manifest_DeltaUFSFiles.txt", $"Manifest_DeltaNonUFSFiles.txt" };
			if (ExcludedFiles.Any(X => X.Equals(SrcFile.GetFileName(), StringComparison.InvariantCultureIgnoreCase)))
			{
				return false;
			}

			// Exclude launch executables for other configurations.  Otherwise, generating packages from a multi-config build will result in a mismatch
			// between launch binaries in the package and those exposed in the game config.  This will result in potential failed attempts to launch the additional
			// binaries in the package by Gauntlet.
			if (OnlyConfig != null && SrcFile.HasExtension(".exe"))
			{
				bool bExcluded = SC.StageTargets.Any(
					X => X.Receipt.Configuration != OnlyConfig &&
					X.Receipt.Launch.GetFileName() == Path.GetFileName(SrcFile.FullName)
				);
				if (bExcluded)
				{
					return false;
				}
			}

			return true;
		}

		private Dictionary<StagedFileReference, FileReference> FindStagedFilesToPackage(ProjectParams Params, DeploymentContext SC, UnrealTargetConfiguration? OnlyConfig = null)
		{
			Dictionary<StagedFileReference, FileReference> FilesToInclude = new();
			foreach (FileReference SrcFile in DirectoryReference.EnumerateFiles(SC.StageDirectory, "*", SearchOption.AllDirectories))
			{
				if (!ShouldPackageFile(SrcFile, Params, SC, OnlyConfig))
				{
					continue;
				}

				StagedFileReference DstFile = StagedFileReference.Combine(SrcFile.MakeRelativeTo(SC.StageDirectory));
				FilesToInclude.Add(DstFile, SrcFile);
			}
			return FilesToInclude;
		}





		internal struct IntelligentInstallRecipe
		{
			public string Id;
			public string[] FeatureIds;
			public string[] Devices;
			public string[] StoreIds;

			internal static IEnumerable<IntelligentInstallRecipe> GetFromConfig(ProjectParams Params, DeploymentContext SC, string IniSection_TargetSettings)
			{
				ConfigHierarchy EngineIni = ConfigCache.ReadHierarchy(ConfigHierarchyType.Engine, DirectoryReference.FromFile(SC.RawProjectPath), SC.StageTargetPlatform.PlatformType, SC.CustomConfig);

				if( EngineIni.GetBool(IniSection_TargetSettings, "bUseFeaturesAndRecipes", out bool bUseFeaturesAndRecipes) &&
					bUseFeaturesAndRecipes)
				{
					// read recipes
					EngineIni.GetArray(IniSection_TargetSettings, "IntelligentDeliveryRecipes", out List<string> Recipes);
					if (Recipes != null && Recipes.Count > 0)
					{
						foreach (string Recipe in Recipes)
						{
							Dictionary<string, string> Properties;
							ConfigHierarchy.TryParse(Recipe, out Properties);

							IntelligentInstallRecipe Mapping;

							if (!TryGetValue(Properties, "Id", out Mapping.Id) || Mapping.Id == "")
							{
								throw new AutomationException("Missing Id in recipe");
							}

							TryGetValue(Properties, "IncludedFeatures", out Mapping.FeatureIds);
							TryGetValue(Properties, "Devices", out Mapping.Devices);
							TryGetValue(Properties, "StoreIds", out Mapping.StoreIds);

							yield return Mapping;
						}
					}

				}
			}
		}


		internal class IntelligentInstallFeature
		{
			public string Id;
			public string[] Tags;
			public bool bHidden;
			public string DisplayNameResourceId;
			public string ImageFileName;

			internal static IEnumerable<IntelligentInstallFeature> GetFromConfig(ProjectParams Params, DeploymentContext SC, string IniSection_TargetSettings)
			{
				ConfigHierarchy EngineIni = ConfigCache.ReadHierarchy(ConfigHierarchyType.Engine, DirectoryReference.FromFile(SC.RawProjectPath), SC.StageTargetPlatform.PlatformType, SC.CustomConfig);

				if (EngineIni.GetBool(IniSection_TargetSettings, "bUseFeaturesAndRecipes", out bool bUseFeaturesAndRecipes) &&
					bUseFeaturesAndRecipes)
				{
					// read features
					HashSet<string> ReferencedTags = new HashSet<string>();
					EngineIni.GetArray(IniSection_TargetSettings, "IntelligentDeliveryFeatures", out List<string> Features);
					if (Features != null && Features.Count > 0)
					{
						foreach (string Feature in Features)
						{
							Dictionary<string, string> Properties;
							ConfigHierarchy.TryParse(Feature, out Properties);

							IntelligentInstallFeature Mapping = new();
							if (!TryGetValue(Properties, "Id", out Mapping.Id) || Mapping.Id == "")
							{
								throw new AutomationException("Missing Id in feature");
							}

							TryGetValue(Properties, "Tags", out string Tags);
							Mapping.Tags = Tags.Split(';', StringSplitOptions.RemoveEmptyEntries);

							TryGetValue(Properties, "bHidden", out Mapping.bHidden);
							if (Mapping.bHidden)
							{
								Mapping.DisplayNameResourceId = null;
								Mapping.ImageFileName = null;
							}
							else
							{
								TryGetValue(Properties, "DefaultDisplayName", out string DefaultDisplayName);

								// read localizable display name (localized strings will be added in GDKGameConfigGenerator)
								Mapping.DisplayNameResourceId = !string.IsNullOrEmpty(DefaultDisplayName) ? $"ms-resource:Feature_{Mapping.Id}" : null;

								// see if there is an image resource for this feature (it will be added in GDKGameConfigGenerator)
								string ImageFilename = $"Feature_{Mapping.Id}.png";
								bool bHasImageFilename = GDKGameConfigGenerator.DoesDefaultResourceBinaryFileExistForPlatform(SC.RawProjectPath, SC.StageTargetPlatform.PlatformType, Logger, ImageFilename, bAllowEngineFallback: false);
								Mapping.ImageFileName = bHasImageFilename ? $@"Resources\{ImageFilename}" : null;
							}

							yield return Mapping;
						}
					}
				}
			}

			internal void AddTag(string Tag)
			{
				if (!Tags.Contains(Tag, StringComparer.OrdinalIgnoreCase))
				{
					Tags = [.. Tags, Tag];
				}
			}
		}


		internal class IntelligentInstallChunk
		{
			public int ChunkId;
			public string ChunkTag = "";

			public string Detail_Devices = "";
			public string Detail_Languages = ""; //NB. these are GDK Culture Ids, not UE Stage Ids
			public bool Detail_bRequiredForLaunch = false;

			// some code readability helpers
			public bool IsOnDemand => (!Detail_bRequiredForLaunch && ChunkTag.Length > 0 && Detail_Languages.Length == 0 && Detail_Devices.Length == 0);
			public string[] Tags => ChunkTag.Split(';', StringSplitOptions.RemoveEmptyEntries | StringSplitOptions.TrimEntries);
			public string[] CultureIDs => Detail_Languages.Split(';', StringSplitOptions.RemoveEmptyEntries | StringSplitOptions.TrimEntries);

			internal static IEnumerable<IntelligentInstallChunk> GetFromConfig(ProjectParams Params, DeploymentContext SC, string IniSection_TargetSettings)
			{
				ConfigHierarchy EngineIni = ConfigCache.ReadHierarchy(ConfigHierarchyType.Engine, DirectoryReference.FromFile(SC.RawProjectPath), SC.StageTargetPlatform.PlatformType, SC.CustomConfig);

				var IntelligentInstallChunks = new List<IntelligentInstallChunk>();

				List<string> IntelligentDeliveryChunks;
				EngineIni.GetArray(IniSection_TargetSettings, "IntelligentDeliveryChunks", out IntelligentDeliveryChunks);

				if (IntelligentDeliveryChunks != null)
				{
					Dictionary<string, string> CultureIds = new Dictionary<string, string>();

					if (EngineIni.GetString(IniSection_TargetSettings, "StageIdOverrides", out string StageIdOverridesString) && ConfigHierarchy.TryParseAsMap(StageIdOverridesString, out Dictionary<string, string> StageIdOverrides))
					{
						CultureIds = StageIdOverrides;
					}

					// parse all chunks
					foreach (var IntelligentDeliveryChunk in IntelligentDeliveryChunks)
					{
						Dictionary<string, string> Properties;
						ConfigHierarchy.TryParse(IntelligentDeliveryChunk, out Properties);

						IntelligentInstallChunk ChunkEntry = new IntelligentInstallChunk();

						// read chunk id
						string ChunkId;
						if (!Properties.TryGetValue("ChunkId", out ChunkId) || !int.TryParse(ChunkId, out ChunkEntry.ChunkId))
						{
							Logger.LogWarning("Couldn't parse ChunkId from IntelligentDeliveryChunks in section {IniSection_TargetSettings}", IniSection_TargetSettings);
							ChunkEntry.ChunkId = -1;
						}
						else
						{
							ChunkEntry.ChunkId += OSChunkIdOffset;
						}

						// read tags
						if (!Properties.TryGetValue("Tags", out ChunkEntry.ChunkTag))
						{
							ChunkEntry.ChunkTag = "";
						}

						// read devices
						string RawDevices;
						if (Properties.TryGetValue("Devices", out RawDevices))
						{
							string[] Devices;
							if (ConfigHierarchy.TryParse(RawDevices, out Devices) && Devices != null && Devices.Length > 0)
							{
								ChunkEntry.Detail_Devices = string.Join(";", Devices);
							}
						}

						// read stage ids
						string RawStageIds;
						if (Properties.TryGetValue("StageIds", out RawStageIds))
						{
							string[] StageIds;
							if (ConfigHierarchy.TryParse(RawStageIds, out StageIds) && StageIds != null && StageIds.Length > 0)
							{
								foreach (string StageId in StageIds)
								{
									// look up the CultureId from the given CultureToStage Id
									string CultureId;
									if (!CultureIds.TryGetValue(StageId, out CultureId))
									{
										CultureId = StageId;
									}
									ChunkEntry.Detail_Languages += CultureId + ";";
								}
							}
							ChunkEntry.Detail_Languages = ChunkEntry.Detail_Languages.TrimEnd(';');
						}

						// read launch flag
						string RequiredForLaunch;
						if (!Properties.TryGetValue("bRequiredForLaunch", out RequiredForLaunch) || !bool.TryParse(RequiredForLaunch, out ChunkEntry.Detail_bRequiredForLaunch))
						{
							ChunkEntry.Detail_bRequiredForLaunch = false;
						}

						IntelligentInstallChunks.Add(ChunkEntry);
					}
				}

				return IntelligentInstallChunks;
			}
		}


		protected class GDKBundleSettings : BundleUtils.BundleSettings
		{
			public int ChunkID { get; set; }
			public string PlatformChunkName { get; set; }
		}

		// Chunk 2 maps to UE chunk 0 and also includes all content not mapped explicitly to another chunk index.
		const int OSChunkIdOffset = 2; // We need an offset to map from UE chunk to GDK chunk because a GDK chunk ID of 0 is not valid. This is 2 for historical CUv1 reasons (and even CUv3 doesn't support chunk ID reassignment without causing a full re-download)

		/// The CustomChunkMapping table is used to assign files to specific chunks using wildcard path matching.
		/// +CustomChunkMapping=(Pattern="shootergame/content/movies/*",ChunkId=1)
		/// +CustomChunkMapping=(Pattern="shootergame/content/movies/movie1.mp4",ChunkId=1)
		protected struct CustomChunkMapping
		{
			public string Pattern;
			public int ChunkId;

			internal static IEnumerable<CustomChunkMapping> GetFromConfig(ProjectParams Params, DeploymentContext SC, string IniSection_TargetSettings)
			{
				const string Key = "CustomChunkMapping";

				ConfigHierarchy EngineIni = ConfigCache.ReadHierarchy(ConfigHierarchyType.Engine, DirectoryReference.FromFile(SC.RawProjectPath), SC.StageTargetPlatform.PlatformType, SC.CustomConfig);

				var Section = EngineIni.FindSection(IniSection_TargetSettings);
				if (Section != null && Section.TryGetValues(Key, out IReadOnlyList<string> ChunkMappingList))
				{
					foreach (var ChunkMapping in ChunkMappingList)
					{
						if (!ConfigHierarchy.TryParse(ChunkMapping, out Dictionary<string, string> Properties))
							throw new AutomationException("Failed to parse properties from chunk mapping.");

						CustomChunkMapping Pair;
						if (!TryGetValue(Properties, "Pattern", out Pair.Pattern))
							throw new AutomationException($"Failed to retrieve Pattern value from {Key} in config section \"{IniSection_TargetSettings}\".");

						if (!TryGetValue(Properties, "ChunkId", out Pair.ChunkId))
							throw new AutomationException($"Failed to retrieve ChunkId value from {Key} in config section \"{IniSection_TargetSettings}\".");

						yield return Pair;
					}
				}
			}
		}



		protected struct DLCChunks
		{
			// these must match the the GDKTargetSettings field
			public string DLCName;
			public int ChunkId;

			internal static Dictionary<string,IEnumerable<int>> GetFromConfig(ProjectParams Params, DeploymentContext SC, string IniSection_TargetSettings)
			{
				ConfigHierarchy EngineIni = ConfigCache.ReadHierarchy(ConfigHierarchyType.Engine, DirectoryReference.FromFile(SC.RawProjectPath), SC.StageTargetPlatform.PlatformType, SC.CustomConfig);

				if (EngineIni.TryGetValuesGeneric(IniSection_TargetSettings, "DLCChunks", out DLCChunks[] DLCChunks) && DLCChunks != null)
				{
					foreach (DLCChunks DLCChunk in DLCChunks)
					{
						if (string.IsNullOrEmpty(DLCChunk.DLCName))
						{
							throw new AutomationException("Missing DLCName in DLCChunks item", DLCChunk.DLCName);
						}

						if (DLCChunk.DLCName.Equals("Combined", StringComparison.InvariantCultureIgnoreCase) || Enum.TryParse<UnrealTargetConfiguration>(DLCChunk.DLCName, out UnrealTargetConfiguration _))
						{
							throw new AutomationException("Invalid DLC name {0} - reserved for use internally", DLCChunk.DLCName);
						}

						if (DLCChunk.ChunkId <= 0)
						{
							throw new AutomationException("Invalid or missing DLC chunk id {0} for {1}... must be > 0", DLCChunk.ChunkId, DLCChunk.DLCName);
						}
					}

					return DLCChunks.GroupBy( C => C.DLCName)
						.ToDictionary(
							G => G.Key,
							G => G.Select(C => C.ChunkId)
						);
				}

				return [];
			}
		}






		class PackageFileDesc
		{
			public StagedFileReference File;
			public string RelativeSourceDir;
			public int ChunkId;
			public bool bAllowInManifest;
			public bool bAllowInLayout;
			public bool bAutoGenerated;
		};



		private void GeneratePackageMetadataInternal(ProjectParams Params, DeploymentContext SC, IEnumerable<StagedFileReference> FilesToInclude, IEnumerable<StagedFileReference> GeneratedFilesToInclude, DirectoryReference LayoutDirectory, ILogger Logger, string DLCPackage)
		{
			ConfigHierarchy EngineIni = ConfigCache.ReadHierarchy(ConfigHierarchyType.Engine, DirectoryReference.FromFile(SC.RawProjectPath), SC.StageTargetPlatform.PlatformType, SC.CustomConfig);

			IReadOnlyDictionary<string, GDKBundleSettings> InstallBundles = null;
			if (BundleUtils.HasPlatformBundleSource(SC.RawProjectPath.Directory, SC.StageTargetPlatform.PlatformType, SC.CustomConfig) || BundleUtils.HasPlatformDLCBundleSource(SC.RawProjectPath.Directory, SC.StageTargetPlatform.PlatformType, SC.CustomConfig))
			{
				Logger.LogInformation("Using Intelligent Delivery Chunk mappings from install bundle config.");

				BundleUtils.LoadBundleConfig(SC.RawProjectPath.Directory, SC.StageTargetPlatform.PlatformType, SC.CustomConfig, out InstallBundles, (Settings, InBundleConfig, BundleSection) =>
				{
					if (InBundleConfig.GetInt32(BundleSection, "PlatformChunkID", out int ChunkID))
					{
						Settings.ChunkID = ChunkID;
					}
					else
					{
						Settings.ChunkID = 0;
					}

					if (InBundleConfig.GetString(BundleSection, "PlatformChunkName", out string ChunkTag))
					{
						Settings.PlatformChunkName = ChunkTag;
					}
					else
					{
						Settings.PlatformChunkName = "";
					}
				});
			}

			// if we are creating DLC packages automatically, we need to make sure that the correct chunks end up in the correct packages
			IEnumerable<int> AllowedChunkIds = null;
			IEnumerable<int> DisallowedChunkIds = null;
			EngineIni.GetBool(IniSection_TargetSettings, "bAutoCreateDLCPackages", out bool bAutoCreateDLCPackages);
			bool bIsDLC = false;
			if (bAutoCreateDLCPackages)
			{
				Dictionary<string, IEnumerable<int>> DLCChunkMap = DLCChunks.GetFromConfig(Params, SC, IniSection_TargetSettings);
				if (DLCPackage == null)
				{
					DisallowedChunkIds = DLCChunkMap.SelectMany( X => X.Value ).Distinct();
				}
				else
				{
					DLCChunkMap.TryGetValue(DLCPackage, out AllowedChunkIds);
					bIsDLC = true;
				}
			}
			bool CanPackageChunk( int ChunkID )
			{
				if (DisallowedChunkIds != null && DisallowedChunkIds.Contains(ChunkID))
				{
					return false;
				}

				if (AllowedChunkIds != null && !AllowedChunkIds.Contains(ChunkID))
				{
					return false;
				}

				return true;
			}

			// read intelligent install data, unless we are making DLC
			var IntelligentInstallChunks   = bIsDLC ? [] : IntelligentInstallChunk.GetFromConfig(Params, SC, IniSection_TargetSettings).ToList();
			var IntelligentInstallFeatures = bIsDLC ? [] : IntelligentInstallFeature.GetFromConfig(Params, SC, IniSection_TargetSettings).ToList();
			var IntelligentInstallRecipes  = bIsDLC ? [] : IntelligentInstallRecipe.GetFromConfig(Params, SC, IniSection_TargetSettings).ToList();

			EngineIni.GetBool(IniSection_TargetSettings, "bUseFeaturesAndRecipes", out bool bUseFeaturesAndRecipes);
			bUseFeaturesAndRecipes &= !bIsDLC;



			DirectoryReference.CreateDirectory(LayoutDirectory);
			FileReference PackageManifestFile = FileReference.Combine(LayoutDirectory, "package.manifest");
			FileReference LayoutXmlFile = FileReference.Combine(LayoutDirectory, "layout.xml");

			string RelativeLayoutDir = SC.StageDirectory.MakeRelativeTo(LayoutDirectory);


			// Build a map of all the files to include in the package, and which chunks they belong to.
			IReadOnlyDictionary<StagedFileReference, PackageFileDesc> FileChunkMapping;
			{
				var LocalFileChunkMapping = new Dictionary<StagedFileReference, PackageFileDesc>();
				FileChunkMapping = LocalFileChunkMapping;

				foreach (var File in FilesToInclude)
				{
					// Assume all files are in chunk 0, except for pak files with a suffixed chunk number (i.e. pakchunkN-platform.pak).
					var PakFileMatch = Regex.Match(File.Name, @".*[/\\]pakchunk([0-9]+)[a-zA-Z]*-[a-zA-Z0-9]+\.(pak|utoc||uondemandtoc|ucas)");
					int ChunkId = PakFileMatch.Success ? int.Parse(PakFileMatch.Groups[1].Value) : 0;

					LocalFileChunkMapping.Add(File, new PackageFileDesc()
					{
						File = File,
						RelativeSourceDir = Path.Combine(RelativeLayoutDir, File.Directory.Name),
						ChunkId = ChunkId,
						bAllowInManifest = true,
						bAllowInLayout = true,
						bAutoGenerated = false,
					});
				}

				// add all the generated files to include
				if (GeneratedFilesToInclude != null)
				{
					foreach (var File in GeneratedFilesToInclude)
					{
						LocalFileChunkMapping.Add(File, new PackageFileDesc()
						{
							File = File,
							RelativeSourceDir = "./" + File.Directory.Name,
							ChunkId = 0,
							bAllowInManifest = false,
							bAllowInLayout = true,
							bAutoGenerated = true,
						});
					}
				}

				// Add the package.manifest files to the chunk mapping
				{
					StagedFileReference StagedManifestFile = StagedFileReference.Combine(PackageManifestFile.MakeRelativeTo(LayoutDirectory));
					LocalFileChunkMapping.Add(StagedManifestFile, new PackageFileDesc()
					{
						File = StagedManifestFile,
						RelativeSourceDir = "./" + StagedManifestFile.Directory.Name,
						ChunkId = 0,
						bAllowInManifest = false,
						bAllowInLayout = true,
						bAutoGenerated = true,
					});
				}
			}

			// apply appropriate chunk remapping
			Dictionary<Int32, HashSet<string>> BundleDefinedImplicitChunkTags = new();  // Maps from chunk id to tag for bundle when there's no explict IntelligentDeliveryChunk definition
			if (InstallBundles != null)
			{
				HashSet<string> LaunchFeatures = [];

				// Override the mappings we built above for anything that matches the regex
				foreach (var Pair in FileChunkMapping)
				{
					string Filename = Path.GetFileName(Pair.Key.Name);
					GDKBundleSettings Bundle = BundleUtils.MatchBundleSettings(Filename, InstallBundles);
					if (Bundle != null)
					{
						Pair.Value.ChunkId = Bundle.ChunkID;
						if (!CanPackageChunk(Bundle.ChunkID))
						{
							continue;
						}

						Logger.LogInformation("Mapping {Filename} to Chunk {ChunkId} for Bundle {BundleName} {DLCPackage}", Filename, Bundle.ChunkID, Bundle.Name, DLCPackage ?? "");

						if (Bundle.ChunkID != -1 && Bundle.PlatformChunkName != "")
						{
							string ExistingTag = IntelligentInstallChunks.FirstOrDefault(X =>
								(X.ChunkId - OSChunkIdOffset) == Bundle.ChunkID &&
								!X.Tags.Contains(Bundle.PlatformChunkName, StringComparer.OrdinalIgnoreCase) &&
								!X.CultureIDs.Contains(Bundle.PlatformChunkName, StringComparer.OrdinalIgnoreCase))
								?.ChunkTag;
							if (ExistingTag != null)
							{
								throw new AutomationException($"Bundle '{Bundle.Name}' with PlatformChunkId={Bundle.ChunkID} has PlatformChunkName={Bundle.PlatformChunkName} but that tag is not referenced in tags or mapped CultureIds for IntelligentDeliveryChunks chunkId={Bundle.ChunkID} {ExistingTag}"); // @todo: could merge these tags, but that'd be a more invasive refactor & isn't supported on some other platforms
							}

							if (!BundleDefinedImplicitChunkTags.ContainsKey(Bundle.ChunkID))
							{
								BundleDefinedImplicitChunkTags.Add(Bundle.ChunkID, new());
							}
							BundleDefinedImplicitChunkTags[Bundle.ChunkID].Add(Bundle.PlatformChunkName);

							// create an implicit feature if necessary
							if (bUseFeaturesAndRecipes)
							{
								IntelligentInstallChunk ExistingChunk = IntelligentInstallChunks.FirstOrDefault( X => (X.ChunkId - OSChunkIdOffset) == Bundle.ChunkID );
								bool bIsLanguageChunk = (ExistingChunk != null) && ExistingChunk.Detail_Languages.Any();
								bool bIsLaunchChunk = (ExistingChunk != null) && ExistingChunk.Detail_bRequiredForLaunch;

								if (!bIsLanguageChunk)
								{
									// find the associated feature, or create it if it doesn't exist
									// note that we do this even for launch chunks because the bundle system needs all bundles to have a Feature so they can be mounted
									IntelligentInstallFeature Feature = IntelligentInstallFeatures.FirstOrDefault(X => X.Tags.Contains(Bundle.PlatformChunkName, StringComparer.OrdinalIgnoreCase));
									if (Feature == null)
									{
										IntelligentInstallFeatures.Add( Feature = new IntelligentInstallFeature
										{
											Id = Bundle.PlatformChunkName,
											Tags = [Bundle.PlatformChunkName],
											bHidden = true,
										});
									}
									else
									{
										Feature.AddTag(Bundle.PlatformChunkName);
									}

									if (bIsLaunchChunk)
									{
										LaunchFeatures.Add(Feature.Id);
									}
								}
							}
						}
					}
					else
					{
						Logger.LogInformation("Mapping {Filename} to Chunk {ChunkId} for unknown bundle", Filename, Pair.Value.ChunkId);
					}
				}

				// Create a default Recipe that contains the launch Features, if there isn't one already
				if (bUseFeaturesAndRecipes && IntelligentInstallRecipes.Count == 0 && LaunchFeatures.Count > 0)
				{
					Logger.LogInformation("Creating default Recipe for launch Features {features}", string.Join(',',LaunchFeatures));

					IntelligentInstallRecipes.Add(
						new IntelligentInstallRecipe
						{
							Id = "default",
							FeatureIds = [.. LaunchFeatures],
						}
					);			
				}
			}
			else
			{
				// If we have custom chunk mappings, override the mappings we built above for anything that matches the wildcard patterns.
				foreach (var Mapping in CustomChunkMapping.GetFromConfig(Params, SC, IniSection_TargetSettings))
				{
					string FullPathPattern = CombinePaths(SC.StageDirectory.FullName, Mapping.Pattern);

					var MatchedFiles = (FileFilter.FindWildcardIndex(FullPathPattern) != -1)
						? FileFilter.ResolveWildcard(FullPathPattern)
						: new List<FileReference>() { new FileReference(FullPathPattern) };

					foreach (var MatchedFile in MatchedFiles)
					{
						var FileRef = new StagedFileReference(MatchedFile.MakeRelativeTo(SC.StageDirectory));
						if (FileChunkMapping.ContainsKey(FileRef))
						{
							FileChunkMapping[FileRef].ChunkId = Mapping.ChunkId;
						}
					}
				}
			}

			// remove anything that's not autogenerated from the layout if it's for a different package
			foreach (var Pair in FileChunkMapping)
			{
				if (!Pair.Value.bAutoGenerated && !CanPackageChunk(Pair.Value.ChunkId))
				{
					Pair.Value.bAllowInManifest = false;
					Pair.Value.bAllowInLayout = false;
				}
			}


			// Generate a contiguous list of chunks up to the maximum ID listed in either the FileChunkMappings or IntelligentInstallChunks
			int MaxChunkId = (new[] { 0 })
				.Concat(FileChunkMapping.Values.Where(c => c.bAllowInLayout).Select(c => c.ChunkId))
				.Concat(IntelligentInstallChunks.Select(c => c.ChunkId - OSChunkIdOffset))
				.Where(c => c >= 0)
				.Distinct()
				.Max();
			var AllChunkIds = Enumerable.Range(0, MaxChunkId + 1).ToList();
			AllChunkIds.Sort();

			// Generate a lookup table of all chunk details for all chunks
			var ChunkInfo = AllChunkIds.ToDictionary(
				ChunkId => ChunkId,
				ChunkId => (from Mapping in IntelligentInstallChunks
							where Mapping.ChunkId - OSChunkIdOffset == ChunkId
							select Mapping)
							.FirstOrDefault() ?? new IntelligentInstallChunk
							{
								ChunkId = ChunkId + OSChunkIdOffset,
								ChunkTag = BundleDefinedImplicitChunkTags.TryGetValue(ChunkId, out HashSet<string> Tags) ? string.Join(';', Tags) : ""
							}
			);

			// Create a collection of all chunks that are required for launch
			var InitialChunkIds = (from Mapping in IntelligentInstallChunks
								   where Mapping.Detail_bRequiredForLaunch
								   select Mapping.ChunkId - OSChunkIdOffset).Distinct().ToList();
			InitialChunkIds.Sort();
			if (InitialChunkIds.Count == 0)
			{
				if (bIsDLC)
				{
					InitialChunkIds.AddRange(AllChunkIds);
				}
				else
				{
					int MinChunkId = AllChunkIds.Min();
					if (IntelligentInstallChunks.Count > 0)
					{
						Logger.LogTrace($"No initial chunk ID specified. Assuming chunk {MinChunkId} is the initial chunk.");
					}
					InitialChunkIds.Add(MinChunkId);
				}
			}

			// Gather the tags that can be used in the layout.xml
			var AllowedLayoutTags = ChunkInfo.Values.SelectMany(Chunk => Chunk.Tags).ToHashSet(StringComparer.InvariantCultureIgnoreCase);
			if (IntelligentInstallFeatures.Any())
			{
				// do not allow tags that are not referenced by a feature. These extra tags are useful for the manifest generated below, but will cause packaging failures in layout.xml
				var ReferencedTags = IntelligentInstallFeatures
					.Select(Feature => Feature.Tags)
					.SelectMany(X => X)
					.ToHashSet(StringComparer.InvariantCultureIgnoreCase);

				AllowedLayoutTags.IntersectWith(ReferencedTags);
			}

			// Generate the package.manifest file
			{
				static string ToManifestFile(StagedFileReference File) => File.Name.Replace('/', '\\');

				var StagedFilesByChunk =
					(from File in FileChunkMapping
					 where File.Value.bAllowInManifest && File.Value.ChunkId >= 0
					 orderby File.Value.ChunkId ascending
					 group File by File.Value.ChunkId)
						.ToDictionary(g => g.Key, g => g.ToList());

				var ManifestObj = new Dictionary<string, object>()
				{
					["chunks"] = StagedFilesByChunk
						.Select(Chunk => new Dictionary<string, object>()
						{
							["id"] = Chunk.Key,
							["Tag"] = ChunkInfo[Chunk.Key].ChunkTag,
							["Devices"] = ChunkInfo[Chunk.Key].Detail_Devices,
							["Languages"] = ChunkInfo[Chunk.Key].Detail_Languages,
							["IsInitial"] = InitialChunkIds.Contains(Chunk.Key),

							["files"] = Chunk.Value.Select(File => ToManifestFile(File.Key))
						})
				};

				if (IntelligentInstallFeatures.Any())
				{
					ManifestObj["features"] = IntelligentInstallFeatures
						.Select(Feature => new Dictionary<string, object>()
						{
							["Id"] = Feature.Id,
							["Tags"] = string.Join(';', Feature.Tags)
						});
				}

				var JsonOptions = new JsonSerializerOptions { WriteIndented = true };
				var Contents = JsonSerializer.Serialize(ManifestObj, JsonOptions);
				File.WriteAllText(PackageManifestFile.FullName, Contents, new UTF8Encoding(encoderShouldEmitUTF8Identifier: false));
			}



			// Generate layout.xml file
			{
				bool bHasRecipes = IntelligentInstallFeatures.Any() || IntelligentInstallRecipes.Any();

				string ToSourcePath(string SrcDir) => @$"{SrcDir.Replace('/', '\\')}";
				string ToDestPath(StagedFileReference File) => @$"\{File.Directory.Name.Replace('/', '\\')}";

				var StagedFilesByChunk =
					(from File in FileChunkMapping
					 where File.Value.bAllowInLayout && File.Value.ChunkId >= 0
					 orderby File.Value.ChunkId ascending
					 orderby InitialChunkIds.Contains(File.Value.ChunkId) descending // launch chunks first
					 group File by File.Value.ChunkId)
						.ToDictionary(g => g.Key, g => g.ToList());

				var LayoutXml = new XElement("Package",
					IntelligentInstallRecipes.Any() ? new XElement("Recipes",
						from Recipe in IntelligentInstallRecipes
						select new XElement("Recipe",
							new XAttribute("Id", Recipe.Id),
							opt_XAttribute("IncludedFeatures", Recipe.FeatureIds),
							opt_XAttribute("Devices", Recipe.Devices),
							opt_XAttribute("StoreId", Recipe.StoreIds)
						)) : null,

					IntelligentInstallFeatures.Any() ? new XElement("Features",
						from Feature in IntelligentInstallFeatures.Where( F => F.Tags.Length > 0 ) // skip features with no tags
						select new XElement("Feature",
							new XAttribute("Id", Feature.Id),
							opt_XAttribute("Tags", Feature.Tags),
							opt_XAttribute("Hidden", Feature.bHidden),
							opt_XAttribute("DisplayName", Feature.DisplayNameResourceId),
							opt_XAttribute("Image", Feature.ImageFileName)
						)) : null,

					from Chunk in StagedFilesByChunk
					select new XElement("Chunk",
						new XAttribute("Id", Chunk.Key + OSChunkIdOffset),
						opt_XAttribute("OnDemand", ChunkInfo[Chunk.Key].IsOnDemand && !bHasRecipes && !InitialChunkIds.Contains(Chunk.Key)), // OnDemand tags only apply when not using Features & Recipes. Launch chunks can't be OnDemand
						opt_XAttribute("Languages", ChunkInfo[Chunk.Key].Detail_Languages),
						opt_XAttribute("Devices", ChunkInfo[Chunk.Key].Detail_Devices),
						opt_XAttribute("Tags", ChunkInfo[Chunk.Key].Tags.Intersect(AllowedLayoutTags).ToArray()),                         // Filter chunk's tags to only include the permitted ones
						opt_XAttribute("Marker", "Launch", bAdditionalPrerequisite: (Chunk.Key == InitialChunkIds.Max())),                   // Last initial chunk gets the Launch marker
						from FilePair in Chunk.Value
						select new XElement("FileGroup",
							new XAttribute("Include", Path.GetFileName(FilePair.Key.Name)),
							new XAttribute("SourcePath", ToSourcePath(FilePair.Value.RelativeSourceDir)),
							new XAttribute("DestinationPath", ToDestPath(FilePair.Value.File))
						)
					)
				);

				LayoutXml.Save(LayoutXmlFile.FullName);
			}
		}



		/// <summary>
		/// Generates the layout.xml and associated files required for packaging in the given layout directory
		/// </summary>
		public void GenerateSinglePackageMetadata(ProjectParams Params, DeploymentContext SC, DirectoryReference PackageMetadataDir, string DLCPackage, UnrealTargetConfiguration? OnlyConfig = null)
		{
			string PackageConfig = (DLCPackage == null) ? (OnlyConfig?.ToString() ?? "Combined") : DLCPackage;
			PackageMetadataDir = DirectoryReference.Combine(PackageMetadataDir, PackageConfig);

			FileUtils.ForceDeleteDirectory(PackageMetadataDir);
			FileUtils.CreateDirectoryTree(PackageMetadataDir);

			// Gather all the files we'll be including in the package
			var FilesUnion = SC.FilesToStage.NonUFSFiles.Union(SC.FilesToStage.NonUFSSystemFiles);
			if (!Params.UsePak(Platform.GetPlatform(PlatformType)))
			{
				// Don't include UFS files in the layout.xml if we're cooking on-the-fly or using ZenStore.
				// UFS files are provided by the connection to the server.
				if (!Params.CookOnTheFly && !Params.ShouldTreatAsFileServer(SC))
				{
					FilesUnion = FilesUnion.Union(SC.FilesToStage.UFSFiles);
				}

				FilesUnion = FilesUnion.Union(SC.CrashReporterUFSFiles);
			}
			else
			{
				var DLCName = Params.HasDLCName ? Params.DLCFile.GetFileNameWithoutExtension() : null;

				var PaksDir = Params.HasDLCName
					? Path.Combine(SC.StageDirectory.FullName, SC.ShortProjectName, "Plugins", DLCName, "Content", "Paks", SC.FinalCookPlatform)
					: Path.Combine(SC.StageDirectory.FullName, SC.ShortProjectName, "Content", "Paks");
				var FilesInPaksDir = (new[] { "*.pak", "*.ucas", "*.utoc", "*.uondemandtoc", "*.sig" }).SelectMany(Pattern => FindFiles(Pattern, false, PaksDir));

				FilesUnion = FilesUnion.Union(
					from f in FilesInPaksDir
					let Reference = new FileReference(Path.Combine(SC.StageDirectory.FullName, f))
					select new KeyValuePair<StagedFileReference, FileReference>(new StagedFileReference(Reference.MakeRelativeTo(SC.StageDirectory)), Reference)
				);

				// crash reporter is handled as a special case in CopyBuildToStagingDirectory
				var CrashReporterPak = FileReference.Combine(SC.RuntimeRootDir, "Engine", "Programs", "CrashReportClient", "Content", "Paks", "CrashReportClient.pak");
				if (SC.CrashReporterUFSFiles.Any() && FileReference.Exists(CrashReporterPak))
				{
					FilesUnion = FilesUnion.Append(new KeyValuePair<StagedFileReference, FileReference>(new StagedFileReference(CrashReporterPak.MakeRelativeTo(SC.StageDirectory)), CrashReporterPak));
				}
			}

			// Only include the debug files in non-distribution builds, and when not running UAT with the "-nodebuginfo" flag.
			// We add debug files here regardless of Params.Distribution because we may be generating multiple configurations,
			// some allowing debug files, some don't.
			if (!Params.NoDebugInfo && !Params.SeparateDebugInfo && !Params.Distribution)
			{
				FilesUnion = FilesUnion.Union(SC.FilesToStage.NonUFSDebugFiles);
			}

			// generate a filtered list of the files that should be included in the package
			IEnumerable<StagedFileReference> FilesToInclude = FilesUnion.Where(X =>
				ShouldPackageFile(X.Value, Params, SC, OnlyConfig))
				.Select(f => f.Key);

			IEnumerable<StagedFileReference> GeneratedFilesToInclude = [];

			// Generate MicrosoftGame.config if we are not generating metadata in the staging root (if we are we'll want to re-use the staging manifest)
			if (PackageMetadataDir != SC.StageDirectory)
			{
				// exclude any staged manifest resources as we're generating new ones
				IEnumerable<StagedFileReference> ExistingStagedManifestFiles = GetGDKManifestFilesInDirectory(SC.StageDirectory)
					.Select(File => StagedFileReference.Combine(File.MakeRelativeTo(SC.StageDirectory)));
				FilesToInclude = FilesToInclude.Except(ExistingStagedManifestFiles);

				// generate the manifest and gather the files
				List<AppXManifestExecutable> Executables = GDKAutomationUtils.GetGameConfigExecutables(Params, SC, Purpose: GDKAutomationUtils.GameConfigPurpose.ForStaging, IniSection_TargetSettings, OnlyConfig);
				GDKGameConfigGenerator.Create(PlatformType, Logger, PackageMetadataDir, SC.ShortProjectName, Params.RawProjectPath, Executables, bDeleteIntermediates: true, DLCFile: Params.DLCFile, DLCPackage: DLCPackage, CustomConfig: SC.CustomConfig);
				GeneratedFilesToInclude = GeneratedFilesToInclude.Union( GetGDKManifestFilesInDirectory(PackageMetadataDir)
					.Select(File => StagedFileReference.Combine(File.MakeRelativeTo(PackageMetadataDir)))
				);
			}

			// Add any platform specific files that need to be included in the package
			GeneratedFilesToInclude = GeneratedFilesToInclude.Union(GetPlatformFilesToPackage(Params, SC, PackageMetadataDir, OnlyConfig));


			// Generate the packaging metadata
			GeneratePackageMetadataInternal(Params, SC, FilesToInclude, GeneratedFilesToInclude, PackageMetadataDir, Logger, DLCPackage);
		}

		protected virtual IEnumerable<StagedFileReference> GetPlatformFilesToPackage(ProjectParams Params, DeploymentContext SC, DirectoryReference PackageMetadataDir, UnrealTargetConfiguration? OnlyConfig)
		{
			return [];
		}


		#region helper functions

		/// attempt to parse out a typed value from the given dictionary & key
		protected static bool TryGetValue<T>(Dictionary<string, string> Dictionary, string Key, out T Value)
		{
			try
			{
				Value = (T)TypeDescriptor.GetConverter(typeof(T)).ConvertFromInvariantString(Dictionary[Key]);
				return true;
			}
			catch
			{
				Value = default(T);
				return false;
			}
		}

		/// attempt to parse out an array of strings from the given dictionary & key
		protected static bool TryGetValue(Dictionary<string, string> Dictionary, string Key, out string[] Value)
		{
			if (TryGetValue(Dictionary, Key, out string RawValue))
			{
				if (ConfigHierarchy.TryParse(RawValue, out Value))
				{
					return true;
				}
			}

			Value = Array.Empty<string>();
			return false;
		}

		/// create an XAttribute as long as the value is not null and the additionl prerequisite is true
		protected static XAttribute opt_XAttribute(string Key, string OptValue, bool bAdditionalPrerequisite = true)
		{
			if (!bAdditionalPrerequisite || string.IsNullOrEmpty(OptValue))
			{
				return null;
			}
			return new XAttribute(Key, OptValue);
		}

		/// create an XAttribute of semicolon-separated values as long as there are values
		protected static XAttribute opt_XAttribute(string Key, string[] Values)
		{
			if (Values == null || Values.Length == 0)
			{
				return null;
			}
			return new XAttribute(Key, string.Join(';', Values));
		}

		/// create an XAttribute as long as the value is true
		protected static XAttribute opt_XAttribute(string Key, bool OptValue)
		{
			if (!OptValue)
			{
				return null;
			}
			return new XAttribute(Key, OptValue);
		}

		#endregion

		#endregion
	}
}
