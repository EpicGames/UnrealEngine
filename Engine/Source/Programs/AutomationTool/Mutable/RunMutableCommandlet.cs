// Copyright Epic Games, Inc. All Rights Reserved.

using Gauntlet;


namespace RunMutableCommandlet.Automation
{
	/// <summary>
	/// Simple UnrealTestNode designed to allow the caller to specify a timeout value for the execution of the targetted mutable commandlet
	/// </summary>	
	public sealed class RunMutableCommandlet : UnrealTestNode<UnrealTestConfiguration>
	{
		/// <summary>
		/// The name of the commandlet to run
		/// </summary>
		string CommandletName = "";

		/// <summary>
		/// The arguments that will get fed to the comandlet. 
		/// </summary>
		string CommandletArguments = "";

		/// <summary>
		/// The max amount of time (seconds) that the commandlet will be allowed to run.
		/// </summary>
		float CommandletTimeoutTime = 600f;


		public RunMutableCommandlet(UnrealTestContext InContext) : 
			base(InContext)
		{
			// Parse provided arguments
			CommandletName = InContext.TestParams.ParseValue("commandlet=", CommandletName);
			CommandletArguments = InContext.TestParams.ParseValue("arguments=", CommandletArguments);
			CommandletTimeoutTime = InContext.TestParams.ParseValue("timeout=", CommandletTimeoutTime);
			
			// Errors and warnings of the commandlet will be treated as errors and warnings of the test itself
			Flags = BehaviorFlags.PromoteWarnings | BehaviorFlags.PromoteErrors;	
		}


		public override UnrealTestConfiguration GetConfiguration()
		{
			UnrealTestConfiguration Configuration = base.GetConfiguration();

			UnrealTestRole EditorRole = Configuration.RequireRole(UnrealTargetRole.Editor);

			EditorRole.CommandLineParams.AddUnique("run", CommandletName);
			EditorRole.CommandLine += CommandletArguments;

			// Error out if the max duration time is reached
			Configuration.MaxDuration = CommandletTimeoutTime;
			Configuration.MaxDurationReachedResult = EMaxDurationReachedResult.Failure;

			// Expose errors and warnings raised by the commandlet
			Configuration.ShowErrorsInSummary = true;
			Configuration.ShowWarningsInSummary = true;

			// All errors should be reported
			Configuration.FailOnEnsures = true;

			Configuration.LogCategoriesForEvents.AddRange(new string[]  {
				"LogMutable",
				"LogMutableCore"
			});
			Configuration.VerboseLogCategories = "LogMutable,LogMutableCore";
			return Configuration;
		}

		protected override UnrealProcessResult GetExitCodeAndReason(StopReason InReason, UnrealLog InLog, UnrealRoleArtifacts InArtifacts, out string ExitReason, out int ExitCode)
		{
			// Force the Engine Initialized flag to true (it seems not to change even in good runs of the commandlet)
			InLog.EngineInitialized = true;

			// Use the exit code of the commandlet as the exit code of the test.
			if (InLog.TestExitCode != 0) 
			{
				ExitCode = InLog.TestExitCode;
			}

			return base.GetExitCodeAndReason(InReason, InLog, InArtifacts, out ExitReason, out ExitCode);
		}
	}

}
