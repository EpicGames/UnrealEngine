// Copyright Epic Games, Inc. All Rights Reserved.
using System;
using System.Collections.Generic;
using System.IO;
using System.Reflection;
using AutomationTool;
using UnrealBuildTool;
using Microsoft.Extensions.Logging;

using static AutomationTool.CommandUtils;
using UnrealBuildBase;

namespace EpicGames.OneSkyLocalization.Config
{
	/// <summary>
	/// Class for retrieving OneSky localization configuration data
	/// </summary>
	public class OneSkyConfigHelper
	{
		// List of configs is cached off for fetching from multiple times
		private static Dictionary<string, OneSkyConfigData> Configs;

		public static OneSkyConfigData Find(string ConfigName)
		{
			if (Configs == null)
			{
				// Load all secret configs by trying to instantiate all classes derived from OneSkyConfigData from all loaded DLLs.
				// Note that we're using the default constructor on the secret config types.
				Configs = new Dictionary<string, OneSkyConfigData>();
				Assembly[] LoadedAssemblies = AppDomain.CurrentDomain.GetAssemblies();
				foreach (var Dll in LoadedAssemblies)
				{
					Type[] AllTypes = Dll.SafeGetLoadedTypes();
					foreach (var PotentialConfigType in AllTypes)
					{
						if (PotentialConfigType != typeof(OneSkyConfigData) && typeof(OneSkyConfigData).IsAssignableFrom(PotentialConfigType))
						{
							try
							{
								OneSkyConfigData Config = Activator.CreateInstance(PotentialConfigType) as OneSkyConfigData;
								if (Config != null)
								{
									Configs.Add(Config.Name, Config);
								}
							}
							catch
							{
								Logger.LogWarning("Unable to create OneSky config data: {Name}", PotentialConfigType.Name);
							}
						}
					}
				}
			}
			OneSkyConfigData LoadedConfig;
			Configs.TryGetValue(ConfigName, out LoadedConfig);
			if (LoadedConfig == null)
			{
				throw new AutomationException("Unable to find requested OneSky config data: {0}", ConfigName);
			}
			return LoadedConfig;
		}
	}

	/// <summary>
	/// Class for storing OneSky Localization configuration data
	/// </summary>
	public class OneSkyConfigData
	{
		public OneSkyConfigData(string InName, string InApiKey, string InApiSecret)
		{
			Name = InName;
			ApiKey = InApiKey;
			ApiSecret = InApiSecret;
		}

		public string Name;
		public string ApiKey;
		public string ApiSecret;

		public void SpewValues()
		{
			Logger.LogInformation("Name : {Name}", Name);
			Logger.LogInformation("ApiKey : {ApiKey}", ApiKey);
			//CommandUtils.LogConsole("ApiSecret : {0}", ApiSecret);  // This should probably never be revealed.
		}
	}
}
