// Copyright Epic Games, Inc. All Rights Reserved.
using System;
using System.Collections.Generic;
using System.IO;
using System.Linq;
using AutomationTool;
using UnrealBuildTool;
using OneSky;
using EpicGames.Localization;
using EpicGames.OneSkyLocalization.Config;
using System.Threading.Tasks;

namespace EpicGames.OneSkyLocalization
{
	[Help("OneSkyConfigName", "Name of the config data to use (see OneSkyConfigHelper).")]
	[Help("OneSkyProjectGroupName", "Name of the project group in OneSky.")]
	public class OneSkyLocalizationProvider : LocalizationProvider
	{
		public OneSkyLocalizationProvider(LocalizationProviderArgs InArgs)
			: base(InArgs)
		{
			var OneSkyConfigName = Command.ParseParamValue("OneSkyConfigName");
			if (OneSkyConfigName == null)
			{
				throw new AutomationException("Missing required command line argument: 'OneSkyConfigName'");
			}

			var OneSkyProjectGroupName = Command.ParseParamValue("OneSkyProjectGroupName");
			if (OneSkyProjectGroupName == null)
			{
				throw new AutomationException("Missing required command line argument: 'OneSkyProjectGroupName'");
			}

			OneSkyConfigData OneSkyConfig = OneSkyConfigHelper.Find(OneSkyConfigName);
			OneSkyService = new OneSkyService(OneSkyConfig.ApiKey, OneSkyConfig.ApiSecret);
			OneSkyProjectGroup = GetOneSkyProjectGroup(OneSkyProjectGroupName);
		}

		public static string StaticGetLocalizationProviderId()
		{
			return "OneSky";
		}

		public override string GetLocalizationProviderId()
		{
			return StaticGetLocalizationProviderId();
		}

		public async override Task DownloadProjectFromLocalizationProvider(string ProjectName, ProjectImportExportInfo ProjectImportInfo)
		{
			var OneSkyProject = GetOneSkyProject(ProjectName);
			DownloadLatestPOFiles(OneSkyProject, null, ProjectImportInfo);
			foreach (var Platform in ProjectImportInfo.SplitPlatformNames)
			{
				DownloadLatestPOFiles(OneSkyProject, Platform, ProjectImportInfo);
			}

			await Task.CompletedTask;
		}

		private void DownloadLatestPOFiles(Project OneSkyProject, string Platform, ProjectImportExportInfo ProjectImportInfo)
		{
			var OneSkyFileName = GetOneSkyFilename(ProjectImportInfo.PortableObjectName, Platform);
			var OneSkyFile = OneSkyProject.UploadedFiles.FirstOrDefault(f => f.Filename == OneSkyFileName);

			// Export
			if (OneSkyFile != null)
			{
				var CulturesToExport = new List<string>();
				foreach (var OneSkyCulture in OneSkyProject.EnabledCultures)
				{
					// Skip the native culture, as OneSky has mangled it
					if (OneSkyCulture == ProjectImportInfo.NativeCulture)
					{
						continue;
					}

					// Only export the OneSky cultures that we care about for this project
					if (ProjectImportInfo.CulturesToGenerate.Contains(OneSkyCulture))
					{
						CulturesToExport.Add(OneSkyCulture);
					}
				}

				var DestinationDirectory = String.IsNullOrEmpty(Platform)
						? new DirectoryInfo(CommandUtils.CombinePaths(RootWorkingDirectory, ProjectImportInfo.DestinationPath))
						: new DirectoryInfo(CommandUtils.CombinePaths(RootWorkingDirectory, ProjectImportInfo.DestinationPath, ProjectImportExportInfo.PlatformLocalizationFolderName, Platform));
				ExportOneSkyFileToDirectory(OneSkyFile, DestinationDirectory, ProjectImportInfo.PortableObjectName, CulturesToExport, ProjectImportInfo.bUseCultureDirectory);
			}
		}

		private UploadedFile.ExportTranslationState ExportOneSkyTranslationWithRetry(UploadedFile OneSkyFile, string Culture, MemoryStream MemoryStream)
		{
			const int MAX_COUNT = 3;

			long StartingMemPos = MemoryStream.Position;
			int Count = 0;
			for (;;)
			{
				try
				{
					return OneSkyFile.ExportTranslation(Culture, MemoryStream).Result;
				}
				catch (Exception)
				{
					if (++Count < MAX_COUNT)
					{
						MemoryStream.Position = StartingMemPos;
						Console.WriteLine("ExportOneSkyTranslation attempt {0}/{1} failed. Retrying...", Count, MAX_COUNT);
						continue;
					}

					Console.WriteLine("ExportOneSkyTranslation attempt {0}/{1} failed.", Count, MAX_COUNT);
					break;
				}
			}

			return UploadedFile.ExportTranslationState.Failure;
		}

		private void ExportOneSkyFileToDirectory(UploadedFile OneSkyFile, DirectoryInfo DestinationDirectory, string DestinationFilename, IEnumerable<string> Cultures, bool bUseCultureDirectory)
		{
			foreach (var Culture in Cultures)
			{
				var CultureDirectory = (bUseCultureDirectory) ? new DirectoryInfo(Path.Combine(DestinationDirectory.FullName, Culture)) : DestinationDirectory;
				if (!CultureDirectory.Exists)
				{
					CultureDirectory.Create();
				}

				using (var MemoryStream = new MemoryStream())
				{
					var ExportTranslationState = ExportOneSkyTranslationWithRetry(OneSkyFile, Culture, MemoryStream);
					if (ExportTranslationState == UploadedFile.ExportTranslationState.Success)
					{
						var ExportFile = new FileInfo(Path.Combine(CultureDirectory.FullName, DestinationFilename));

						// Write out the updated PO file so that the gather commandlet will import the new data from it
						{
							var ExportFileWasReadOnly = false;
							if (ExportFile.Exists)
							{
								// We're going to clobber the existing PO file, so make sure it's writable (it may be read-only if in Perforce)
								ExportFileWasReadOnly = ExportFile.IsReadOnly;
								ExportFile.IsReadOnly = false;
							}

							MemoryStream.Position = 0;
							using (Stream FileStream = ExportFile.Open(FileMode.Create))
							{
								MemoryStream.CopyTo(FileStream);
								Console.WriteLine("[SUCCESS] Exported '{0}' as '{1}' ({2})", OneSkyFile.Filename, ExportFile.FullName, Culture);
							}

							if (ExportFileWasReadOnly)
							{
								ExportFile.IsReadOnly = true;
							}
						}

						// Also update the back-up copy so we can diff against what we got from OneSky, and what the gather commandlet produced
						{
							var ExportFileCopy = new FileInfo(Path.Combine(ExportFile.DirectoryName, String.Format("{0}_FromOneSky{1}", Path.GetFileNameWithoutExtension(ExportFile.Name), ExportFile.Extension)));

							var ExportFileCopyWasReadOnly = false;
							if (ExportFileCopy.Exists)
							{
								// We're going to clobber the existing PO file, so make sure it's writable (it may be read-only if in Perforce)
								ExportFileCopyWasReadOnly = ExportFileCopy.IsReadOnly;
								ExportFileCopy.IsReadOnly = false;
							}

							ExportFile.CopyTo(ExportFileCopy.FullName, true);

							if (ExportFileCopyWasReadOnly)
							{
								ExportFileCopy.IsReadOnly = true;
							}

							// Add/check out backed up POs from OneSky.
							if (CommandUtils.P4Enabled)
							{
								UnrealBuild.AddBuildProductsToChangelist(PendingChangeList, new List<string>() { ExportFileCopy.FullName });
							}
						}
					}
					else if (ExportTranslationState == UploadedFile.ExportTranslationState.NoContent)
					{
						Console.WriteLine("[SUCCESS] Skipped '{0}' ({1}) as it has no translations", OneSkyFile.Filename, Culture);
					}
					else
					{
						Console.WriteLine("[FAILED] Failed to get translation data for '{0}' ({1})", OneSkyFile.Filename, Culture);
					}
				}
			}
		}

		public async override Task UploadProjectToLocalizationProvider(string ProjectName, ProjectImportExportInfo ProjectExportInfo)
		{
			var OneSkyProject = GetOneSkyProject(ProjectName);

			// Upload the .po file for the native culture first
			UploadFileToOneSky(OneSkyProject, ProjectExportInfo.NativeCulture, null, ProjectExportInfo);
			foreach (var Platform in ProjectExportInfo.SplitPlatformNames)
			{
				UploadFileToOneSky(OneSkyProject, ProjectExportInfo.NativeCulture, Platform, ProjectExportInfo);
			}

			if (bUploadAllCultures)
			{
				// Upload the remaining .po files for the other cultures
				foreach (var Culture in ProjectExportInfo.CulturesToGenerate)
				{
					// Skip native culture as we uploaded it above
					if (Culture != ProjectExportInfo.NativeCulture)
					{
						UploadFileToOneSky(OneSkyProject, Culture, null, ProjectExportInfo);
						foreach (var Platform in ProjectExportInfo.SplitPlatformNames)
						{
							UploadFileToOneSky(OneSkyProject, Culture, Platform, ProjectExportInfo);
						}
					}
				}
			}

			await Task.CompletedTask;
		}

		private UploadedFile UploadOneSkyTranslationWithRetry(OneSky.Project OneSkyProject, string OneSkyFileName, string Culture, FileStream FileStream)
		{
			const int MAX_COUNT = 3;

			long StartingFilePos = FileStream.Position;
			int Count = 0;
			for (;;)
			{
				try
				{
					return OneSkyProject.Upload(OneSkyFileName, FileStream, Culture).Result;
				}
				catch (Exception)
				{
					if (++Count < MAX_COUNT)
					{
						FileStream.Position = StartingFilePos;
						Console.WriteLine("UploadOneSkyTranslation attempt {0}/{1} failed. Retrying...", Count, MAX_COUNT);
						continue;
					}

					Console.WriteLine("UploadOneSkyTranslation attempt {0}/{1} failed.", Count, MAX_COUNT);
					break;
				}
			}

			return null;
		}

		private void UploadFileToOneSky(OneSky.Project OneSkyProject, string Culture, string Platform, ProjectImportExportInfo ProjectExportInfo)
		{
			var SourceDirectory = String.IsNullOrEmpty(Platform)
					? new DirectoryInfo(CommandUtils.CombinePaths(RootWorkingDirectory, ProjectExportInfo.DestinationPath))
					: new DirectoryInfo(CommandUtils.CombinePaths(RootWorkingDirectory, ProjectExportInfo.DestinationPath, ProjectImportExportInfo.PlatformLocalizationFolderName, Platform));
			var CultureDirectory = (ProjectExportInfo.bUseCultureDirectory) ? new DirectoryInfo(Path.Combine(SourceDirectory.FullName, Culture)) : SourceDirectory;

			var FileToUpload = new FileInfo(Path.Combine(CultureDirectory.FullName, ProjectExportInfo.PortableObjectName));
			using (var FileStream = FileToUpload.OpenRead())
			{
				// Read the BOM
				var UTF8BOM = new byte[3];
				FileStream.Read(UTF8BOM, 0, 3);

				// We want to ignore the utf8 BOM
				if (UTF8BOM[0] != 0xef || UTF8BOM[1] != 0xbb || UTF8BOM[2] != 0xbf)
				{
					FileStream.Position = 0;
				}

				var OneSkyFileName = GetOneSkyFilename(ProjectExportInfo.PortableObjectName, Platform);

				Console.WriteLine("Uploading: '{0}' as '{1}' ({2})", FileToUpload.FullName, OneSkyFileName, Culture);

				var UploadedFile = UploadOneSkyTranslationWithRetry(OneSkyProject, OneSkyFileName, Culture, FileStream);

				if (UploadedFile == null)
				{
					Console.WriteLine("[FAILED] Uploading: '{0}' ({1})", FileToUpload.FullName, Culture);
				}
				else
				{
					Console.WriteLine("[SUCCESS] Uploading: '{0}' ({1})", FileToUpload.FullName, Culture);
				}
			}
		}

		private ProjectGroup GetOneSkyProjectGroup(string ProjectGroupName)
		{
			var OneSkyProjectGroup = OneSkyService.ProjectGroups.FirstOrDefault(g => g.Name == ProjectGroupName);

			if (OneSkyProjectGroup == null)
			{
				OneSkyProjectGroup = new ProjectGroup(ProjectGroupName, "en");
				OneSkyService.ProjectGroups.Add(OneSkyProjectGroup);
			}

			return OneSkyProjectGroup;
		}

		private OneSky.Project GetOneSkyProject(string ProjectName, string ProjectDescription = "")
		{
			OneSky.Project OneSkyProject = OneSkyProjectGroup.Projects.FirstOrDefault(p => p.Name == ProjectName);

			if (OneSkyProject == null)
			{
				ProjectType projectType = OneSkyService.ProjectTypes.First(pt => pt.Code == "website");

				OneSkyProject = new OneSky.Project(ProjectName, ProjectDescription, projectType);
				OneSkyProjectGroup.Projects.Add(OneSkyProject);
			}

			return OneSkyProject;
		}

		private string GetOneSkyFilename(string BaseFilename, string Platform)
		{
			var OneSkyFileName = BaseFilename;
			if (!String.IsNullOrEmpty(Platform))
			{
				// Apply the platform suffix. XLoc will take care of merging the files from different platforms together.
				var OneSkyFileNameWithSuffix = Path.GetFileNameWithoutExtension(OneSkyFileName) + "_" + Platform + Path.GetExtension(OneSkyFileName);
				OneSkyFileName = OneSkyFileNameWithSuffix;
			}
			if (!String.IsNullOrEmpty(LocalizationBranchName))
			{
				// Apply the branch suffix. OneSky will take care of merging the files from different branches together.
				var OneSkyFileNameWithSuffix = Path.GetFileNameWithoutExtension(OneSkyFileName) + "_" + LocalizationBranchName + Path.GetExtension(OneSkyFileName);
				OneSkyFileName = OneSkyFileNameWithSuffix;
			}
			if (!String.IsNullOrEmpty(RemoteFilenamePrefix))
			{
				// Apply the prefix (this is used to avoid collisions with plugins that use the same name for their PO files)
				OneSkyFileName = RemoteFilenamePrefix + "_" + OneSkyFileName;
			}
			return OneSkyFileName;
		}

		private OneSkyService OneSkyService;
		private ProjectGroup OneSkyProjectGroup;
	}
}
