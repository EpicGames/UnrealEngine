// Copyright Epic Games, Inc. All Rights Reserved.
// This software is provided "as-is," without any express or implied warranty. 
// In no event shall the author, nor Epic Games, Inc. be held liable for any damages arising from the use of this software.
// This software will not be supported.
// Use at your own risk.
using System;
using System.Linq;
using System.Threading;
using System.Threading.Tasks;
using System.Diagnostics;
using System.Reflection;
using EpicGames.Core;
using EpicGames.LogMatchers;
using System.IO;
using System.Collections.Generic;
using UnrealBuildBase;
using Microsoft.Extensions.Logging;

namespace AutomationToolDriver
{
	/// <summary>
	/// Main entry point
	/// </summary>
	public partial class Program
	{
		/// <summary>
		/// Parses command line parameter.
		/// </summary>
		/// <param name="CurrentParam">Parameter</param>
		/// <param name="CurrentCommand">Recently parsed command</param>
		/// <param name="Logger">Logger for output</param>
		/// <returns>True if the parameter has been successfully parsed.</returns>
		private static void ParseParam(string CurrentParam, CommandInfo CurrentCommand, ILogger Logger)
		{
			if (AutomationToolCommandLine.IsParameterIgnored(CurrentParam))
            {
				return;
            }

			bool bGlobalParam = AutomationToolCommandLine.TrySetGlobal(CurrentParam);

			// Global value parameters, handled explicitly
			string Option_ScriptsForProject = "-ScriptsForProject";	// restrict game search directories to the the specified uproject
			string Option_ScriptDir = "-ScriptDir";					// additional search directory for .csproj files
			string Option_AutomationProjects = "-AutomationProject";// filter the found .csproj projects to build and load, specified as name1+name2+...+nameN
			string Option_Telemetry = "-Telemetry";
			string Option_WaitForStdStreams = "-WaitForStdStreams";

			// The parameter was not found in the list of global parameters, continue looking...
			if (CurrentParam.StartsWith(Option_ScriptsForProject + "=", StringComparison.InvariantCultureIgnoreCase))
			{
				if (AutomationToolCommandLine.IsSetUnchecked(Option_ScriptsForProject))
				{
					throw new Exception("The -ScriptsForProject argument may only be specified once");
				}

				string ProjectFileName = CurrentParam.Substring(CurrentParam.IndexOf('=') + 1).Replace("\"", "");
				FileReference ProjectReference = NativeProjectsBase.FindProjectFile(ProjectFileName, Logger);

				if (ProjectReference != null)
				{
					AutomationToolCommandLine.SetUnchecked(Option_ScriptsForProject, ProjectReference.FullName);
					Logger.LogDebug("Found project file: {0}", ProjectReference.FullName);
				}
				else if (Path.IsPathFullyQualified(ProjectFileName))
				{
					throw new Exception($"Project '{ProjectFileName}' does not exist");
				}
				else
				{
					throw new Exception($"Project '{ProjectFileName}' does not exist relative to any entries in *.uprojectdirs");
				}
			}
			else if (CurrentParam.StartsWith(Option_ScriptDir + "=", StringComparison.InvariantCultureIgnoreCase))
			{
				string ScriptDir = CurrentParam.Substring(CurrentParam.IndexOf('=') + 1);
				if (Directory.Exists(ScriptDir))
				{
					List<string> OutAdditionalScriptDirectories = (List<string>)AutomationToolCommandLine.GetValueUnchecked(Option_ScriptDir) ?? new List<string>();
					OutAdditionalScriptDirectories.Add(Path.GetFullPath(ScriptDir));
					AutomationToolCommandLine.SetUnchecked(Option_ScriptDir, OutAdditionalScriptDirectories);
					Logger.LogDebug("Found additional script dir: {0}", Path.GetFullPath(ScriptDir));
				}
				else
				{
					DirectoryReference ScriptDirReference = NativeProjectsBase.FindRelativeDirectoryReference(ScriptDir, Logger);

					if (ScriptDirReference != null)
					{
						List<string> OutAdditionalScriptDirectories = (List<string>)AutomationToolCommandLine.GetValueUnchecked(Option_ScriptDir) ?? new List<string>();
						OutAdditionalScriptDirectories.Add(ScriptDirReference.FullName);
						AutomationToolCommandLine.SetUnchecked(Option_ScriptDir, OutAdditionalScriptDirectories);
						Logger.LogDebug("Found additional script dir: {0}", ScriptDirReference.FullName);
					}
					else if (Path.IsPathFullyQualified(ScriptDir))
					{
						throw new Exception($"Specified ScriptDir doesn't exist: {ScriptDir}");
					}
					else
					{
						throw new Exception($"Specified ScriptDir doesn't exist relative to any entries in *.uprojectdirs: {ScriptDir}");
					}
				}
			}
			else if (CurrentParam.StartsWith(Option_AutomationProjects + "=", StringComparison.InvariantCultureIgnoreCase))
            {
				string AutomationProjectString = CurrentParam.Substring(CurrentParam.IndexOf('=') + 1);
				List<string> AutomationProjects = AutomationProjectString.Split("+".ToCharArray()).ToList();
				if (AutomationProjects.Count > 0)
				{
					// AutomationUtils is a necessity, see MainProc()
					if (!AutomationProjects.Contains("AutomationUtils"))
					{
						AutomationProjects.Add("AutomationUtils");
					}
					AutomationToolCommandLine.SetUnchecked(Option_AutomationProjects, AutomationProjects);
				}
			}
			else if (CurrentParam.StartsWith(Option_Telemetry + "=", StringComparison.InvariantCultureIgnoreCase))
            {
				string TelemetryPath = CurrentParam.Substring(CurrentParam.IndexOf('=') + 1);
				AutomationToolCommandLine.SetUnchecked(Option_Telemetry, TelemetryPath);
			}
			else if (CurrentParam.StartsWith(Option_WaitForStdStreams + "=", StringComparison.InvariantCultureIgnoreCase))
			{
				string WaitTime = CurrentParam.Substring(CurrentParam.IndexOf('=') + 1);
				AutomationToolCommandLine.SetUnchecked(Option_WaitForStdStreams, WaitTime);
			}
			else if (CurrentParam.StartsWith("-"))
			{
				if (CurrentCommand != null)
				{
					CurrentCommand.Arguments.Add(CurrentParam.Substring(1));
				}
				else if (!bGlobalParam)
				{
					throw new Exception($"Unknown parameter {CurrentParam} in the command line that does not belong to any command.");
				}
			}
			else if (CurrentParam.Contains("="))
			{
				// Environment variable
				int ValueStartIndex = CurrentParam.IndexOf('=') + 1;
				string EnvVarName = CurrentParam.Substring(0, ValueStartIndex - 1);
				if (String.IsNullOrEmpty(EnvVarName))
				{
					throw new Exception($"Unable to parse environment variable that has no name. Error when parsing command line param {CurrentParam}");
				}
				string EnvVarValue = CurrentParam.Substring(ValueStartIndex);

				Logger.LogDebug($"SetEnvVar {EnvVarName}={EnvVarValue}");
				Environment.SetEnvironmentVariable(EnvVarName, EnvVarValue);
			}
		}

		private static string ParseString(string Key, string Value)
		{
			if (!String.IsNullOrEmpty(Key))
			{
				if (Value == "true" || Value == "false")
				{
					return "-" + Key;
				}
				else
				{
					string param = "-" + Key + "=";
					if (Value.Contains(" "))
					{
						param += "\"" + Value + "\"";
					}
					else
					{
						param += Value;
					}
					return param;
				}
			}
			else
			{
				return Value;
			}
		}

		private static string ParseList(string Key, List<object> Value)
		{
			string param = "-" + Key + "=";
			bool bStart = true;
			foreach (var Val in Value)
			{
				if (!bStart)
				{
					param += "+";
				}
				param += Val as string;
				bStart = false;
			}
			return param;
		}

		private static void ParseDictionary(Dictionary<string, object> Value, List<string> Arguments)
		{
			foreach (KeyValuePair<string, object> Pair in Value)
			{
				if ((Pair.Value as string) != null && !string.IsNullOrEmpty(Pair.Value as string))
				{
					Arguments.Add(ParseString(Pair.Key, Pair.Value as string));
				}
				else if (Pair.Value.GetType() == typeof(bool))
				{
					if ((bool)Pair.Value)
					{
						Arguments.Add("-" + Pair.Key);
					}
				}
				else if ((Pair.Value as List<object>) != null)
				{
					Arguments.Add(ParseList(Pair.Key, Pair.Value as List<object>));
				}
				else if ((Pair.Value as Dictionary<string, object>) != null)
				{
					string param = "-" + Pair.Key + "=\"";
					List<string> Args = new List<string>();
					ParseDictionary(Pair.Value as Dictionary<string, object>, Args);
					bool bStart = true;
					foreach (var Arg in Args)
					{
						if (!bStart)
						{
							param += " ";
						}
						param += Arg.Replace("\"", "\'");
						bStart = false;
					}
					param += "\"";
					Arguments.Add(param);
				}
			}
		}

		private static void ParseProfile(ref string[] CommandLine)
		{
			// find if there is a profile file to read
			string Profile = "";
			List<string> Arguments = new List<string>();
			for (int Index = 0; Index < CommandLine.Length; ++Index)
			{
				if (CommandLine[Index].StartsWith("-profile="))
				{
					Profile = CommandLine[Index].Substring(CommandLine[Index].IndexOf('=') + 1);
				}
				else
				{
					Arguments.Add(CommandLine[Index]);
				}
			}

			if (!string.IsNullOrEmpty(Profile))
			{
				if (File.Exists(Profile))
				{
					// find if the command has been specified
					var text = File.ReadAllText(Profile);
					Dictionary<string, object> RawObject = fastJSON.JSON.Instance.Parse(text) as Dictionary<string, object>;
					List<object> Params = RawObject["scripts"] as List<object>;
					foreach (var Script in Params)
					{
						string ScriptName = (Script as Dictionary<string, object>)["script"] as string;
						if (!string.IsNullOrEmpty(ScriptName) && !Arguments.Contains(ScriptName))
						{
							Arguments.Add(ScriptName);
						}
						(Script as Dictionary<string, object>).Remove("script");
						ParseDictionary((Script as Dictionary<string, object>), Arguments);
					}
				}
			}

			CommandLine = Arguments.ToArray();
		}

		/// <summary>
		/// Parse the command line and create a list of commands to execute.
		/// </summary>
		/// <param name="Arguments">Command line</param>
		/// <param name="Logger">Logger for output</param>
		public static void ParseCommandLine(string[] Arguments, ILogger Logger)
		{
			AutomationToolCommandLine = new ParsedCommandLine(
				new Dictionary<string, string>(StringComparer.InvariantCultureIgnoreCase)
				{
					{"-Verbose", "Enables verbose logging"},
					{"-VeryVerbose", "Enables very verbose logging"},
					{"-TimeStamps", ""},
					{"-Submit", "Allows UAT command to submit changes"},
					{"-NoSubmit", "Prevents any submit attempts"},
					{"-NoP4", "Disables Perforce functionality {default if not run on a build machine}"},
					{"-P4", "Enables Perforce functionality {default if run on a build machine}"},
					{"-IgnoreDependencies", ""},
					{"-Help", "Displays help"},
					{"-List", "Lists all available commands"},
					{"-NoKill", "Does not kill any spawned processes on exit"},
					{"-UTF8Output", ""},
					{"-AllowStdOutLogVerbosity", ""},
					{"-NoAutoSDK", ""},
					{"-CompileOnly", "Run without a command to compile all out-of-date script modules."},
					{"-Compile", "Force all script modules to be compiled"},
					{"-NoCompile", "Do not attempt to compile any script modules - attempts to run with whatever is up to date" },
					{"-IgnoreBuildRecords", "Ignore build records (Intermediate/ScriptModule/ProjectName.json) files when determining if script modules are up to date" },
					{"-UseLocalBuildStorage", @"Allows you to use local storage for your root build storage dir {default of P:\Builds {on PC} is changed to Engine\Saved\LocalBuilds}. Used for local testing."},
					{"-WaitForDebugger", "Waits for a debugger to be attached, and breaks once debugger successfully attached."},
					{"-BuildMachine", "" },
					{"-WaitForUATMutex", "" },
					{"-WaitForStdStreams", "Time in milliseconds to wait for std streams to close in child processes." }
				},
				new HashSet<string>(StringComparer.InvariantCultureIgnoreCase) { "-msbuild-verbose", "-NoCompileUAT" }
			);

			ParseProfile(ref Arguments);

			Logger.LogInformation("Parsing command line: {CommandLine}", CommandLine.FormatCommandLine(Arguments));

			CommandInfo CurrentCommand = null;
			for (int Index = 0; Index < Arguments.Length; ++Index)
			{
				// Guard against empty arguments passed as "" on the command line
				string Param = Arguments[Index];
				if(Param.Length > 0) 
				{
					if (Param.StartsWith("-") || Param.Contains("="))
					{
						ParseParam(Arguments[Index], CurrentCommand, Logger);
					}
					else
					{
						CurrentCommand = new CommandInfo(Arguments[Index]);
						AutomationToolCommandLine.CommandsToExecute.Add(CurrentCommand);
					}
				}
			}

			// Validate
			var Result =
				AutomationToolCommandLine.CommandsToExecute.Count > 0 ||
				AutomationToolCommandLine.IsSetGlobal("-Help") ||
				AutomationToolCommandLine.IsSetGlobal("-List") ||
				AutomationToolCommandLine.IsSetGlobal("-CompileOnly");
			if (AutomationToolCommandLine.CommandsToExecute.Count > 0)
			{
				Logger.LogDebug("Found {NumScripts} scripts to execute:", AutomationToolCommandLine.CommandsToExecute.Count);
				foreach (CommandInfo Command in AutomationToolCommandLine.CommandsToExecute)
				{
					Logger.LogDebug("  {Command}", Command.ToString());
				}
			}
			else if (!Result)
			{
				throw new Exception("Failed to find scripts to execute in the command line params.");
			}
			if (AutomationToolCommandLine.IsSetGlobal("-NoP4") && AutomationToolCommandLine.IsSetGlobal("-P4"))
			{
				throw new Exception("'-NoP4' and '-P4' can't be set simultaneously.");
			}
			if (AutomationToolCommandLine.IsSetGlobal("-NoSubmit") && AutomationToolCommandLine.IsSetGlobal("-Submit"))
			{
				throw new Exception("'-NoSubmit' and '-Submit' can't be set simultaneously.");
			}
		}

		static ParsedCommandLine AutomationToolCommandLine;

		/// <summary>
		/// Main entry point
		/// </summary>
		/// <remarks>Do not add [STAThread] here. It will cause deadlocks in platform automation code.</remarks>
		public static async Task<int> Main(string[] Arguments)
		{
			ILogger Logger = Log.Logger;

			// Initialize the log system, buffering the output until we can create the log file
			Log.AddStartupTraceListener();
			Logger.LogInformation("Starting AutomationTool...");

			// Populate AutomationToolCommandLine and CommandsToExecute
			try
			{
				ParseCommandLine(Arguments, Logger);
			}
			catch (Exception Ex)
			{
				Logger.LogError(Ex, "ERROR: " + Ex.Message);
				return (int)ExitCode.Error_Arguments;
			}

			// Wait for a debugger to be attached
			if (AutomationToolCommandLine.IsSetGlobal("-WaitForDebugger"))
			{
				Console.WriteLine("Waiting for debugger to be attached...");
				while (Debugger.IsAttached == false)
				{
					Thread.Sleep(100);
				}
				Debugger.Break();
			}

			Stopwatch Timer = Stopwatch.StartNew();

			// Ensure UTF8Output flag is respected, since we are initializing logging early in the program.
			if (AutomationToolCommandLine.IsSetGlobal("-UTF8Output"))
			{
				Console.OutputEncoding = new System.Text.UTF8Encoding(false, false);
			}

			// Parse the log level argument
			if (AutomationToolCommandLine.IsSetGlobal("-Verbose"))
			{
				Log.OutputLevel = LogEventType.Verbose;
			}
			if (AutomationToolCommandLine.IsSetGlobal("-VeryVerbose"))
			{
				Log.OutputLevel = LogEventType.VeryVerbose;
			}

			// Configure log timestamps
			Log.IncludeTimestamps = AutomationToolCommandLine.IsSetGlobal("-Timestamps");

			Log.EventParser.AddMatchers(LogMatcherCollections.UBTMatchers);

			// when running frmo RunUAT.sh (Mac/Linux) we need to install a Ctrl-C handler, or hitting Ctrl-C from a terminal
			// can leave dotnet process in a zombie state (some order of process destruction is failing)
			// by putting this in, the Ctrl-C may not be handled immediately, but it shouldn't leave a zombie process
			if (OperatingSystem.IsMacOS() || OperatingSystem.IsLinux())
			{
				Console.CancelKeyPress += delegate
				{
					Console.WriteLine("AutomationTool: Ctrl-C pressed. Exiting...");
				};
			}

			// Enter the main program section
			ExitCode ReturnCode = ExitCode.Error_Unknown;
			try
			{
				// Set the working directory to the Unreal root directory
				DirectoryReference.SetCurrentDirectory(Unreal.RootDirectory);

				// Ensure we can resolve any external assemblies as necessary.
				DirectoryReference AutomationToolLocation = new DirectoryReference(Path.GetDirectoryName(Assembly.GetEntryAssembly().GetOriginalLocation()));
				AssemblyUtils.InstallRecursiveAssemblyResolver(AutomationToolLocation.FullName);

				// Log the operating environment. Since we usually compile to AnyCPU, we may be executed using different system paths under WOW64.
				Logger.LogDebug("Running on {Platform} as a {Bitness}-bit process.", RuntimePlatform.Current.ToString(), Environment.Is64BitProcess ? 64 : 32);

				// Log if we're running from the launcher
				string ExecutingAssemblyLocation = Assembly.GetExecutingAssembly().Location;
				if (string.Compare(ExecutingAssemblyLocation, Assembly.GetEntryAssembly().GetOriginalLocation(), StringComparison.OrdinalIgnoreCase) != 0)
				{
					Logger.LogDebug("Executed from AutomationToolLauncher ({Location})", ExecutingAssemblyLocation);
				}
				Logger.LogDebug("CWD={Cwd}", Environment.CurrentDirectory);

				// Log the application version
				FileVersionInfo Version = AssemblyUtils.ExecutableVersion;
				Logger.LogDebug("{ProductName} ver. {ProductVersion}", Version.ProductName, Version.ProductVersion);

				bool bWaitForUATMutex = AutomationToolCommandLine.IsSetGlobal("-WaitForUATMutex");

				// Don't allow simultaneous execution of AT (in the same branch)
				ReturnCode = await ProcessSingleton.RunSingleInstanceAsync(MainProc, bWaitForUATMutex, Log.Logger);
			}
			catch (Exception Ex)
			{
				Logger.LogError(Ex, "Unhandled exception: {Message}", ExceptionUtils.FormatException(Ex));
			}
			finally
			{
				// If there was an error before a log writer was attached, do so now
				if (!Log.HasFileWriter() && ReturnCode != ExitCode.Success)
				{
					FileReference errorLogFile = FileReference.Combine(Unreal.WritableEngineDirectory, "Programs", "AutomationTool", "Saved", "Logs", "ErrorLog.txt");
					Log.AddFileWriterWithoutBackup("UnhandledFileLogListener", errorLogFile);
					Logger.LogInformation("AutomationTool encountered an error before a log file was set, see '{File}' for more details", errorLogFile.FullName);
				}

				// Write the exit code
				Logger.LogInformation("AutomationTool executed for {Time}", Timer.Elapsed.ToString("h'h 'm'm 's's'"));
				Logger.LogInformation("AutomationTool exiting with ExitCode={ExitCode} ({ExitReason})", (int)ReturnCode, ReturnCode);

				// Can't use NoThrow here because the code logs exceptions. We're shutting down logging!
				Trace.Close();
			}
			return (int)ReturnCode;
		}

		static async Task<ExitCode> MainProc()
		{
			ILogger Logger = Log.Logger;
			Logger.LogInformation("Initializing script modules...");
			Stopwatch Timer = Stopwatch.StartNew();
			string ScriptsForProject = (string)AutomationToolCommandLine.GetValueUnchecked("-ScriptsForProject");
			List<string> AdditionalScriptDirs = (List<string>) AutomationToolCommandLine.GetValueUnchecked("-ScriptDir");
			List<string> AutomationProjects = (List<string>) AutomationToolCommandLine.GetValueUnchecked("-AutomationProject");
			bool bForceCompile = AutomationToolCommandLine.IsSetGlobal("-Compile");
			bool bNoCompile = AutomationToolCommandLine.IsSetGlobal("-NoCompile");
			bool bUseBuildRecords = !AutomationToolCommandLine.IsSetGlobal("-IgnoreBuildRecords");
			List<CommandInfo> Commands = AutomationToolCommandLine.IsSetGlobal("-List")
				? null
				: AutomationToolCommandLine.CommandsToExecute;
			bool bBuildSuccess;
			HashSet<FileReference> ScriptModuleAssemblyPaths = CompileScriptModule.InitializeScriptModules(
					Rules.RulesFileType.AutomationModule,
					ScriptsForProject,
					AdditionalScriptDirs,
					AutomationProjects,
					bForceCompile,
					bNoCompile,
					bUseBuildRecords, 
					out bBuildSuccess,
					Logger);

			if (!bBuildSuccess)
			{
				return ExitCode.Error_Unknown;
			}

			// when the engine is installed, or UAT is invoked with -NoCompile, we expect to find at least one script module (AutomationUtils is a necessity)
			if (ScriptModuleAssemblyPaths.Count == 0)
			{
				throw new Exception("Found no script module records.");
			}

			// Load AutomationUtils.Automation.dll
			FileReference AssemblyPath = ScriptModuleAssemblyPaths.FirstOrDefault(x => x.GetFileNameWithoutExtension().Contains("AutomationUtils.Automation"));
			AssemblyUtils.InstallRecursiveAssemblyResolver(AssemblyPath.Directory.FullName);
			Assembly AutomationUtilsAssembly = AssemblyPath != null ? Assembly.LoadFrom(AssemblyPath.FullName) : null;

			if (AutomationUtilsAssembly == null)
			{
				throw new Exception("Did not find an AutomationUtils.Automation.dll");
			}

			// Call into AutomationTool.Automation.ProcessAsync()
			Type AutomationTools_Automation = AutomationUtilsAssembly.GetType("AutomationTool.Automation");
			MethodInfo Automation_Process = AutomationTools_Automation.GetMethod("ProcessAsync");
			Logger.LogInformation("Total script module initialization time: {InitTime:0.00} s.", Timer.Elapsed.TotalSeconds);
			return await (Task<ExitCode>) Automation_Process.Invoke(null, [AutomationToolCommandLine, ScriptModuleAssemblyPaths]);
		}
	}
}
