// Copyright Epic Games, Inc. All Rights Reserved.

using System.Reflection;
using System.Runtime.CompilerServices;
using System.Runtime.InteropServices;

// General Information about an assembly is controlled through the following 
// set of attributes. Change these attribute values to modify the information
// associated with an assembly.
[assembly: AssemblyTitle("AutomationTool")]
[assembly: AssemblyDescription("")]
[assembly: AssemblyConfiguration("")]

// The following GUID is for the ID of the typelib if this project is exposed to COM
[assembly: Guid("013C6878-CB31-4C3D-8CF9-EBF26ABD69B0")]
