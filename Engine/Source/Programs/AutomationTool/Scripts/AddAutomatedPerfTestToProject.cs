// Copyright Epic Games, Inc. All Rights Reserved.
using System;
using System.Collections.Generic;
using System.Text;
using System.IO;
using AutomationTool;
using UnrealBuildTool;
using UnrealBuildBase;
using EpicGames.Core;
using System.Linq;
using System.Text.Json;
using Microsoft.Extensions.Logging;
using Microsoft.VisualStudio.OLE.Interop;
using System.Threading.Tasks;
using System.Security.AccessControl;
using Microsoft.Build.Evaluation.Context;

[Help("UAT command to enable PerfTesting hook ups for a project.")]
[Help("-Project", "Path to the project to be instrumented.")]
[Help("-Overwrite", "If set, will overwrite any template files that already exist.")]
class AddAutomatedPerfTestToProject : BuildCommand
{
	/// <summary>
	/// The plugin name itself
	/// </summary>
	const string AptPluginName = "AutomatedPerfTesting";

	/// <summary>
	/// Path to the AutomatePerfTesting plugin
	/// </summary>
	const string AptPathFromRoot = "Engine/Plugins/Performance/AutomatedPerfTesting";

	/// <summary>
	/// Path to the AutomatePerfTesting templates
	/// </summary>
	const string AptTemplatesFromPluginRoot = "Resources/TemplatesForHookup";

	/// <summary>
	/// Name of the Gauntlet settings that needs to be copied verbatim
	/// </summary>
	const string AptGauntletIncludeFile = "GauntletSettings.xml";

	/// <summary>
	/// Name of the template file with project setings that will be modified
	/// </summary>
	const string AptProjectBuildGraphTemplateFile = "AutoPerfTests.xml";

	/// <summary>
	/// Name of the template file for running tests locally on Windows
	/// </summary>
	const string AptLocalTestBatchFile = "RunLocalTests.bat";
	
	/// <summary>
	/// Name of the template file for running tests locally on Mac/Linux
	/// </summary>
	const string AptLocalTestShellFile = "RunLocalTests.sh";
	
	/// <summary>
	/// Path to the engine's build/batchfiles directory relative to the engine root
	/// </summary>
	const string EngineBatchFilesFromRoot = "Engine/Build/BatchFiles";
	
	/// <summary>
	/// Name of the config section for APT
	/// </summary>
	const string AptConfigSectionName = "/Script/AutomatedPerfTesting.AutomatedSequencePerfTestProjectSettings";

	protected bool EnablePlugin(string ProjectFilePath)
	{
		Logger.LogInformation("Enabling the plugin for the project.");

		JsonObject ProjectJson = JsonObject.Read(new FileReference(ProjectFilePath));

		JsonObject AptReference = new JsonObject();
		AptReference.AddOrSetFieldValue("Name", AptPluginName);
		AptReference.AddOrSetFieldValue("Enabled", true);
		List<JsonObject> NewPluginArray = new List<JsonObject>();
		if(ProjectJson.TryGetObjectArrayField("Plugins", out JsonObject[] Plugins))
		{
			Logger.LogInformation("Project has some plugins configured for it, checking if APT is among them.");

			foreach(JsonObject PluginDesc in Plugins) 
			{
				string PluginName;
				if (PluginDesc.TryGetStringField("Name",out PluginName) && PluginName == AptPluginName) 
				{
					Logger.LogInformation("Project already has AutomatedPerfTest plugin enabled.");
					return true;
				}
				NewPluginArray.Add(PluginDesc);
			}
		}
		else
		{
			Logger.LogInformation("Project had no plugins configured for it, adding a block.");
		}

		Logger.LogInformation("Adding APT to the list of project plugins.");
		NewPluginArray.Add(AptReference);
		ProjectJson.AddOrSetFieldValue("Plugins", NewPluginArray.ToArray());

		string TempFile = ProjectFilePath + ".temp";	// unlikely to have more than one upgraders running at the same time
		File.WriteAllText(TempFile,ProjectJson.ToJsonString());

		ProjectJson = null;
		File.Delete(ProjectFilePath);
		File.Move(TempFile, ProjectFilePath);
		return true;
	}

	protected bool ModifyProjectConfigFiles(string ProjectFilePath)
	{
		Logger.LogInformation("Modifying project settings.");

		string ProjectDir = Path.GetDirectoryName(ProjectFilePath);

		// load ini file
		string DefaultEngineIniFile = Path.Combine(ProjectDir,"Config","DefaultEngine.ini");
		if(!File.Exists(DefaultEngineIniFile)) 
		{
			Logger.LogInformation("Project does not seem to have DefaultEngine.ini.");
			return true;
		}

		ConfigFile DefaultEngineIni = new ConfigFile(new FileReference(DefaultEngineIniFile));

		ConfigFileSection Section = null;
		if (DefaultEngineIni.TryGetSection(AptConfigSectionName, out Section))
		{
			Logger.LogInformation("Project already has APT settings in its DefaultEngine.ini.");
			return true;
		}

		Section = DefaultEngineIni.FindOrAddSection(AptConfigSectionName);
		Section.Lines.Add(new ConfigLine(ConfigLineAction.Add, "MapsAndSequencesToTest", "ComboName = \"PerfSeqeunce\", Map = \"/...\", Sequence = \"...\""));
		Section.Lines.Add(new ConfigLine(ConfigLineAction.Set,"SequenceStartDelay", "5.0000"));

		DefaultEngineIni.Write(new FileReference(DefaultEngineIniFile));

		;// +MapsAndSequencesToTest = (ComboName = "PerfSequence", Map = "/Game/Maps/LV_Main.LV_Main", Sequence = "/Game/Cinematics/LS_ReplaySequence.LS_ReplaySequence")
		;// SequenceStartDelay = 5.000000

		return true;
	}

	/// <summary>
	/// Return a project name from the path to .uproject
	/// </summary>
	protected string GetProjectName(string ProjectFilePath)
	{
		return Path.GetFileNameWithoutExtension(ProjectFilePath);
	}

	/// <summary>
	/// Return a project path relative to the current engine root, with forward slashes.
	/// </summary>
	protected string GetProjectRelativePath(string ProjectFilePath) 
	{
		string ProjectPath = Path.GetDirectoryName(ProjectFilePath);
		ProjectPath = Path.GetRelativePath(Unreal.RootDirectory.ToString(), ProjectPath);
		ProjectPath = CommandUtils.ConvertSeparators(PathSeparator.Slash, ProjectPath);
		return ProjectPath;
	}
	
	/// <summary>
	/// Return the path to RunUAT relative to the input file
	/// </summary>
	protected string GetRunUATRelativePath(string InFilePath) 
	{
		string InDirectoryPath = Path.GetDirectoryName(InFilePath);
		string RunUATRelativePath = Path.GetRelativePath(InDirectoryPath, Path.Combine(Unreal.RootDirectory.ToString(), EngineBatchFilesFromRoot));
		RunUATRelativePath = CommandUtils.ConvertSeparators(PathSeparator.Slash, RunUATRelativePath);
		return RunUATRelativePath;
	}

	protected bool ModifyBuildGraphFiles(string ProjectFilePath, bool Overwrite=false)
	{
		// work out the target directory
		string ProjectDir = Path.GetDirectoryName(ProjectFilePath);

		string BuildGraphTemplates = Path.Combine(AptPathFromRoot, AptTemplatesFromPluginRoot);

		string BuildGraphDest = Path.Combine(ProjectDir, "Build");
		string BuildGraphGauntletIncDest = Path.Combine(BuildGraphDest, "Inc");

		Directory.CreateDirectory(BuildGraphDest);
		// copy Gauntlet settings file
		{
			Directory.CreateDirectory(BuildGraphGauntletIncDest);

			string DestFile = Path.Combine(BuildGraphGauntletIncDest,AptGauntletIncludeFile);
			if(File.Exists(DestFile))
			{
				Logger.LogInformation("Project already has file {0}.",DestFile);
			}
			else
			{
				string SrcFile = Path.Combine(BuildGraphTemplates,AptGauntletIncludeFile);
				if (File.Exists(DestFile) && Overwrite)
				{
					File.Delete(DestFile);
				}
				File.Copy(SrcFile,DestFile);
			}
		}

		// copy example per-project BuildGraph file
		{
			string DestFile = Path.Combine(BuildGraphDest,AptProjectBuildGraphTemplateFile);
			if(File.Exists(DestFile) && !Overwrite)
			{
				Logger.LogInformation("Project already has file {0}.",DestFile);
			}
			else
			{
				string SrcFile = Path.Combine(BuildGraphTemplates,AptProjectBuildGraphTemplateFile);
				string Template = File.ReadAllText(SrcFile);
				Template = Template.Replace("**REPLACE_PROJECTNAME**", GetProjectName(ProjectFilePath));
				Template = Template.Replace("**REPLACE_PROJECTPATH**",GetProjectRelativePath(ProjectFilePath));
				File.WriteAllText(DestFile, Template);
			}
		}

		return true;
	}

	protected bool ModifyLocalTestsBatchFile(string ProjectFilePath, bool Overwrite=false)
	{
		// work out the target directory
		string ProjectDir = Path.GetDirectoryName(ProjectFilePath);

		string TemplatesFolder = Path.Combine(AptPathFromRoot, AptTemplatesFromPluginRoot);

		string BatchFileDest = Path.Combine(ProjectDir, "Build", "BatchFiles");
		
		// copy example batch files
		Directory.CreateDirectory(BatchFileDest);
		{
			List<string> BatchFiles = [AptLocalTestBatchFile, AptLocalTestShellFile];
			
			foreach (string DestFileName in BatchFiles)
			{
				string DestFile = Path.Combine(BatchFileDest,DestFileName);
				if(File.Exists(DestFile) && !Overwrite)
				{
					Logger.LogInformation("Project already has file {0}.",DestFile);
				}
				else
				{
					string SrcFile = Path.Combine(TemplatesFolder,DestFileName);
					string Template = File.ReadAllText(SrcFile);
					Template = Template.Replace("**REPLACE_PROJECTNAME**", GetProjectName(ProjectFilePath));
					Template = Template.Replace("**REPLACE_PROJECTPATH**",GetProjectRelativePath(ProjectFilePath));
					Template = Template.Replace("**RUN_UAT_RELATIVEPATH**",GetRunUATRelativePath(DestFile));
					File.WriteAllText(DestFile, Template);
				}	
			}
		}
		return true;
	}

	public override void ExecuteBuild()
	{
		Logger.LogInformation("Adding AutomatedPerfTest hookups for a project.");

		// Parse the project
		string Project = ParseParamValue("Project");
		if (Project == null)
		{
			throw new AutomationException("Project to instrument was not specified, use -Project=<path_to_uproject>. Lookup via Default.uprojectdirs is not supported");
		}

		bool Overwrite = ParseParam("Overwrite");

		Logger.LogInformation("Project {0} will be configured for AutomatedPerfTest plugin.", Project);
		
		EnablePlugin(Project);
		ModifyProjectConfigFiles(Project);
		ModifyBuildGraphFiles(Project, Overwrite);
		ModifyLocalTestsBatchFile(Project, Overwrite);

		Logger.LogInformation("Successfully added.");
	}
}
