// Copyright Epic Games, Inc. All Rights Reserved.
using System;
using System.Collections.Generic;
using System.Text;
using System.IO;
using AutomationTool;
using UnrealBuildTool;
using System.Linq;
using Microsoft.Extensions.Logging;

using static AutomationTool.CommandUtils;

/*
 - You can also use the full program to test compiling all or a subset of libs:
 - From Engine/Build/BatchFiles, do:
 -   RunUAT AnalyzeThirdPartyLibs [-libs=lib1+lib2+lib3] [-changelist=NNNN]
 */

class PlatformLibraryInfo
{
	public List<string> PathParticles = new List<string>();
	public List<string> Manifest = new List<string>();
	public long TotalSize = 0;
	public string PlatformName;

	public bool PartOfPlatform(string Filename)
	{
		foreach (string Particle in PathParticles)
		{
			if (Filename.IndexOf(Particle, StringComparison.InvariantCultureIgnoreCase) != -1)
			{
				return true;
			}
		}

		return (PathParticles.Count == 0) ? true : false;
	}

	public PlatformLibraryInfo(string PlatformName, params string[] Values)
	{
		this.PlatformName = PlatformName;
		PathParticles.AddRange(Values);
	}

	public void AddFile(string Filename, long Size)
	{
		Manifest.Add(Filename);
		TotalSize += Size;
	}
};


class ThirdPartyLibraryInfo
{
	public List<string> Manifest = new List<string>();

	public long GetSize(List<PlatformLibraryInfo> Platforms)
	{
		long TotalSize = 0;

		foreach (string Filename in Manifest)
		{
			FileInfo FI = new FileInfo(Filename);
			long Size = FI.Length;

			foreach (PlatformLibraryInfo Platform in Platforms)
			{
				if (Platform.PartOfPlatform(Filename))
				{
					Platform.AddFile(Filename, Size);
				}
			}

			TotalSize += Size;
		}

		return TotalSize;
	}

	public void FindLargeFiles(List<string> AllowedExtensions, long MinSize)
	{
		foreach (string Filename in Manifest)
		{
			FileInfo FI = new FileInfo(Filename);
			long Size = FI.Length;

			if (Size > MinSize)
			{
				bool bAllowed = false;
				foreach (string Extension in AllowedExtensions)
				{
					if (Filename.EndsWith(Extension, StringComparison.InvariantCultureIgnoreCase))
					{
						bAllowed = true;
						break;
					}
				}

				if (!bAllowed)
				{
					Logger.LogWarning("{Filename} is {Arg1} with an unexpected extension", Filename, AnalyzeThirdPartyLibs.ToMegabytes(Size));
				}
			}
		}
	}

	public ThirdPartyLibraryInfo(string Root)
	{
		string[] Files = Directory.GetFiles(Root, "*.*", SearchOption.AllDirectories);
		Manifest.AddRange(Files);
	}
}

[Help("Analyzes third party libraries")]
[Help("Libs", "[Optional] + separated list of libraries to compile; if not specified this job will build all libraries it can find builder scripts for")]
[Help("Changelist", "[Optional] a changelist to check out into; if not specified, a changelist will be created")]
class AnalyzeThirdPartyLibs : BuildCommand
{
	// path to the third party directory
	static private string LibDir = "Engine/Source/ThirdParty";

	// batch/script file to look for when compiling
	public static string ToMegabytes(long Size)
	{
		double SizeKB = Size / 1024.0;
		double SizeMB = SizeKB / 1024.0;

		return String.Format("{0:N2} MB", SizeMB);
	}

	public override void ExecuteBuild()
	{
		Logger.LogInformation("************************* Analyze Third Party Libs");

		// figure out what batch/script to run
		if (UnrealBuildTool.BuildHostPlatform.Current.Platform != UnrealTargetPlatform.Win64 &&
			UnrealBuildTool.BuildHostPlatform.Current.Platform != UnrealTargetPlatform.Mac &&
			UnrealBuildTool.BuildHostPlatform.Current.Platform != UnrealTargetPlatform.Linux)
		{
			throw new AutomationException("Unknown runtime platform!");
		}

		// go to the third party lib dir
		CommandUtils.PushDir(LibDir);

		// figure out what libraries to evaluate
		string LibsToEvaluateString = ParseParamValue("Libs");

		// Determine which libraries to evaluate
		List<string> LibsToEvaluate = new List<string>();
		if (string.IsNullOrEmpty(LibsToEvaluateString))
		{
			// loop over third party directories looking for the right batch files
			foreach (string Dir in Directory.EnumerateDirectories("."))
			{
				LibsToEvaluate.Add(Path.GetFileName(Dir));
			}
		}
		else
		{
			// just split up the param and make sure the batch file exists
			string[] Libs = LibsToEvaluateString.Split('+');
			foreach (string Dir in Libs)
			{
				LibsToEvaluate.Add(Path.GetFileName(Dir));
			}
		}

		// Make a list of platforms
		List<PlatformLibraryInfo> Platforms = new List<PlatformLibraryInfo>();
		Platforms.Add(new PlatformLibraryInfo("Windows", "Windows", "Win64", "VS20"));
		Platforms.Add(new PlatformLibraryInfo("Mac", "Osx", "Mac"));
		Platforms.Add(new PlatformLibraryInfo("iOS", "IOS"));
		Platforms.Add(new PlatformLibraryInfo("Android", "Android"));
		Platforms.Add(new PlatformLibraryInfo("Linux", "Linux"));

		Platforms.Add(new PlatformLibraryInfo("VS2013", "VS2013", "vs12"));
		Platforms.Add(new PlatformLibraryInfo("VS2015", "VS2015", "vs14"));

		List<long> LastSizes = new List<long>();
		foreach (var Platform in Platforms)
		{
			LastSizes.Add(0);
		}

		// now go through and evaluate each package
		long TotalSize = 0;
		foreach (string Lib in LibsToEvaluate)
		{
			ThirdPartyLibraryInfo Info = new ThirdPartyLibraryInfo(Lib);

			long Size = Info.GetSize(Platforms);

			Logger.LogInformation("Library {Lib} is {Arg1}", Lib, ToMegabytes(Size));

			long Total = 0;
			for (int Index = 0; Index < Platforms.Count; ++Index)
			{
				PlatformLibraryInfo Platform = Platforms[Index];
				long Growth = Platform.TotalSize - LastSizes[Index];
				Logger.LogInformation("  {Arg0} is {Arg1}", Platform.PlatformName, ToMegabytes(Growth));

				LastSizes[Index] = Platform.TotalSize;
				Total += Growth;
			}
			Logger.LogInformation("  Platform neutral is probably {Arg0} (specific sum {Arg1})", ToMegabytes(Size - Total), ToMegabytes(Total));

			TotalSize += Size;
		}


		// Make a list of known large file types
		List<string> LargeFileExtensions = new List<string>();
		LargeFileExtensions.AddRange(new string[] { ".pdb", ".a", ".lib", ".dll", ".dylib", ".bc", ".so" });

		// Hackery, look for big files (re-traverses everything)
		Logger.LogInformation("----");
		foreach (string Lib in LibsToEvaluate)
		{
			ThirdPartyLibraryInfo Info = new ThirdPartyLibraryInfo(Lib);
			Info.FindLargeFiles(LargeFileExtensions, 1024 * 1024);
		}

		Logger.LogInformation("----");
		foreach (var Platform in Platforms)
		{
			Logger.LogInformation("  {Arg0} is {Arg1} (estimate)", Platform.PlatformName, ToMegabytes(Platform.TotalSize));
		}
		Logger.LogInformation("  OVERALL is {Arg0} (accurate)", ToMegabytes(TotalSize));

		// undo the LibDir push
		CommandUtils.PopDir();

		PrintRunTime();
	}
}
