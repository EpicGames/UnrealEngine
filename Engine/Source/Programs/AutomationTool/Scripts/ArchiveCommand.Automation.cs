// Copyright Epic Games, Inc. All Rights Reserved.
using System;
using System.Collections.Generic;
using System.IO;
using System.Threading;
using System.Reflection;
using AutomationTool;
using UnrealBuildTool;
using EpicGames.Core;
using Microsoft.Extensions.Logging;

namespace AutomationScripts
{
	public partial class Project : CommandUtils
	{
		public static void CreateArchiveManifest(ProjectParams Params, DeploymentContext SC)
		{
			if (!Params.Archive)
			{
				return;
			}
			var ThisPlatform = SC.StageTargetPlatform;

			if (SC.CustomDeployment == null || !SC.CustomDeployment.GetFilesToArchive(Params, SC))
			{
				ThisPlatform.GetFilesToArchive(Params, SC);
			}
			SC.CustomStagingHandlers.ForEach(H => H.GetFilesToArchive(Params, SC));

			//@todo add any archive meta data files as needed

			if (Params.ArchiveMetaData)
			{
				// archive the build.version file for extra info for testing, etc
				string BuildVersionFile = CombinePaths(SC.LocalRoot.FullName, "Engine", "Build", "Build.version");
				SC.ArchiveFiles(Path.GetDirectoryName(BuildVersionFile), Path.GetFileName(BuildVersionFile));
			}
		}

		public static void ApplyArchiveManifest(ProjectParams Params, DeploymentContext SC)
		{
			if (SC.ArchivedFiles.Count > 0)
			{
				foreach (var Pair in SC.ArchivedFiles)
				{
					FileReference Src = new FileReference(Pair.Key);
					FileReference Dest = FileReference.Combine(SC.ArchiveDirectory, Pair.Value);
					CopyFileOrSymlink(Src, Dest);
				}
			}
		}

		public static void Archive(ProjectParams Params)
		{
			Params.ValidateAndLog();
			if (!Params.Archive)
			{
				return;
			}

			Logger.LogInformation("********** ARCHIVE COMMAND STARTED **********");
			var StartTime = DateTime.UtcNow;

			Logger.LogInformation("Archiving to {Arg0}", Params.ArchiveDirectoryParam);

			if (!Params.NoClient)
			{
				var DeployContextList = CreateDeploymentContext(Params, false, false);
				foreach (var SC in DeployContextList)
				{
					CreateArchiveManifest(Params, SC);
					ApplyArchiveManifest(Params, SC);
					SC.StageTargetPlatform.ProcessArchivedProject(Params, SC);
				}
			}
			if (Params.DedicatedServer)
			{
				ProjectParams ServerParams = new ProjectParams(Params);
				ServerParams.Devices = new ParamList<string>(ServerParams.ServerDevice);
				var DeployContextList = CreateDeploymentContext(ServerParams, true, false);
				foreach (var SC in DeployContextList)
				{
					CreateArchiveManifest(Params, SC);
					ApplyArchiveManifest(Params, SC);
					SC.StageTargetPlatform.ProcessArchivedProject(Params, SC);
				}
			}
			Logger.LogInformation("Archive command time: {0:0.00} s", (DateTime.UtcNow - StartTime).TotalMilliseconds / 1000);
			Logger.LogInformation("********** ARCHIVE COMMAND COMPLETED **********");
		}
	}
}
