// Copyright Epic Games, Inc. All Rights Reserved.

using System.Runtime.CompilerServices;

[assembly: InternalsVisibleTo("AutomationTool.Tests")]
