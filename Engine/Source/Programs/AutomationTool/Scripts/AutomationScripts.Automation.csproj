<Project Sdk="Microsoft.NET.Sdk">
  <Import Project="..\..\Shared\UnrealEngine.csproj.props" />
  
  <PropertyGroup>
    <OutputType>Library</OutputType>
    <GenerateAssemblyInfo>false</GenerateAssemblyInfo>
    <GenerateTargetFrameworkAttribute>false</GenerateTargetFrameworkAttribute>
    <RootNamespace>AutomationTool</RootNamespace>
    <AssemblyName>AutomationScripts.Automation</AssemblyName>
    <WarningsNotAsErrors>612,618</WarningsNotAsErrors>
    <OutputPath Condition="'$(ArtifactsPath)' == ''">..\..\..\..\Binaries\DotNET\AutomationTool\AutomationScripts\Scripts</OutputPath>
    <TreatWarningsAsErrors>true</TreatWarningsAsErrors>
  </PropertyGroup>
  <ItemGroup>
    <Reference Include="OneSky, Version=1.0.0.0, Culture=neutral, processorArchitecture=MSIL">
      <HintPath>..\..\..\..\Binaries\DotNET\OneSky.dll</HintPath>
      <SpecificVersion>False</SpecificVersion>
      <Private>False</Private>
    </Reference>
  </ItemGroup>

  <ItemGroup>
    <Compile Include="..\..\Shared\MetaData.cs">
      <Link>Properties\MetaData.cs</Link>
    </Compile>
  </ItemGroup>
  <ItemGroup>
    <ProjectReference Include="..\..\Shared\EpicGames.Build\EpicGames.Build.csproj" PrivateAssets="All"><Private>false</Private></ProjectReference>
    <ProjectReference Include="..\..\Shared\EpicGames.Core\EpicGames.Core.csproj" PrivateAssets="All"><Private>false</Private></ProjectReference>
    <ProjectReference Include="..\..\Shared\EpicGames.ProjectStore\EpicGames.ProjectStore.csproj" PrivateAssets="All" />
    <ProjectReference Include="..\..\UnrealBuildTool\UnrealBuildTool.csproj" PrivateAssets="All"><Private>false</Private></ProjectReference>
    <ProjectReference Include="..\AutomationUtils\AutomationUtils.Automation.csproj" PrivateAssets="All"><Private>false</Private></ProjectReference>
    <ProjectReference Include="..\Localization\Localization.Automation.csproj" PrivateAssets="All"><Private>false</Private></ProjectReference>
  </ItemGroup>
  <ItemGroup>
    <PackageReference Include="Microsoft.VisualStudio.OLE.Interop" Version="17.14.40260" />
    <PackageReference Include="AWSSDK.S3" Version="3.7.501.14" PrivateAssets="all" />
  </ItemGroup>
</Project>