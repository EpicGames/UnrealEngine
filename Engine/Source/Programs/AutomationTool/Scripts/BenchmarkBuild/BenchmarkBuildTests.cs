// Copyright Epic Games, Inc. All Rights Reserved.

using AutomationTool;
using System;
using System.Collections.Generic;
using System.Linq;
using System.Text;
using System.Threading.Tasks;
using EpicGames.Core;
using UnrealBuildTool;
using UnrealBuildBase;

namespace AutomationTool.Benchmark
{
	
	/// <summary>
	/// Task that builds a target
	/// </summary>
	class BenchmarkBuildTask : BenchmarkTaskBase
	{
		private BuildTarget				Command;
		private UBTBuildOptions			BuildOptions;
		private UnrealTargetPlatform	TargetPlatform;

		public static bool SupportsAcceleration
		{
			get
			{
				return BuildHostPlatform.Current.Platform == UnrealTargetPlatform.Win64 ||
					(BuildHostPlatform.Current.Platform == UnrealTargetPlatform.Mac);
			}
		}

		public static string AccelerationName
		{
			get
			{
				if (BuildHostPlatform.Current.Platform == UnrealTargetPlatform.Win64)
				{
					return "XGE";
				}
				else if (BuildHostPlatform.Current.Platform == UnrealTargetPlatform.Mac)
				{
					return "FASTBuild";
				}
				else
				{
					return "none";
				}
			}
		}

		private string _TaskName;
		public override string TaskName
		{
			get
			{
				return _TaskName;
			}
		}
		public BenchmarkBuildTask(FileReference InProjectFile, string InTarget, UnrealTargetPlatform InPlatform, XGETaskOptions InXgeOption, string InUBTArgs="", int CoreCount=0, UBTBuildOptions InOptions = UBTBuildOptions.None)
			: base(InProjectFile)
		{
			bool IsVanillaUnreal = InProjectFile == null;

			string ModuleName = IsVanillaUnreal ? "Unreal" : InProjectFile.GetFileNameWithoutAnyExtensions();

			_TaskName = string.Format("Build {0} {1}", InTarget, InPlatform);

			BuildOptions = InOptions;
			TargetPlatform = InPlatform;

			Command = new BuildTarget();
			Command.ProjectName = IsVanillaUnreal ? null : ModuleName;
			Command.Platforms = TargetPlatform.ToString();
			Command.Targets = InTarget;
			Command.NoTools = true;	
			Command.UBTArgs = InUBTArgs;

			bool WithAccel = InXgeOption == XGETaskOptions.WithXGE;

			if (!WithAccel || !SupportsAcceleration)
			{
				string Arg = string.Format("No{0}", AccelerationName);

				Command.UBTArgs += " -" + Arg;
				//TaskModifiers.Add(Arg);
				Command.Params = new[] { Arg }; // need to also pass it to this

				// If no cores were specified use the machines CPU count rather than letting UBT pick a value. The latter may 
				// not be deterministic.
				int NumCores = CoreCount > 0 ? CoreCount : Environment.ProcessorCount;

				TaskModifiers.Add(string.Format("{0}c", NumCores));
				Command.UBTArgs += string.Format(" -MaxParallelActions={0}", NumCores);
			}
			else
			{
				TaskModifiers.Add(AccelerationName);
			}		
			
			if (!string.IsNullOrEmpty(InUBTArgs))
			{
				TaskModifiers.Add(InUBTArgs);
			}
		}

		protected bool CleanBuildTarget()
		{
			var BuildCommand = new UnrealBuild(null);
			var BuildTarget = Command.ProjectTargetFromTargetName(
				Command.Targets,
				ProjectFile,
				new [] { TargetPlatform },
				new [] { UnrealTargetConfiguration.Development }
				);
			BuildCommand.CleanWithUBT(BuildTarget.TargetName, TargetPlatform, UnrealTargetConfiguration.Development, ProjectFile);
			return true;
		}

		protected override bool PerformPrequisites()
		{
			if (!base.PerformPrequisites())
			{
				return false;
			}

			if (BuildOptions.HasFlag(UBTBuildOptions.PreClean))
			{
				return CleanBuildTarget();
			}

			return true;
		}

		protected override bool PerformTask()
		{	
			ExitCode Result = Command.Execute();

			return Result == ExitCode.Success;
		}
	}

	class BenchmarkCleanBuildTask : BenchmarkBuildTask
	{
		string _TaskName;

		public BenchmarkCleanBuildTask(FileReference InProjectFile, string InTarget, UnrealTargetPlatform InPlatform)
			: base(InProjectFile, InTarget, InPlatform, XGETaskOptions.None, "", 0, UBTBuildOptions.None)
		{
			string ModuleName = InProjectFile == null ? "Unreal" : InProjectFile.GetFileNameWithoutAnyExtensions();
			_TaskName = string.Format("Clean {0} {1}", InTarget, InPlatform);
		}

		public override string TaskName
		{
			get
			{
				return _TaskName;
			}
		}

		protected override bool PerformTask()
		{
			return CleanBuildTarget();
		}
	}
}

