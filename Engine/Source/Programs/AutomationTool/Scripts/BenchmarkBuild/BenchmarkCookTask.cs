// Copyright Epic Games, Inc. All Rights Reserved.

using AutomationTool;
using System;
using System.Collections.Generic;
using System.IO;
using System.Linq;
using System.Text;
using System.Threading.Tasks;
using EpicGames.Core;
using UnrealBuildTool;
using UnrealBuildBase;
using Microsoft.Extensions.Logging;

namespace AutomationTool.Benchmark
{
	class BenchmarkCookTask : BenchmarkEditorTaskBase
	{
		protected string	CookPlatformName;

		string				CookArgs;

		bool				CookAsClient;
		public override string TaskName
		{
			get
			{
				return string.Format("Cook {0}", CookPlatformName);
			}
		}
		public BenchmarkCookTask(ProjectTargetInfo InTargetInfo, ProjectTaskOptions InOptions, bool InSkipBuild)
			: base(InTargetInfo, InOptions, InSkipBuild)
		{
			CookArgs = InOptions.Args;
			CookAsClient = InTargetInfo.BuildTargetAsClient;

			var PlatformToCookPlatform = new Dictionary<UnrealTargetPlatform, string> {
				{ UnrealTargetPlatform.Win64, "WindowsClient" },
				{ UnrealTargetPlatform.Mac, "MacClient" },
				{ UnrealTargetPlatform.Linux, "LinuxClient" },
				{ UnrealTargetPlatform.Android, "Android_ASTCClient" }
			};

			CookPlatformName = InTargetInfo.TargetPlatform.ToString();

			if (PlatformToCookPlatform.ContainsKey(InTargetInfo.TargetPlatform))
			{
				CookPlatformName = PlatformToCookPlatform[InTargetInfo.TargetPlatform];
			}
		}

		protected override string GetEditorTaskArgs()
		{
			string Arguments = "";

			if (CookAsClient)
			{
				Arguments += " -client";
			}

			if (CookArgs.Length > 0)
			{
				Arguments += " " + CookArgs;
			}

			return Arguments;
		}

		protected override bool PerformTask()
		{
			string Arguments = GetBasicEditorCommandLine(false);
			// will throw an exception if it fails
			CommandUtils.RunCommandlet(ProjectTarget.ProjectFile, UnrealExe:null, "Cook", String.Format("-TargetPlatform={0} {1}", CookPlatformName, Arguments));
			return true;
		}
	}

	/// <summary>
	/// Iterative cooking test
	/// </summary>
	class BenchmarkIterativeCookTask : BenchmarkCookTask
	{
		public BenchmarkIterativeCookTask(ProjectTargetInfo InTargetInfo, ProjectTaskOptions InOptions, bool InSkipBuild)
			: base(InTargetInfo, InOptions, InSkipBuild)
		{
		}

		public override string TaskName
		{
			get
			{
				return string.Format("Iterative Cook {0}", CookPlatformName);
			}
		}

		protected override bool PerformPrequisites()
		{
			if (!base.PerformPrequisites())
			{
				return false;
			}

			DirectoryReference ContentDir = DirectoryReference.Combine(ProjectTarget.ProjectFile.Directory, "Content");

			var Files = DirectoryReference.EnumerateFiles(ContentDir, "*.uasset", System.IO.SearchOption.AllDirectories);

			var AssetFile = Files.FirstOrDefault();

			if (AssetFile == null)
			{
				Logger.LogError("Could not find asset file to touch under {ContentDir}", ContentDir);
				return false;
			}

			FileInfo Fi = AssetFile.ToFileInfo();

			bool ReadOnly = Fi.IsReadOnly;

			if (ReadOnly)
			{
				Fi.IsReadOnly = false;
			}

			Fi.LastWriteTime = DateTime.Now;

			if (ReadOnly)
			{
				Fi.IsReadOnly = true;
			}

			return true;
		}

		protected override string GetEditorTaskArgs()
		{
			string Args = base.GetEditorTaskArgs();

			Args += " -iterate";

			return Args;
		}
	}
}
