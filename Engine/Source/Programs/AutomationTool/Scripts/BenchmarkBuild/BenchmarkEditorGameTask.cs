// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Text;
using UnrealBuildTool;

namespace AutomationTool.Benchmark
{
	class BenchmarkEditorGameTask : BenchmarkEditorStartupTask
	{
		public BenchmarkEditorGameTask(ProjectTargetInfo InTargetInfo, ProjectTaskOptions InOptions, bool InSkipBuild)
			: base(InTargetInfo, InOptions, InSkipBuild)
		{
		}

		public override string TaskName
		{
			get
			{
				return "EditorGame";
			}
		}

		protected override string GetEditorTaskArgs()
		{
			return "-windowed -game -execcmds=\"automation Quit\"";
		}
	}
}
