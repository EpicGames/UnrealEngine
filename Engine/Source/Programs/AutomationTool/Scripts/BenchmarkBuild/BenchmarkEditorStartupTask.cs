// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Linq;
using System.Text;
using System.Threading;
using System.Threading.Tasks;
using EpicGames.Core;
using UnrealBuildBase;
using UnrealBuildTool;

namespace AutomationTool.Benchmark
{
	class BenchmarkEditorStartupTask : BenchmarkEditorTaskBase
	{
		public BenchmarkEditorStartupTask(ProjectTargetInfo InTargetInfo, ProjectTaskOptions InOptions, bool InSkipBuild)
			: base(InTargetInfo, InOptions, InSkipBuild)
		{
		}
		public override string TaskName
		{
			get
			{
				return "Start Editor";
			}
		}

		protected override string GetEditorTaskArgs()
		{
			return "-execcmds=\"automation Quit\"";
		}
	}
}
