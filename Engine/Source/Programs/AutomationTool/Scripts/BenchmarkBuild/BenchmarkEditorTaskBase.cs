// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.IO;
using System.Linq;
using System.Text;
using System.Threading;
using System.Threading.Tasks;
using EpicGames.Core;
using UnrealBuildBase;
using UnrealBuildTool;
using Microsoft.Extensions.Logging;

namespace AutomationTool.Benchmark
{

	/// <summary>
	/// Options when running a task
	/// </summary>
	class ProjectTaskOptions
	{
		public DDCTaskOptions DDCOptions { get; protected set; }

		public XGETaskOptions XGEOptions { get; protected set; }

		public string Map { get; protected set; }

		public int CoreLimit { get; protected set; }

		public string Args { get; protected set; }


		public ProjectTaskOptions(DDCTaskOptions InDDCOptions, XGETaskOptions InXGEOptions, string InArgs, string InMap = "", int InCoreLimit = 0)
		{
			DDCOptions = InDDCOptions;
			XGEOptions = InXGEOptions;
			Args = InArgs.Trim().Replace("  ", " "); ;
			CoreLimit = InCoreLimit;
			Map = InMap;
		}
	}

	/// <summary>
	/// Abstract classs for running editor-based tasks for benchmarking
	/// </summary>
	abstract class BenchmarkEditorTaskBase : BenchmarkTaskBase
	{
		protected ProjectTargetInfo ProjectTarget = null;

		protected ProjectTaskOptions TaskOptions = null;

		protected bool SkipBuild = false;

		protected static string MakeValidTaskFileName(string name)
		{
			string invalidChars = System.Text.RegularExpressions.Regex.Escape(new string(System.IO.Path.GetInvalidFileNameChars()));
			string invalidRegStr = string.Format(@"([{0}]*\.+$)|([{0}]+)", invalidChars);

			return System.Text.RegularExpressions.Regex.Replace(name, invalidRegStr, "_");
		}

		protected BenchmarkEditorTaskBase(ProjectTargetInfo InTargetInfo, ProjectTaskOptions InOptions, bool InSkipBuild)
			: base(InTargetInfo.ProjectFile)
		{
			ProjectTarget = InTargetInfo;
			TaskOptions = InOptions;
			SkipBuild = InSkipBuild;

			if (!string.IsNullOrEmpty(TaskOptions.Map))
			{
				TaskModifiers.Add(TaskOptions.Map);
			}

			if (TaskOptions.DDCOptions == DDCTaskOptions.None || TaskOptions.DDCOptions.HasFlag(DDCTaskOptions.WarmDDC))
			{
				TaskModifiers.Add("WarmDDC");
			}

			if (TaskOptions.DDCOptions.HasFlag(DDCTaskOptions.ColdDDC))
			{
				TaskModifiers.Add("ColdDDC");
			}

			if (TaskOptions.DDCOptions.HasFlag(DDCTaskOptions.ColdDDCNoShared))
			{
				TaskModifiers.Add("ColdDDC-NoShared");
			}

			if (TaskOptions.DDCOptions.HasFlag(DDCTaskOptions.HotDDC))
			{
				TaskModifiers.Add("HotDDC");
			}

			if (TaskOptions.DDCOptions.HasFlag(DDCTaskOptions.NoShaderDDC))
			{
				TaskModifiers.Add("NoShaderDDC");
			}

			if (TaskOptions.DDCOptions.HasFlag(DDCTaskOptions.KeepMemoryDDC))
			{
				TaskModifiers.Add("WithBootDDC");
			}

			if (TaskOptions.XGEOptions.HasFlag(XGETaskOptions.NoEditorXGE))
			{
				//TaskModifiers.Add("NoXGE");
			}

			if (TaskOptions.XGEOptions.HasFlag(XGETaskOptions.WithEditorXGE))
			{
				TaskModifiers.Add("XGE");
			}

			if (!string.IsNullOrEmpty(TaskOptions.Args))
			{
				TaskModifiers.Add(TaskOptions.Args);
			}

			if (TaskOptions.CoreLimit > 0)
			{
				TaskModifiers.Add(string.Format("{0}c", TaskOptions.CoreLimit));
			}
		}

		protected string GetEditorPath()
		{
			return HostPlatform.Current.GetUnrealExePath("UnrealEditor.exe");
		}

		protected string GetBasicEditorCommandLine(bool bIsWarming)
		{
			string ProjectArg = ProjectTarget.ProjectFile != null ? ProjectTarget.ProjectFile.ToString() : "";
			string MapArg = string.IsNullOrEmpty(TaskOptions.Map) ? "" : TaskOptions.Map;
			string LogArg = string.Format("-log={0}.log", MakeValidTaskFileName(FullName).Replace(" ", "_"));
			string Arguments = string.Format("{0} {1} {2} -benchmark -stdout -FullStdOutLogOutput -unattended {3}", ProjectArg, MapArg, TaskOptions.Args, LogArg);

			if (!bIsWarming && TaskOptions.DDCOptions.HasFlag(DDCTaskOptions.NoShaderDDC))
			{
				Arguments += " -noshaderddc";
			}

			if (TaskOptions.DDCOptions.HasFlag(DDCTaskOptions.ColdDDCNoShared))
			{
				Arguments += " -ddc=noshared";
			}

			if (TaskOptions.XGEOptions.HasFlag(XGETaskOptions.NoEditorXGE))
			{
				Arguments += " -noxgeshadercompile";
			}

			if (TaskOptions.CoreLimit > 0)
			{
				Arguments += string.Format(" -CoreLimit={0}", TaskOptions.CoreLimit);
			}

			return Arguments;
		}

		protected virtual string GetEditorTaskArgs()
		{
			return "";
		}

		private Dictionary<string, string> StoredEnvVars = new Dictionary<string, string>();
		private List<DirectoryReference> CachePaths = new List<DirectoryReference>();

		private string GetXPlatformEnvironmentKey(string InKey)
		{
			// Mac uses _ in place of -
			if (BuildHostPlatform.Current.Platform != UnrealTargetPlatform.Win64)
			{
				InKey = InKey.Replace("-", "_");
			}

			return InKey;
		}

		static IProcessResult CurrentProcess = null;
		static DateTime LastStdOutTime = DateTime.Now;
		//static bool TestCompleted = false;

		/// <summary>
		/// A filter that suppresses all output od stdout/stderr
		/// </summary>
		/// <param name="Message"></param>
		/// <returns></returns>
		static string EndOnMapCheckFilter(string Message)
		{
			if (CurrentProcess != null)
			{
				lock (CurrentProcess)
				{
					if (Message.Contains("TEST COMPLETE"))
					{
						Logger.LogInformation("Automation test reported as complete.");
						//TestCompleted = true;
					}

					LastStdOutTime = DateTime.Now;
				}
			}
			return Message;
		}

		protected bool RunEditorAndWaitForMapLoad(bool bIsWarming)
		{
			string EditorPath = GetEditorPath();

			string Arguments = GetBasicEditorCommandLine(bIsWarming);
			Arguments = Arguments + " " + GetEditorTaskArgs();

			var RunOptions = CommandUtils.ERunOptions.AllowSpew | CommandUtils.ERunOptions.NoWaitForExit;

			var SpewDelegate = new ProcessResult.SpewFilterCallbackType(EndOnMapCheckFilter);

			//TestCompleted = false;
			LastStdOutTime = DateTime.Now;
			CurrentProcess = CommandUtils.Run(EditorPath, Arguments, Options: RunOptions, SpewFilterCallback: SpewDelegate);

			int TimeoutMins = 20;

			while (!CurrentProcess.HasExited)
			{
				Thread.Sleep(5 * 1000);

				lock (CurrentProcess)
				{
					if ((DateTime.Now - LastStdOutTime).TotalMinutes >= TimeoutMins)
					{
						Logger.LogError("Gave up waiting for task after {TimeoutMins} minutes of no output", TimeoutMins);
						CurrentProcess.ProcessObject.Kill();
					}
				}
			}

			int ExitCode = CurrentProcess.ExitCode;
			CurrentProcess = null;

			return ExitCode == 0;
		}

		protected override bool PerformTask()
		{
			return RunEditorAndWaitForMapLoad(false);
		}

		static HashSet<string> BuiltEditors = new HashSet<string>();

		protected override bool PerformPrequisites()
		{

			string ProjectName = ProjectTarget.ProjectFile.GetFileNameWithoutAnyExtensions();
			// build editor
			if (SkipBuild == false)
			{
				BuildTarget Command = new BuildTarget();
				Command.ProjectName = ProjectTarget.ProjectFile != null ? ProjectName : null;
				Command.Platforms = BuildHostPlatform.Current.Platform.ToString();
				Command.Targets = "Editor";
				Command.NoTools = false;

				if (Command.Execute() != ExitCode.Success)
				{
					Logger.LogError("Failed to build editor!");
					return false;
				}
				else
				{
					BuiltEditors.Add(ProjectName);
				}
		
			}
			else if (BuiltEditors.Contains(ProjectName) == false)
			{
				Logger.LogError("SkipBuild = true but no editor has been built for {ProjectName}", ProjectName);
				return false;
			}	

			if (TaskOptions.DDCOptions.HasFlag(DDCTaskOptions.ColdDDC) || TaskOptions.DDCOptions.HasFlag(DDCTaskOptions.ColdDDCNoShared))
			{
				StoredEnvVars.Clear();
				CachePaths.Clear();

				// We put our temp DDC paths in here
				DirectoryReference BasePath = DirectoryReference.Combine(Unreal.EngineDirectory, "BenchmarkDDC");

				// For Linux and Mac the ENV vars will be UE_BootDataCachePath and UE_LocalDataCachePath
				IEnumerable<string> DDCEnvVars = new string[] { GetXPlatformEnvironmentKey("UE-BootDataCachePath"), GetXPlatformEnvironmentKey("UE-LocalDataCachePath") };

				if (TaskOptions.DDCOptions.HasFlag(DDCTaskOptions.KeepMemoryDDC))
				{
					DDCEnvVars = DDCEnvVars.Where(E => !E.Contains("UE-Boot"));
				}

				// get all current environment vars and set them to our temp dir
				foreach (var Key in DDCEnvVars)
				{
					// save current key
					StoredEnvVars.Add(Key, Environment.GetEnvironmentVariable(Key));

					// create a new dir for this key
					DirectoryReference Dir = DirectoryReference.Combine(BasePath, Key);

					if (DirectoryReference.Exists(Dir))
					{
						DirectoryReference.Delete(Dir, true);
					}

					DirectoryReference.CreateDirectory(Dir);

					// save this dir and set it as the env var
					CachePaths.Add(Dir);
					Environment.SetEnvironmentVariable(Key, Dir.FullName);
				}

				if (TaskOptions.DDCOptions.HasFlag(DDCTaskOptions.ColdDDCNoShared))
				{
					string Key = "UE-SharedDataCachePath";
					StoredEnvVars.Add(Key, Environment.GetEnvironmentVariable(Key));
					Environment.SetEnvironmentVariable(Key, "None");
				}

				// remove project files
				if (ProjectTarget.ProjectFile != null)
				{
					DirectoryReference ProjectDDC = DirectoryReference.Combine(ProjectTarget.ProjectFile.Directory, "DerivedDataCache");
					CommandUtils.DeleteDirectory_NoExceptions(ProjectDDC.FullName);

					// remove S3 files
					DirectoryReference S3DDC = DirectoryReference.Combine(ProjectTarget.ProjectFile.Directory, "Saved", "S3DDC");
					CommandUtils.DeleteDirectory_NoExceptions(S3DDC.FullName);
				}
			}

			// if they want a hot DDC then do the test one time with no timing
			if (TaskOptions.DDCOptions.HasFlag(DDCTaskOptions.HotDDC))
			{
				RunEditorAndWaitForMapLoad(true);
			}

			return base.PerformPrequisites();
		}

		protected override void PerformCleanup()
		{
			// restore keys
			foreach (var KV in StoredEnvVars)
			{
				Environment.SetEnvironmentVariable(KV.Key, KV.Value);
			}

			foreach (var Dir in CachePaths)
			{
				CommandUtils.DeleteDirectory_NoExceptions(Dir.FullName);
			}

			CachePaths.Clear();
			StoredEnvVars.Clear();
		}
	}
}
