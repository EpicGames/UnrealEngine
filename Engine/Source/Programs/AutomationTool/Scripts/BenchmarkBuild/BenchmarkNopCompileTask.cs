// Copyright Epic Games, Inc. All Rights Reserved.

using AutomationTool;
using System;
using System.Collections.Generic;
using System.IO;
using System.Linq;
using System.Text;
using System.Threading.Tasks;
using EpicGames.Core;
using UnrealBuildTool;

namespace AutomationTool.Benchmark
{
	class BenchmarkNopCompileTask : BenchmarkBuildTask
	{


		private string _TaskName;
		public override string TaskName
		{
			get
			{
				return _TaskName;
			}
		}

		public BenchmarkNopCompileTask(FileReference InProjectFile, string InTarget, UnrealTargetPlatform InPlatform, XGETaskOptions InXGEOptions)
			: base(InProjectFile, InTarget, InPlatform, InXGEOptions, "", 0)
		{
			_TaskName = string.Format("Nop {0} {1}", InTarget, InPlatform);
		}
	}
}
