// Copyright Epic Games, Inc. All Rights Reserved.

using AutomationTool;
using System;
using System.Collections.Generic;
using System.IO;
using System.Linq;
using System.Text;
using System.Threading.Tasks;
using EpicGames.Core;
using UnrealBuildTool;
using UnrealBuildBase;
using Microsoft.Extensions.Logging;

namespace AutomationTool.Benchmark
{
	class BenchmarkSingleCompileTask : BenchmarkBuildTask
	{
		FileReference SourceFile = null;

		private string _TaskName;
		public override string TaskName
		{
			get
			{
				return _TaskName;
			}
		}
		public BenchmarkSingleCompileTask(FileReference InProjectFile, string InTarget, UnrealTargetPlatform InPlatform, XGETaskOptions InXgeOption)
			: base(InProjectFile, InTarget, InPlatform, InXgeOption, "", 0)
		{
			string ModuleName = InProjectFile == null ? "Unreal" : InProjectFile.GetFileNameWithoutAnyExtensions();

			_TaskName = string.Format("Incr {0} {1}", InTarget, InPlatform);

			string ProjectName = null;

			// Try to find a source file in the project
			if (InProjectFile != null)
			{
				ProjectName = InProjectFile.GetFileNameWithoutAnyExtensions();
				DirectoryReference SourceDir = DirectoryReference.Combine(InProjectFile.Directory, "Source", ProjectName);

				if (DirectoryReference.Exists(SourceDir))
				{
					var Files = DirectoryReference.EnumerateFiles(SourceDir, "*.cpp", System.IO.SearchOption.AllDirectories);
					SourceFile = Files.FirstOrDefault();
				}
			}

			// if we didn't, use an engine one
			if (SourceFile == null)
			{
				if (InProjectFile == null)
				{
					ProjectName = "UnrealEngine";
				}
				SourceFile = FileReference.Combine(Unreal.EngineDirectory, "Source/Runtime/Engine/Private/UnrealEngine.cpp");
			}

			Logger.LogDebug("Will compile {SourceFile} for single-file compilation test for {ProjectName}", SourceFile, ProjectName);
		}

		protected override bool PerformPrequisites()
		{
			if (!base.PerformPrequisites())
			{
				return false;
			}

			FileInfo Fi = SourceFile.ToFileInfo();

			bool ReadOnly = Fi.IsReadOnly;

			if (ReadOnly)
			{
				Fi.IsReadOnly = false;
			}

			Fi.LastWriteTime = DateTime.Now;

			if (ReadOnly)
			{
				Fi.IsReadOnly = true;
			}

			return true;
		}
	}
}

