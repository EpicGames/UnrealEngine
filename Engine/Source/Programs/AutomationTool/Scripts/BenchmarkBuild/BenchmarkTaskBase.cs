// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.IO;
using System.Linq;
using System.Text;
using System.Threading.Tasks;
using EpicGames.Core;
using UnrealBuildBase;
using UnrealBuildTool;
using Microsoft.Extensions.Logging;

namespace AutomationTool.Benchmark
{
	[Flags]
	public enum UBTBuildOptions
	{
		None = 0,
		PreClean = 1 << 0,          // don't preclean before the job (useful for testing)
		PostClean = 1 << 1,         // clean after the job (default for building multiple clients)
		CleanOnly = 1 << 2,
	}

	[Flags]
	public enum DDCTaskOptions
	{
		None = 0,
		WarmDDC = 1 << 0,
		ColdDDCNoShared = 1 << 1,
		ColdDDC = 1 << 2,
		NoShaderDDC = 1 << 3,
		HotDDC = 1 << 4,
		KeepMemoryDDC = 1 << 6,
	}

	[Flags]
	public enum XGETaskOptions
	{
		None = 0,
		NoXGE = 1 << 1,
		WithXGE = 1 << 2,
		NoEditorXGE = 1 << 3,         // don't use XGE for shader compilation
		WithEditorXGE = 1 << 4,         // don't use XGE for shader compilation
	}

	/// <summary>
	/// Simple class that describes a target in a project
	/// </summary>
	class ProjectTargetInfo
	{
		public FileReference ProjectFile { get; private set; }

		public UnrealTargetPlatform TargetPlatform { get; private set; }

		public bool BuildTargetAsClient { get; private set; }

		public ProjectTargetInfo(FileReference InFileReference, UnrealTargetPlatform InTargetPlatform, bool InBuildTargetAsClient)
		{
			ProjectFile = InFileReference;
			TargetPlatform = InTargetPlatform;
			BuildTargetAsClient = InBuildTargetAsClient;
		}
	}

	/// <summary>
	/// Base class for running tasks
	/// </summary>
	abstract class BenchmarkTaskBase
	{
		/// <summary>
		/// Default logger instance
		/// </summary>
		protected static ILogger Logger => Log.Logger;

		/// <summary>
		/// True or false based on whether the task failed
		/// </summary>
		public bool Failed { get; protected set; }

		/// <summary>
		/// Failure message
		/// </summary>
		public string FailureString { get; protected set; }

		/// <summary>
		/// Don't report this test
		/// </summary>
		public bool SkipReport { get; set; }

		/// <summary>
		/// Time the task took (does not include prequisites)
		/// </summary>
		public TimeSpan TaskTime { get; protected set; }

		/// <summary>
		/// Time the task started
		/// </summary>
		public DateTime StartTime { get; protected set; }

		/// <summary>
		/// Perform any prerequisites the task requires
		/// </summary>
		virtual protected bool PerformPrequisites() { return true;  }

		/// <summary>
		/// Perform post-task cleanup
		/// </summary>
		virtual protected void PerformCleanup() { }

		/// <summary>
		/// Perform the actual task that is measured
		/// </summary>
		protected abstract bool PerformTask();	
		
		/// <summary>
		/// Path to the project file we're using. May be null for Unreal editor, client etc
		/// </summary>
		public FileReference ProjectFile { get; private set; }


		/// <summary>
		/// Return the name of the module this is being performed on
		/// </summary>
		/// <returns></returns>
		public string ProjectName { get { return ProjectFile == null ? "Unreal" : ProjectFile.GetFileNameWithoutAnyExtensions(); } }

		/// <summary>
		/// Return the name of this task
		/// </summary>
		/// <returns></returns>
		public abstract string TaskName { get;  }

		private readonly List<string> InternalModifiers = new List<string>();

		/// <summary>
		/// A list of modifiers that can be considered when
		/// </summary>
		protected List<string> TaskModifiers { get { return InternalModifiers; }  }

		/// <summary>
		/// Return the full name of this task for reporting
		/// </summary>
		/// <returns></returns>
		public string ModifierString 
		{ 
			get
			{
				string Str = String.Empty;

				if (TaskModifiers.Count > 0)
				{
					Str = string.Join(" ", TaskModifiers);
				}

				return Str;
			} 
		}

		/// <summary>
		/// Returns a full name to use in reporting and logging
		/// </summary>
		/// <returns></returns>
		public string TaskNameWithModifiers
		{
			get
			{
				string Name = TaskName;

				if (TaskModifiers.Count > 0)
				{
					Name = string.Format("{0} ({1})", Name, ModifierString);
				}

				return Name;
			}
		}



		/// <summary>
		/// Return the full name of this task for reporting
		/// </summary>
		/// <returns></returns>
		public string FullName { get { return String.Format("{0} {1}", ProjectName, TaskNameWithModifiers); } }		

		/// <summary>
		/// Simple constructor
		/// </summary>
		/// <param name="InProjectFile"></param>
		public BenchmarkTaskBase(FileReference InProjectFile)
		{
			ProjectFile = InProjectFile;
		}

		/// <summary>
		/// Run the task. Performs any prerequisites, then the actual task itself
		/// </summary>
		public void Run()
		{
			try
			{
				if (PerformPrequisites())
				{
					StartTime = DateTime.Now;

					if (!PerformTask())
					{
						FailureString = "Task Failed";
						Failed = true;
					}
				}
				else
				{
					FailureString = "Prequisites Failed";
					Failed = true;
				}
				
			}
			catch (Exception Ex)
			{
				FailureString = string.Format("Exception: {0}", Ex.ToString());
				Failed = true;
			}

			if (StartTime != DateTime.MinValue)
			{
				TaskTime = DateTime.Now - StartTime;
			}
			
			if (Failed)
			{
				Logger.LogError("{FullName} failed. {FailureString}", FullName, FailureString);
			}

			try
			{
				PerformCleanup();
			}
			catch (Exception Ex)
			{
				Logger.LogError("Cleanup of {FullName} failed. {Ex}", FullName, Ex);
			}
		}

		/// <summary>
		/// Report how long the task took
		/// </summary>
		public void Report()
		{
			if (!Failed)
			{
				Logger.LogInformation("Task {FullName}:\t\t\t\t{Arg1}", FullName, TaskTime.ToString(@"hh\:mm\:ss"));
			}
			else
			{
				Logger.LogInformation("Task {FullName}::\t\t\t\t{Arg1} Failed. {FailureString}", FullName, TaskTime.ToString(@"hh\:mm\:ss"), FailureString);
			}
		}

		public override string ToString()
		{
			return FullName;
		}
	}
}
