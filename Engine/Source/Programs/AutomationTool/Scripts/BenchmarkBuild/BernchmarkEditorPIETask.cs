// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Text;
using UnrealBuildTool;

namespace AutomationTool.Benchmark
{
	class BenchmarPIEEditorTask : BenchmarkEditorStartupTask
	{
		public BenchmarPIEEditorTask(ProjectTargetInfo InTargetInfo, ProjectTaskOptions InOptions, bool InSkipBuild)
			: base(InTargetInfo, InOptions, InSkipBuild)
		{
		}
		public override string TaskName
		{
			get
			{
				return "Start PIE";
			}
		}

		protected override string GetEditorTaskArgs()
		{
			return "-execcmds=\"automation IgnoreLogEvents;runtest Project.Maps.PIE;Quit\"";
		}
	}
}
