// Copyright Epic Games, Inc. All Rights Reserved.

using AutomationTool;
using AutomationUtils;
using EpicGames.Core;
using EpicGames.ProjectStore;
using EpicGames.Serialization;
using Microsoft.Extensions.DependencyInjection;
using Microsoft.Extensions.Logging;
using System;
using System.Collections.Generic;
using System.ComponentModel;
using System.Linq;
using System.Reflection;
using System.Diagnostics;
using System.IO;
using UnrealBuildBase;
using System.Text.RegularExpressions;
using UnrealBuildTool;
using EpicGames.Horde;
using System.Reflection.Metadata;

class BuildInfo
{
	public string BuildVariant = null;
	public string Platform = null;
}

public static class EnumExtensions
{
	public static EnumDescription ConvertEnumWithDescription<T>() where T : struct, IConvertible
	{
		if (!typeof(T).IsEnum)
		{
			throw new ArgumentException("Type T must be an Enum");
		}

		var enumType = typeof(T).Name;
		var valueDescriptions = Enum.GetValues(typeof(T))
			.Cast<Enum>()
			.ToDictionary(Convert.ToInt32, GetEnumDescription);
		var descriptionValues = Enum.GetValues(typeof(T))
			.Cast<Enum>()
			.Select(x => GetEnumDescription(x)).ToList();

		return new EnumDescription
		{
			Type = enumType,
			ValueDescriptions = valueDescriptions,
			Descriptions = descriptionValues
		};
	}

	public static string GetEnumDescription(Enum value)
	{
		if (value == null)
		{
			return String.Empty;
		}

		FieldInfo fi = value.GetType().GetField(value.ToString());
		DescriptionAttribute[] attributes = (DescriptionAttribute[])fi.GetCustomAttributes(typeof(DescriptionAttribute), false);

		return attributes.Length > 0 ? attributes[0].Description : string.Empty;
	}

	public static string GetEnumNameOrDescription(Enum value)
	{
		var Description = GetEnumDescription(value);
		return Description == string.Empty ? value.ToString() : Description;
	}
}

public class EnumDescription
{
	public string Type { get; set; }
	public IDictionary<int, string> ValueDescriptions { get; set; }
	public IList<string> Descriptions { get; set; }
}

class BuildAcquire : BuildCommand, IProjectParamsHelpers
{
	public enum BuildVariant
	{
		[Description("Development")]
		Development,
		[Description("Debug")]
		Debug,
		[Description("Test")]
		Test,
		[Description("Shipping")]
		Shipping
	}

	public enum BuildTypeEnum
	{
		[Description("staged-build")]
		Staged,
		[Description("packaged-build")]
		Packaged,
		[Description("oplog")]
		Snapshot,
		Unspecified
	}

	public BuildAcquire()
	{
		CloudStorageQueryBuilder query = new CloudStorageQueryBuilder();
		query.Equals("BuildGroup", 1);
		CbObject obj = query.Done();
	}

	public override void ExecuteBuild()
	{
		Initialize();

		var Params = new ProjectParams
			(
				Command: this,
				RawProjectPath: ProjectPath
			);

		Params.Validate();

		if (DownloadBuild)
		{
			DownloadBuilds(Params);
		}

		if (DownloadSnapshot)
		{
			ImportSnapshot(Params);
		}

		if (SyncUGS && !DryRun)
		{
			if (!IsEditorRunning(Params))
			{
				CommandUtils.Run(UGSExePath.ToString(), $"sync {Changelist}");
			}
			else
			{
				throw new AutomationException("The editor is running, cannot sync using UGS. Close the editor and try again");
			}
		}
	}

	private void Initialize()
	{
		ProjectPath = ParseProjectParam();
		if (ProjectPath == null)
		{
			throw new AutomationException("No project file specified. Use -project=<project>.");
		}

		if (!Enum.TryParse(ParseOptionalStringParam("BuildConfigType"), out BuildConfigType))
		{
			BuildConfigType = BuildVariant.Development;
		}

		DryRun = ParseParam("DryRun");
		SyncUGS = ParseParam("SyncUGS");
		ClosestSnapshot = ParseParam("ClosestSnapshot");
		DownloadSnapshot = (BuildType == BuildTypeEnum.Snapshot) || ClosestSnapshot;
		DownloadClient = ParseParam("DownloadClient");
		DownloadServer = ParseParam("DownloadServer");
		DownloadGame = ParseParam("DownloadGame");
		BuildPlatform = ParseParamValue("BuildPlatform");
		SkipContentInStaged = ParseParam("SkipContentInStaged");
		DownloadBuild = DownloadClient || DownloadServer || DownloadGame || (!String.IsNullOrEmpty(BuildPlatform) && !DownloadSnapshot && BuildType != BuildTypeEnum.Unspecified);

		Stream = ParseParamValue("Stream");
		if (Stream == null)
		{
			throw new AutomationException("No stream specified, cannot continue. Use -Stream=<Stream>");
		}

		Changelist = ParseParamValue("CL");
		if (Changelist == null)
		{
			throw new AutomationException("No CL number specified. Use -CL=<changelist>");
		}

		IsPreflight = ParseParam("Preflight");
		Buildgroup = ParseParamValue("Buildgroup", null);

		if (!DownloadBuild && !SyncUGS && !DownloadSnapshot)
		{
			throw new AutomationException("No action to perform specified on the commandline.");
		}

		UGSExePath = FileReference.Combine(new DirectoryReference(Environment.GetFolderPath(Environment.SpecialFolder.LocalApplicationData)), "UnrealGameSync", "Latest", "ugs.exe");
		if (SyncUGS)
		{
			if (!FileReference.Exists(UGSExePath))
			{
				throw new AutomationException("UGS not found at {}, cannot continue", UGSExePath.ToString());
			}
		}

		ZenExePath = new FileReference(Path.Combine(Unreal.EngineDirectory.FullName, "Binaries", HostPlatform.Current.HostEditorPlatform.ToString(), String.Format("zen{0}", RuntimePlatform.ExeExtension)));
		if (DownloadSnapshot)
		{
			if (!FileReference.Exists(ZenExePath))
			{
				throw new AutomationException("Zen executable not found at {}, cannot continue", ZenExePath.ToString());
			}
		}

		CloudStorageService = CommandUtils.ServiceScope.ServiceProvider.GetService<ICloudStorage>();
		if (CloudStorageService == null)
		{
			throw new NotImplementedException("Cannot retrieve cloud storage from service provider, unable to continue");
		}

		CloudStorageService.Reconfigure(options =>
		{
			options.ReadFromIni(ProjectPath);
			options.EchoZenCliToStdout = true;
		});

		CloudStorageService.LoginAsync(unattended: true);
	}

	private List<FindBuildResponse> FilterBuilds(in List<FindBuildResponse> foundBuilds, string CookPlatform)
	{
		List<FindBuildResponse> Builds = foundBuilds
				.Where(build => !build.Name.Contains("-zen-", StringComparison.OrdinalIgnoreCase))
				.Where(build => CookPlatform.Equals(build.Metadata.Find("cookPlatform").AsString(), StringComparison.OrdinalIgnoreCase)).ToList();

		return Builds;
	}

	private void DownloadBuilds(ProjectParams Params)
	{
		string BuildTypeDesc = EnumExtensions.GetEnumDescription(BuildType);
		if (String.IsNullOrEmpty(BuildTypeDesc))
		{
			throw new AutomationException("Build type description is uncrecognized");
		}

		CloudStorageQueryBuilder query = new CloudStorageQueryBuilder();
		query.Equals("type", BuildTypeDesc);
		query.Equals("commit", Changelist);
		query.Equals("isPreflight", IsPreflight ? "true" : "false");
		if (!string.IsNullOrEmpty(Buildgroup))
		{
			query.Equals("buildgroup", Buildgroup);
		}

		string[] BuildPlatforms = BuildPlatform?.Split('+');
		string[] Overlay = ParseOptionalStringParam("Overlay")?.Split('+');

		CbObject obj = query.Done();

		string SanitizedStream = StringId.Sanitize(Stream).ToString().Replace(".", "-", StringComparison.Ordinal);

		List<FindBuildResponse> foundBuilds = CloudStorageService.FindBuilds(ProjectPath.GetFileNameWithoutExtension(), BuildTypeDesc, SanitizedStream, obj).ToList();
        List<FindBuildResponse> BuildsToDownload = new();
		List<FindBuildResponse> OverlayBuilds = new();
		
		if (foundBuilds.Count > 0)
		{
			if (BuildPlatforms?.Length > 0)
			{
				foreach (string BuildPlatformEntry in BuildPlatforms)
				{
					BuildsToDownload.Add(foundBuilds.Where(Build => Build.Name.Equals(BuildPlatformEntry, StringComparison.OrdinalIgnoreCase)).First());
				}
			}
			else
			{
				var ServerPlatformStrings = Params.ServerTargetPlatformInstances.Select(platform => platform.GetCookPlatform(true, false)).ToList();
				var ClientPlatformStrings = Params.ClientTargetPlatformInstances.Select(platform => platform.GetCookPlatform(false, true)).ToList();
				var GamePlatformStrings = Params.ClientTargetPlatformInstances.Select(platform => platform.GetCookPlatform(false, false)).ToList();

				if (DownloadServer)
				{
					ServerPlatformStrings.ForEach(platform => BuildsToDownload.AddRange(FilterBuilds(foundBuilds, CookPlatform: platform)));
				}

				if (DownloadClient)
				{
					ClientPlatformStrings.ForEach(platform => BuildsToDownload.AddRange(FilterBuilds(foundBuilds, CookPlatform: platform)));
				}

				if (DownloadGame)
				{
					GamePlatformStrings.ForEach(platform => BuildsToDownload.AddRange(FilterBuilds(foundBuilds, CookPlatform: platform)));
				}
			}

			BuildsToDownload = BuildsToDownload.DistinctBy(Build => Build.Name).ToList();

			if (Overlay?.Length > 0)
			{
				foreach (string OverlayBuild in Overlay)
				{
					OverlayBuilds.Add(foundBuilds.Where(Build => Build.Name.Equals(OverlayBuild, StringComparison.OrdinalIgnoreCase)).First());
				}
			}
		}

		if (DryRun)
		{
			Logger.LogInformation("Will be performing a dry run");
		}

		if (BuildsToDownload.Count == 0)
		{
			Logger.LogWarning("No builds found at CL {} matching the parameters", Changelist);
			return;
		}

        foreach (FindBuildResponse Build in BuildsToDownload)
        {
            Logger.LogInformation("Build to download {Name}:{BuildId}", Build.Name, Build.BuildId);
        }

        foreach (FindBuildResponse Build in BuildsToDownload)
        {
			DirectoryReference BaseDir;
			//DirectoryReference FinalDownloadDir = DirectoryReference.Combine(new DirectoryReference(Params.BaseStageDirectory), Build.Metadata.Find("cookPlatform").AsString());
			if (BuildType == BuildTypeEnum.Snapshot)
			{
				BaseDir = new DirectoryReference(Path.Combine(Params.RawProjectPath.Directory.FullName, "Saved", "Cooked"));
			}
			else
			{
				BaseDir = new DirectoryReference(Params.BaseStageDirectory);
			}

			DirectoryReference FinalDownloadDir = DirectoryReference.Combine(BaseDir, Build.Metadata.Find("cookPlatform").AsString());
            
			Logger.LogInformation("Downloading {Name}:{BuildId} to {FinalDownloadDir}", Build.Name, Build.BuildId, FinalDownloadDir);

			if (!DryRun)
			{
				List<string> ExcludeFiles = null;
				if(SkipContentInStaged)
				{
					ExcludeFiles =
					[
						"*.pak",
						"*.utoc",
						"*.ucas",
						"*.uregs"
					];
				}

				CloudStorageService.DownloadBuildEx(Build.BucketId, Build.BuildId, FinalDownloadDir, downloadOptions: new CloudBuildDownloadOptions
				{
					AssumeHttp2 = true,
					BoostWorkers = true
				}, 
				includeFiles: null, excludeFiles: ExcludeFiles);

				foreach (FindBuildResponse OverlayBuild in OverlayBuilds)
				{
					CloudStorageService.DownloadBuild(OverlayBuild.BucketId, OverlayBuild.BuildId, FinalDownloadDir, downloadOptions: new CloudBuildDownloadOptions
					{
						AssumeHttp2 = true,
						BoostWorkers = true,
						Append = true
					});
				}
			}
        }
    }

	private struct CloudStorageConfig
	{
		public string Host = "";
		public string BuildsNamespace = "";
		public string OAuthProviderIdentifier = "";

		public CloudStorageConfig()
		{
		}
	}

	private List<string> GetCookPlatforms(ProjectParams Params)
	{
		var PlatformsSet = new HashSet<string>();

		if (!Params.NoClient)
		{
			foreach (var ClientPlatform in Params.ClientTargetPlatforms)
			{
				var PlatformDescriptor = Params.GetCookedDataPlatformForClientTarget(ClientPlatform);
				string CookPlatform = Platform.Platforms[PlatformDescriptor].GetCookPlatform(false, Params.Client);
				PlatformsSet.Add(CookPlatform);
			}
		}

		if (Params.DedicatedServer)
		{
			foreach (var ServerPlatform in Params.ServerTargetPlatforms)
			{
				var PlatformDescriptor = Params.GetCookedDataPlatformForServerTarget(ServerPlatform);
				string CookPlatform = Platform.Platforms[PlatformDescriptor].GetCookPlatform(true, false);
				PlatformsSet.Add(CookPlatform);
			}
		}

		if (!String.IsNullOrEmpty(BuildPlatform))
		{
			PlatformsSet.AddRange(BuildPlatform.Split('+'));
		}

		return PlatformsSet.ToList();
	}

	private void ImportSnapshot(ProjectParams Params)
	{

		ConfigHierarchy HostPlatformConfig = Params.EngineConfigs[HostPlatform.Current.HostEditorPlatform];
		if (!HostPlatformConfig.TryGetValueGeneric("StorageServers", "Cloud", out CloudStorageConfig cloudConfig))
		{
			throw new AutomationException("Failed to read config values for storage servers");
		}

		UnrealCloudDDCBuildIndex Index = new UnrealCloudDDCBuildIndex(cloudConfig.Host.Split(";")[0], cloudConfig.BuildsNamespace, cloudConfig.OAuthProviderIdentifier, (string app, string commandLine) => CommandUtils.Run(app, commandLine));
		DirectoryReference CookOutputDir;
		if (String.IsNullOrEmpty(Params.CookOutputDir))
		{
			CookOutputDir = new DirectoryReference(Path.Combine(Params.RawProjectPath.Directory.FullName, "Saved", "Cooked", "{Platform}"));
		}
		else
		{
			CookOutputDir = new DirectoryReference(Params.CookOutputDir);
		}

		foreach (string CookPlatform in GetCookPlatforms(Params))
		{
			string Platform;
			string Runtime;

			if (CookPlatform.EndsWith("client", StringComparison.OrdinalIgnoreCase))
			{
				Platform = CookPlatform.Substring(0, CookPlatform.Length - "client".Length);
				Runtime = "client";
			}
			else if (CookPlatform.EndsWith("server", StringComparison.OrdinalIgnoreCase))
			{
				Platform = CookPlatform.Substring(0, CookPlatform.Length - "server".Length);
				Runtime = "server";
			}
			else
			{
				Platform = CookPlatform;
				Runtime = "game";
			}

			EpicGames.ProjectStore.Build Snapshot = Index.GetBuild(Stream, ProjectPath.GetFileNameWithoutExtension(), Platform, Runtime, Int32.Parse(Changelist), ClosestSnapshot);
			if (!DryRun)
			{
				Snapshot.Import(Params.RawProjectPath, CookOutputDir, false, false);
			}
		}
	}

	private bool IsEditorRunning(ProjectParams Params)
	{
		Process[] ProcessArr = Process.GetProcesses();
		Regex EditorRegex = new Regex(@"UnrealEditor.*\.exe");
		foreach (var process in ProcessArr)
		{
			// Skip the system and the idle process
			if (process.Id == 0 || process.Id == 4)
			{
				continue;
			}

			try
			{
				if (process.HasExited)
				{
					continue;
				}

				FileReference ProcessFile = new FileReference(process.MainModule.FileName);
				string ProcessFileName = ProcessFile.GetFileName();
				if (EditorRegex.Match(ProcessFileName).Success)
				{
					return ProcessFile.IsUnderDirectory(Unreal.EngineDirectory);
				}
			}
			catch (Win32Exception)
			{
				// A Win32 exception means that this is a system process without any Modules
				continue;
			}
		}

		return false;
	}

	private ICloudStorage CloudStorageService;

	//string _BuildType;
	BuildTypeEnum? _BuildType;
	private BuildTypeEnum? BuildType
	{
		get
		{
			if (_BuildType == null)
			{
				String CmdBuildType = ParseOptionalStringParam("BuildType");
				if (String.IsNullOrEmpty(CmdBuildType))
				{
					_BuildType = BuildTypeEnum.Unspecified;
				}
				else if (!Enum.TryParse(CmdBuildType, true, out BuildTypeEnum OutBuildType))
				{
					EnumDescription BuildTypeDescriptions = EnumExtensions.ConvertEnumWithDescription<BuildTypeEnum>();
					int BuildTypeIndex = BuildTypeDescriptions.Descriptions.IndexOf(CmdBuildType);
					if (BuildTypeIndex < 0)
					{
						String PossibleValues = String.Join("', '", Enum.GetNames(typeof(BuildTypeEnum)));
						throw new AutomationException("Unrecognized build type {0}. Possible build types are '{1}'.", CmdBuildType, PossibleValues);
					}
					else
					{
						_BuildType = (BuildTypeEnum)BuildTypeIndex;
					}
				}
				else
				{
					//_BuildType = EnumExtensions.GetEnumNameOrDescription(OutBuildType);
					_BuildType = OutBuildType;
				}
			}

			return _BuildType;
		}
	}

	string Stream;
	string Changelist;
	string Buildgroup = null;
	bool IsPreflight = false;
	FileReference ProjectPath;
	string BuildPlatform;
	bool DownloadServer;
	bool DownloadClient;
	bool DownloadGame;
	bool DownloadSnapshot;
	bool ClosestSnapshot;
	bool SkipContentInStaged;

	bool SyncUGS;
	bool DownloadBuild;
	bool DryRun;

	FileReference UGSExePath;
	FileReference ZenExePath;
	// TODO: Add support for build config once established how PL2 wants to pass in build configs. Currently it uses build platform which contains the full set: platform, config, executable type
	BuildVariant BuildConfigType;
}
