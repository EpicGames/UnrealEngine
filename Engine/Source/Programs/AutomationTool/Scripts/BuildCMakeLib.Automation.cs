// Copyright Epic Games, Inc. All Rights Reserved.
using AutomationTool;
using System;
using System.Collections.Generic;
using System.IO;
using System.Linq;
using System.Reflection;
using System.Text;
using EpicGames.Core;
using UnrealBuildTool;
using UnrealBuildBase;
using Microsoft.Extensions.Logging;
using System.Runtime.InteropServices;

// TODO Currently this only supports one lib and one platform at a time.
// The reason for this is that the the library version and additional arguments (which are per platform) is passed on the command line.
// This could be improved by loading the configuration (per lib, and per lib+platform) for each lib, either from a .cs or an .ini and aligning those values with the Build.cs.
// Additionally this is adapted from the BuildPhysX automation script, so some of this could be aligned there as well.
[Help("Builds a third party library using the CMake build system.")]
[Help("TargetLibRoot", "Override the path to the library's root dir.")]
[Help("TargetLib", "Specify the target library to build.")]
[Help("TargetLibVersion", "Specify the target library version to build.")]
[Help("TargetLibSourcePath", "Override the path to source, if external to the engine. (eg. -TargetLibSourcePath=path). Default is empty.")]
[Help("TargetPlatform", "Specify the name of the target platform to build (eg. -TargetPlatform=IOS).")]
[Help("TargetArchitecture", "Specify the name of the target architecture to build (eg. -TargetArchitecture=x86_64).")]
[Help("TargetConfigs", "Specify a list of configurations to build, separated by '+' characters (eg. -TargetConfigs=release+debug). Default is release+debug.")]
[Help("BinOutputPath", "Override the path to output binaries to. (eg. -BinOutputPath=bin). Default is empty.")]
[Help("LibOutputPath", "Override the path to output libraries to. (eg. -LibOutputPath=lib). Default is empty.")]
[Help("CMakeGenerator", "Specify the CMake generator to use.")]
[Help("CMakeProjectIncludeFile", "Specify the name of the CMake project include file to use, first looks in current directory then looks in global directory.")]
[Help("CMakeAdditionalArguments", "Specify the additional arguments to pass to CMake when generating the build system.")]
[Help("MakeTarget", "Override the target to pass to make.")]
[Help("SkipCreateChangelist", "Do not create a P4 changelist for source or libs. If this argument is not supplied source and libs will be added to a Perforce changelist.")]
[Help("SkipSubmit", "Do not perform P4 submit of source or libs. If this argument is not supplied source and libs will be automatically submitted to Perforce. If SkipCreateChangelist is specified, this argument applies by default.")]
[Help("RoboMerge", "Which RoboMerge action to apply to the submission. If we're skipping submit, this is not used.")]
[RequireP4]
[DoesNotNeedP4CL]
public sealed class BuildCMakeLib : BuildCommand
{
	public class TargetLib
	{
		public string Name = "";
		public string Version = "";
		public string RootDir = "";
		public string SourcePath = "";
		public string BinOutputPath = "";
		public string LibOutputPath = "";
		public string CMakeProjectIncludeFile = "";
		public string CMakeAdditionalArguments = "";
		public string MakeTarget = "";

		public virtual Dictionary<string, string> BuildMap => new Dictionary<string, string>(StringComparer.OrdinalIgnoreCase)
		{
			{ "debug",   "Debug"   },
			{ "release", "Release" }
		};

		public virtual Dictionary<string, string> BuildSuffix => new Dictionary<string, string>(StringComparer.OrdinalIgnoreCase)
		{
			{ "debug",   "" },
			{ "release", "" }
		};

		public static DirectoryReference ThirdPartySourceDirectory = DirectoryReference.Combine(Unreal.RootDirectory, "Engine", "Source", "ThirdParty");

		public DirectoryReference GetLibSourceDirectory()
		{
			if (string.IsNullOrEmpty(SourcePath))
			{
				return DirectoryReference.Combine(ThirdPartySourceDirectory, Name, Version);
			}
			else
			{
				return new DirectoryReference(SourcePath);
			}
		}

		public override string ToString() => Name;
	}

	public static void MakeFreshDirectoryIfRequired(DirectoryReference Directory)
	{
		if (!DirectoryReference.Exists(Directory))
		{
			DirectoryReference.CreateDirectory(Directory);
		}
		else
		{
			InternalUtils.SafeDeleteDirectory(Directory.FullName);
			DirectoryReference.CreateDirectory(Directory);
		}
	}

	public abstract class TargetPlatform : CommandUtils
	{
		public static DirectoryReference CMakeRootDirectory = DirectoryReference.Combine(Unreal.RootDirectory, "Engine", "Extras", "ThirdPartyNotUE", "CMake");
		public static DirectoryReference MakeRootDirectory = DirectoryReference.Combine(Unreal.RootDirectory, "Engine", "Extras", "ThirdPartyNotUE", "GNU_Make", "make-3.81");

		private DirectoryReference PlatformEngineRoot => IsPlatformExtension
			? DirectoryReference.Combine(Unreal.RootDirectory, "Engine", "Platforms", PlatformOrGroupName)
			: DirectoryReference.Combine(Unreal.RootDirectory, "Engine");

		private DirectoryReference GetTargetLibRootDirectory(TargetLib TargetLib)
		{
			if (!string.IsNullOrEmpty(TargetLib.RootDir))
			{
				return new DirectoryReference(TargetLib.RootDir);
			}
			return DirectoryReference.Combine(PlatformEngineRoot, "Source", "ThirdParty", TargetLib.Name, TargetLib.Version);
		}

		private DirectoryReference GetTargetLibBaseRootDirectory(TargetLib TargetLib)
		{
			if (!string.IsNullOrEmpty(TargetLib.RootDir))
			{
				return new DirectoryReference(TargetLib.RootDir);
			}
			return DirectoryReference.Combine(Unreal.RootDirectory, "Engine", "Source", "ThirdParty", TargetLib.Name, TargetLib.Version);
		}

		private DirectoryReference GetTargetLibBuildScriptDirectory(TargetLib TargetLib)
		{
			// Some libraries use BuildForUE4 instead of BuildForUE, check this here
			DirectoryReference BuildForUEDirectory = DirectoryReference.Combine(GetTargetLibRootDirectory(TargetLib), "BuildForUE");
			if (!DirectoryReference.Exists(BuildForUEDirectory))
			{
				// If not available then check BuildForUE4
				BuildForUEDirectory = DirectoryReference.Combine(GetTargetLibRootDirectory(TargetLib), "BuildForUE4");
			}

			return BuildForUEDirectory;
		}

		private DirectoryReference GetTargetLibBaseBuildScriptDirectory(TargetLib TargetLib)
		{
			// Some libraries use BuildForUE4 instead of BuildForUE, check this here
			DirectoryReference BuildForUEDirectory = DirectoryReference.Combine(GetTargetLibBaseRootDirectory(TargetLib), "BuildForUE");
			if (!DirectoryReference.Exists(BuildForUEDirectory))
			{
				// If not available then check BuildForUE4
				BuildForUEDirectory = DirectoryReference.Combine(GetTargetLibBaseRootDirectory(TargetLib), "BuildForUE4");
			}

			return BuildForUEDirectory;
		}

		protected DirectoryReference GetTargetLibPlatformCMakeDirectory(TargetLib TargetLib)
		{
			// Possible "standard" locations for the CMakesLists.txt are BuildForUE/Platform, BuildForUE or the source root

			// First check for an overridden CMakeLists.txt in the BuildForUE/Platform directory
			DirectoryReference CMakeDirectory = GetTargetLibBuildScriptDirectory(TargetLib);
			if (!FileReference.Exists(FileReference.Combine(CMakeDirectory, IsPlatformExtension ? "" : PlatformOrGroupName, "CMakeLists.txt")))
			{
				// If not available then check BuildForUE
				CMakeDirectory = GetTargetLibBaseBuildScriptDirectory(TargetLib);
				if (!FileReference.Exists(FileReference.Combine(CMakeDirectory, "CMakeLists.txt")))
				{
					// If not available then check the lib source root
					CMakeDirectory = TargetLib.GetLibSourceDirectory();
				}
				if (!FileReference.Exists(FileReference.Combine(CMakeDirectory, "CMakeLists.txt")))
				{
					// If not available then check the cmake directory
					CMakeDirectory = DirectoryReference.Combine(TargetLib.GetLibSourceDirectory(), "cmake");
				}
				if (!FileReference.Exists(FileReference.Combine(CMakeDirectory, "CMakeLists.txt")))
				{
					throw new AutomationException("No CMakeLists.txt found to build (in {0}, {1}, {2}, or {3}).",
						GetTargetLibBuildScriptDirectory(TargetLib),
						GetTargetLibBaseBuildScriptDirectory(TargetLib),
						TargetLib.GetLibSourceDirectory(),
						DirectoryReference.Combine(TargetLib.GetLibSourceDirectory(), "cmake"));
				}
			}

			return CMakeDirectory;
		}

		protected DirectoryReference GetProjectsDirectory(TargetLib TargetLib, string TargetConfiguration) =>
			DirectoryReference.Combine(GetTargetLibRootDirectory(TargetLib), "Intermediate",
				IsPlatformExtension ? "" : PlatformOrGroupName,
				VariantDirectory ?? "",
				SeparateProjectPerConfig ? TargetLib.BuildMap[TargetConfiguration] : "");

		protected FileReference GetToolchainPath(TargetLib TargetLib, string TargetConfiguration)
		{
			string ToolchainName = GetToolchainName(TargetLib, TargetConfiguration);

			if (ToolchainName == null)
			{
				return null;
			}
			else if (File.Exists(ToolchainName))
			{
				return new FileReference(ToolchainName);
			}

			// First check for an overriden toolchain in the BuildForUE/Platform directory
			FileReference ToolChainPath = FileReference.Combine(GetTargetLibBuildScriptDirectory(TargetLib), IsPlatformExtension ? "" : PlatformOrGroupName, ToolchainName);
			if (!FileReference.Exists(ToolChainPath))
			{
				// If not available then use the top level toolchain path
				ToolChainPath = FileReference.Combine(PlatformEngineRoot, "Source", "ThirdParty", "CMake", "PlatformScripts", IsPlatformExtension ? "" : PlatformOrGroupName, ToolchainName);
			}

			return ToolChainPath;
		}

		protected FileReference GetProjectIncludePath(TargetLib TargetLib, string TargetConfiguration)
		{
			return FileReference.Combine(GetTargetLibBuildScriptDirectory(TargetLib), IsPlatformExtension ? "" : PlatformOrGroupName, TargetLib.CMakeProjectIncludeFile);
		}

		protected DirectoryReference GetOutputLibraryDirectory(TargetLib TargetLib, string TargetConfiguration)
		{
			return DirectoryReference.Combine(GetTargetLibRootDirectory(TargetLib), TargetLib.LibOutputPath, IsPlatformExtension ? "" : PlatformOrGroupName, VariantDirectory ?? "", TargetConfiguration ?? "");
		}

		protected DirectoryReference GetOutputBinaryDirectory(TargetLib TargetLib, string TargetConfiguration)
		{
			return DirectoryReference.Combine(GetTargetLibRootDirectory(TargetLib), TargetLib.BinOutputPath, IsPlatformExtension ? "" : PlatformOrGroupName, VariantDirectory ?? "", TargetConfiguration ?? "");
		}

		public abstract string PlatformOrGroupName { get; }

		public virtual bool HasBinaries => false;
		public virtual bool UseResponseFiles => false;
		public virtual string TargetBuildPlatform => "";

		public virtual string DebugDatabaseExtension => null;
		public virtual string DynamicLibraryExtension => null;
		public virtual string StaticLibraryExtension => null;
		public virtual string SymbolExtension => null;

		public abstract bool IsPlatformExtension { get; }
		public abstract bool SeparateProjectPerConfig { get; }
		public abstract string CMakeGeneratorName { get; }

		public virtual string VariantDirectory => null;

		public virtual string FriendlyName
		{
			get { return VariantDirectory == null ? PlatformOrGroupName : string.Format("{0}-{1}", PlatformOrGroupName, VariantDirectory); }
		}

		public virtual string CMakeCommand =>
			BuildHostPlatform.Current.Platform.IsInGroup(UnrealPlatformGroup.Microsoft)
			? FileReference.Combine(CMakeRootDirectory, "bin", "cmake.exe").FullName
			: BuildHostPlatform.Current.Platform == UnrealTargetPlatform.Mac
			? FileReference.Combine(CMakeRootDirectory, "bin", "cmake").FullName
			: "cmake";

		public virtual string MakeCommand => null;

		public virtual string GetToolchainName(TargetLib TargetLib, string TargetConfiguration) => FriendlyName + ".cmake";

		public virtual string GetCMakeSetupArguments(TargetLib TargetLib, string TargetConfiguration)
		{
			DirectoryReference CMakeTargetDirectory = GetProjectsDirectory(TargetLib, TargetConfiguration);

			string Args = "-B \"" + CMakeTargetDirectory.FullName + "\"";
			Args += " -S \"" + GetTargetLibPlatformCMakeDirectory(TargetLib).FullName + "\"";
			Args += " -G \"" + CMakeGeneratorName + "\"";

			if (SeparateProjectPerConfig)
			{
				Args += " -DCMAKE_BUILD_TYPE=\"" + TargetConfiguration + "\"";
			}

			if (MakeCommand != null)
			{
				string ResolvedMakeCommand = WhichApp(MakeCommand);
				if (ResolvedMakeCommand != null)
				{
					Args += " -DCMAKE_MAKE_PROGRAM=\"" + ResolvedMakeCommand + "\"";
				}
			}

			FileReference ToolchainPath = GetToolchainPath(TargetLib, TargetConfiguration);
			if (ToolchainPath != null && FileReference.Exists(ToolchainPath))
			{
				Args += " -DCMAKE_TOOLCHAIN_FILE=\"" + ToolchainPath.FullName + "\"";
			}

			FileReference ProjectIncludePath = GetProjectIncludePath(TargetLib, TargetConfiguration);
			if (ProjectIncludePath != null && FileReference.Exists(ProjectIncludePath))
			{
				Args += " -DCMAKE_PROJECT_INCLUDE_FILE=\"" + ProjectIncludePath.FullName + "\"";
			}

			Args += " -DCMAKE_ARCHIVE_OUTPUT_DIRECTORY=\"" + GetOutputLibraryDirectory(TargetLib, TargetConfiguration) + "\"";
			Args += " -DCMAKE_LIBRARY_OUTPUT_DIRECTORY=\"" + GetOutputBinaryDirectory(TargetLib, TargetConfiguration) + "\"";
			Args += " -DCMAKE_RUNTIME_OUTPUT_DIRECTORY=\"" + GetOutputBinaryDirectory(TargetLib, TargetConfiguration) + "\"";

			if (TargetLib.CMakeAdditionalArguments != null)
			{
				Args += " " + TargetLib.CMakeAdditionalArguments.Replace("${TARGET_CONFIG}", TargetConfiguration ?? "");
			}

			return Args;
		}

		// Returns extra environment variables passed to every cmake invocation, or null for none.
		protected virtual Dictionary<string, string> GetCMakeEnvironment() => null;

		public virtual string GetCMakeBuildArguments(TargetLib TargetLib, string TargetConfiguration)
		{
			StringBuilder CMakeArgs = new StringBuilder();

			DirectoryReference ConfigDirectory = GetProjectsDirectory(TargetLib, TargetConfiguration);
			CMakeArgs.Append("--build \"").Append(ConfigDirectory).Append("\"");

			if (!String.IsNullOrEmpty(TargetConfiguration))
			{
				CMakeArgs.Append(" --config \"").Append(TargetConfiguration).Append("\"");
			}

			if (!String.IsNullOrEmpty(TargetLib.MakeTarget))
			{
				CMakeArgs.Append(" --target \"").Append(TargetLib.MakeTarget).Append("\"");
			}

			CMakeArgs.Append(" --parallel ").Append(Environment.ProcessorCount);

			return CMakeArgs.ToString();
		}

		public virtual IEnumerable<FileReference> EnumerateOutputFiles(DirectoryReference BaseDir, string SearchPrefix, TargetLib TargetLib)
		{
			if (!DirectoryReference.Exists(BaseDir))
			{
				yield break;
			}

			foreach (FileReference File in DirectoryReference.EnumerateFiles(BaseDir, SearchPrefix))
			{
				var FileNameUpper = File.GetFileName().ToUpper();
				if (FileNameUpper.Contains(TargetLib.Name.ToUpper()))
				{
					yield return File;
				}
			}
		}

		public virtual IEnumerable<FileReference> EnumerateOutputFiles(TargetLib TargetLib, string TargetConfiguration)
		{
			string SearchPrefix = "*" + TargetLib.BuildSuffix[TargetConfiguration] + ".";

			IEnumerable<FileReference> Results = Enumerable.Empty<FileReference>();

			// Scan static libraries directory
			if (StaticLibraryExtension != null)
			{
				DirectoryReference OutputLibraryDirectory = GetOutputLibraryDirectory(TargetLib, TargetConfiguration);

				Results = Results.Concat(EnumerateOutputFiles(OutputLibraryDirectory, SearchPrefix + StaticLibraryExtension, TargetLib));
				if (DebugDatabaseExtension != null)
				{
					Results = Results.Concat(EnumerateOutputFiles(OutputLibraryDirectory, SearchPrefix + DebugDatabaseExtension, TargetLib));
				}
			}

			// Scan dynamic libraries directory
			if (DynamicLibraryExtension != null)
			{
				DirectoryReference OutputBinaryDirectory = GetOutputBinaryDirectory(TargetLib, TargetConfiguration);

				Results = Results.Concat(EnumerateOutputFiles(OutputBinaryDirectory, SearchPrefix + DynamicLibraryExtension, TargetLib));
				if (DebugDatabaseExtension != null)
				{
					Results = Results.Concat(EnumerateOutputFiles(OutputBinaryDirectory, SearchPrefix + DebugDatabaseExtension, TargetLib));
				}
				if (SymbolExtension != null)
				{
					Results = Results.Concat(EnumerateOutputFiles(OutputBinaryDirectory, SearchPrefix + SymbolExtension, TargetLib));
				}
			}

			return Results;
		}

		public virtual void SetupTargetLib(TargetLib TargetLib, string TargetConfiguration)
		{
			Logger.LogInformation("Building {Arg0} for {FriendlyName} ({Arg2})...", TargetLib.Name, FriendlyName, TargetConfiguration ?? "");

			DirectoryReference CMakeTargetDirectory = GetProjectsDirectory(TargetLib, TargetConfiguration);
			MakeFreshDirectoryIfRequired(CMakeTargetDirectory);

			Logger.LogInformation("Generating projects for {Arg0} for {FriendlyName}", TargetLib.Name, FriendlyName);

			string CMakeArgs = GetCMakeSetupArguments(TargetLib, TargetConfiguration);
			if (Run(CMakeCommand, CMakeArgs, Env: GetCMakeEnvironment()).ExitCode != 0)
			{
				throw new AutomationException("Unable to generate projects for {0}.", TargetLib.ToString() + ", " + FriendlyName);
			}
		}

		public virtual void BuildTargetLib(TargetLib TargetLib, string TargetConfiguration)
		{
			if (Run(CMakeCommand, GetCMakeBuildArguments(TargetLib, TargetConfiguration), Env: GetCMakeEnvironment()).ExitCode != 0)
			{
				throw new AutomationException("Unable to build target {0}, {1}.", TargetLib.ToString(), FriendlyName);
			}
		}

		public virtual void CleanupTargetLib(TargetLib TargetLib, string TargetConfiguration)
		{
			if (string.IsNullOrEmpty(TargetConfiguration))
			{
				InternalUtils.SafeDeleteDirectory(DirectoryReference.Combine(GetTargetLibRootDirectory(TargetLib), "Intermediate").FullName);
			}
			else
			{
				DirectoryReference CMakeTargetDirectory = GetProjectsDirectory(TargetLib, TargetConfiguration);
				InternalUtils.SafeDeleteDirectory(CMakeTargetDirectory.FullName);
			}
		}
	}

	public abstract class NMakeTargetPlatform : TargetPlatform
	{
		public override bool SeparateProjectPerConfig => true;

		public override string CMakeGeneratorName => "NMake Makefiles";
	}

	public abstract class MakefileTargetPlatform : TargetPlatform
	{
		public override bool SeparateProjectPerConfig => true;

		public override string CMakeGeneratorName => "Unix Makefiles";

		public override string MakeCommand =>
			BuildHostPlatform.Current.Platform.IsInGroup(UnrealPlatformGroup.Microsoft)
			? FileReference.Combine(MakeRootDirectory, "bin", "make.exe").FullName
			: BuildHostPlatform.Current.Platform.IsInGroup(UnrealPlatformGroup.Apple)
			? FileReference.Combine(MakeRootDirectory, "bin", "make").FullName
			: "make";
	}

	public abstract class NinjaBuildTargetPlatform : TargetPlatform
	{
		public override bool SeparateProjectPerConfig => true;

		public override string CMakeGeneratorName => "Ninja";

		public override string MakeCommand =>
			BuildHostPlatform.Current.Platform.IsInGroup(UnrealPlatformGroup.Microsoft)
			? FileReference.Combine(Unreal.RootDirectory, "Engine", "Extras", "ThirdPartyNotUE", "ninja-build", "ninja.exe").FullName
			: "ninja";		
	}

	public abstract class VSTargetPlatform : TargetPlatform
	{
		public override bool SeparateProjectPerConfig => false;

		// Clear value of msbuild related vars in DotNetTask.cs
		protected override Dictionary<string, string> GetCMakeEnvironment() => new()
			{
				["MSBuildSDKsPath"] = String.Empty,
				["MSBuildExtensionsPath"] = String.Empty,
				["MSBUILD_EXE_PATH"] = String.Empty,
			};
	}

	public abstract class VS2017TargetPlatform : VSTargetPlatform
	{
		public override string CMakeGeneratorName => "Visual Studio 15 2017";
	}

	public abstract class VS2019TargetPlatform : VSTargetPlatform
	{
		public override string CMakeGeneratorName => "Visual Studio 16 2019";
	}

	public abstract class VS2022TargetPlatform : VSTargetPlatform
	{
		public override string CMakeGeneratorName => "Visual Studio 17 2022";
	}

	public abstract class XcodeTargetPlatform : TargetPlatform
	{
		public override bool SeparateProjectPerConfig => false;
		public override string CMakeGeneratorName => "Xcode";
	}

	private TargetLib GetTargetLib()
	{
		TargetLib TargetLib = new TargetLib();

		TargetLib.Name = ParseParamValue("TargetLib", "");
		TargetLib.Version = ParseParamValue("TargetLibVersion", "");
		TargetLib.RootDir = ParseParamValue("TargetRootDir", "");
		TargetLib.SourcePath = ParseParamValue("TargetLibSourcePath", "");
		TargetLib.BinOutputPath = ParseParamValue("BinOutputPath", "Binaries");
		TargetLib.LibOutputPath = ParseParamValue("LibOutputPath", "");
		TargetLib.CMakeProjectIncludeFile = ParseParamValue("CMakeProjectIncludeFile", "");
		TargetLib.CMakeAdditionalArguments = ParseParamValue("CMakeAdditionalArguments", "");
		TargetLib.MakeTarget = ParseParamValue("MakeTarget", "");

		if (string.IsNullOrEmpty(TargetLib.Name) || string.IsNullOrEmpty(TargetLib.Version))
		{
			throw new AutomationException("Must specify both -TargetLib and -TargetLibVersion");
		}

		return TargetLib;
	}

	public List<string> GetTargetConfigurations()
	{
		List<string> TargetConfigs = new List<string>();

		string TargetConfigFilter = ParseParamValue("TargetConfigs", "debug+release");
		if (TargetConfigFilter != null)
		{
			foreach (string TargetConfig in TargetConfigFilter.Split(new char[] { '+' }, StringSplitOptions.RemoveEmptyEntries))
			{
				TargetConfigs.Add(TargetConfig);
			}
		}

		return TargetConfigs;
	}

	private TargetPlatform GetTargetPlatform()
	{
		// Grab all the non-abstract subclasses of TargetPlatform from the executing assembly.
		var AvailablePlatformTypes = from Assembly in ScriptManager.AllScriptAssemblies
									 from Type in Assembly.GetTypes()
									 where !Type.IsAbstract && Type.IsAssignableTo(typeof(TargetPlatform))
									 select Type;

		var PlatformTypeMap = new Dictionary<string, Type>();

		foreach (var Type in AvailablePlatformTypes)
		{
			int Index = Type.Name.IndexOf('_');
			if (Index == -1)
			{
				throw new BuildException("Invalid BuildCMakeLib target platform type found: {0}", Type);
			}

			PlatformTypeMap.Add(Type.Name, Type);
		}

		TargetPlatform TargetPlatform = null;

		// TODO For now the CMakeGenerateor and TargetPlatform are combined.
		string TargetPlatformName = ParseParamValue("TargetPlatform", "");
		string TargetArchitecture = ParseParamValue("TargetArchitecture", null);
		string CMakeGenerator = ParseParamValue("CMakeGenerator", "");
		var SelectedPlatform = string.Format("{0}TargetPlatform_{1}", CMakeGenerator, TargetPlatformName);

		if (!PlatformTypeMap.ContainsKey(SelectedPlatform))
		{
			throw new BuildException("Unknown BuildCMakeLib target platform specified: {0}", SelectedPlatform);
		}

		var SelectedType = PlatformTypeMap[SelectedPlatform];
		var Constructors = SelectedType.GetConstructors();
		if (Constructors.Length != 1)
		{
			throw new BuildException("BuildCMakeLib build platform implementation type \"{0}\" should have exactly one constructor.", SelectedType);
		}

		var Parameters = Constructors[0].GetParameters();
		if (Parameters.Length >= 2)
		{
			throw new BuildException("The constructor for the target platform type \"{0}\" must take exactly zero or one arguments.", TargetPlatformName);
		}

		if (Parameters.Length == 1 && Parameters[0].ParameterType != typeof(string))
		{
			throw new BuildException("The constructor for the target platform type \"{0}\" has an invalid argument type. The type must be a string.", TargetPlatformName);
		}

		var Args = new object[Parameters.Length];
		if (Args.Length > 0)
		{
			if (!string.IsNullOrEmpty(TargetArchitecture))
			{
				Args[0] = TargetArchitecture;
			}
			else if (Parameters[0].HasDefaultValue)
			{
				Args[0] = Parameters[0].DefaultValue;
			}
			else
			{
				throw new BuildException("The target architecture is a required argument for the target platform \"{0}\".", TargetPlatformName);
			}
		}

		TargetPlatform = (TargetPlatform)Activator.CreateInstance(SelectedType, Args);
		if (TargetPlatform == null)
		{
			throw new BuildException("The target platform \"{0}\" could not be constructed.", SelectedPlatform);
		}

		return TargetPlatform;
	}

	private static string RemoveOtherMakeAndCygwinFromPath(string WindowsPath)
	{
		string[] PathComponents = WindowsPath.Split(';');
		string NewPath = "";
		foreach(string PathComponent in PathComponents)
		{
			// Everything that contains /bin or /sbin is suspicious, check if it has make in it
			if (PathComponent.Contains("\\bin") || PathComponent.Contains("/bin") || PathComponent.Contains("\\sbin") || PathComponent.Contains("/sbin"))
			{
				if (File.Exists(PathComponent + "/make.exe") || File.Exists(PathComponent + "make.exe") || File.Exists(PathComponent + "/cygwin1.dll"))
				{
					Logger.LogInformation("Removing {PathComponent} from PATH since it contains possibly colliding make.exe", PathComponent);
					continue;
				}
			}

			NewPath = NewPath + ';' + PathComponent + ';';
		}

		return NewPath;
	}

	private void SetupBuildEnvironment()
	{
		// ================================================================================
		// ThirdPartyNotUE
		// NOTE: these are Windows executables
		if (BuildHostPlatform.Current.Platform.IsInGroup(UnrealPlatformGroup.Microsoft))
		{
			string CMakePath = DirectoryReference.Combine(TargetPlatform.CMakeRootDirectory, "bin").FullName;
			string MakePath = DirectoryReference.Combine(TargetPlatform.MakeRootDirectory, "bin").FullName;

			string PrevPath = Environment.GetEnvironmentVariable("PATH");
			// mixing bundled make and cygwin make is no good. Try to detect and remove cygwin paths.
			string PathWithoutCygwin = RemoveOtherMakeAndCygwinFromPath(PrevPath);
			Environment.SetEnvironmentVariable("PATH", CMakePath + ";" + MakePath + ";" + PathWithoutCygwin);
			Environment.SetEnvironmentVariable("PATH", CMakePath + ";" + MakePath + ";" + Environment.GetEnvironmentVariable("PATH"));
			Logger.LogInformation("set {Arg0}={Arg1}", "PATH", Environment.GetEnvironmentVariable("PATH"));
		}
	}

	public override void ExecuteBuild()
	{
		bool bAutoCreateChangelist = true;
		if (ParseParam("SkipCreateChangelist"))
		{
			bAutoCreateChangelist = false;
		}

		bool bAutoSubmit = bAutoCreateChangelist;
		if (ParseParam("SkipSubmit"))
		{
			bAutoSubmit = false;
		}

		// if we don't pass anything, we'll just merge by default
		string RobomergeCommand = ParseParamValue("Robomerge", "").ToLower();
		if (!string.IsNullOrEmpty(RobomergeCommand))
		{
			// for merge default action, add flag to make sure buildmachine commit isn't skipped
			if (RobomergeCommand == "merge")
			{
				RobomergeCommand = "#robomerge[all] #DisregardExcludedAuthors";
			}
			// otherwise add hashtags
			else if (RobomergeCommand == "ignore")
			{
				RobomergeCommand = "#robomerge #ignore";
			}
			else if (RobomergeCommand == "null")
			{
				RobomergeCommand = "#robomerge #null";
			}
			// otherwise the submit will likely fail.
			else
			{
				throw new AutomationException("Invalid Robomerge param passed in {0}.  Must be \"merge\", \"null\", or \"ignore\"", RobomergeCommand);
			}
		}

		SetupBuildEnvironment();

		TargetPlatform Platform = GetTargetPlatform();

		TargetLib TargetLib = GetTargetLib();

		List<string> TargetConfigurations = GetTargetConfigurations();

		HashSet<FileReference> FilesToReconcile = new HashSet<FileReference>();

		try
		{
			if (Platform.SeparateProjectPerConfig)
			{
				foreach (string TargetConfiguration in TargetConfigurations)
				{
					Platform.SetupTargetLib(TargetLib, TargetConfiguration);
				}
			}
			else
			{
				Platform.SetupTargetLib(TargetLib, null);
			}

			foreach (string TargetConfiguration in TargetConfigurations)
			{
				foreach (FileReference FileToDelete in Platform.EnumerateOutputFiles(TargetLib, TargetConfiguration).Distinct())
				{
					FilesToReconcile.Add(FileToDelete);

					// Also clean the output files
					InternalUtils.SafeDeleteFile(FileToDelete.FullName);
				}
			}

			foreach (string TargetConfiguration in TargetConfigurations)
			{
				Platform.BuildTargetLib(TargetLib, TargetConfiguration);
			}
		}
		finally
		{
			Platform.CleanupTargetLib(TargetLib, null);
		}

		const int InvalidChangeList = -1;
		int P4ChangeList = InvalidChangeList;

		if (bAutoCreateChangelist)
		{
			string LibDeploymentDesc = TargetLib.Name + " " + Platform.FriendlyName;

			var Builder = new StringBuilder();
			Builder.AppendFormat("BuildCMakeLib.Automation: Deploying {0} libs.{1}", LibDeploymentDesc, Environment.NewLine);
			Builder.AppendLine("#rb none");
			Builder.AppendLine("#lockdown Nick.Penwarden");
			Builder.AppendLine("#tests none");
			Builder.AppendLine("#jira none");
			Builder.AppendLine("#okforgithub ignore");
			if (!string.IsNullOrEmpty(RobomergeCommand))
			{
				Builder.AppendLine(RobomergeCommand);
			}

			P4ChangeList = P4.CreateChange(P4Env.Client, Builder.ToString());
		}

		if (P4ChangeList != InvalidChangeList)
		{
			foreach (FileReference FileToReconcile in FilesToReconcile)
			{
				P4.Reconcile(P4ChangeList, FileToReconcile.FullName);
			}

			if (bAutoSubmit)
			{
				if (!P4.TryDeleteEmptyChange(P4ChangeList))
				{
					Logger.LogInformation("{Text}", "Submitting changelist " + P4ChangeList.ToString());
					int SubmittedChangeList = InvalidChangeList;
					P4.Submit(P4ChangeList, out SubmittedChangeList);
				}
				else
				{
					Logger.LogInformation("Nothing to submit!");
				}
			}
		}
	}
}

class VS2017TargetPlatform_Win64 : BuildCMakeLib.VS2017TargetPlatform
{
	public override string PlatformOrGroupName => nameof(UnrealTargetPlatform.Win64);
	public override string DebugDatabaseExtension => "pdb";
	public override string DynamicLibraryExtension => "dll";
	public override string StaticLibraryExtension => "lib";
	public override bool IsPlatformExtension => false;
}

class VS2019TargetPlatform_Win64 : BuildCMakeLib.VS2019TargetPlatform
{
	public override string PlatformOrGroupName => nameof(UnrealTargetPlatform.Win64);
	public override string DebugDatabaseExtension => "pdb";
	public override string DynamicLibraryExtension => "dll";
	public override string StaticLibraryExtension => "lib";
	public override string VariantDirectory => Architecture == "Win64" ? "" : Architecture.ToLower();
	public override bool IsPlatformExtension => false;

	private readonly string Architecture;

	public VS2019TargetPlatform_Win64(string Architecture = "Win64")
	{
		this.Architecture = Architecture;
	}

	public override string GetCMakeSetupArguments(BuildCMakeLib.TargetLib TargetLib, string TargetConfiguration)
	{
		return base.GetCMakeSetupArguments(TargetLib, TargetConfiguration)
			+ (Architecture == "Win64" ? "" : string.Format(" -A {0}", Architecture));
	}
}

class VS2022TargetPlatform_Win64 : BuildCMakeLib.VS2022TargetPlatform
{
	public override string PlatformOrGroupName => nameof(UnrealTargetPlatform.Win64);
	public override string DebugDatabaseExtension => "pdb";
	public override string DynamicLibraryExtension => "dll";
	public override string StaticLibraryExtension => "lib";
	public override string VariantDirectory => Architecture == "Win64" ? "" : Architecture.ToLower();
	public override bool IsPlatformExtension => false;

	private readonly string Architecture;

	public VS2022TargetPlatform_Win64(string Architecture = "Win64")
	{
		this.Architecture = Architecture;
	}

	public override string GetCMakeSetupArguments(BuildCMakeLib.TargetLib TargetLib, string TargetConfiguration)
	{
		return base.GetCMakeSetupArguments(TargetLib, TargetConfiguration)
			+ (Architecture == "Win64" ? "" : string.Format(" -A {0}", Architecture));
	}
}

class NMakeTargetPlatform_Win64 : BuildCMakeLib.NMakeTargetPlatform
{
	public override string PlatformOrGroupName => nameof(UnrealTargetPlatform.Win64);
	public override string DebugDatabaseExtension => "pdb";
	public override string DynamicLibraryExtension => "dll";
	public override string StaticLibraryExtension => "lib";
	public override bool IsPlatformExtension => false;
}

class MakefileTargetPlatform_Win64 : BuildCMakeLib.MakefileTargetPlatform
{
	public override string PlatformOrGroupName => nameof(UnrealTargetPlatform.Win64);
	public override string DebugDatabaseExtension => "pdb";
	public override string DynamicLibraryExtension => "dll";
	public override string StaticLibraryExtension => "lib";
	public override bool IsPlatformExtension => false;
}

class NinjaBuildTargetPlatform_Win64 : BuildCMakeLib.NinjaBuildTargetPlatform
{
	public override string PlatformOrGroupName => nameof(UnrealTargetPlatform.Win64);
	public override string DebugDatabaseExtension => "pdb";
	public override string DynamicLibraryExtension => "dll";
	public override string StaticLibraryExtension => "lib";
	public override bool IsPlatformExtension => false;
}

class NMakeTargetPlatform_Unix : BuildCMakeLib.NMakeTargetPlatform
{
	public override string PlatformOrGroupName => nameof(UnrealPlatformGroup.Unix);
	public override string StaticLibraryExtension => "a";
	public override string VariantDirectory => Architecture;
	public override bool IsPlatformExtension => false;

	private readonly string Architecture;

	public NMakeTargetPlatform_Unix(string Architecture)
	{
		this.Architecture = Architecture;
	}
}

class MakefileTargetPlatform_Unix : BuildCMakeLib.MakefileTargetPlatform
{
	public override string PlatformOrGroupName => nameof(UnrealPlatformGroup.Unix);
	public override string StaticLibraryExtension => "a";
	public override string VariantDirectory => Architecture;
	public override bool IsPlatformExtension => false;

	private readonly string Architecture;

	public MakefileTargetPlatform_Unix(string Architecture)
	{
		this.Architecture = Architecture;
	}
}

class NinjaBuildTargetPlatform_Unix : BuildCMakeLib.NinjaBuildTargetPlatform
{
	public override string PlatformOrGroupName => nameof(UnrealPlatformGroup.Unix);
	public override string StaticLibraryExtension => "a";
	public override string VariantDirectory => Architecture;
	public override bool IsPlatformExtension => false;

	private readonly string Architecture;

	public NinjaBuildTargetPlatform_Unix(string Architecture)
	{
		this.Architecture = Architecture;
	}
}

class NMakeTargetPlatform_Linux : NMakeTargetPlatform_Unix
{
	public override string PlatformOrGroupName => nameof(UnrealTargetPlatform.Linux);

	public NMakeTargetPlatform_Linux(string Architecture)
		: base(Architecture)
	{
	}
}

class MakefileTargetPlatform_Linux : MakefileTargetPlatform_Unix
{
	public override string PlatformOrGroupName => nameof(UnrealTargetPlatform.Linux);

	public MakefileTargetPlatform_Linux(string Architecture)
		: base(Architecture)
	{
	}
}

class NinjaBuildTargetPlatform_Linux : NinjaBuildTargetPlatform_Unix
{
	public override string PlatformOrGroupName => nameof(UnrealTargetPlatform.Linux);

	public NinjaBuildTargetPlatform_Linux(string Architecture)
		: base(Architecture)
	{
	}
}

class MakefileTargetPlatform_LinuxArm64 : MakefileTargetPlatform_Unix
{
	public override string PlatformOrGroupName => nameof(UnrealTargetPlatform.LinuxArm64);

	public MakefileTargetPlatform_LinuxArm64(string Architecture)
		: base(Architecture)
	{
	}
}

class NinjaBuildTargetPlatform_LinuxArm64 : NinjaBuildTargetPlatform_Unix
{
	public override string PlatformOrGroupName => nameof(UnrealTargetPlatform.LinuxArm64);

	public NinjaBuildTargetPlatform_LinuxArm64(string Architecture)
		: base(Architecture)
	{
	}
}

class XcodeTargetPlatform_Mac : BuildCMakeLib.XcodeTargetPlatform
{
	public override string PlatformOrGroupName => nameof(UnrealTargetPlatform.Mac);
	public override string StaticLibraryExtension => "a";
	public override bool IsPlatformExtension => false;

	private readonly string Architecture;

	public XcodeTargetPlatform_Mac(string Architecture)
	{
		this.Architecture = Architecture;
	}

	public override string GetCMakeSetupArguments(BuildCMakeLib.TargetLib TargetLib, string TargetConfiguration)
	{
		return base.GetCMakeSetupArguments(TargetLib, TargetConfiguration)
			+ string.Format(" -DCMAKE_OSX_ARCHITECTURES={0}", Architecture);
	}
}

class MakefileTargetPlatform_Mac : BuildCMakeLib.MakefileTargetPlatform
{
	public override string PlatformOrGroupName => nameof(UnrealTargetPlatform.Mac);
	public override string StaticLibraryExtension => "a";
	public override string VariantDirectory => Architecture;
	public override bool IsPlatformExtension => false;

	private readonly string Architecture;

	public MakefileTargetPlatform_Mac(string Architecture)
	{
		this.Architecture = Architecture;
	}

	public override string GetCMakeSetupArguments(BuildCMakeLib.TargetLib TargetLib, string TargetConfiguration)
	{
		return base.GetCMakeSetupArguments(TargetLib, TargetConfiguration)
			+ string.Format(" -DCMAKE_OSX_ARCHITECTURES={0}", Architecture);
	}
}

class NinjaBuildTargetPlatform_Mac : BuildCMakeLib.NinjaBuildTargetPlatform
{
	public override string PlatformOrGroupName => nameof(UnrealTargetPlatform.Mac);
	public override string StaticLibraryExtension => "a";
	public override string VariantDirectory => Architecture;
	public override bool IsPlatformExtension => false;

	private readonly string Architecture;

	public NinjaBuildTargetPlatform_Mac(string Architecture)
	{
		this.Architecture = Architecture;
	}

	public override string GetCMakeSetupArguments(BuildCMakeLib.TargetLib TargetLib, string TargetConfiguration)
	{
		return base.GetCMakeSetupArguments(TargetLib, TargetConfiguration)
			+ string.Format(" -DCMAKE_OSX_ARCHITECTURES={0}", Architecture);
	}
}

public class XcodeTargetPlatform_IOS : BuildCMakeLib.XcodeTargetPlatform
{
	public override string PlatformOrGroupName => nameof(UnrealTargetPlatform.IOS);
	public override string StaticLibraryExtension => "a";
	public override bool IsPlatformExtension => false;
	protected virtual string CMakeSystemName => "iOS";

	public override string GetCMakeSetupArguments(BuildCMakeLib.TargetLib TargetLib, string TargetConfiguration)
	{
		return base.GetCMakeSetupArguments(TargetLib, TargetConfiguration)
			+ $" -DCMAKE_SYSTEM_NAME={CMakeSystemName}";
	}
}

public class MakefileTargetPlatform_IOS : BuildCMakeLib.MakefileTargetPlatform
{
	public override string PlatformOrGroupName => nameof(UnrealTargetPlatform.IOS);
	public override string StaticLibraryExtension => "a";
	public override string VariantDirectory => Architecture;
	public override bool IsPlatformExtension => false;
	protected virtual string CMakeSystemName => "iOS";
	protected virtual string SdkName => Architecture == "x86_64" || Architecture == "iossimulator" ? "iphonesimulator" : "iphoneos";
	protected virtual string ClangArchitecture => Architecture.Contains("x86_64") ? "x86_64" : "arm64";

	protected readonly string Architecture;

	public MakefileTargetPlatform_IOS(string Architecture)
	{
		this.Architecture = Architecture;
	}

	public override string GetCMakeSetupArguments(BuildCMakeLib.TargetLib TargetLib, string TargetConfiguration)
	{
		return base.GetCMakeSetupArguments(TargetLib, TargetConfiguration)
			+ $" -DCMAKE_SYSTEM_NAME={CMakeSystemName}"
			+ $" -DCMAKE_OSX_ARCHITECTURES={ClangArchitecture}"
			+ $" -DCMAKE_OSX_SYSROOT={GetSysRoot()}";
	}

	private string GetSysRoot()
	{
		IProcessResult Result = Run("xcrun", string.Format("-sdk {0} --show-sdk-path", SdkName));
		string SysRoot = (Result.Output ?? "").Trim();
		if (Result.ExitCode != 0 || !DirectoryExists(SysRoot))
		{
			throw new AutomationException("Failed to locate {3} SDK \"{0}\":{1}{2}", SdkName, Environment.NewLine, SysRoot, CMakeSystemName);
		}
		return SysRoot;
	}

}

public class NinjaBuildTargetPlatform_IOS : BuildCMakeLib.NinjaBuildTargetPlatform
{
	public override string PlatformOrGroupName => nameof(UnrealTargetPlatform.IOS);
	public override string StaticLibraryExtension => "a";
	public override string VariantDirectory => Architecture;
	public override bool IsPlatformExtension => false;
	protected virtual string CMakeSystemName => "iOS";
	protected virtual string SdkName => Architecture == "x86_64" || Architecture == "iossimulator" ? "iphonesimulator" : "iphoneos";
	protected virtual string ClangArchitecture => Architecture.Contains("x86_64") ? "x86_64" : "arm64";

	protected readonly string Architecture;

	public NinjaBuildTargetPlatform_IOS(string Architecture)
	{
		this.Architecture = Architecture;
	}

	public override string GetCMakeSetupArguments(BuildCMakeLib.TargetLib TargetLib, string TargetConfiguration)
	{
		return base.GetCMakeSetupArguments(TargetLib, TargetConfiguration)
			+ $" -DCMAKE_SYSTEM_NAME={CMakeSystemName}"
			+ $" -DCMAKE_OSX_ARCHITECTURES={ClangArchitecture}"
			+ $" -DCMAKE_OSX_SYSROOT={GetSysRoot()}";
	}

	private string GetSysRoot()
	{
		IProcessResult Result = Run("xcrun", string.Format("-sdk {0} --show-sdk-path", SdkName));
		string SysRoot = (Result.Output ?? "").Trim();
		if (Result.ExitCode != 0 || !DirectoryExists(SysRoot))
		{
			throw new AutomationException("Failed to locate {3} SDK \"{0}\":{1}{2}", SdkName, Environment.NewLine, SysRoot, CMakeSystemName);
		}
		return SysRoot;
	}
}

class XcodeTargetPlatform_TVOS : XcodeTargetPlatform_IOS
{
	public override string PlatformOrGroupName => nameof(UnrealTargetPlatform.TVOS);
	protected override string CMakeSystemName => "tvOS";
}

class MakefileTargetPlatform_TVOS : MakefileTargetPlatform_IOS
{
	public override string PlatformOrGroupName => nameof(UnrealTargetPlatform.TVOS);
	protected override string CMakeSystemName => "tvOS";
	protected override string SdkName => Architecture == "x86_64" || Architecture == "tvossimulator" || Architecture == "iossimulator" ? "appletvsimulator" : "appletvos";

	public MakefileTargetPlatform_TVOS(string Architecture) : base(Architecture)
	{
	}
}

class NinjaBuildTargetPlatform_TVOS : NinjaBuildTargetPlatform_IOS
{
	public override string PlatformOrGroupName => nameof(UnrealTargetPlatform.TVOS);
	protected override string CMakeSystemName => "tvOS";
	protected override string SdkName => Architecture == "x86_64" || Architecture == "tvossimulator" || Architecture == "iossimulator" ? "appletvsimulator" : "appletvos";

	public NinjaBuildTargetPlatform_TVOS(string Architecture) : base(Architecture)
	{
	}
}

class NMakeTargetPlatform_Android : BuildCMakeLib.NMakeTargetPlatform
{
	public override string PlatformOrGroupName => nameof(UnrealTargetPlatform.Android);
	public override string StaticLibraryExtension => "a";
	public override string VariantDirectory => Architecture;
	public override bool IsPlatformExtension => false;

	private readonly string Architecture;

	public NMakeTargetPlatform_Android(string Architecture)
	{
		this.Architecture = Architecture;
	}
}

class MakefileTargetPlatform_Android : BuildCMakeLib.MakefileTargetPlatform
{
	public override string PlatformOrGroupName => nameof(UnrealTargetPlatform.Android);
	public override string StaticLibraryExtension => "a";
	public override string VariantDirectory => Architecture;
	public override bool IsPlatformExtension => false;

	private readonly string Architecture;

	public MakefileTargetPlatform_Android(string Architecture)
	{
		this.Architecture = Architecture;
	}
}

class NinjaBuildTargetPlatform_Android : BuildCMakeLib.NinjaBuildTargetPlatform
{
	public override string PlatformOrGroupName => nameof(UnrealTargetPlatform.Android);
	public override string StaticLibraryExtension => "a";
	public override string VariantDirectory => Architecture;
	public override bool IsPlatformExtension => false;

	private readonly string Architecture;

	public NinjaBuildTargetPlatform_Android(string Architecture)
	{
		this.Architecture = Architecture;
	}
}

class VS2019TargetPlatform_Android : BuildCMakeLib.VS2019TargetPlatform
{
	public override string PlatformOrGroupName => nameof(UnrealTargetPlatform.Android);
	public override string StaticLibraryExtension => "a";
	public override bool IsPlatformExtension => false;
}
