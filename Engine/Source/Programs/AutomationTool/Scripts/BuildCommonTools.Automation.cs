// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.IO;
using System.Collections.Generic;
using System.Linq;
using System.Text;
using AutomationTool;
using UnrealBuildTool;
using EpicGames.Core;
using Microsoft.Extensions.Logging;

[Help("Builds common tools used by the engine which are not part of typical editor or game builds. Useful when syncing source-only on GitHub.")]
[Help("platforms=<X>+<Y>+...", "Specifies on or more platforms to build for (defaults to the current host platform)")]
[Help("manifest=<Path>", "Writes a manifest of all the build products to the given path")]
public class BuildCommonTools : BuildCommand
{
	public override void ExecuteBuild()
	{
		Logger.LogInformation("************************* BuildCommonTools");

		List<UnrealTargetPlatform> Platforms = new List<UnrealTargetPlatform>();

		// Add all the platforms if specified
		if (ParseParam("allplatforms"))
		{
			Platforms = UnrealTargetPlatform.GetValidPlatforms().ToList();
		}
		else
		{
			// Get the list of platform names
			string[] PlatformNames = ParseParamValue("platforms", BuildHostPlatform.Current.Platform.ToString()).Split('+');

			// Parse the platforms
			foreach (string PlatformName in PlatformNames)
			{
				UnrealBuildTool.UnrealTargetPlatform Platform;
				if (!UnrealTargetPlatform.TryParse(PlatformName, out Platform))
				{
					throw new AutomationException("Unknown platform specified on command line - '{0}' - valid platforms are {1}", PlatformName, String.Join("/", UnrealTargetPlatform.GetValidPlatformNames()));
				}
				Platforms.Add(Platform);
			}
		}

		// Get the agenda
		List<string> ExtraBuildProducts = new List<string>();
		UnrealBuild.BuildAgenda Agenda = MakeAgenda(Platforms.ToArray(), ExtraBuildProducts);

		// Build everything. We don't want to touch version files for GitHub builds -- these are "programmer builds" and won't have a canonical build version
		UnrealBuild Builder = new UnrealBuild(this);
		Builder.Build(Agenda, InUpdateVersionFiles: false);

		// Add UAT and UBT to the build products
		Builder.AddUATFilesToBuildProducts();
		Builder.AddUBTFilesToBuildProducts();

		// Add all the extra build products
		foreach(string ExtraBuildProduct in ExtraBuildProducts)
		{
			Builder.AddBuildProduct(ExtraBuildProduct);
		}

		// Make sure all the build products exist
		UnrealBuild.CheckBuildProducts(Builder.BuildProductFiles);

		// Write the manifest if needed
		string ManifestPath = ParseParamValue("manifest");
		if(ManifestPath != null)
		{
			SortedSet<string> Files = new SortedSet<string>();
			foreach(string BuildProductFile in Builder.BuildProductFiles)
			{
				Files.Add(BuildProductFile);
			}
			File.WriteAllLines(ManifestPath, Files.ToArray());
		}
	}

	public static UnrealBuild.BuildAgenda MakeAgenda(UnrealBuildTool.UnrealTargetPlatform[] Platforms, List<string> ExtraBuildProducts)
	{
		// Create the build agenda
		UnrealBuild.BuildAgenda Agenda = new UnrealBuild.BuildAgenda();

		// C# binaries
		Agenda.SwarmAgentProject = @"Engine\Source\Programs\UnrealSwarm\SwarmAgent.sln";
		Agenda.SwarmCoordinatorProject = @"Engine\Source\Programs\UnrealSwarm\SwarmCoordinator.sln";
		Agenda.DotNetProjects.Add(@"Engine/Source/Editor/SwarmInterface/DotNET/SwarmInterface.csproj");
		Agenda.DotNetProjects.Add(@"Engine/Source/Programs/UnrealControls/UnrealControls.csproj");

		// Windows binaries
		if(Platforms.Contains(UnrealBuildTool.UnrealTargetPlatform.Win64))
		{
			Agenda.AddTarget("CrashReportClient", UnrealBuildTool.UnrealTargetPlatform.Win64, UnrealBuildTool.UnrealTargetConfiguration.Development);
			Agenda.AddTarget("UnrealPak", UnrealBuildTool.UnrealTargetPlatform.Win64, UnrealBuildTool.UnrealTargetConfiguration.Development);
			Agenda.AddTarget("UnrealLightmass", UnrealBuildTool.UnrealTargetPlatform.Win64, UnrealBuildTool.UnrealTargetConfiguration.Development);
			Agenda.AddTarget("InterchangeWorker", UnrealBuildTool.UnrealTargetPlatform.Win64, UnrealBuildTool.UnrealTargetConfiguration.Development);
			Agenda.AddTarget("ShaderCompileWorker", UnrealBuildTool.UnrealTargetPlatform.Win64, UnrealBuildTool.UnrealTargetConfiguration.Development);
			Agenda.AddTarget("UnrealVersionSelector", UnrealBuildTool.UnrealTargetPlatform.Win64, UnrealBuildTool.UnrealTargetConfiguration.Shipping);
			Agenda.AddTarget("BootstrapPackagedGame", UnrealBuildTool.UnrealTargetPlatform.Win64, UnrealBuildTool.UnrealTargetConfiguration.Shipping);
			Agenda.AddTarget("EpicWebHelper", UnrealBuildTool.UnrealTargetPlatform.Win64, UnrealBuildTool.UnrealTargetConfiguration.Development);
		}

		// Mac binaries
		if(Platforms.Contains(UnrealBuildTool.UnrealTargetPlatform.Mac))
		{
			Agenda.AddTarget("CrashReportClient", UnrealBuildTool.UnrealTargetPlatform.Mac, UnrealBuildTool.UnrealTargetConfiguration.Development, InAddArgs: "-CopyAppBundleBackToDevice");
			Agenda.AddTarget("UnrealPak", UnrealBuildTool.UnrealTargetPlatform.Mac, UnrealBuildTool.UnrealTargetConfiguration.Development, InAddArgs: "-CopyAppBundleBackToDevice");
			Agenda.AddTarget("UnrealLightmass", UnrealBuildTool.UnrealTargetPlatform.Mac, UnrealBuildTool.UnrealTargetConfiguration.Development, InAddArgs: "-CopyAppBundleBackToDevice");
			Agenda.AddTarget("InterchangeWorker", UnrealBuildTool.UnrealTargetPlatform.Mac, UnrealBuildTool.UnrealTargetConfiguration.Development, InAddArgs: "-CopyAppBundleBackToDevice");
			Agenda.AddTarget("ShaderCompileWorker", UnrealBuildTool.UnrealTargetPlatform.Mac, UnrealBuildTool.UnrealTargetConfiguration.Development, InAddArgs: "-CopyAppBundleBackToDevice");
			Agenda.AddTarget("UnrealEditorServices", UnrealBuildTool.UnrealTargetPlatform.Mac, UnrealBuildTool.UnrealTargetConfiguration.Development, InAddArgs: "-CopyAppBundleBackToDevice");
			Agenda.AddTarget("EpicWebHelper", UnrealBuildTool.UnrealTargetPlatform.Mac, UnrealBuildTool.UnrealTargetConfiguration.Development, InAddArgs: "-CopyAppBundleBackToDevice");
		}

		// Linux binaries
		if (Platforms.Contains(UnrealBuildTool.UnrealTargetPlatform.Linux))
		{
			Agenda.AddTarget("CrashReportClient", UnrealBuildTool.UnrealTargetPlatform.Linux, UnrealBuildTool.UnrealTargetConfiguration.Development);
			Agenda.AddTarget("EpicWebHelper", UnrealBuildTool.UnrealTargetPlatform.Linux, UnrealBuildTool.UnrealTargetConfiguration.Development);
		}

		// iOS binaries
		if(Platforms.Contains(UnrealBuildTool.UnrealTargetPlatform.IOS))
		{
			Agenda.DotNetProjects.Add(@"Engine/Source/Programs/iOS/iPhonePackager/iPhonePackager.csproj");
			ExtraBuildProducts.Add(CommandUtils.CombinePaths(CommandUtils.CmdEnv.LocalRoot, @"Engine/Binaries/DotNET/iOS/iPhonePackager.exe"));
		}

		// Platform extensions
		foreach (UnrealBuildTool.UnrealTargetPlatform UBTPlatform in Platforms)
		{
			AutomationTool.Platform AutomationPlatform = Platform.GetPlatform(UBTPlatform);
			AutomationPlatform.MakeAgenda(Agenda, ExtraBuildProducts);
		}

		return Agenda;
	}
}

public class ZipProjectUp : BuildCommand
{
	public override void ExecuteBuild()
	{
		// Get Directories
		string ProjectDirectory = ParseParamValue("project", "");
		string InstallDirectory = ParseParamValue("install", "");
		ProjectDirectory = Path.GetDirectoryName(ProjectDirectory);

		Logger.LogInformation("Started zipping project up");
		Logger.LogInformation("Project directory: {ProjectDirectory}", ProjectDirectory);
		Logger.LogInformation("Install directory: {InstallDirectory}", InstallDirectory);
		Logger.LogInformation("Packaging up the project...");

		// Setup filters
		FileFilter Filter = new FileFilter();
		Filter.Include("/Config/...");
		Filter.Include("/Content/...");
		Filter.Include("/Plugins/...");
		Filter.Exclude("/Plugins/.../Intermediate/...");
		Filter.Exclude("/Plugins/.../Binaries/...");
		Filter.Include("/Shaders/...");
		Filter.Include("/Source/...");
		Filter.Include("*.uproject");

		ZipFiles(new FileReference(InstallDirectory), new DirectoryReference(ProjectDirectory), Filter);

		Logger.LogInformation("Completed zipping project up");
	}
}
