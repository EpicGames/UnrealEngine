// Copyright Epic Games, Inc. All Rights Reserved.
using System;
using System.Collections.Generic;
using System.IO;
using System.Threading;
using System.Reflection;
using AutomationTool;
using AutomationScripts;
using UnrealBuildTool;
using EpicGames.Core;
using Microsoft.Extensions.Logging;

[Help(@"Builds/Cooks/Runs a project.

For non-uprojects project targets are discovered by compiling target rule files found in the project folder.
If -map is not specified, the command looks for DefaultMap entry in the project's DefaultEngine.ini and if not found, in BaseEngine.ini.
If no DefaultMap can be found, the command falls back to /Engine/Maps/Entry.")]
[Help("project=Path", @"Project path (required), i.e: -project=QAGame, -project=Samples\BlackJack\BlackJack.uproject, -project=D:\Projects\MyProject.uproject")]
[Help("destsample", "Destination Sample name")]
[Help("foreigndest", "Foreign Destination")]
[Help(typeof(ProjectParams))]
[Help(typeof(UnrealBuild))]
[Help(typeof(CodeSign))]
public class BuildCookRun : BuildCommand, IProjectParamsHelpers
{
	public override void ExecuteBuild()
	{
		var StartTime = DateTime.UtcNow;

		// these need to be done first
		var bForeign = ParseParam("foreign");
		var bForeignCode = ParseParam("foreigncode");
		if (bForeign)
		{
			MakeForeignSample();
		}
		else if (bForeignCode)
		{
			MakeForeignCodeSample();
		}
		var Params = SetupParams();

		DoBuildCookRun(Params);

		Logger.LogInformation("BuildCookRun time: {0:0.00} s", (DateTime.UtcNow - StartTime).TotalMilliseconds / 1000);
	}

	protected ProjectParams SetupParams()
	{
		Logger.LogInformation("Setting up ProjectParams for {ProjectPath}", ProjectPath);

		var Params = new ProjectParams
		(
			Command: this,
			// Shared
			RawProjectPath: ProjectPath,
			RunChunkAssigner: ParseParam("RunChunkAssigner")
		);

		var WithTagFilter = ParseParam("WithTagFilter");
		if (WithTagFilter)
		{
			Params.StageTagMappingConfigFiles = true;
		}

		var DirectoriesToCook = ParseParamValue("cookdir");
		if (!String.IsNullOrEmpty(DirectoriesToCook))
		{
			Params.DirectoriesToCook = new ParamList<string>(DirectoriesToCook.Split('+'));
		}

		var ProjectProgramOverride = ParseParamValue("projectprogramoverride");
		if (!String.IsNullOrEmpty(ProjectProgramOverride))
		{
			Params.RawProgramProjectOverride = ParseProjectString(ProjectProgramOverride);
		}

		var DDCGraph = ParseParamValue("ddc");
		if (!String.IsNullOrEmpty(DDCGraph))
		{
			Params.DDCGraph = DDCGraph;
		}


        var InternationalizationPreset = ParseParamValue("i18npreset");
        if (!String.IsNullOrEmpty(InternationalizationPreset))
        {
            Params.InternationalizationPreset = InternationalizationPreset;
        }

        var CulturesToCook = ParseParamValue("cookcultures");
        if (!String.IsNullOrEmpty(CulturesToCook))
        {
            Params.CulturesToCook = new ParamList<string>(CulturesToCook.Split('+'));
        }

		var ReferenceContainerGlobalFileName = ParseParamValue("ReferenceContainerGlobalFileName");
		if (!String.IsNullOrEmpty(ReferenceContainerGlobalFileName))
		{
			Params.ReferenceContainerGlobalFileName = ReferenceContainerGlobalFileName;
		}

		var ReferenceContainerAdditionalPath = ParseParamValue("ReferenceContainerAdditionalPath");
		if (!String.IsNullOrEmpty(ReferenceContainerAdditionalPath))
		{
			Params.ReferenceContainerAdditionalPath = ReferenceContainerAdditionalPath;
		}

		var ReferenceContainerChangesCSVFileName = ParseParamValue("ReferenceContainerChangesCSVFileName");
		if (!String.IsNullOrEmpty(ReferenceContainerChangesCSVFileName))
		{
			Params.ReferenceContainerChangesCSVFileName = ReferenceContainerChangesCSVFileName;
		}

		var ReferenceContainerCryptoKeys = ParseParamValue("ReferenceContainerCryptoKeys");
		if (!String.IsNullOrEmpty(ReferenceContainerCryptoKeys))
		{
			Params.ReferenceContainerCryptoKeys = ReferenceContainerCryptoKeys;
		}

		if (Params.DedicatedServer)
		{
			foreach (var ServerPlatformInstance in Params.ServerTargetPlatformInstances)
			{
				ServerPlatformInstance.PlatformSetupParams(ref Params);
			}
		}
		else
		{
			foreach (var ClientPlatformInstance in Params.ClientTargetPlatformInstances)
			{
				ClientPlatformInstance.PlatformSetupParams(ref Params);
			}
		}

		List<string> PluginsToEnable = new List<string>();
		string EnablePlugins = ParseParamValue("EnablePlugins", null);
		if (!string.IsNullOrEmpty(EnablePlugins))
		{
			PluginsToEnable.AddRange(EnablePlugins.Split(new[] { ',', '+' }, StringSplitOptions.RemoveEmptyEntries));
		}

		if (PluginsToEnable.Count > 0)
		{
			Params.AdditionalCookerOptions += " -EnablePlugins=\"" + string.Join(",", PluginsToEnable) + "\"";
			Params.AdditionalBuildOptions += " -EnablePlugins=\"" + string.Join(",", PluginsToEnable) + "\"";
		}

		Params.ValidateAndLog();
		return Params;
	}

	/// <summary>
	/// In case the command line specified multiple map names with a '+', selects the first map from the list.
	/// </summary>
	/// <param name="Maps">Map(s) specified in the commandline.</param>
	/// <returns>First map or an empty string.</returns>
	private static string GetFirstMap(string Maps)
	{
		string Map = String.Empty;
		if (!String.IsNullOrEmpty(Maps))
		{
			var AllMaps = Maps.Split(new char[] { '+' }, StringSplitOptions.RemoveEmptyEntries);
			if (!IsNullOrEmpty(AllMaps))
			{
				Map = AllMaps[0];
			}
		}
		return Map;
	}

	private string GetTargetName(Type TargetRulesType)
	{
		const string TargetPostfix = "Target";
		var Name = TargetRulesType.Name;
		if (Name.EndsWith(TargetPostfix, StringComparison.InvariantCultureIgnoreCase))
		{
			Name = Name.Substring(0, Name.Length - TargetPostfix.Length);
		}
		return Name;
	}

	private string GetDefaultMap(ProjectParams Params)
	{
		const string EngineEntryMap = "/Engine/Maps/Entry";
		Logger.LogInformation("Trying to find DefaultMap in ini files");
		string DefaultMap = null;
		var ProjectFolder = GetDirectoryName(Params.RawProjectPath.FullName);
		var DefaultGameEngineConfig = CombinePaths(ProjectFolder, "Config", "DefaultEngine.ini");
		if (FileExists(DefaultGameEngineConfig))
		{
			Logger.LogInformation("Looking for DefaultMap in {DefaultGameEngineConfig}", DefaultGameEngineConfig);
			DefaultMap = GetDefaultMapFromIni(DefaultGameEngineConfig, Params.DedicatedServer);
			if (DefaultMap == null && Params.DedicatedServer)
			{
				DefaultMap = GetDefaultMapFromIni(DefaultGameEngineConfig, false);
			}
		}
		else
		{
			var BaseEngineConfig = CombinePaths(CmdEnv.LocalRoot, "Config", "BaseEngine.ini");
			if (FileExists(BaseEngineConfig))
			{
				Logger.LogInformation("Looking for DefaultMap in {BaseEngineConfig}", BaseEngineConfig);
				DefaultMap = GetDefaultMapFromIni(BaseEngineConfig, Params.DedicatedServer);
				if (DefaultMap == null && Params.DedicatedServer)
				{
					DefaultMap = GetDefaultMapFromIni(BaseEngineConfig, false);
				}
			}
		}
		// We check for null here becase null == not found
		if (DefaultMap == null)
		{
			Logger.LogInformation("No DefaultMap found, assuming: {EngineEntryMap}", EngineEntryMap);
			DefaultMap = EngineEntryMap;
		}
		else
		{
			Logger.LogInformation("Found DefaultMap={DefaultMap}", DefaultMap);
		}
		return DefaultMap;
	}

	private string GetDefaultMapFromIni(string IniFilename, bool DedicatedServer)
	{
		var IniLines = ReadAllLines(IniFilename);
		string DefaultMap = null;

		string ConfigKeyStr = "GameDefaultMap";
		if (DedicatedServer)
		{
			ConfigKeyStr = "ServerDefaultMap";
		}

		foreach (var Line in IniLines)
		{
			if (Line.StartsWith(ConfigKeyStr, StringComparison.InvariantCultureIgnoreCase))
			{
				var DefaultMapPair = Line.Split('=');
				DefaultMap = DefaultMapPair[1].Trim();
			}
			if (DefaultMap != null)
			{
				break;
			}
		}
		return DefaultMap;
	}

	protected void DoBuildCookRun(ProjectParams Params)
	{
		int WorkingCL = -1;
		if (P4Enabled && GlobalCommandLine.Submit && AllowSubmit)
		{
			WorkingCL = P4.CreateChange(P4Env.Client, String.Format("{0} build from changelist {1}", Params.ShortProjectName, P4Env.Changelist));
		}

        Project.Build(this, Params, WorkingCL, ProjectBuildTargets.All);
		Project.Cook(Params);
		Project.CopyBuildToStagingDirectory(Params);
		Project.Package(Params, WorkingCL);
		Project.Archive(Params);
		Project.Deploy(Params);
		PrintRunTime();
		Project.Run(Params);
		Project.GetFile(Params);

		// Check everything in!
		if (WorkingCL != -1)
		{
			int SubmittedCL;
			P4.Submit(WorkingCL, out SubmittedCL, true, true);
		}
	}

	private void MakeForeignSample()
	{
		string Sample = "BlankProject";
		var DestSample = ParseParamValue("DestSample", "CopiedBlankProject");
		var Src = CombinePaths(CmdEnv.LocalRoot, "Samples", "SampleGames", Sample);
		if (!DirectoryExists(Src))
		{
			throw new AutomationException("Can't find source directory to make foreign sample {0}.", Src);
		}

		var Dest = ParseParamValue("ForeignDest", CombinePaths(@"C:\testue\foreign\", DestSample + "_ _Dir"));
		Logger.LogInformation("Make a foreign sample {Src} -> {Dest}", Src, Dest);

		CloneDirectory(Src, Dest);

		DeleteDirectory_NoExceptions(CombinePaths(Dest, "Intermediate"));
		DeleteDirectory_NoExceptions(CombinePaths(Dest, "Saved"));

		RenameFile(CombinePaths(Dest, Sample + ".uproject"), CombinePaths(Dest, DestSample + ".uproject"));

		var IniFile = CombinePaths(Dest, "Config", "DefaultEngine.ini");
		var Ini = new VersionFileUpdater(new FileReference(IniFile));
		Ini.ReplaceLine("GameName=", DestSample);
		Ini.Commit();
	}

	private void MakeForeignCodeSample()
	{
		string Sample = "PlatformerGame";
		string DestSample = "PlatformerGame";
		var Src = CombinePaths(CmdEnv.LocalRoot, Sample);
		if (!DirectoryExists(Src))
		{
			throw new AutomationException("Can't find source directory to make foreign sample {0}.", Src);
		}

		var Dest = ParseParamValue("ForeignDest", CombinePaths(@"C:\testue\foreign\", DestSample + "_ _Dir"));
		Logger.LogInformation("Make a foreign sample {Src} -> {Dest}", Src, Dest);

		CloneDirectory(Src, Dest);
		DeleteDirectory_NoExceptions(CombinePaths(Dest, "Intermediate"));
		DeleteDirectory_NoExceptions(CombinePaths(Dest, "Saved"));
		DeleteDirectory_NoExceptions(CombinePaths(Dest, "Plugins", "FootIK", "Intermediate"));

		//RenameFile(CombinePaths(Dest, Sample + ".uproject"), CombinePaths(Dest, DestSample + ".uproject"));

		var IniFile = CombinePaths(Dest, "Config", "DefaultEngine.ini");
		var Ini = new VersionFileUpdater(new FileReference(IniFile));
		Ini.ReplaceLine("GameName=", DestSample);
		Ini.Commit();
	}

	private FileReference ProjectFullPath;
	public virtual FileReference ProjectPath
	{
		get
		{
			if (ProjectFullPath == null)
			{
				ProjectFullPath = ParseProjectParam();

				if (ProjectFullPath == null)
				{
					throw new AutomationException("No project file specified. Use -project=<project>.");
				}
			}

			return ProjectFullPath;
		}
	}
}
