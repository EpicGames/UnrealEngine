// Copyright Epic Games, Inc. All Rights Reserved.

using AutomationTool;
using UnrealBuildTool;
using System;
using System.Collections.Generic;
using System.Linq;
using System.Text;
using System.IO;
using EpicGames.Core;
using UnrealBuildBase;
using Microsoft.Extensions.Logging;

using static AutomationTool.CommandUtils;

public class BuildDerivedDataCache : BuildCommand
{
	public override void ExecuteBuild()
	{
		// Get the list of platform names
		string[] FeaturePacks = ParseParamValue("FeaturePacks").Split(';');
		string TempDir = ParseParamValue("TempDir");
		UnrealTargetPlatform HostPlatform = BuildHostPlatform.Current.Platform;
		string TargetPlatforms = ParseParamValue("TargetPlatforms");
		string SavedDir = ParseParamValue("SavedDir");
		string BackendName = ParseParamValue("BackendName", "CreateInstalledEnginePak");
		string RelativePakPath = ParseParamValue("RelativePakPath", "Engine/DerivedDataCache/Compressed.ddp");
		bool bSkipEngine = ParseParam("SkipEngine");
		string EngineContentExtraArgs = ParseParamValue("EngineContentExtraArgs", string.Empty);
		string FeaturePackExtraArgs = ParseParamValue("FeaturePackExtraArgs", string.Empty);

		// Get paths to everything within the temporary directory
		string EditorExe = CommandUtils.GetEditorCommandletExe(TempDir, HostPlatform);
		string OutputPakFile = CommandUtils.CombinePaths(TempDir, RelativePakPath);
		string OutputCsvFile = Path.ChangeExtension(OutputPakFile, ".csv");


		List<string> ProjectPakFiles = new List<string>();
		List<string> FeaturePackPaths = new List<string>();
		// loop through all the projects first and bail out if one of them doesn't exist.
		foreach (string FeaturePack in FeaturePacks)
		{
			if (!String.IsNullOrWhiteSpace(FeaturePack))
			{
				string FeaturePackPath = CommandUtils.CombinePaths(Unreal.RootDirectory.FullName, FeaturePack);
				if (!CommandUtils.FileExists(FeaturePackPath))
				{
					throw new AutomationException($"Could not find project: {FeaturePack}");
				}
				FeaturePackPaths.Add(FeaturePackPath);
			}
		}

		// loop through all the paths and generate ddc data for them
		foreach (string FeaturePackPath in FeaturePackPaths)
		{
			StringBuilder FeaturePackParameters = new StringBuilder();
			string ProjectSpecificPlatforms = TargetPlatforms;
			FileReference FileRef = new FileReference(FeaturePackPath);
			string GameName = FileRef.GetFileNameWithoutAnyExtensions();
			ProjectDescriptor Project = ProjectDescriptor.FromFile(FileRef);

			FeaturePackParameters.Append($"-fill -DDC={BackendName} -ProjectOnly");
			if (!string.IsNullOrEmpty(FeaturePackExtraArgs))
			{
				FeaturePackParameters.Append($" {FeaturePackExtraArgs} ");
			}

			if (Project.TargetPlatforms != null && Project.TargetPlatforms.Length > 0)
			{
				// Restrict target platforms used to those specified in project file
				List<string> FilteredPlatforms = new List<string>();

				// Always include the editor platform for cooking
				string EditorCookPlatform = Platform.GetPlatform(HostPlatform).GetEditorCookPlatform();
				if (TargetPlatforms.Contains(EditorCookPlatform))
				{
					FilteredPlatforms.Add(EditorCookPlatform);
				}

				foreach (string TargetPlatform in Project.TargetPlatforms)
				{
					if (TargetPlatforms.Contains(TargetPlatform))
					{
						FilteredPlatforms.Add(TargetPlatform);
					}
				}
				if (FilteredPlatforms.Count == 0)
				{
					Logger.LogInformation("Did not find any project specific platforms for FeaturePack {GameName} out of supplied TargetPlatforms {ProjectSpecificPlatforms}, skipping it!", GameName, ProjectSpecificPlatforms);
					continue;
				}
				ProjectSpecificPlatforms = CommandUtils.CombineCommandletParams(FilteredPlatforms.Distinct().ToArray());
			}
			Logger.LogInformation("Generating DDC data for {GameName} on {ProjectSpecificPlatforms}", GameName, ProjectSpecificPlatforms);
			CommandUtils.DDCCommandlet(FileRef, EditorExe, null, ProjectSpecificPlatforms, FeaturePackParameters.ToString());

			string ProjectPakFile = CommandUtils.CombinePaths(Path.GetDirectoryName(OutputPakFile), $"Compressed-{GameName}.ddp");
			CommandUtils.DeleteFile(ProjectPakFile);
			CommandUtils.RenameFile(OutputPakFile, ProjectPakFile);

			string ProjectCsvFile = Path.ChangeExtension(ProjectPakFile, ".csv");
			CommandUtils.DeleteFile(ProjectCsvFile);
			CommandUtils.RenameFile(OutputCsvFile, ProjectCsvFile);

			ProjectPakFiles.Add(Path.GetFileName(ProjectPakFile));

		}

		// Before running the Engine, delete any stale saved Config files from previous runs of the Engine. The list of enabled plugins can change, and the config file needs to be
		// recomputed after they do, but there is currently nothing that makes that happen automatically
		CommandUtils.DeleteDirectory(CommandUtils.CombinePaths(TempDir, "Engine", "Saved", "Config"));

		// Generate DDC for the editor, and merge all the other PAK files ini
		List<string> EngineContentArgs = new() 
		{ 
			"-fill",
			$"-DDC={BackendName}",
			$"-MergePaks={CommandUtils.MakePathSafeToUseWithCommandLine(String.Join("+", ProjectPakFiles))}" 
		};

		if (!string.IsNullOrEmpty(EngineContentExtraArgs))
		{
			EngineContentArgs.Add(EngineContentExtraArgs);
		}

		if (bSkipEngine)
		{
			EngineContentArgs.Add("-projectonly");
		}

		Logger.LogInformation("Generating DDC data for engine content on {TargetPlatforms}", TargetPlatforms);
		CommandUtils.DDCCommandlet(null, EditorExe, null, TargetPlatforms, String.Join(" ", EngineContentArgs));

		string SavedPakFile = CommandUtils.CombinePaths(SavedDir, RelativePakPath);
		CommandUtils.CopyFile(OutputPakFile, SavedPakFile);
	}
}

