// Copyright Epic Games, Inc. All Rights Reserved.
using System;
using System.Collections.Generic;
using System.IO;
using System.Threading;
using System.Linq;
using System.Reflection;
using Microsoft.Win32;
using System.Diagnostics;
using System.Text.RegularExpressions;
using AutomationTool;
using UnrealBuildTool;
using EpicGames.Core;
using UnrealBuildBase;
using Microsoft.Extensions.Logging;

[Help("Builds Hlslcc using CMake build system.")]
[Help("TargetPlatforms", "Specify a list of target platforms to build, separated by '+' characters (eg. -TargetPlatforms=Win64+Linux+Mac). Architectures are specified with '-'. Default is Win64+Linux.")]
[Help("TargetConfigs", "Specify a list of configurations to build, separated by '+' characters (eg. -TargetConfigs=Debug+RelWithDebInfo). Default is Debug+RelWithDebInfo.")]
[Help("TargetWindowsCompilers", "Specify a list of target compilers to use when building for Windows, separated by '+' characters (eg. -TargetCompilers=VisualStudio2022). Default is VisualStudio2022.")]
[Help("SkipBuild", "Do not perform build step. If this argument is not supplied libraries will be built (in accordance with TargetLibs, TargetPlatforms and TargetWindowsCompilers).")]
[Help("SkipDeployLibs", "Do not perform library deployment to the engine. If this argument is not supplied libraries will be copied into the engine.")]
[Help("SkipDeploySource", "Do not perform source deployment to the engine. If this argument is not supplied source will be copied into the engine.")]
[Help("SkipCreateChangelist", "Do not create a P4 changelist for source or libs. If this argument is not supplied source and libs will be added to a Perforce changelist.")]
[Help("SkipSubmit", "Do not perform P4 submit of source or libs. If this argument is not supplied source and libs will be automatically submitted to Perforce. If SkipCreateChangelist is specified, this argument applies by default.")]
[Help("Robomerge", "Which robomerge action to apply to the submission. If we're skipping submit, this is not used.")]
[RequireP4]
class BuildHlslcc : BuildCommand
{
	const int InvalidChangeList = -1;

	private struct TargetPlatformData
	{
		public UnrealTargetPlatform Platform;
		public string Architecture;

		public TargetPlatformData(UnrealTargetPlatform InPlatform)
		{
			Platform = InPlatform;
			// Linux never has an empty architecture. If we don't care then it's x86_64-unknown-linux-gnu
			Architecture = (Platform == UnrealTargetPlatform.Linux) ? "x86_64-unknown-linux-gnu" : "";
		}
		public TargetPlatformData(UnrealTargetPlatform InPlatform, string InArchitecture)
		{
			Platform = InPlatform;
			Architecture = InArchitecture;
		}

		public override string ToString()
		{
			return Architecture == "" ? Platform.ToString() : Platform.ToString() + "_" + Architecture;
		}
	}

	// Apex libs that do not have an APEX prefix in their name
	private static string[] APEXSpecialLibs = { "NvParameterized", "RenderDebug" };

	// We cache our own MSDev and MSBuild executables
	private static FileReference MsDev22Exe;
	private static FileReference MsBuildExe;

	// Cache directories under the PhysX/ directory
	private static DirectoryReference SourceRootDirectory = DirectoryReference.Combine(Unreal.RootDirectory, "Engine", "Source", "ThirdParty", "hlslcc", "hlslcc");
	private static DirectoryReference RootOutputLibDirectory = DirectoryReference.Combine(SourceRootDirectory, "lib");
	private static DirectoryReference ThirdPartySourceDirectory = DirectoryReference.Combine(Unreal.RootDirectory, "Engine", "Source", "ThirdParty");

	private static string GetCMakeNameAndSetupEnv(TargetPlatformData TargetData)
	{
		DirectoryReference CMakeRootDirectory = DirectoryReference.Combine(Unreal.RootDirectory, "Engine", "Extras", "ThirdPartyNotUE", "CMake");
		if (BuildHostPlatform.Current.Platform == UnrealTargetPlatform.Linux)
		{
			return "cmake";
		}

		Environment.SetEnvironmentVariable("CMAKE_ROOT", DirectoryReference.Combine(CMakeRootDirectory, "share").ToString());
		Logger.LogInformation("set {Arg0}={Arg1}", "CMAKE_ROOT", Environment.GetEnvironmentVariable("CMAKE_ROOT"));

		if (TargetData.Platform == UnrealTargetPlatform.Mac)
		{
			return FileReference.Combine(CMakeRootDirectory, "bin", "cmake").ToString();
		}
		else
		{
			return FileReference.Combine(CMakeRootDirectory, "bin", "cmake.exe").ToString();
		}
	}

	private static string GetCMakeTargetDirectoryName(TargetPlatformData TargetData, WindowsCompiler TargetWindowsCompiler)
	{
		string VisualStudioDirectoryName;
		switch (TargetWindowsCompiler)
		{
			case WindowsCompiler.VisualStudio2022:
				VisualStudioDirectoryName = "VS2015";
				break;
			default:
				throw new AutomationException(String.Format("Non-CMake or unsupported windows compiler '{0}' supplied to GetCMakeTargetDirectoryName", TargetWindowsCompiler));
		}

		if (TargetData.Platform == UnrealTargetPlatform.Win64)
		{
			// Note slashes need to be '/' as this gets string-composed in the CMake script with other paths
			return "Win64/" + VisualStudioDirectoryName;
		}
		else
		{
			return TargetData.Platform.ToString();
		}
	}

	private static DirectoryReference GetProjectDirectory(TargetPlatformData TargetData, WindowsCompiler TargetWindowsCompiler = WindowsCompiler.VisualStudio2022)
	{
		DirectoryReference Directory = SourceRootDirectory;

		return DirectoryReference.Combine(Directory, "compiler", GetCMakeTargetDirectoryName(TargetData, TargetWindowsCompiler));
	}

	private static string GetBundledLinuxLibCxxFlags(string ToolchainPath)
	{
		string ToolchainFlags = string.Format("--sysroot={0} -B{0}/usr/lib/ -B{0}/usr/lib64", ToolchainPath);
		string CFlags = "\"" + ToolchainFlags + "\"";
		string CxxFlags = "\"-I " + ThirdPartySourceDirectory + "/Linux/LibCxx/include -I " + ThirdPartySourceDirectory + "/Linux/LibCxx/include/c++/v1 " + ToolchainFlags + "\"";
		string CxxLinkerFlags = "\"-stdlib=libc++ -nodefaultlibs -L " + ThirdPartySourceDirectory + "/Linux/LibCxx/lib/Linux/x86_64-unknown-linux-gnu/ " + ThirdPartySourceDirectory + "/Linux/LibCxx/lib/Linux/x86_64-unknown-linux-gnu/libc++.a " + ThirdPartySourceDirectory + "/Linux/LibCxx/lib/Linux/x86_64-unknown-linux-gnu/libc++abi.a -lm -lc -lgcc_s -target x86_64-unknown-linux-gnu " + ToolchainFlags + "\"";

		return "-DCMAKE_C_FLAGS=" + CFlags + " -DCMAKE_CXX_FLAGS=" + CxxFlags + " -DCMAKE_EXE_LINKER_FLAGS=" + CxxLinkerFlags + " -DCAMKE_MODULE_LINKER_FLAGS=" + CxxLinkerFlags + " -DCMAKE_SHARED_LINKER_FLAGS=" + CxxLinkerFlags + " ";
	}

	private static string GetLinuxToolchainSettings(TargetPlatformData TargetData)
	{
		if (BuildHostPlatform.Current.Platform == UnrealTargetPlatform.Linux)
		{
			string ExtraSettings = "";

			// native builds try to use bundled toolchain
			string ToolchainPath = Environment.GetEnvironmentVariable("LINUX_MULTIARCH_ROOT");
			if (string.IsNullOrEmpty(ToolchainPath))
			{
				// try to find bundled toolchain
				DirectoryReference ToolchainDir  = DirectoryReference.Combine(ThirdPartySourceDirectory, "..", "..", "Extras", "ThirdPartyNotUE", "SDKs", "HostLinux", "Linux_x64");
				Logger.LogInformation("LINUX_MULTIARCH_ROOT not defined. Looking for Linux toolchain in {ToolchainDir}...", ToolchainDir);

				IEnumerable<DirectoryReference> AvailableToolchains = DirectoryReference.EnumerateDirectories(ToolchainDir);
				if (AvailableToolchains.Count() > 0)
				{
					// grab first available
					ToolchainPath = AvailableToolchains.First() + "/x86_64-unknown-linux-gnu";
				}
			}

			if (string.IsNullOrEmpty(ToolchainPath))
			{
				Logger.LogInformation("Bundled toolchain not found. Using system clang.");
			}
			else
			{
				ExtraSettings = GetBundledLinuxLibCxxFlags(ToolchainPath);
				ToolchainPath += "/bin/";
			}

			Logger.LogInformation("Using toolchain: {ToolchainPath}", ToolchainPath);

			return string.Format(" -DCMAKE_C_COMPILER={0}clang -DCMAKE_CXX_COMPILER={0}clang++ ", ToolchainPath) + ExtraSettings;
		}

		// otherwise, use a per-architecture file.
		return " -DCMAKE_TOOLCHAIN_FILE=\"" + SourceRootDirectory + "\\..\\..\\PhysX3\\Externals\\CMakeModules\\Linux\\LinuxCrossToolchain.multiarch.cmake\"" + " -DARCHITECTURE_TRIPLE=" + TargetData.Architecture;
	}

	private static string GetCMakeArguments(TargetPlatformData TargetData, string BuildConfig = "", WindowsCompiler TargetWindowsCompiler = WindowsCompiler.VisualStudio2022)
	{
		string VisualStudioName;
		switch (TargetWindowsCompiler)
		{
			case WindowsCompiler.VisualStudio2022:
				VisualStudioName = "Visual Studio 17 2022";
				break;
			default:
				throw new AutomationException(String.Format("Non-CMake or unsupported platform '{0}' supplied to GetCMakeArguments", TargetData.ToString()));
		}

		string OutputFlags = "";

		// Enable response files for platforms that require them.
		// Response files are used for include paths etc, to fix max command line length issues.
		if (TargetData.Platform == UnrealTargetPlatform.Linux)
		{
			OutputFlags += " -DUSE_RESPONSE_FILES=1";
		}

		DirectoryReference CMakeFilePath = DirectoryReference.Combine(SourceRootDirectory, "projects");
		if (TargetData.Platform == UnrealTargetPlatform.Win64)
		{
			return CMakeFilePath.ToString() + " -G \"" + VisualStudioName + "\" -Ax64 -DTARGET_BUILD_PLATFORM=windows" + OutputFlags;
		}
		else if (TargetData.Platform == UnrealTargetPlatform.Linux)
		{
			return CMakeFilePath.ToString() + " --no-warn-unused-cli -G \"Unix Makefiles\" -DTARGET_BUILD_PLATFORM=linux " + " -DCMAKE_BUILD_TYPE=" + BuildConfig + GetLinuxToolchainSettings(TargetData) + OutputFlags;
		}
		else if (TargetData.Platform == UnrealTargetPlatform.Mac)
		{
			return CMakeFilePath.ToString() + " -G \"Xcode\" -DTARGET_BUILD_PLATFORM=mac" + OutputFlags;
		}
		else
		{
			throw new AutomationException(String.Format("Non-CMake or unsupported platform '{0}' supplied to GetCMakeArguments", TargetData.ToString()));
		}
	}

	private static string GetMsDevExe(TargetPlatformData TargetData)
	{
		if (TargetData.Platform == UnrealTargetPlatform.Win64)
		{
			return MsDev22Exe.ToString();
		}

		throw new AutomationException(String.Format("Non-MSBuild or unsupported platform '{0}' supplied to GetMsDevExe", TargetData.ToString()));
	}

	private static string GetMsBuildExe(TargetPlatformData TargetData)
	{
		if (TargetData.Platform == UnrealTargetPlatform.Win64)
		{
			return MsBuildExe.ToString();
		}

		throw new AutomationException(String.Format("Non-MSBuild or unsupported platform '{0}' supplied to GetMsBuildExe", TargetData.ToString()));
	}

	private static string GetTargetLibSolutionName()
	{
		return "hlslcc.sln";
	}

	private static FileReference GetTargetLibSolutionFileName(TargetPlatformData TargetData, WindowsCompiler TargetWindowsCompiler)
	{
		DirectoryReference Directory = GetProjectDirectory(TargetData, TargetWindowsCompiler);
		return FileReference.Combine(Directory, GetTargetLibSolutionName());
	}

	private static bool DoesPlatformUseMSBuild(TargetPlatformData TargetData)
	{
		return TargetData.Platform == UnrealTargetPlatform.Win64;
	}

	private static bool DoesPlatformUseMakefiles(TargetPlatformData TargetData)
	{
		return TargetData.Platform == UnrealTargetPlatform.Linux;
	}

	private static bool DoesPlatformUseXcode(TargetPlatformData TargetData)
	{
		return TargetData.Platform == UnrealTargetPlatform.Mac;
	}

	private List<TargetPlatformData> GetTargetPlatforms()
	{
		List<TargetPlatformData> TargetPlatforms = new List<TargetPlatformData>();

		// Remove any platforms that aren't enabled on the command line
		string TargetPlatformFilter = ParseParamValue("TargetPlatforms", "Win64+Linux");
		if (TargetPlatformFilter != null)
		{
			foreach (string TargetPlatformName in TargetPlatformFilter.Split(new char[] { '+' }, StringSplitOptions.RemoveEmptyEntries))
			{
				string[] TargetPlatformAndArch = TargetPlatformName.Split(new char[] { '-' }, StringSplitOptions.RemoveEmptyEntries);

				UnrealTargetPlatform TargetPlatform;
				if (!UnrealTargetPlatform.TryParse(TargetPlatformAndArch[0], out TargetPlatform))
				{
					throw new AutomationException(String.Format("Unknown target platform '{0}' specified on command line", TargetPlatformName));
				}
				else
				{
					if (TargetPlatformAndArch.Count() == 2)
					{
						TargetPlatforms.Add(new TargetPlatformData(TargetPlatform, TargetPlatformAndArch[1]));
					}
					else if (TargetPlatformAndArch.Count() > 2)
					{
						// Linux archs are OS triplets, so have multiple dashes
						string DashedArch = TargetPlatformAndArch[1];
						for (int Idx = 2; Idx < TargetPlatformAndArch.Count(); ++Idx)
						{
							DashedArch += "-" + TargetPlatformAndArch[Idx];
						}
						TargetPlatforms.Add(new TargetPlatformData(TargetPlatform, DashedArch));
					}
					else
					{
						TargetPlatforms.Add(new TargetPlatformData(TargetPlatform));
					}
				}
			}
		}

		return TargetPlatforms;
	}

	public List<string> GetTargetConfigurations()
	{
		List<string> TargetConfigs = new List<string>();
		// Remove any configs that aren't enabled on the command line
		string TargetConfigFilter = ParseParamValue("TargetConfigs", "Debug+RelWithDebInfo");
		if (TargetConfigFilter != null)
		{
			foreach (string TargetConfig in TargetConfigFilter.Split(new char[] { '+' }, StringSplitOptions.RemoveEmptyEntries))
			{
				TargetConfigs.Add(TargetConfig);
			}
		}

		return TargetConfigs;
	}

	private List<WindowsCompiler> GetTargetWindowsCompilers()
	{
		List<WindowsCompiler> TargetWindowsCompilers = new List<WindowsCompiler>();
		string TargetWindowsCompilersFilter = ParseParamValue("TargetWindowsCompilers", "VisualStudio2022");
		if (TargetWindowsCompilersFilter != null)
		{
			foreach (string TargetWindowsCompilerName in TargetWindowsCompilersFilter.Split(new char[] { '+' }, StringSplitOptions.RemoveEmptyEntries))
			{
				WindowsCompiler TargetWindowsCompiler;
				if (!Enum.TryParse(TargetWindowsCompilerName, out TargetWindowsCompiler))
				{
					throw new AutomationException(String.Format("Unknown target windows compiler '{0}' specified on command line", TargetWindowsCompilerName));
				}
				else
				{
					TargetWindowsCompilers.Add(TargetWindowsCompiler);
				}
			}
		}
		return TargetWindowsCompilers;
	}

	private static void MakeFreshDirectoryIfRequired(DirectoryReference Directory)
	{
		if (!DirectoryReference.Exists(Directory))
		{
			DirectoryReference.CreateDirectory(Directory);
		}
		else
		{

			InternalUtils.SafeDeleteDirectory(Directory.FullName);
			DirectoryReference.CreateDirectory(Directory);
		}
	}

	public static int RunLocalProcess(Process LocalProcess)
	{
		int ExitCode = -1;

		// release all process resources
		using (LocalProcess)
		{
			LocalProcess.StartInfo.UseShellExecute = false;
			LocalProcess.StartInfo.RedirectStandardOutput = true;

			try
			{
				// Start the process up and then wait for it to finish
				LocalProcess.Start();
				LocalProcess.BeginOutputReadLine();

				if (LocalProcess.StartInfo.RedirectStandardError)
				{
					LocalProcess.BeginErrorReadLine();
				}

				LocalProcess.WaitForExit();
				ExitCode = LocalProcess.ExitCode;
			}
			catch (Exception ex)
			{
				throw new AutomationException(ex, "Failed to start local process for action (\"{0}\"): {1} {2}", ex.Message, LocalProcess.StartInfo.FileName, LocalProcess.StartInfo.Arguments);
			}
		}

		return ExitCode;
	}

	public static int RunLocalProcessAndLogOutput(ProcessStartInfo StartInfo)
	{
		Process LocalProcess = new Process();
		LocalProcess.StartInfo = StartInfo;
		LocalProcess.OutputDataReceived += (Sender, Line) => { if (Line != null && Line.Data != null) Logger.LogInformation("{Line}", Line.Data); };
		return RunLocalProcess(LocalProcess);
	}

	private static void SetupBuildForTargetLibAndPlatform(TargetPlatformData TargetData, List<string> TargetConfigurations, List<WindowsCompiler> TargetWindowsCompilers, bool bCleanOnly)
	{
		string CMakeName = GetCMakeNameAndSetupEnv(TargetData);

		if (TargetData.Platform == UnrealTargetPlatform.Win64)
		{
			// for windows platforms we support building against multiple compilers
			foreach (WindowsCompiler TargetWindowsCompiler in TargetWindowsCompilers)
			{
				DirectoryReference CMakeTargetDirectory = GetProjectDirectory(TargetData, TargetWindowsCompiler);
				MakeFreshDirectoryIfRequired(CMakeTargetDirectory);

				if (!bCleanOnly)
				{
					Logger.LogInformation("{Text}", "Generating projects for lib " + TargetData.ToString());

					ProcessStartInfo StartInfo = new ProcessStartInfo();
					StartInfo.FileName = CMakeName;
					StartInfo.WorkingDirectory = CMakeTargetDirectory.ToString();
					StartInfo.Arguments = GetCMakeArguments(TargetData, "", TargetWindowsCompiler);

					if (RunLocalProcessAndLogOutput(StartInfo) != 0)
					{
						throw new AutomationException(String.Format("Unable to generate projects for {0}.", TargetData.ToString()));
					}
				}
			}
		}
		else if (TargetData.Platform == UnrealTargetPlatform.Linux)
		{
			foreach (string BuildConfig in TargetConfigurations)
			{
				DirectoryReference CMakeTargetDirectory = GetProjectDirectory(TargetData);
				CMakeTargetDirectory = DirectoryReference.Combine(CMakeTargetDirectory, BuildConfig);
				MakeFreshDirectoryIfRequired(CMakeTargetDirectory);

				if (!bCleanOnly)
				{
					Logger.LogInformation("{Text}", "Generating projects for lib " + TargetData.ToString());

					ProcessStartInfo StartInfo = new ProcessStartInfo();
					StartInfo.FileName = CMakeName;
					StartInfo.WorkingDirectory = CMakeTargetDirectory.ToString();
					StartInfo.Arguments = GetCMakeArguments(TargetData, BuildConfig);

					System.Console.WriteLine("Working in '{0}'", StartInfo.WorkingDirectory);
					Logger.LogInformation("Working in '{Arg0}'", StartInfo.WorkingDirectory);

					System.Console.WriteLine("{0} {1}", StartInfo.FileName, StartInfo.Arguments);
					Logger.LogInformation("{Arg0} {Arg1}", StartInfo.FileName, StartInfo.Arguments);

					if (RunLocalProcessAndLogOutput(StartInfo) != 0)
					{
						throw new AutomationException(String.Format("Unable to generate projects for {0}.", TargetData.ToString()));
					}
				}
			}
		}
		else if (TargetData.Platform == UnrealTargetPlatform.Mac)
		{
			DirectoryReference CMakeTargetDirectory = GetProjectDirectory(TargetData);
			MakeFreshDirectoryIfRequired(CMakeTargetDirectory);

			if (!bCleanOnly)
			{
				Logger.LogInformation("{Text}", "Generating projects for lib " + TargetData.ToString());

				ProcessStartInfo StartInfo = new ProcessStartInfo();
				StartInfo.FileName = CMakeName;
				StartInfo.WorkingDirectory = CMakeTargetDirectory.ToString();
				StartInfo.Arguments = GetCMakeArguments(TargetData);

				if (RunLocalProcessAndLogOutput(StartInfo) != 0)
				{
					throw new AutomationException(String.Format("Unable to generate projects for {0}.", TargetData.ToString()));
				}
			}
		}
		else
		{
			throw new AutomationException(String.Format("Unable to generate projects for {0}.", TargetData.ToString()));
		}
	}

	private static string GetMsDevExe(WindowsCompiler Version)
	{
		IEnumerable<DirectoryReference> VSPaths = WindowsExports.TryGetVSInstallDirs(Version);
		if (VSPaths != null)
		{
			return FileReference.Combine(VSPaths.First(), "Common7", "IDE", "Devenv.com").FullName;
		}
		return null;
	}

	private static string GetMsBuildExe(WindowsCompiler Version)
	{
		IEnumerable<DirectoryReference> VSPaths = WindowsExports.TryGetVSInstallDirs(Version);
		if (VSPaths != null)
		{
			return FileReference.Combine(VSPaths.First(), "MSBuild", "Current", "Bin", "MSBuild.exe").FullName;
		}
		return null;
	}

	private static string RemoveOtherMakeAndCygwinFromPath(string WindowsPath)
	{
		string[] PathComponents = WindowsPath.Split(';');
		string NewPath = "";
		foreach (string PathComponent in PathComponents)
		{
			// everything what contains /bin or /sbin is suspicious, check if it has make in it
			if (PathComponent.Contains("\\bin") || PathComponent.Contains("/bin") || PathComponent.Contains("\\sbin") || PathComponent.Contains("/sbin"))
			{
				if (File.Exists(PathComponent + "/make.exe") || File.Exists(PathComponent + "make.exe") || File.Exists(PathComponent + "/cygwin1.dll"))
				{
					// gotcha!
					Logger.LogInformation("Removing {PathComponent} from PATH since it contains possibly colliding make.exe", PathComponent);
					continue;
				}
			}

			NewPath = NewPath + ';' + PathComponent + ';';
		}

		return NewPath;
	}
	private static void SetupStaticBuildEnvironment()
	{
		if (OperatingSystem.IsWindows())
		{
			string VS2022Path = GetMsDevExe(WindowsCompiler.VisualStudio2022);
			if (VS2022Path != null)
			{
				MsDev22Exe = new FileReference(GetMsDevExe(WindowsCompiler.VisualStudio2022));
				MsBuildExe = new FileReference(GetMsBuildExe(WindowsCompiler.VisualStudio2022));
			}

			// ================================================================================
			// ThirdPartyNotUE
			// NOTE: these are Windows executables
			if (BuildHostPlatform.Current.Platform == UnrealTargetPlatform.Win64)
			{
				DirectoryReference ThirdPartyNotUERootDirectory = DirectoryReference.Combine(Unreal.RootDirectory, "Engine", "Extras", "ThirdPartyNotUE");
				string CMakePath = DirectoryReference.Combine(ThirdPartyNotUERootDirectory, "CMake", "bin").ToString();
				string MakePath = DirectoryReference.Combine(ThirdPartyNotUERootDirectory, "GNU_Make", "make-3.81", "bin").ToString();

				string PrevPath = Environment.GetEnvironmentVariable("PATH");
				// mixing bundled make and cygwin make is no good. Try to detect and remove cygwin paths.
				string PathWithoutCygwin = RemoveOtherMakeAndCygwinFromPath(PrevPath);
				Environment.SetEnvironmentVariable("PATH", CMakePath + ";" + MakePath + ";" + PathWithoutCygwin);
				Environment.SetEnvironmentVariable("PATH", CMakePath + ";" + MakePath + ";" + Environment.GetEnvironmentVariable("PATH"));
				Logger.LogInformation("set {Arg0}={Arg1}", "PATH", Environment.GetEnvironmentVariable("PATH"));
			}
		}
	}

	private static void BuildMSBuildTarget(TargetPlatformData TargetData, List<string> TargetConfigurations, WindowsCompiler TargetWindowsCompiler = WindowsCompiler.VisualStudio2022)
	{
		string SolutionFile = GetTargetLibSolutionFileName(TargetData, TargetWindowsCompiler).ToString();
		string MSDevExe = GetMsDevExe(TargetData);

		if (!FileExists(SolutionFile))
		{
			throw new AutomationException(String.Format("Unabled to build Solution {0}. Solution file not found.", SolutionFile));
		}
		if (String.IsNullOrEmpty(MSDevExe))
		{
			throw new AutomationException(String.Format("Unabled to build Solution {0}. devenv.com not found.", SolutionFile));
		}

		foreach (string BuildConfig in TargetConfigurations)
		{
			string CmdLine = String.Format("\"{0}\" /build \"{1}\"", SolutionFile, BuildConfig);
			RunAndLog(BuildCommand.CmdEnv, MSDevExe, CmdLine);
		}
	}

	private static void BuildMakefileTarget(TargetPlatformData TargetData, List<string> TargetConfigurations)
	{
		// FIXME: use absolute path
		string MakeCommand = "make";

		// FIXME: "j -16" should be tweakable
		//string MakeOptions = "-j 1";
		string MakeOptions = "-j 16";

		// Bundled GNU make does not pass job number to subprocesses on Windows, work around that...
		if (BuildHostPlatform.Current.Platform == UnrealTargetPlatform.Win64)
		{
			// Redefining the MAKE variable will cause the -j flag to be passed to child make instances.
			MakeOptions = string.Format("{1} \"MAKE={0} {1}\"", MakeCommand, MakeOptions);
		}

		// makefile build has "projects" for every configuration. However, we abstract away from that by assuming GetProjectDirectory points to the "meta-project"
		foreach (string BuildConfig in TargetConfigurations)
		{
			DirectoryReference MetaProjectDirectory = GetProjectDirectory(TargetData);
			DirectoryReference ConfigDirectory = DirectoryReference.Combine(MetaProjectDirectory, BuildConfig);
			string Makefile = FileReference.Combine(ConfigDirectory, "Makefile").ToString();
			if (!FileExists(Makefile))
			{
				throw new AutomationException(String.Format("Unabled to build {0} - file not found.", Makefile));
			}

			ProcessStartInfo StartInfo = new ProcessStartInfo();
			StartInfo.FileName = MakeCommand;
			StartInfo.WorkingDirectory = ConfigDirectory.ToString();
			StartInfo.Arguments = MakeOptions;

			Logger.LogInformation("Working in: {Arg0}", StartInfo.WorkingDirectory);
			Logger.LogInformation("{Arg0} {Arg1}", StartInfo.FileName, StartInfo.Arguments);

			if (RunLocalProcessAndLogOutput(StartInfo) != 0)
			{
				throw new AutomationException(String.Format("Unabled to build {0}. Build process failed.", Makefile));
			}
		}
	}

	private static void BuildXcodeTarget(TargetPlatformData TargetData, List<string> TargetConfigurations)
	{
		DirectoryReference Directory = GetProjectDirectory(TargetData);
		string ProjectName = "Hlslcc";

		string ProjectFile = FileReference.Combine(Directory, ProjectName + ".xcodeproj").ToString();

		if (!DirectoryExists(ProjectFile))
		{
			throw new AutomationException(String.Format("Unabled to build project {0}. Project file not found.", ProjectFile));
		}

		foreach (string BuildConfig in TargetConfigurations)
		{
			string CmdLine = String.Format("-project \"{0}\" -target=\"ALL_BUILD\" -configuration {1} -quiet", ProjectFile, BuildConfig);
			RunAndLog(BuildCommand.CmdEnv, "/usr/bin/xcodebuild", CmdLine);
		}
	}

	private static void BuildTargetLibForPlatform(TargetPlatformData TargetData, List<string> TargetConfigurations, List<WindowsCompiler> TargetWindowsCompilers)
	{
		if (DoesPlatformUseMSBuild(TargetData))
		{
			foreach (WindowsCompiler TargetWindowsCompiler in TargetWindowsCompilers)
			{
				BuildMSBuildTarget(TargetData, TargetConfigurations, TargetWindowsCompiler);
			}
		}
		else if (DoesPlatformUseXcode(TargetData))
		{
			BuildXcodeTarget(TargetData, TargetConfigurations);
		}
		else if (DoesPlatformUseMakefiles(TargetData))
		{
			BuildMakefileTarget(TargetData, TargetConfigurations);
		}
		else
		{
			throw new AutomationException(String.Format("Unsupported target platform '{0}' passed to BuildTargetLibForPlatform", TargetData));
		}
	}

	private static DirectoryReference GetPlatformLibDirectory(TargetPlatformData TargetData, WindowsCompiler TargetWindowsCompiler)
	{
		string VisualStudioName = string.Empty;
		string ArchName = string.Empty;

		if (DoesPlatformUseMSBuild(TargetData))
		{
			switch (TargetWindowsCompiler)
			{
				case WindowsCompiler.VisualStudio2022:
					VisualStudioName = "VS2015";
					break;
				default:
					throw new AutomationException(String.Format("Unsupported visual studio compiler '{0}' supplied to GetOutputLibDirectory", TargetWindowsCompiler));
			}
		}

		if (TargetData.Platform == UnrealTargetPlatform.Win64)
		{
			ArchName = "Win64";
		}
		else if (TargetData.Platform == UnrealTargetPlatform.Linux)
		{
			ArchName = "Linux/" + TargetData.Architecture;
		}
		else if (TargetData.Platform == UnrealTargetPlatform.Mac)
		{
			ArchName = "Mac";
		}
		else
		{
			throw new AutomationException(String.Format("Unsupported platform '{0}' supplied to GetOutputLibDirectory", TargetData.ToString()));
		}

		return DirectoryReference.Combine(RootOutputLibDirectory, ArchName, VisualStudioName);
	}

	private static string GetPlatformLibExtension(TargetPlatformData TargetData)
	{
		if (TargetData.Platform == UnrealTargetPlatform.Win64)
		{
			return "lib";
		}
		else if (TargetData.Platform == UnrealTargetPlatform.Linux || TargetData.Platform == UnrealTargetPlatform.Mac)
		{
			return "a";
		}

		throw new AutomationException(String.Format("No lib extension for platform '{0}'", TargetData.Platform.ToString()));
	}

	private static void FindOutputFilesHelper(HashSet<FileReference> OutputFiles, DirectoryReference BaseDir, string SearchPrefix)
	{
		if (!DirectoryReference.Exists(BaseDir))
		{
			return;
		}

		foreach (FileReference FoundFile in DirectoryReference.EnumerateFiles(BaseDir, SearchPrefix))
		{
			OutputFiles.Add(FoundFile);
		}
	}

	private static void FindOutputFiles(HashSet<FileReference> OutputFiles, TargetPlatformData TargetData, string TargetConfiguration, WindowsCompiler TargetWindowsCompiler = WindowsCompiler.VisualStudio2022)
	{
		string SearchSuffix = "";
		if (TargetConfiguration == "Debug")
		{
			SearchSuffix = "d";
		}
		if (TargetData.Platform == UnrealTargetPlatform.Win64)
		{
			SearchSuffix += "_64";
		}

		string SearchPrefix = "*" + SearchSuffix + ".";

		DirectoryReference LibDir = GetPlatformLibDirectory(TargetData, TargetWindowsCompiler);
		FindOutputFilesHelper(OutputFiles, LibDir, SearchPrefix + GetPlatformLibExtension(TargetData));

		if (TargetData.Platform == UnrealTargetPlatform.Win64)
		{
			FindOutputFilesHelper(OutputFiles, LibDir, SearchPrefix + "pdb");
		}
	}

	private static bool PlatformSupportsTargetLib(TargetPlatformData TargetData)
	{
		if (TargetData.Platform == UnrealTargetPlatform.Win64)
		{
			return BuildHostPlatform.Current.Platform == UnrealTargetPlatform.Win64;
		}
		else if (TargetData.Platform == UnrealTargetPlatform.Mac)
		{
			return BuildHostPlatform.Current.Platform == UnrealTargetPlatform.Mac;
		}
		else if (TargetData.Platform == UnrealTargetPlatform.Linux)
		{
			// only x86_64 Linux supports it.
			if (!TargetData.Architecture.StartsWith("x86_64"))
			{
				return false;
			}

			// cross and native compilation is supported
			return (BuildHostPlatform.Current.Platform == UnrealTargetPlatform.Win64 || BuildHostPlatform.Current.Platform == UnrealTargetPlatform.Linux);
		}

		return false;
	}

	public override void ExecuteBuild()
	{
		SetupStaticBuildEnvironment();

		bool bBuildSolutions = true;
		if (ParseParam("SkipBuildSolutions"))
		{
			bBuildSolutions = false;
		}

		bool bBuildLibraries = true;
		if (ParseParam("SkipBuild"))
		{
			bBuildLibraries = false;
		}

		bool bAutoCreateChangelist = true;
		if (ParseParam("SkipCreateChangelist"))
		{
			bAutoCreateChangelist = false;
		}

		bool bAutoSubmit = false; // bAutoCreateChangelist;
		if (ParseParam("SkipSubmit"))
		{
			bAutoSubmit = false;
		}

		// if we don't pass anything, we'll just merge by default
		string RobomergeCommand = ParseParamValue("Robomerge", "").ToLower();
		if (!string.IsNullOrEmpty(RobomergeCommand))
		{
			// for merge default action, add flag to make sure buildmachine commit isn't skipped
			if (RobomergeCommand == "merge")
			{
				RobomergeCommand = "#robomerge[all] #DisregardExcludedAuthors";
			}
			// otherwise add hashtags
			else if (RobomergeCommand == "ignore")
			{
				RobomergeCommand = "#robomerge #ignore";
			}
			else if (RobomergeCommand == "null")
			{
				RobomergeCommand = "#robomerge #null";
			}
			// otherwise the submit will likely fail.
			else
			{
				throw new AutomationException("Invalid Robomerge param passed in {0}.  Must be \"merge\", \"null\", or \"ignore\"", RobomergeCommand);
			}
		}

		// get the platforms we want to build for
		List<TargetPlatformData> TargetPlatforms = GetTargetPlatforms();

		// get the platforms we want to build for
		List<WindowsCompiler> TargetWindowsCompilers = GetTargetWindowsCompilers();

		// get the configurations we want to build for
		List<string> TargetConfigurations = GetTargetConfigurations();

		if (bBuildSolutions)
		{
			// build target lib for all platforms
			foreach (TargetPlatformData TargetData in TargetPlatforms)
			{
				if (!PlatformSupportsTargetLib(TargetData))
				{
					continue;
				}

				SetupBuildForTargetLibAndPlatform(TargetData, TargetConfigurations, TargetWindowsCompilers, false);
			}
		}

		HashSet<FileReference> FilesToReconcile = new HashSet<FileReference>();
		if (bBuildLibraries)
		{
			// build target lib for all platforms
			foreach (TargetPlatformData TargetData in TargetPlatforms)
			{
				if (!PlatformSupportsTargetLib(TargetData))
				{
					continue;
				}

				HashSet<FileReference> FilesToDelete = new HashSet<FileReference>();
				foreach (string TargetConfiguration in TargetConfigurations)
				{
					// Delete output files before building them
					if (TargetData.Platform == UnrealTargetPlatform.Win64)
					{
						foreach (WindowsCompiler TargetCompiler in TargetWindowsCompilers)
						{
							FindOutputFiles(FilesToDelete, TargetData, TargetConfiguration, TargetCompiler);
						}
					}
					else
					{
						FindOutputFiles(FilesToDelete, TargetData, TargetConfiguration);
					}
				}
				foreach (FileReference FileToDelete in FilesToDelete)
				{
					FilesToReconcile.Add(FileToDelete);
					InternalUtils.SafeDeleteFile(FileToDelete.ToString());
				}

				BuildTargetLibForPlatform(TargetData, TargetConfigurations, TargetWindowsCompilers);

				if (DoesPlatformUseMSBuild(TargetData))
				{
					foreach (WindowsCompiler TargetWindowsCompiler in TargetWindowsCompilers)
					{
						CopyLibsToFinalDestination(TargetData, TargetConfigurations, TargetWindowsCompiler);
					}
				}
				else
				{
					CopyLibsToFinalDestination(TargetData, TargetConfigurations);
				}
			}
		}

		int P4ChangeList = InvalidChangeList;
		if (bAutoCreateChangelist)
		{
			string RobomergeLine = string.Empty;
			if (!string.IsNullOrEmpty(RobomergeCommand))
			{
				RobomergeLine = Environment.NewLine + RobomergeCommand;
			}
			P4ChangeList = P4.CreateChange(P4Env.Client, "BuildHlslcc.Automation: Deploying hlslcc libs." + Environment.NewLine + "#rb none" + Environment.NewLine + "#lockdown Nick.Penwarden" + Environment.NewLine + "#tests none" + Environment.NewLine + "#jira none" + Environment.NewLine + "#okforgithub ignore" + RobomergeLine);
		}

		if (P4ChangeList != InvalidChangeList)
		{
			foreach (string TargetConfiguration in TargetConfigurations)
			{
				//Add any new files that p4 is not yet tracking.
				foreach (TargetPlatformData TargetData in TargetPlatforms)
				{
					if (!PlatformSupportsTargetLib(TargetData))
					{
						continue;
					}

					if (TargetData.Platform == UnrealTargetPlatform.Win64)
					{
						foreach (WindowsCompiler TargetCompiler in TargetWindowsCompilers)
						{
							FindOutputFiles(FilesToReconcile, TargetData, TargetConfiguration, TargetCompiler);
						}
					}
					else
					{
						FindOutputFiles(FilesToReconcile, TargetData, TargetConfiguration);
					}
				}
			}

			foreach (FileReference FileToReconcile in FilesToReconcile)
			{
				P4.Reconcile(P4ChangeList, FileToReconcile.ToString());
			}
		}

		if (bAutoSubmit && (P4ChangeList != InvalidChangeList))
		{
			if (!P4.TryDeleteEmptyChange(P4ChangeList))
			{
				Logger.LogInformation("{Text}", "Submitting changelist " + P4ChangeList.ToString());
				int SubmittedChangeList = InvalidChangeList;
				P4.Submit(P4ChangeList, out SubmittedChangeList);
			}
			else
			{
				Logger.LogInformation("Nothing to submit!");
			}
		}
	}

	private void CopyLibsToFinalDestination(TargetPlatformData TargetData, List<string> TargetConfigurations, WindowsCompiler TargetWindowsCompiler = WindowsCompiler.VisualStudio2022)
	{
		foreach (string TargetConfiguration in TargetConfigurations)
		{
			DirectoryReference OutputLibPath = GetPlatformLibDirectory(TargetData, TargetWindowsCompiler);
			string OutputLibName = "";
			if (TargetData.Platform == UnrealTargetPlatform.Linux || TargetData.Platform == UnrealTargetPlatform.Mac)
			{
				OutputLibName = "lib";
			}

			OutputLibName += "hlslcc";

			if (TargetConfiguration == "Debug")
			{
				OutputLibName += "d";
			}

			if (TargetData.Platform == UnrealTargetPlatform.Win64)
			{
				OutputLibName += "_64";
			}

			OutputLibName += "." + GetPlatformLibExtension(TargetData);

			DirectoryReference ProjectDirectory = GetProjectDirectory(TargetData, TargetWindowsCompiler);
			string SourceFileName = "";
			if (TargetData.Platform == UnrealTargetPlatform.Win64)
			{
				SourceFileName = "hlslcc.lib";
			}
			else if (TargetData.Platform == UnrealTargetPlatform.Linux || TargetData.Platform == UnrealTargetPlatform.Mac)
			{
				SourceFileName = "libhlslcc.a";
			}

			FileReference SourceFile = FileReference.Combine(ProjectDirectory, TargetConfiguration, SourceFileName);
			FileReference DestFile = FileReference.Combine(OutputLibPath, OutputLibName);
			FileReference.Copy(SourceFile, DestFile);

			if (TargetData.Platform == UnrealTargetPlatform.Win64)
			{
				FileReference PdbSourceFile = SourceFile.ChangeExtension("pdb");
				FileReference PdbDestFile = DestFile.ChangeExtension("pdb");
				FileReference.Copy(PdbSourceFile, PdbDestFile);
			}
		}
	}
}
