// Copyright Epic Games, Inc. All Rights Reserved.
using System;
using System.Collections.Generic;
using System.IO;
using System.Linq;
using System.Reflection;
using AutomationTool;
using UnrealBuildTool;
using EpicGames.Core;
using System.Text;
using UnrealBuildBase;
using Microsoft.Extensions.Logging;

[Help("Builds a plugin, and packages it for distribution")]
[Help("Plugin", "Specify the path to the descriptor file for the plugin that should be packaged")]
[Help("Dependencies", "Specify the path to the plugin descriptor file for a dependency of the Plugin argument - only required if it is not accessible in the Engine/Plugins directory. Can be specified multiple times, one per dependency.")]
[Help("NoHostPlatform", "Prevent compiling for the editor platform on the host")]
[Help("HostPlatforms", "Specify a list of host platforms to build, separated by '+' characters (eg. -HostPlatforms=Win32+Win64). Default is the current host platforms")]
[Help("TargetPlatforms", "Specify a list of target platforms to build, separated by '+' characters (eg. -TargetPlatforms=Win32+Win64). Default is all the Rocket target platforms.")]
[Help("Package", "The path which the build artifacts should be packaged to, ready for distribution.")]
[Help("PackageAppendPluginSubdir", "Package the build artifacts in a subfolder named after the plugin.")]
[Help("StrictIncludes", "Disables precompiled headers and unity build in order to check all source files have self-contained headers.")]
[Help("EngineDir=<RootDirectory>", "Root Directory of the engine that will be used to build plugin(s) (optional)")]
[Help("Unversioned", "Do not embed the current engine version into the descriptor")]
[Help("Architecture_<Platform>=<Architecture[s]>", "Control architecture to compile for a platform (eg. -Architecture_Mac=arm64+x86). Default is to use UBT defaults for the platform.")]
public sealed class BuildPlugin : BuildCommand
{
	const string MacDefaultArchitectures = "arm64+x64";
	const string AndroidDefaultArchitectures = "arm64+x64";

	string UnrealBuildToolDllRelativePath = @"Engine/Binaries/DotNET/UnrealBuildTool/UnrealBuildTool.dll";
	FileReference UnrealBuildToolDll;
	static private Dictionary<UnrealTargetPlatform, string> PlatformToArchitectureMap = new Dictionary<UnrealTargetPlatform, string>();

	public override void ExecuteBuild()
	{
		// See if an engine dir was specified, fall back on default if not.
		DirectoryReference EngineDirParam = ParseOptionalDirectoryReferenceParam("EngineDir");
		UnrealBuildToolDll = EngineDirParam == null ? UnrealBuild.UnrealBuildToolDll : new(CommandUtils.CombinePaths(EngineDirParam.ToString(), UnrealBuildToolDllRelativePath));

		// Get the plugin filename
		string PluginParam = ParseParamValue("Plugin");
		if(PluginParam == null)
		{
			throw new AutomationException("Missing -Plugin=... argument");
		}

		// Check it exists
		FileReference PluginFile = new FileReference(PluginParam);
		if (!FileReference.Exists(PluginFile))
		{
			throw new AutomationException("Plugin '{0}' not found", PluginFile.FullName);
		}

		// Get the output directory
		string PackageParam = ParseParamValue("Package");
		if (PackageParam == null)
		{
			throw new AutomationException("Missing -Package=... argument");
		}

		string[] Dependencies = ParseParamValues("Dependencies");
		List<FileReference> DependencyPlugins = new(Dependencies.Length);
		foreach (string Dependency in Dependencies)
		{
			FileReference DependencyPlugin = new FileReference(Dependency);
			if (!FileReference.Exists(DependencyPlugin))
			{
				throw new AutomationException("Dependency plugin '{0}' not found", DependencyPlugin);
			}
			DependencyPlugins.Add(DependencyPlugin);
		}

		// Option for verifying that all include directive s
		bool bStrictIncludes = ParseParam("StrictIncludes");

		// Make sure the packaging directory is valid
		DirectoryReference PackageDir = new DirectoryReference(PackageParam);
		
		// If specified, package the build artifacts in a subfolder named after the initial plugin directory. (ie. MyPlugin/Manifest.uplugin will be packaged under {PackageDir}/MyPlufin)
		if (ParseParam("PackageAppendPluginSubdir"))
		{
			string PluginSubfolder = PluginFile.Directory.GetDirectoryName();
			PackageDir = DirectoryReference.Combine(PackageDir, PluginSubfolder);
		}
		
		if (PluginFile.IsUnderDirectory(PackageDir))
		{
			throw new AutomationException("Packaged plugin output directory must be different to source");
		}
		if (PackageDir.IsUnderDirectory(DirectoryReference.Combine(Unreal.RootDirectory, "Engine")))
		{
			throw new AutomationException("Output directory for packaged plugin must be outside engine directory");
		}

		// Clear the output directory of existing stuff
		if (DirectoryReference.Exists(PackageDir))
		{
			CommandUtils.DeleteDirectoryContents(PackageDir.FullName);
		}
		else
		{
			DirectoryReference.CreateDirectory(PackageDir);
		}

		// Create a placeholder FilterPlugin.ini with instructions on how to use it
		FileReference SourceFilterFile = FileReference.Combine(PluginFile.Directory, "Config", "FilterPlugin.ini");
		if (!FileReference.Exists(SourceFilterFile))
		{
			List<string> Lines = new List<string>();
			Lines.Add("[FilterPlugin]");
			Lines.Add("; This section lists additional files which will be packaged along with your plugin. Paths should be listed relative to the root plugin directory, and");
			Lines.Add("; may include \"...\", \"*\", and \"?\" wildcards to match directories, files, and individual characters respectively.");
			Lines.Add(";");
			Lines.Add("; Examples:");
			Lines.Add(";    /README.txt");
			Lines.Add(";    /Extras/...");
			Lines.Add(";    /Binaries/ThirdParty/*.dll");
			DirectoryReference.CreateDirectory(SourceFilterFile.Directory);
			CommandUtils.WriteAllLines_NoExceptions(SourceFilterFile.FullName, Lines.ToArray());
		}

		// Create a host project for the plugin. For script generator plugins, we need to have UHT be able to load it, which can only happen if it's enabled in a project.
		FileReference HostProjectFile = FileReference.Combine(PackageDir, "HostProject", "HostProject.uproject");
		FileReference HostProjectPluginFile = CreateHostProject(HostProjectFile, PluginFile, DependencyPlugins);

		// Read the plugin
		Logger.LogInformation("Reading plugin from {HostProjectPluginFile}...", HostProjectPluginFile);
		PluginDescriptor Plugin = PluginDescriptor.FromFile(HostProjectPluginFile);

		// Get the arguments for the compile
		StringBuilder AdditionalArgs = new StringBuilder();
		if (bStrictIncludes)
		{
			Logger.LogInformation("Building with precompiled headers and unity disabled");
			AdditionalArgs.Append(" -NoPCH -NoSharedPCH -DisableUnity");
		}

		// check if any architectures were specified
		foreach (UnrealTargetPlatform Platform in UnrealTargetPlatform.GetValidPlatforms())
		{
			// by default, don't specify any architecture when building (unless user requested with -architecture_Platform=), except for
			// any special cases set at the top of this file
			string DefaultValue = null;
			if (Platform == UnrealTargetPlatform.Mac)
			{
				DefaultValue = MacDefaultArchitectures;
			}
			else if (Platform == UnrealTargetPlatform.Android)
			{
				DefaultValue = AndroidDefaultArchitectures;
			}
			PlatformToArchitectureMap[Platform] = ParseParamValue($"architecture_{Platform}", DefaultValue);
		}

		// Compile the plugin for all the target platforms
		IReadOnlyList<UnrealTargetPlatform> HostPlatforms = GetHostPlatforms(this);
		List<UnrealTargetPlatform> TargetPlatforms = GetTargetPlatforms(this, BuildHostPlatform.Current.Platform);
		FileReference[] BuildProducts = CompilePlugin(UnrealBuildToolDll, HostProjectFile, HostProjectPluginFile, Plugin, HostPlatforms, TargetPlatforms, AdditionalArgs.ToString());

		// Package up the final plugin data
		PackagePlugin(HostProjectPluginFile, BuildProducts, PackageDir, ParseParam("unversioned"), TargetPlatforms);

		// Remove the host project
		if(!ParseParam("NoDeleteHostProject"))
		{
			CommandUtils.DeleteDirectory(HostProjectFile.Directory.FullName);
		}
	}

	private FileReference CreateHostProject(FileReference HostProjectFile, FileReference PluginFile, IReadOnlyList<FileReference> DependencyPlugins)
	{
		DirectoryReference HostProjectDir = HostProjectFile.Directory;
		DirectoryReference.CreateDirectory(HostProjectDir);

		// Create the new project descriptor
		StringBuilder PluginsArray = new($"{{ \"Name\": \"{PluginFile.GetFileNameWithoutExtension()}\", \"Enabled\": true }}");
		foreach (FileReference DependencyPlugin in DependencyPlugins)
		{
			PluginsArray.Append($", {{ \"Name\": \"{DependencyPlugin.GetFileNameWithoutExtension()}\", \"Enabled\": true }}");
		}

		File.WriteAllText(HostProjectFile.FullName, $"{{ \"FileVersion\": 3, \"Plugins\": [ {PluginsArray} ] }}");

		// Get the plugin directory in the host project, and copy all the files in
		DirectoryReference HostProjectPluginDir = DirectoryReference.Combine(HostProjectDir, "Plugins", PluginFile.GetFileNameWithoutExtension());
		CommandUtils.ThreadedCopyFiles(PluginFile.Directory.FullName, HostProjectPluginDir.FullName);
		CommandUtils.DeleteDirectory(true, DirectoryReference.Combine(HostProjectPluginDir, "Intermediate").FullName);

		foreach (FileReference DependencyPlugin in DependencyPlugins)
		{
			DirectoryReference HostProjectDependencyDir = DirectoryReference.Combine(HostProjectDir, "Plugins",  DependencyPlugin.GetFileNameWithoutExtension());
			CommandUtils.ThreadedCopyFiles(DependencyPlugin.Directory.FullName, HostProjectDependencyDir.FullName);
			CommandUtils.DeleteDirectory(true, DirectoryReference.Combine(HostProjectDependencyDir, "Intermediate").FullName);
		}

		// Return the path to the plugin file in the host project
		return FileReference.Combine(HostProjectPluginDir, PluginFile.GetFileName());
	}

	public abstract class TargetPlatform : CommandUtils
	{
		[Obsolete("Deprecated in UE5.1; function signature changed")]
		public abstract void CompilePluginWithUBT(string UBTExe, FileReference HostProjectFile, FileReference HostProjectPluginFile, PluginDescriptor Plugin, string TargetName, TargetType TargetType, UnrealTargetPlatform Platform, UnrealTargetConfiguration Configuration, List<FileReference> ManifestFileNames, string InAdditionalArgs);

		public abstract void CompilePluginWithUBT(FileReference UnrealBuildToolDll, FileReference HostProjectFile, FileReference HostProjectPluginFile, PluginDescriptor Plugin, string TargetName, TargetType TargetType, UnrealTargetPlatform Platform, UnrealTargetConfiguration Configuration, List<FileReference> ManifestFileNames, string InAdditionalArgs);

	};

	private static TargetPlatform GetTargetPlatform( UnrealTargetPlatform Platform )
	{
		// Grab all the non-abstract subclasses of TargetPlatform from the executing assembly.
		var AvailablePlatformTypes = from Assembly in ScriptManager.AllScriptAssemblies
									 from Type in Assembly.GetTypes()
									 where !Type.IsAbstract && Type.IsAssignableTo(typeof(TargetPlatform))
									 select Type;

		var PlatformTypeMap = new Dictionary<string, Type>();

		foreach (var Type in AvailablePlatformTypes)
		{
			int Index = Type.Name.IndexOf('_');
			if (Index == -1)
			{
				throw new BuildException("Invalid BuildPluginCommand target platform type found: {0}", Type);
			}

			PlatformTypeMap.Add(Type.Name, Type);
		}

		var SelectedPlatform = $"BuildPlugin_{Platform.ToString()}";
		if (!PlatformTypeMap.ContainsKey(SelectedPlatform))
		{
			return null;
		}

		var SelectedType = PlatformTypeMap[SelectedPlatform];
		TargetPlatform TargetPlatform = (TargetPlatform)Activator.CreateInstance(SelectedType);
		if (TargetPlatform == null)
		{
			throw new BuildException("The target platform \"{0}\" could not be constructed.", SelectedPlatform);
		}

		return TargetPlatform;
	}



	[Obsolete("Deprecated in UE5.1; function signature changed")]
	public static FileReference[] CompilePlugin(string UBTExe, FileReference HostProjectFile, FileReference HostProjectPluginFile, PluginDescriptor Plugin, List<UnrealTargetPlatform> HostPlatforms, List<UnrealTargetPlatform> TargetPlatforms, string AdditionalArgs = "")
	{
		List<FileReference> ManifestFileNames = new List<FileReference>();

		// Build the host platforms
		if(HostPlatforms.Count > 0)
		{
			Logger.LogInformation("Building plugin for host platforms: {Arg0}", String.Join(", ", HostPlatforms));
			foreach (UnrealTargetPlatform HostPlatform in HostPlatforms)
			{
				CompilePluginWithUBT(UBTExe, HostProjectFile, HostProjectPluginFile, Plugin, "UnrealEditor", TargetType.Editor, HostPlatform, UnrealTargetConfiguration.Development, ManifestFileNames, AdditionalArgs);
			}
		}

		// Add the supported game targets
		if(TargetPlatforms.Count > 0)
		{
			List<UnrealTargetPlatform> SupportedTargetPlatforms = TargetPlatforms.FindAll(Plugin.SupportsTargetPlatform);
			Logger.LogInformation("Building plugin for target platforms: {Arg0}", String.Join(", ", SupportedTargetPlatforms));
			foreach (UnrealTargetPlatform TargetPlatform in SupportedTargetPlatforms)
			{
				string AdditionalTargetArgs = AdditionalArgs;
				CompilePluginWithUBT(UBTExe, HostProjectFile, HostProjectPluginFile, Plugin, "UnrealGame", TargetType.Game, TargetPlatform, UnrealTargetConfiguration.Development, ManifestFileNames, AdditionalTargetArgs);
				CompilePluginWithUBT(UBTExe, HostProjectFile, HostProjectPluginFile, Plugin, "UnrealGame", TargetType.Game, TargetPlatform, UnrealTargetConfiguration.Shipping, ManifestFileNames, AdditionalTargetArgs);
			}
		}

		// Package the plugin to the output folder
		HashSet<FileReference> BuildProducts = new HashSet<FileReference>();
		foreach(FileReference ManifestFileName in ManifestFileNames)
		{
			BuildManifest Manifest = CommandUtils.ReadManifest(ManifestFileName);
			BuildProducts.UnionWith(Manifest.BuildProducts.Select(x => new FileReference(x)));
		}
		return BuildProducts.ToArray();
	}

	public static FileReference[] CompilePlugin(FileReference UnrealBuildToolDll, FileReference HostProjectFile, FileReference HostProjectPluginFile, PluginDescriptor Plugin, IReadOnlyList<UnrealTargetPlatform> HostPlatforms, List<UnrealTargetPlatform> TargetPlatforms, string AdditionalArgs = "")
	{
		List<FileReference> ManifestFileNames = new List<FileReference>();

		// Build the host platforms
		if(HostPlatforms.Count > 0)
		{
			Logger.LogInformation("Building plugin for host platforms: {Arg0}", String.Join(", ", HostPlatforms));
			foreach (UnrealTargetPlatform HostPlatform in HostPlatforms)
			{
				CompilePluginWithUBT(UnrealBuildToolDll, HostProjectFile, HostProjectPluginFile, Plugin, "UnrealEditor", TargetType.Editor, HostPlatform, UnrealTargetConfiguration.Development, ManifestFileNames, AdditionalArgs);
			}
		}

		// Add the supported game targets
		if(TargetPlatforms.Count > 0)
		{
			List<UnrealTargetPlatform> SupportedTargetPlatforms = TargetPlatforms.FindAll(Plugin.SupportsTargetPlatform);
			Logger.LogInformation("Building plugin for target platforms: {Arg0}", String.Join(", ", SupportedTargetPlatforms));
			foreach (UnrealTargetPlatform TargetPlatform in SupportedTargetPlatforms)
			{
				string AdditionalTargetArgs = AdditionalArgs;
				CompilePluginWithUBT(UnrealBuildToolDll, HostProjectFile, HostProjectPluginFile, Plugin, "UnrealGame", TargetType.Game, TargetPlatform, UnrealTargetConfiguration.Development, ManifestFileNames, AdditionalTargetArgs);
				CompilePluginWithUBT(UnrealBuildToolDll, HostProjectFile, HostProjectPluginFile, Plugin, "UnrealGame", TargetType.Game, TargetPlatform, UnrealTargetConfiguration.Shipping, ManifestFileNames, AdditionalTargetArgs);
			}
		}

		// Package the plugin to the output folder
		HashSet<FileReference> BuildProducts = new HashSet<FileReference>();
		foreach(FileReference ManifestFileName in ManifestFileNames)
		{
			BuildManifest Manifest = CommandUtils.ReadManifest(ManifestFileName);
			BuildProducts.UnionWith(Manifest.BuildProducts.Select(x => new FileReference(x)));
		}
		return BuildProducts.ToArray();
	}

	[Obsolete("Deprecated in UE5.1; function signature has changed")]
	static void CompilePluginWithUBT(string UBTExe, FileReference HostProjectFile, FileReference HostProjectPluginFile, PluginDescriptor Plugin, string TargetName, TargetType TargetType, UnrealTargetPlatform Platform, UnrealTargetConfiguration Configuration, List<FileReference> ManifestFileNames, string InAdditionalArgs)
	{
		// Find a list of modules that need to be built for this plugin
		bool bCompilePlatform = false;
		if (Plugin.Modules != null)
		{
			bool bBuildDeveloperTools = (TargetType == TargetType.Editor || TargetType == TargetType.Program || (Configuration != UnrealTargetConfiguration.Test && Configuration != UnrealTargetConfiguration.Shipping));
			bool bBuildRequiresCookedData = (TargetType != TargetType.Editor && TargetType != TargetType.Program);
			UnrealArchitectures Architectures = PlatformToArchitectureMap.TryGetValue(Platform, out string SpecifiedArchitecture) ? UnrealArchitectures.FromString(SpecifiedArchitecture, null) : null;

			foreach (ModuleDescriptor Module in Plugin.Modules)
			{
				if (Module.IsCompiledInConfiguration(Platform, Configuration, TargetName, TargetType, bBuildDeveloperTools, bBuildRequiresCookedData, Architectures))
				{
					bCompilePlatform = true;
				}
			}
		}

		// Add these modules to the build agenda
		if(bCompilePlatform)
		{
			TargetPlatform TargetPlatform = GetTargetPlatform(Platform);
			if (TargetPlatform != null)
			{
				TargetPlatform.CompilePluginWithUBT(UBTExe, HostProjectFile, HostProjectPluginFile, Plugin, TargetName, TargetType, Platform, Configuration, ManifestFileNames, InAdditionalArgs );
			}
			else
			{
				FileReference ManifestFileName = FileReference.Combine(HostProjectFile.Directory, "Saved", String.Format("Manifest-{0}-{1}-{2}.xml", TargetName, Platform, Configuration));
				ManifestFileNames.Add(ManifestFileName);
				
				string Arguments = String.Format("-plugin={0} -noubtmakefiles -manifest={1} -nohotreload", CommandUtils.MakePathSafeToUseWithCommandLine(HostProjectPluginFile.FullName), CommandUtils.MakePathSafeToUseWithCommandLine(ManifestFileName.FullName));

				if (!String.IsNullOrEmpty(InAdditionalArgs))
				{
					Arguments += InAdditionalArgs;
				}

				CommandUtils.RunUBT(CmdEnv, UBTExe, HostProjectFile, TargetName, Platform, Configuration, Arguments);
			}
		}
	}

	static void CompilePluginWithUBT(FileReference UnrealBuildToolDll, FileReference HostProjectFile, FileReference HostProjectPluginFile, PluginDescriptor Plugin, string TargetName, TargetType TargetType, UnrealTargetPlatform Platform, UnrealTargetConfiguration Configuration, List<FileReference> ManifestFileNames, string InAdditionalArgs)
	{
		// Find a list of modules that need to be built for this plugin
		bool bCompilePlatform = false;
		if (Plugin.Modules != null)
		{
			bool bBuildDeveloperTools = (TargetType == TargetType.Editor || TargetType == TargetType.Program || (Configuration != UnrealTargetConfiguration.Test && Configuration != UnrealTargetConfiguration.Shipping));
			bool bBuildRequiresCookedData = (TargetType != TargetType.Editor && TargetType != TargetType.Program);
			UnrealArchitectures Architectures = PlatformToArchitectureMap.TryGetValue(Platform, out string SpecifiedArchitecture) ? UnrealArchitectures.FromString(SpecifiedArchitecture, null) : null;

			foreach (ModuleDescriptor Module in Plugin.Modules)
			{
				if (Module.IsCompiledInConfiguration(Platform, Configuration, TargetName, TargetType, bBuildDeveloperTools, bBuildRequiresCookedData, Architectures))
				{
					bCompilePlatform = true;
				}
			}
		}

		// Add these modules to the build agenda
		if(bCompilePlatform)
		{
			TargetPlatform TargetPlatform = GetTargetPlatform(Platform);

			if (TargetPlatform != null)
			{
				TargetPlatform.CompilePluginWithUBT(UnrealBuildToolDll, HostProjectFile, HostProjectPluginFile, Plugin, TargetName, TargetType, Platform, Configuration, ManifestFileNames, InAdditionalArgs);
			}
			else
			{
				FileReference ManifestFileName = FileReference.Combine(HostProjectFile.Directory, "Saved", String.Format("Manifest-{0}-{1}-{2}.xml", TargetName, Platform, Configuration));
				ManifestFileNames.Add(ManifestFileName);
				
				string Arguments = String.Format("-plugin={0} -noubtmakefiles -manifest={1} -nohotreload", CommandUtils.MakePathSafeToUseWithCommandLine(HostProjectPluginFile.FullName), CommandUtils.MakePathSafeToUseWithCommandLine(ManifestFileName.FullName));

				if (PlatformToArchitectureMap.TryGetValue(Platform, out string SpecifiedArchitecture) && !string.IsNullOrEmpty(SpecifiedArchitecture))
				{
					Arguments += String.Format(" -architecture={0}", SpecifiedArchitecture);
				}

				if (!String.IsNullOrEmpty(InAdditionalArgs))
				{
					Arguments += InAdditionalArgs;
				}

				CommandUtils.RunUBT(CmdEnv, UnrealBuildToolDll, HostProjectFile, TargetName, Platform, Configuration, Arguments);
			}
		}
	}

	public static void PackagePlugin(FileReference SourcePluginFile, IEnumerable<FileReference> BuildProducts, DirectoryReference TargetDir, bool bUnversioned, IEnumerable<UnrealTargetPlatform> TargetPlatforms)
	{
		DirectoryReference SourcePluginDir = SourcePluginFile.Directory;

		// Copy all the files to the output directory
		FileReference[] SourceFiles = FilterPluginFiles(SourcePluginFile, BuildProducts, TargetPlatforms).ToArray();
		foreach(FileReference SourceFile in SourceFiles)
		{
			FileReference TargetFile = FileReference.Combine(TargetDir, SourceFile.MakeRelativeTo(SourcePluginDir));
			CommandUtils.CopyFile(SourceFile.FullName, TargetFile.FullName);
			CommandUtils.SetFileAttributes(TargetFile.FullName, ReadOnly: false);
		}

		// Get the output plugin filename
		FileReference TargetPluginFile = FileReference.Combine(TargetDir, SourcePluginFile.GetFileName());
		PluginDescriptor NewDescriptor = PluginDescriptor.FromFile(TargetPluginFile);
		NewDescriptor.bEnabledByDefault = null;
		NewDescriptor.bInstalled = true;
		if(!bUnversioned)
		{
			BuildVersion Version;
			if(BuildVersion.TryRead(BuildVersion.GetDefaultFileName(), out Version))
			{
				NewDescriptor.EngineVersion = String.Format("{0}.{1}.0", Version.MajorVersion, Version.MinorVersion);
			}
		}
		NewDescriptor.Save(TargetPluginFile.FullName);
	}

	static void AddRulesFromFileToFilter(FileFilter Filter, FileReference FilterFile)
	{
		if (FileReference.Exists(FilterFile))
		{
			Logger.LogInformation("Reading filter rules from {FilterFile}", FilterFile);
			Filter.ReadRulesFromFile(FilterFile, "FilterPlugin");
		}
	}

	static IEnumerable<FileReference> FilterPluginFiles(FileReference PluginFile, IEnumerable<FileReference> BuildProducts, IEnumerable<UnrealTargetPlatform> TargetPlatforms)
	{
		// Set up the default filter
		FileFilter Filter = new FileFilter();
		Filter.AddRuleForFile(PluginFile, PluginFile.Directory, FileFilterType.Include);
		Filter.AddRuleForFiles(BuildProducts, PluginFile.Directory, FileFilterType.Include);
		Filter.Include("/Binaries/ThirdParty/...");
		Filter.Include("/Resources/...");
		Filter.Include("/Content/...");
		Filter.Include("/Intermediate/Build/.../Inc/...");
		Filter.Include("/Shaders/...");
		Filter.Include("/Source/...");
		Filter.Exclude("/Tests/...");

		// Add custom rules for all platforms
		AddRulesFromFileToFilter(Filter, FileReference.Combine(PluginFile.Directory, "Config", "FilterPlugin.ini"));

		// Add custom rules for targeted platforms.
		foreach (UnrealTargetPlatform Platform in TargetPlatforms)
		{
			Logger.LogInformation("Loading FilterPlugin{Platform}.ini", Platform);
			AddRulesFromFileToFilter(Filter, FileReference.Combine(PluginFile.Directory, "Config", $"FilterPlugin{Platform}.ini"));
		}

		// Apply the standard exclusion rules
		foreach (string RestrictedFolderName in RestrictedFolder.GetNames())
		{
			Filter.AddRule(String.Format(".../{0}/...", RestrictedFolderName), FileFilterType.Exclude);
		}

		// Apply the filter to the plugin directory
		return Filter.ApplyToDirectory(PluginFile.Directory, true);
	}

	static List<UnrealTargetPlatform> GetTargetPlatforms(BuildCommand Command, UnrealTargetPlatform HostPlatform)
	{
		List<UnrealTargetPlatform> TargetPlatforms = new List<UnrealTargetPlatform>();
		if(!Command.ParseParam("NoTargetPlatforms"))
		{
			// Only interested in building for Platforms that support code projects
			TargetPlatforms = PlatformExports.GetRegisteredPlatforms().Where(x => InstalledPlatformInfo.IsValidPlatform(x, EProjectType.Code)).ToList();

			// only build Mac on Mac
			if (HostPlatform != UnrealTargetPlatform.Mac && TargetPlatforms.Contains(UnrealTargetPlatform.Mac))
			{
				TargetPlatforms.Remove(UnrealTargetPlatform.Mac);
			}
			// only build Windows on Windows
			if (HostPlatform != UnrealTargetPlatform.Win64 && TargetPlatforms.Contains(UnrealTargetPlatform.Win64))
			{
				TargetPlatforms.Remove(UnrealTargetPlatform.Win64);
			}
			// build Linux on Windows and Linux
			if (HostPlatform != UnrealTargetPlatform.Win64 && HostPlatform != UnrealTargetPlatform.Linux)
			{
				if (TargetPlatforms.Contains(UnrealTargetPlatform.Linux))
					TargetPlatforms.Remove(UnrealTargetPlatform.Linux);

				if (TargetPlatforms.Contains(UnrealTargetPlatform.LinuxArm64))
					TargetPlatforms.Remove(UnrealTargetPlatform.LinuxArm64);
			}

			// Remove any platforms that aren't enabled on the command line
			string TargetPlatformFilter = Command.ParseParamValue("TargetPlatforms", null);
			if(TargetPlatformFilter != null)
			{
				List<UnrealTargetPlatform> NewTargetPlatforms = new List<UnrealTargetPlatform>();
				foreach (string TargetPlatformName in TargetPlatformFilter.Split(new char[]{ '+' }, StringSplitOptions.RemoveEmptyEntries))
				{
					UnrealTargetPlatform TargetPlatform;
					if (!UnrealTargetPlatform.TryParse(TargetPlatformName, out TargetPlatform))
					{
						throw new AutomationException("Unknown target platform '{0}' specified on command line", TargetPlatformName);
					}
					if(TargetPlatforms.Contains(TargetPlatform))
					{
						NewTargetPlatforms.Add(TargetPlatform);
					}
				}
				TargetPlatforms = NewTargetPlatforms;
			}
		}
		return TargetPlatforms;
	}

	static IReadOnlyList<UnrealTargetPlatform> GetHostPlatforms(BuildCommand Command)
	{
		if (Command.ParseParam("NoHostPlatform"))
		{
			return Array.Empty<UnrealTargetPlatform>();
		}
		var CurrentPlatform = BuildHostPlatform.Current.Platform;
		string HostPlatformFilter = Command.ParseParamValue("HostPlatforms", null);
		if (HostPlatformFilter == null)
		{
			return new[] { CurrentPlatform };
		}
		// Only interested in building for Platforms that support code projects
		HashSet<UnrealTargetPlatform> SupportedHostPlatforms = PlatformExports.GetRegisteredPlatforms().Where(x => InstalledPlatformInfo.IsValidPlatform(x, EProjectType.Code)).ToHashSet();

		// only build Mac on Mac
		if (CurrentPlatform != UnrealTargetPlatform.Mac && SupportedHostPlatforms.Contains(UnrealTargetPlatform.Mac))
		{
			SupportedHostPlatforms.Remove(UnrealTargetPlatform.Mac);
		}
		// only build Windows on Windows
		if (CurrentPlatform != UnrealTargetPlatform.Win64 && SupportedHostPlatforms.Contains(UnrealTargetPlatform.Win64))
		{
			SupportedHostPlatforms.Remove(UnrealTargetPlatform.Win64);
		}
		// build Linux on Windows and Linux
		if (CurrentPlatform != UnrealTargetPlatform.Win64 && CurrentPlatform != UnrealTargetPlatform.Linux)
		{
			SupportedHostPlatforms.Remove(UnrealTargetPlatform.Linux);
			SupportedHostPlatforms.Remove(UnrealTargetPlatform.LinuxArm64);
		}

		List<UnrealTargetPlatform> NewHostPlatforms = new List<UnrealTargetPlatform>();
		foreach (string HostPlatformName in HostPlatformFilter.Split(new char[] { '+' }, StringSplitOptions.RemoveEmptyEntries))
		{
			UnrealTargetPlatform HostPlatform;
			if (!UnrealTargetPlatform.TryParse(HostPlatformName, out HostPlatform))
			{
				throw new AutomationException("Unknown host platform '{0}' specified on command line", HostPlatformName);
			}
			if (SupportedHostPlatforms.Contains(HostPlatform))
			{
				NewHostPlatforms.Add(HostPlatform);
			}
			else
			{
				Logger.LogWarning("HostPlatform {HostPlatformName} not supported on current platform {CurrentPlatform}", HostPlatformName, CurrentPlatform);
			}
		}
		return NewHostPlatforms;
	}
}
