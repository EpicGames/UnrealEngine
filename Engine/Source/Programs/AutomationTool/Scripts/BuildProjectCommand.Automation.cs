// Copyright Epic Games, Inc. All Rights Reserved.
using System;
using System.Collections.Generic;
using System.IO;
using System.Threading;
using System.Reflection;
using System.Linq;
using AutomationTool;
using UnrealBuildTool;
using EpicGames.Core;
using UnrealBuildBase;
using Microsoft.Extensions.Logging;

[Flags]
public enum ProjectBuildTargets
{
	None = 0,
	Editor = 1 << 0,
	ClientCooked = 1 << 1,
	ServerCooked = 1 << 2,
	Bootstrap = 1 << 3,
	CrashReporter = 1 << 4,
	Programs = 1 << 5,
	UnrealPak = 1 << 6,

	// All targets
	All = Editor | ClientCooked | ServerCooked | Bootstrap | CrashReporter | Programs | UnrealPak,
}

namespace AutomationScripts
{
	/// <summary>
	/// Helper command used for compiling.
	/// </summary>
	/// <remarks>
	/// Command line params used by this command:
	/// -cooked
	/// -cookonthefly
	/// -clean
	/// -[platform]
	/// </remarks>
	public partial class Project : CommandUtils
	{
		/// <summary>
		/// PlatformSupportsCrashReporter
		/// </summary>
		/// <param name="InPlatform">The platform of interest</param>
		/// <returns>True if the given platform supports a crash reporter client (i.e. it can be built for it)</returns>
		static public bool PlatformSupportsCrashReporter(UnrealTargetPlatform InPlatform)
		{
			return (
				(InPlatform == UnrealTargetPlatform.Win64) ||
				(InPlatform == UnrealTargetPlatform.Linux) ||
				(InPlatform == UnrealTargetPlatform.LinuxArm64) ||
				(InPlatform == UnrealTargetPlatform.Mac)
				);
		}


		public static void Build(BuildCommand Command, ProjectParams Params, int WorkingCL = -1, ProjectBuildTargets TargetMask = ProjectBuildTargets.All)
		{
			Params.ValidateAndLog();

			if (!Params.Build)
			{
				return;
			}
			if (Unreal.IsEngineInstalled() && !Params.IsCodeBasedProject)
			{
				return;
			}

			Logger.LogInformation("********** BUILD COMMAND STARTED **********");
			var StartTime = DateTime.UtcNow;

			var UnrealBuild = new UnrealBuild(Command);
			var NoProjectAgenda = new UnrealBuild.BuildAgenda();
			// if there's a project, make a new Agenda. we will build one or both at the end
			var ProjectAgenda = Params.CodeBasedUprojectPath == null ? NoProjectAgenda : new UnrealBuild.BuildAgenda();
			var CrashReportPlatforms = new HashSet<UnrealTargetPlatform>();


			Func<UnrealArchitectures, string> GetArchString = Arch => (Arch == null) ? "" : $"-architecture={Arch}";
			string ServerBuildArgs = Params.AdditionalBuildOptions + GetArchString(Params.ServerArchitecture);
			string EditorBuildArgs = Params.AdditionalBuildOptions + GetArchString(Params.EditorArchitecture);
			string ClientBuildArgs = Params.AdditionalBuildOptions + GetArchString(Params.ClientArchitecture);
			string ProgramBuildArgs = Params.AdditionalBuildOptions + GetArchString(Params.ProgramArchitecture);

			// Setup editor targets
			if (Params.HasEditorTargets && (!Params.SkipBuildEditor) && (TargetMask & ProjectBuildTargets.Editor) == ProjectBuildTargets.Editor)
			{
				// @todo Mac: proper platform detection
				UnrealTargetPlatform EditorPlatform = HostPlatform.Current.HostEditorPlatform;
				const UnrealTargetConfiguration EditorConfiguration = UnrealTargetConfiguration.Development;

				ProjectAgenda.AddTargets(Params.EditorTargets.ToArray(), EditorPlatform, EditorConfiguration, Params.CodeBasedUprojectPath, InAddArgs: EditorBuildArgs);

				if (!Unreal.IsEngineInstalled())
				{
					CrashReportPlatforms.Add(EditorPlatform);
					if (Params.EditorTargets.Contains("ShaderCompileWorker") == false)
					{
						ProjectAgenda.AddTargets(new string[] { "ShaderCompileWorker" }, EditorPlatform, EditorConfiguration, Params.CodeBasedUprojectPath, InAddArgs: ProgramBuildArgs);
					}
					if (Params.FileServer && Params.EditorTargets.Contains("UnrealFileServer") == false)
					{
						NoProjectAgenda.AddTargets(new string[] { "UnrealFileServer" }, EditorPlatform, EditorConfiguration, InAddArgs: ProgramBuildArgs);
					}
				}
			}

			// Build any tools we need to stage
			if ((TargetMask & ProjectBuildTargets.UnrealPak) == ProjectBuildTargets.UnrealPak && !Unreal.IsEngineInstalled())
			{
				if (!Params.HasEditorTargets || Params.EditorTargets.Contains("UnrealPak") == false)
				{
					ProjectAgenda.AddTargets(new string[] { "UnrealPak" }, HostPlatform.Current.HostEditorPlatform, UnrealTargetConfiguration.Development, Params.CodeBasedUprojectPath, InAddArgs: ProgramBuildArgs);
				}

				// If IoStoreOnDemand is being forced onto a project then we will also need to compile IasTool for use during staging
				if (Params.ApplyIoStoreOnDemand && (!Params.HasEditorTargets || Params.EditorTargets.Contains("IasTool") == false))
				{
					NoProjectAgenda.AddTargets(new string[] { "IasTool" }, HostPlatform.Current.HostEditorPlatform, UnrealTargetConfiguration.Development, InAddArgs: ProgramBuildArgs);
				}
			}

			// Additional compile arguments
			string AdditionalArgs = "";

			if (string.IsNullOrEmpty(Params.UbtArgs) == false)
			{
				string Arg = Params.UbtArgs;
				Arg = Arg.TrimStart(new char[] { '\"' });
				Arg = Arg.TrimEnd(new char[] { '\"' });
				AdditionalArgs += " " + Arg;
			}

			if (Params.MapFile)
			{
				AdditionalArgs += " -mapfile";
			}

			if (Params.Deploy || Params.Package)
			{
				AdditionalArgs += " -skipdeploy"; // skip deploy step in UBT if we going to do it later anyway
			}

			if (Params.Distribution)
			{
				AdditionalArgs += " -distribution";
			}

			// Config overrides (-ini)
			foreach (string ConfigOverrideParam in Params.ConfigOverrideParams)
			{
				AdditionalArgs += " -";
				AdditionalArgs += ConfigOverrideParam;
			}

			// Setup cooked targets
			if (Params.HasClientCookedTargets && (!Params.SkipBuildClient) && (TargetMask & ProjectBuildTargets.ClientCooked) == ProjectBuildTargets.ClientCooked)
			{
				List<UnrealTargetPlatform> UniquePlatformTypes = Params.ClientTargetPlatforms.ConvertAll(x => x.Type).Distinct().ToList();

				foreach (var BuildConfig in Params.ClientConfigsToBuild)
				{
					foreach (var ClientPlatformType in UniquePlatformTypes)
					{
						UnrealTargetPlatform CrashReportPlatform = Platform.GetPlatform(ClientPlatformType).CrashReportPlatform ?? ClientPlatformType;
						CrashReportPlatforms.Add(CrashReportPlatform);
						string Arch = Params.IsProgramTarget ? ProgramBuildArgs : ClientBuildArgs;
						FileReference ProjectRefrence = Params.CodeBasedUprojectPath;
						ProjectAgenda.AddTargets(Params.ClientCookedTargets.ToArray(), ClientPlatformType, BuildConfig, ProjectRefrence, 
							InAddArgs: $" -remoteini=\"{Params.RawProjectPath.Directory}\" {AdditionalArgs} {Arch}");
					}
				}
			}
			if (Params.HasServerCookedTargets && (TargetMask & ProjectBuildTargets.ServerCooked) == ProjectBuildTargets.ServerCooked)
			{
				List<UnrealTargetPlatform> UniquePlatformTypes = Params.ServerTargetPlatforms.ConvertAll(x => x.Type).Distinct().ToList();

				foreach (var BuildConfig in Params.ServerConfigsToBuild)
				{
					foreach (var ServerPlatformType in UniquePlatformTypes)
					{
						UnrealTargetPlatform CrashReportPlatform = Platform.GetPlatform(ServerPlatformType).CrashReportPlatform ?? ServerPlatformType;
						CrashReportPlatforms.Add(CrashReportPlatform);
						ProjectAgenda.AddTargets(Params.ServerCookedTargets.ToArray(), ServerPlatformType, BuildConfig, Params.CodeBasedUprojectPath, 
							InAddArgs: $" -remoteini=\"{Params.RawProjectPath.Directory}\" {AdditionalArgs} {ServerBuildArgs}");
					}
				}
			}
			if (!Params.NoBootstrapExe && !Unreal.IsEngineInstalled() && (TargetMask & ProjectBuildTargets.Bootstrap) == ProjectBuildTargets.Bootstrap)
			{
				UnrealBuildTool.UnrealTargetPlatform[] BootstrapPackagedGamePlatforms = { UnrealBuildTool.UnrealTargetPlatform.Win64 };
				foreach (UnrealBuildTool.UnrealTargetPlatform BootstrapPackagedGamePlatformType in BootstrapPackagedGamePlatforms)
				{
					if (Params.ClientTargetPlatforms.Contains(new TargetPlatformDescriptor(BootstrapPackagedGamePlatformType)))
					{
						string BootstrapBuildArgs = BootstrapPackagedGamePlatformType.IsInGroup(UnrealPlatformGroup.Windows) 
							? Params.AdditionalBuildOptions // do not pass a specific architecture, so we default to x64
							: ClientBuildArgs
						;

						NoProjectAgenda.AddTarget("BootstrapPackagedGame", BootstrapPackagedGamePlatformType, UnrealBuildTool.UnrealTargetConfiguration.Shipping, InAddArgs: BootstrapBuildArgs);
					}
				}
			}
			if (Params.CrashReporter && !Unreal.IsEngineInstalled() && (TargetMask & ProjectBuildTargets.CrashReporter) == ProjectBuildTargets.CrashReporter)
			{
				foreach (var CrashReportPlatform in CrashReportPlatforms)
				{
					if (PlatformSupportsCrashReporter(CrashReportPlatform))
					{
						NoProjectAgenda.AddTarget("CrashReportClient", CrashReportPlatform, UnrealTargetConfiguration.Shipping, 
							InAddArgs: $" -remoteini=\"{Params.RawProjectPath.Directory}\" {ProgramBuildArgs}");
					}
				}
			}
			if (Params.HasProgramTargets && (TargetMask & ProjectBuildTargets.Programs) == ProjectBuildTargets.Programs)
			{
				List<UnrealTargetPlatform> UniquePlatformTypes = Params.ClientTargetPlatforms.ConvertAll(x => x.Type).Distinct().ToList();

				foreach (var BuildConfig in Params.ClientConfigsToBuild)
				{
					foreach (var ClientPlatformType in UniquePlatformTypes)
					{
						ProjectAgenda.AddTargets(Params.ProgramTargets.ToArray(), ClientPlatformType, BuildConfig, Params.CodeBasedUprojectPath, ProgramBuildArgs);
					}
				}
			}

			// allow all involved platforms to hook into the agenda
			HashSet<UnrealTargetPlatform> UniquePlatforms = new HashSet<UnrealTargetPlatform>();
			UniquePlatforms.UnionWith(Params.ClientTargetPlatforms.Select(x => x.Type));
			UniquePlatforms.UnionWith(Params.ServerTargetPlatforms.Select(x => x.Type));
			foreach (UnrealTargetPlatform TargetPlatform in UniquePlatforms)
			{
				// @todo: pass in both project and no project
				Platform.GetPlatform(TargetPlatform).PreBuildAgenda(UnrealBuild, ProjectAgenda, Params);
			}

			UnrealBuild.Build(ProjectAgenda, InDeleteBuildProducts: Params.Clean, InUpdateVersionFiles: WorkingCL > 0);
			// if we had two separate agendas, build the other one now
			if (NoProjectAgenda != ProjectAgenda)
			{
				foreach (UnrealTargetPlatform TargetPlatform in UniquePlatforms)
				{
					Platform.GetPlatform(TargetPlatform).PreBuildAgenda(UnrealBuild, NoProjectAgenda, Params);
				}
				UnrealBuild.Build(NoProjectAgenda, InDeleteBuildProducts: Params.Clean, InUpdateVersionFiles: WorkingCL > 0);
			}

			if (WorkingCL > 0) // only move UAT files if we intend to check in some build products
			{
				UnrealBuild.AddUATFilesToBuildProducts();
			}
			UnrealBuild.CheckBuildProducts(UnrealBuild.BuildProductFiles);

			if (WorkingCL > 0)
			{
				// Sign everything we built
				CodeSign.SignMultipleIfEXEOrDLL(Command, UnrealBuild.BuildProductFiles);

				// Open files for add or edit
				UnrealBuild.AddBuildProductsToChangelist(WorkingCL, UnrealBuild.BuildProductFiles);
			}

			Logger.LogInformation("Build command time: {0:0.00} s", (DateTime.UtcNow - StartTime).TotalMilliseconds / 1000);
			Logger.LogInformation("********** BUILD COMMAND COMPLETED **********");
		}
	}
}
