// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.IO;
using System.Linq;
using System.Reflection;
using System.Text.RegularExpressions;
using EpicGames.Core;
using UnrealBuildBase;
using UnrealBuildTool;
using Microsoft.Extensions.Logging;

namespace AutomationTool
{

	[Help("Builds the specified targets and configurations for the specified project.")]
	[Help("Example BuildTarget -project=QAGame -target=Editor+Game -platform=Win64+Android -configuration=Development.")]
	[Help("Note: Editor will only ever build for the current platform in a Development config and required tools will be included")]
	[Help("project=<QAGame>", "Project to build. Will search current path and paths in ueprojectdirs. If omitted will build vanilla UnrealEditor")]
	[Help("platform=Win64+Android", "Platforms to build, join multiple platforms using +")]
	[Help("configuration=Development+Test", "Configurations to build, join multiple configurations using +")]
	[Help("target=Editor+Game", "Targets to build, join multiple targets using +")]
	[Help("notools", "Don't build any tools (UnrealPak, Lightmass, ShaderCompiler, CrashReporter")]
	[Help("clean", "Do a clean build")]
	[Help("NoXGE", "Toggle to disable the distributed build process")]
	[Help("DisableUnity", "Toggle to disable the unity build system")]
	public class BuildTarget : BuildCommand
	{
		// exposed as a property so projects can derive and set this directly
		public string ProjectName { get; set; }

		public string Targets { get; set; }

		public string Platforms { get; set; }

		public string Configurations { get; set; }

		public bool	  Clean { get; set; }

		public bool	  NoTools { get; set; }

		public string UBTArgs { get; set; }

		public bool Preview { get; set; }

		// It would be nice to use SingleTargetProperties but we can't get rules
		// For UE types..
		public class SimpleTargetInfo
		{
			public string		TargetName { get; private set; }

			public TargetType	Type { get; private set; }

			public SimpleTargetInfo(string InName, TargetType InType)
			{
				TargetName = InName;
				Type = InType;
			}
		}


		public BuildTarget()
		{
			Platforms = HostPlatform.Current.HostEditorPlatform.ToString();
			Configurations = "Development";
			UBTArgs = "";
		}

		public override ExitCode Execute()
		{
			string[] Arguments = this.Params;

			ProjectName = ParseParamValue("project", ProjectName);
			Targets = ParseParamValue("target", Targets);
			Platforms = ParseParamValue("platform", Platforms);
			Configurations = ParseParamValue("configuration", Configurations);
			Clean = ParseParam("clean") || Clean;
			NoTools = ParseParam("NoTools") || NoTools;
			UBTArgs = ParseParamValue("ubtargs", UBTArgs);
			Preview = ParseParam("preview") || Preview;

			if (string.IsNullOrEmpty(Targets))
			{
				throw new AutomationException("No target specified with -target. Use -help to see all options");
			}

			IEnumerable<string> TargetList = Targets.Split(new char[] { '+' }, StringSplitOptions.RemoveEmptyEntries);
			IEnumerable<UnrealTargetConfiguration> ConfigurationList = null;
			IEnumerable<UnrealTargetPlatform> PlatformList = null;

			try
			{
				ConfigurationList = Configurations.Split(new char[] { '+' }, StringSplitOptions.RemoveEmptyEntries)
																		.Select(C => (UnrealTargetConfiguration)Enum.Parse(typeof(UnrealTargetConfiguration), C, true)).ToArray();
			}
			catch (Exception Ex)
			{
				Logger.LogError("Failed to parse configuration string. {Arg0}", Ex.Message);
				return ExitCode.Error_Arguments;
			}

			try
			{
				PlatformList = Platforms.Split(new char[] { '+' }, StringSplitOptions.RemoveEmptyEntries)
																		.Select(C =>
																		{
																			UnrealTargetPlatform Platform;
																			if (!UnrealTargetPlatform.TryParse(C, out Platform))
																			{
																				throw new AutomationException("No such platform {0}", C);
																			}
																			return Platform;
																		}).ToArray();
			}
			catch (Exception Ex)
			{
				Logger.LogError("Failed to parse configuration string. {Arg0}", Ex.Message);
				return ExitCode.Error_Arguments;
			}

			FileReference ProjectFile = null;

			if (!string.IsNullOrEmpty(ProjectName))
			{
				// find the project
				ProjectFile = ProjectUtils.FindProjectFileFromName(ProjectName);

				if (ProjectFile == null)
				{
					throw new AutomationException("Unable to find uproject file for {0}", ProjectName);
				}
			}

			IEnumerable<SimpleTargetInfo> BuildTargets = TargetList.Select(T => ProjectTargetFromTargetName(T, ProjectFile, PlatformList, ConfigurationList)).ToArray();			

			bool ContainsEditor = BuildTargets.Where(T => T.Type == TargetType.Editor).Any();
			bool SingleBuild = BuildTargets.Count() == 1 && PlatformList.Count() == 1 && ConfigurationList.Count() == 1;

			if (!SingleBuild || (ContainsEditor && !NoTools))
			{
				UnrealBuild Build = new UnrealBuild(this);

				UnrealBuild.BuildAgenda Agenda = new UnrealBuild.BuildAgenda();

				SimpleTargetInfo EditorTarget = BuildTargets.Where(T => T.Type == TargetType.Editor).FirstOrDefault();
				IEnumerable<SimpleTargetInfo> OtherTargets = BuildTargets.Where(T => T.Type != TargetType.Editor);

				UnrealTargetPlatform CurrentPlatform = HostPlatform.Current.HostEditorPlatform;

				if (EditorTarget != null)
				{
					Agenda.AddTarget(EditorTarget.TargetName, CurrentPlatform, UnrealTargetConfiguration.Development, ProjectFile, UBTArgs);

					if (!NoTools)
					{
						Agenda.AddTarget("UnrealPak", CurrentPlatform, UnrealTargetConfiguration.Development, ProjectFile, UBTArgs);
						Agenda.AddTarget("ShaderCompileWorker", CurrentPlatform, UnrealTargetConfiguration.Development, ProjectFile, UBTArgs);
						Agenda.AddTarget("UnrealLightmass", CurrentPlatform, UnrealTargetConfiguration.Development, ProjectFile, UBTArgs);
						Agenda.AddTarget("InterchangeWorker", CurrentPlatform, UnrealTargetConfiguration.Development, ProjectFile, UBTArgs);
						Agenda.AddTarget("CrashReportClient", CurrentPlatform, UnrealTargetConfiguration.Shipping, ProjectFile, UBTArgs);
						Agenda.AddTarget("CrashReportClientEditor", CurrentPlatform, UnrealTargetConfiguration.Shipping, ProjectFile, UBTArgs);
					}
				}

				foreach (SimpleTargetInfo Target in OtherTargets)
				{
					bool IsServer = Target.Type == TargetType.Server;

					IEnumerable<UnrealTargetPlatform> PlatformsToBuild = IsServer ? new UnrealTargetPlatform[] { CurrentPlatform } : PlatformList;

					foreach (UnrealTargetPlatform Platform in PlatformsToBuild)
					{
						foreach (UnrealTargetConfiguration Config in ConfigurationList)
						{
							Agenda.AddTarget(Target.TargetName, Platform, Config, ProjectFile, UBTArgs);
						}
					}
				}

				foreach (var Target in Agenda.Targets)
				{
					Logger.LogInformation("Will {Arg0}build {Target}", Clean ? "clean and " : "", Target);
					if (Clean)
					{
						Target.Clean = Clean;
					}
				}

				if (!Preview)
				{
					Build.Build(Agenda, InUpdateVersionFiles: false);
				}
			}
			else
			{
				UnrealTargetPlatform PlatformToBuild = PlatformList.First();
				UnrealTargetConfiguration ConfigToBuild = ConfigurationList.First();
				string TargetToBuild = BuildTargets.First().TargetName;

				if (!Preview)
				{
					// Compile the editor
					string CommandLine = CommandUtils.UBTCommandline(ProjectFile, TargetToBuild, PlatformToBuild, ConfigToBuild, UBTArgs);

					if (Clean)
					{
						CommandUtils.RunUBT(CommandUtils.CmdEnv, Unreal.UnrealBuildToolDllPath, CommandLine + " -clean");
					}
					CommandUtils.RunUBT(CommandUtils.CmdEnv, Unreal.UnrealBuildToolDllPath, CommandLine);
				}
				else
				{ 
					Logger.LogInformation("Will {Arg0}build {TargetToBuild} {PlatformToBuild} {ConfigToBuild}", Clean ? "clean and " : "", TargetToBuild, PlatformToBuild, ConfigToBuild);
				}
				
			}

			return ExitCode.Success;
		}

		/// <summary>
		/// Takes a target type like "Editor" and returns the actual targetname used by the specified project. If no project is specified then
		/// the name of the UE types (e.g. UnrealEditor) is returned.
		/// </summary>
		/// <param name="InTargetName"></param>
		/// <param name="InProjectFile"></param>
		/// <param name="InPlatformList"></param>
		/// <param name="InConfigurationList"></param>
		/// <returns></returns>
		public SimpleTargetInfo ProjectTargetFromTargetName(string InTargetName, FileReference InProjectFile, IEnumerable<UnrealTargetPlatform> InPlatformList, IEnumerable<UnrealTargetConfiguration> InConfigurationList)
		{
			ProjectProperties Properties = InProjectFile != null ? ProjectUtils.GetProjectProperties(InProjectFile, InPlatformList.ToList(), InConfigurationList.ToList()) : null;

			SimpleTargetInfo ProjectTarget = null;

			IEnumerable<SimpleTargetInfo> AvailableTargets = null;

			if (Properties != null && Properties.bIsCodeBasedProject)
			{
				AvailableTargets = Properties.Targets.Select(T => new SimpleTargetInfo(T.TargetName, T.Rules.Type));
			}
			else
			{
				// default UE targets
				AvailableTargets = new[]
				{
					new SimpleTargetInfo("UnrealEditor", TargetType.Editor),
					new SimpleTargetInfo("UnrealGame", TargetType.Game),
					new SimpleTargetInfo("UnrealClient", TargetType.Client),
					new SimpleTargetInfo("UnrealServer", TargetType.Server),
				};
			}


			// If they asked for ShooterClient etc and that's there, just return that.
			ProjectTarget = AvailableTargets.FirstOrDefault(T => T.TargetName.Equals(InTargetName, StringComparison.OrdinalIgnoreCase));

			if (ProjectTarget == null)
			{
				// find targets that use rules of the desired type
				IEnumerable<SimpleTargetInfo> MatchingTargetTypes = AvailableTargets
					.Where(T => T.Type.ToString().Equals(InTargetName, StringComparison.OrdinalIgnoreCase));

				if (MatchingTargetTypes.Any())
				{
					if (MatchingTargetTypes.Count() == 1)
					{
						ProjectTarget = MatchingTargetTypes.First();
					}
					else
					{
						// if multiple targets, pick the one that starts with project name and contains the target type
						// (Some projects have multiple targets of a given type)
						ProjectTarget = MatchingTargetTypes
										.Where(T => string.CompareOrdinal(T.TargetName, 0, ProjectName, 0, 1) == 0)
										.Where(T => T.TargetName.IndexOf(InTargetName, StringComparison.OrdinalIgnoreCase) > 0)
										.FirstOrDefault();

						// Try to find a target where the target type matches the target name exactly such as "Editor" or "Game"
						if (ProjectTarget == null && string.CompareOrdinal(InTargetName, "Program") != 0)
						{
							ProjectTarget = MatchingTargetTypes
											.Where(T => string.CompareOrdinal(T.Type.ToString(), InTargetName) == 0)
											.FirstOrDefault();
						}
					}
				}
			}		

			// If no target is found, try to build the provided target name. This enables programs to build (Ex. UnrealInsights).
			if(ProjectTarget == null)
			{
				ProjectTarget = new SimpleTargetInfo(InTargetName, TargetType.Program);
			}

			if (ProjectTarget == null)
			{
				throw new AutomationException("{0} is not a valid target in {1}", InTargetName, InProjectFile);
			}
		
			return ProjectTarget;
		}
	}

	[Help("Builds the editor for the specified project.")]
	[Help("Example BuildEditor -project=QAGame")]
	[Help("Note: Editor will only ever build for the current platform in a Development config and required tools will be included")]
	[Help("project=<QAGame>", "Project to build. Will search current path and paths in ueprojectdirs. If omitted will build vanilla UnrealEditor")]
	[Help("notools", "Don't build any tools (UHT, ShaderCompiler, CrashReporter")]
	class BuildEditor : BuildTarget
	{
		public BuildEditor()
		{
			Targets = "Editor";
		}

		public override ExitCode Execute()
		{
			bool DoOpen = ParseParam("open");
			ExitCode Status = base.Execute();

			if (Status == ExitCode.Success && DoOpen)
			{
				OpenEditor OpenCmd = new OpenEditor();
				OpenCmd.ProjectName = this.ProjectName;
				Status = OpenCmd.Execute();
			}

			return Status;			
		}
	}

	[Help("Builds the game for the specified project.")]
	[Help("Example BuildGame -project=QAGame -platform=Win64+Android -configuration=Development.")]
	[Help("project=<QAGame>", "Project to build. Will search current path and paths in ueprojectdirs.")]
	[Help("platform=Wind64+Android", "Platforms to build, join multiple platforms using +")]
	[Help("configuration=Development+Test", "Configurations to build, join multiple configurations using +")]
	[Help("notools", "Don't build any tools (UHT, ShaderCompiler, CrashReporter")]
	class BuildGame : BuildTarget
	{
		public BuildGame()
		{
			Targets = "Game";
		}
	}

	[Help("Builds the server for the specified project.")]
	[Help("Example BuildServer -project=QAGame -platform=Win64 -configuration=Development.")]
	[Help("project=<QAGame>", "Project to build. Will search current path and paths in ueprojectdirs.")]
	[Help("platform=Win64", "Platforms to build, join multiple platforms using +")]
	[Help("configuration=Development+Test", "Configurations to build, join multiple configurations using +")]
	[Help("notools", "Don't build any tools (UHT, ShaderCompiler, CrashReporter")]
	class BuildServer : BuildTarget
	{
		public BuildServer()
		{
			Targets = "Server";
		}
	}
}

