// Copyright Epic Games, Inc. All Rights Reserved.
using System;
using System.Collections.Generic;
using System.Text;
using System.IO;
using AutomationTool;
using UnrealBuildTool;
using Microsoft.Extensions.Logging;




/********************************
 
	-	Check out these files for reference:
		-	//UE5/Main/Engine/Source/ThirdParty/libPNG/UE_BuildThirdPartyLib.bat
		-	//UE5/Main/Engine/Source/ThirdParty/libPNG/UE_BuildThirdPartyLib_Mac.command
	-	Make new files with the same names in the root of your library directory.
		-	The program that builds the libraries will automatically find the script and call it if it exists
		-	You will likely have to open projects in Visual Studio or Xcode to find out the name of the targets, etc
		-	For the Mac script, you should to chmod a+x it
	-	Some notes on writing the files
		-	Since the script is in the root of the lib dir, pushd into the necessary directories to do the compiling
		-	Make sure check out the libraries before you compile them, using the THIRD_PARTY_CHANGELIST env var (the var will be -c xxxxxx)
			-	It's okay to go overboard, the calling program will revert any unchanged files in the changelist
		-	If you are compiling for IOS, please modify (and checkin) the project and make sure that arm64 is being compiled along with armv7/armv7s
	-	To test, just run the batch file from the root of the lib!
		-	Unless you set the THIRD_PARTY_CHANGELIST env var, your script will check out into the default changelist
		-	You can also use the full program to test compiling all or a subset of libs:
			-	From Engine/Build/BatchFiles, do:
				-	RunUAT BuildThirdPartyLibs [-libs=lib1+lib2+lib3] [-changelist=NNNN]
			-	When you use this script, it won't submit to p4, so just revert when your done testing

 *******************************/








[Help("Builds third party libraries, and puts them all into a changelist")]
[Help("Libs", "[Optional] + separated list of libraries to compile; if not specified this job will build all libraries it can find builder scripts for")]
[Help("Changelist", "[Optional] a changelist to check out into; if not specified, a changelist will be created")]
[Help("SearchDir", "[Optional] Directory to search for the specified Libs")]
[RequireP4]
class BuildThirdPartyLibs : BuildCommand
{
	// path to the third party directory
	static private string DefaultLibraryDir = "Engine/Source/ThirdParty";

	// batch/script file to look for when compiling
	static private string WindowsCompileScript = "UE_BuildThirdPartyLib.bat";
	static private string MacCompileScript = "UE_BuildThirdPartyLib_Mac.command";
	static private string LinuxCompileScript = "UE_BuildThirdPartyLib_Linux.sh";

	public override void ExecuteBuild()
	{
		Logger.LogInformation("************************* Build Third Party Libs");

		// figure out what batch/script to run
		string CompileScriptName;
		if (UnrealBuildTool.BuildHostPlatform.Current.Platform == UnrealTargetPlatform.Win64)
		{
			CompileScriptName = WindowsCompileScript;
		}
		else if (UnrealBuildTool.BuildHostPlatform.Current.Platform == UnrealTargetPlatform.Mac)
		{
			CompileScriptName = MacCompileScript;
		}
		else if (UnrealBuildTool.BuildHostPlatform.Current.Platform == UnrealTargetPlatform.Linux)
		{
			CompileScriptName = LinuxCompileScript;
		}
		else
		{
			throw new AutomationException("Unknown runtime platform!");
		}

		// look for changelist on the command line
		int WorkingCL = Int32.Parse(ParseParamValue("Changelist", "-1"));
		
		// if not specified, make one
		if (WorkingCL == -1)
		{
			WorkingCL = P4.CreateChange(P4Env.Client, String.Format("Third party libs built from changelist {0}", P4Env.Changelist));
		}
		Logger.LogInformation("Build from {Arg0}    Working in {WorkingCL}", P4Env.Changelist, WorkingCL);

		// go to the third party lib dir
		string SearchLibraryDir = ParseParamValue("SearchDir", DefaultLibraryDir);
		CommandUtils.PushDir(SearchLibraryDir);

		// figure out what libraries to compile
		string LibsToCompileString = ParseParamValue("Libs");

		// hunt down build batch files if the caller didn't specify a list to compile
		List<string> LibsToCompile = new List<string>();
		if (string.IsNullOrEmpty(LibsToCompileString))
		{
			// loop over third party directories looking for the right batch files
			foreach (string Dir in Directory.EnumerateDirectories("."))
			{
				if (File.Exists(Path.Combine(Dir, CompileScriptName)))
				{
					LibsToCompile.Add(Path.GetFileName(Dir));
				}
			}
		}
		else
		{
			// just split up the param and make sure the batch file exists
			string[] Libs = LibsToCompileString.Split('+');
			bool bHadError = false;
			foreach (string Dir in Libs)
			{
				if (File.Exists(Path.Combine(Dir, CompileScriptName)))
				{
					LibsToCompile.Add(Path.GetFileName(Dir));
				}
				else
				{
					Logger.LogError("Error: Requested lib {Dir} does not have a {CompileScriptName}", Dir, CompileScriptName);
					bHadError = true;
				}
			}
			if (bHadError)
			{
				// error out so that we don't fail to build some and have it lost in the noise
				throw new AutomationException("One or more libs were not set up to compile.");
			}
		}

		// set an envvar so that the inner batch files can check out files into a shared changelist
		Environment.SetEnvironmentVariable("THIRD_PARTY_CHANGELIST", string.Format("-c {0}", WorkingCL));

		// now go through and run each batch file, 
		foreach (string Lib in LibsToCompile)
		{
			Logger.LogInformation("Building {Lib}", Lib);

			// go into the lib dir
			CommandUtils.PushDir(Lib);

			// run the builder batch file
			CommandUtils.RunAndLog(CmdEnv, CompileScriptName, "", "ThirdPartyLib_" + Lib);

			// go back to ThirdParty dir
			CommandUtils.PopDir();
		}

		// undo the SearchLibraryDir push
		CommandUtils.PopDir();

		PrintRunTime();			

		// revert any unchanged files
		P4.RevertUnchanged(WorkingCL);

		if (AllowSubmit)
		{
			int SubmittedCL;
			P4.Submit(WorkingCL, out SubmittedCL, true, true);
			Logger.LogInformation("Submitted changelist {SubmittedCL}", SubmittedCL);
		}
	}
}
