// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Diagnostics;
using System.IO;
using System.Linq;
using System.Text;
using System.Text.RegularExpressions;
using System.Threading.Tasks;
using EpicGames.Core;
using UnrealBuildBase;
using UnrealBuildTool;
using Microsoft.Extensions.Logging;

namespace AutomationTool
{
	[Help("Checks that all source files have balanced macros for enabling/disabling optimization, warnings, etc...")]
	[Help("Project=<Path>", "Path to an additional project file to consider")]
	[Help("File=<Path>", "Path to a file to parse in isolation, for testing")]
	[Help("OverrideFileList=<Path>", "Path to a text file with paths to the files you want to parse")]
	[Help("Ignore=<Name>", "File name (without path) to exclude from testing")]
	class CheckBalancedMacros : BuildCommand
	{
		/// <summary>
		/// List of directories relative to the root that can contain source files
		/// </summary>
		public static readonly string[] SourceDirectories = { "Platforms", "Plugins", "Restricted", "Shaders", "Source" };

		/// <summary>
		/// List of macros that should be paired up
		/// </summary>
		static readonly string[,] MacroPairs = new string[,]
		{
			{
				"PRAGMA_DISABLE_OPTIMIZATION",
				"PRAGMA_ENABLE_OPTIMIZATION"
			},
			{
				"UE_DISABLE_OPTIMIZATION_SHIP",
				"UE_ENABLE_OPTIMIZATION_SHIP"
			},
			{
				"PRAGMA_DISABLE_DEPRECATION_WARNINGS",
				"PRAGMA_ENABLE_DEPRECATION_WARNINGS"
			},
			{
				"THIRD_PARTY_INCLUDES_START",
				"THIRD_PARTY_INCLUDES_END"
			},
			{
				"PRAGMA_DISABLE_SHADOW_VARIABLE_WARNINGS",
				"PRAGMA_ENABLE_SHADOW_VARIABLE_WARNINGS"
			},
			{
				"PRAGMA_DISABLE_UNSAFE_TYPECAST_WARNINGS",
				"PRAGMA_RESTORE_UNSAFE_TYPECAST_WARNINGS"
			},
			{
				"PRAGMA_DISABLE_UNREACHABLE_CODE_WARNINGS",
				"PRAGMA_RESTORE_UNREACHABLE_CODE_WARNINGS"
			},
			{
				"PRAGMA_FORCE_UNSAFE_TYPECAST_WARNINGS",
				"PRAGMA_RESTORE_UNSAFE_TYPECAST_WARNINGS"
			},			
			{
				"PRAGMA_DISABLE_UNDEFINED_IDENTIFIER_WARNINGS",
				"PRAGMA_ENABLE_UNDEFINED_IDENTIFIER_WARNINGS"
			},
			{
				"PRAGMA_DISABLE_MISSING_VIRTUAL_DESTRUCTOR_WARNINGS",
				"PRAGMA_ENABLE_MISSING_VIRTUAL_DESTRUCTOR_WARNINGS"
			},
			{
				"BEGIN_FUNCTION_BUILD_OPTIMIZATION",
				"END_FUNCTION_BUILD_OPTIMIZATION"
			},
			{
				"BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION",
				"END_SLATE_FUNCTION_BUILD_OPTIMIZATION"
			},
		};

		/// <summary>
		/// Regexes for LOCTEXT_NAMESPACE preprocessor identification
		/// This could be generalized like the above if we had other pairings we wanted to manage
		/// </summary>
		static readonly Regex OpenLoctextNamespaceRegex = new Regex(@"\G#define\s+LOCTEXT_NAMESPACE");
		static readonly Regex CloseLoctextNamespaceRegex = new Regex(@"\G#undef\s+LOCTEXT_NAMESPACE");

		/// <summary>
		/// List of files to ignore for balanced macros. Additional filenames may be specified on the command line via -Ignore=...
		/// </summary>
		HashSet<string> IgnoreFileNames = new HashSet<string>(StringComparer.OrdinalIgnoreCase)
		{
			"PreWindowsApi.h",
			"PostWindowsApi.h",
			"USDIncludesStart.h",
			"USDIncludesEnd.h",
			"PreOpenCVHeaders.h",
			"PostOpenCVHeaders.h",
		};

		/// <summary>
		/// Main entry point for the command
		/// </summary>
		public override void ExecuteBuild()
		{
			// Build a lookup of flags to set and clear for each identifier
			Dictionary<string, List<int>> IdentifierToIndex = new Dictionary<string, List<int>>();
			for(int Idx = 0; Idx < MacroPairs.GetLength(0); Idx++)
			{
				for (int SubIdx = 0; SubIdx < 2; SubIdx++)
				{
					ref string Key = ref MacroPairs[Idx, SubIdx];
					if (!IdentifierToIndex.ContainsKey(Key))
					{
						IdentifierToIndex[Key] = new List<int>();
					}

					IdentifierToIndex[Key].Add(SubIdx == 0 ? Idx : ~Idx);
				}
			}

			// Check if we want to just parse a single file
			string FileParam = ParseParamValue("File");
			string OverrideFileList = ParseParamValue("OverrideFileList=", null); // Specify a file list of individual files you want to check instead of the entire directory

			if (FileParam != null && OverrideFileList != null)
			{
				throw new AutomationException("File and OverrideFileList parameters cannot be passed at the same time.");
			}

			if (FileParam != null)
			{
				// Check the file exists
				FileReference File = new FileReference(FileParam);
				if (!FileReference.Exists(File))
				{
					throw new AutomationException("File '{0}' does not exist", File);
				}
				CheckSourceFile(File, IdentifierToIndex, new object());
			}
			else if (OverrideFileList != null)
			{
				Logger.LogInformation("Finding files from OverrideFileList {File}", OverrideFileList);

				FileReference FileListToCheck = new FileReference(OverrideFileList);
				if (!FileReference.Exists(FileListToCheck))
				{
					throw new AutomationException("FileList '{0}' does not exist", FileListToCheck);
				}

				string[] FilesToCheck = FileReference.ReadAllLines(FileListToCheck);
				List<FileReference> SourceFiles = new List<FileReference>();

				foreach (string File in FilesToCheck.Where(x => !String.IsNullOrWhiteSpace(x) && (x.EndsWith(".h") || x.EndsWith(".cpp"))))
				{
					SourceFiles.Add(new FileReference(File));
				}

				// Loop through all the source files
				using (ThreadPoolWorkQueue Queue = new ThreadPoolWorkQueue())
				{
					object LogLock = new object();
					foreach (FileReference SourceFile in SourceFiles)
					{
						Queue.Enqueue(() => CheckSourceFile(SourceFile, IdentifierToIndex, LogLock));
					}

					using (LogStatusScope Scope = new LogStatusScope("Checking source files..."))
					{
						while (!Queue.Wait(10 * 1000))
						{
							Scope.SetProgress("{0}/{1}", SourceFiles.Count - Queue.NumRemaining, SourceFiles.Count);
						}
					}
				}
			}
			else
			{
				// Add the additional files to be ignored
				foreach(string IgnoreFileName in ParseParamValues("Ignore"))
				{
					IgnoreFileNames.Add(IgnoreFileName);
				}

				// Create a list of all the root directories
				HashSet<DirectoryReference> RootDirs = new HashSet<DirectoryReference>();
				RootDirs.Add(Unreal.EngineDirectory);

				// Add the enterprise directory
				DirectoryReference EnterpriseDirectory = DirectoryReference.Combine(Unreal.RootDirectory, "Enterprise");
				if(DirectoryReference.Exists(EnterpriseDirectory))
				{
					RootDirs.Add(EnterpriseDirectory);
				}

				// Add the project directories
				string[] ProjectParams = ParseParamValues("Project");
				foreach(string ProjectParam in ProjectParams)
				{
					FileReference ProjectFile = new FileReference(ProjectParam);
					if(!FileReference.Exists(ProjectFile))
					{
						throw new AutomationException("Unable to find project '{0}'", ProjectFile);
					}
					RootDirs.Add(ProjectFile.Directory);
				}

				// Recurse through the tree
				Logger.LogInformation("Finding source files...");
				List<FileReference> SourceFiles = new List<FileReference>();
				using(ThreadPoolWorkQueue Queue = new ThreadPoolWorkQueue())
				{
					foreach(DirectoryReference RootDir in RootDirs)
					{
						foreach (String Directory in SourceDirectories)
						{
							DirectoryInfo SourceDir = new DirectoryInfo(Path.Combine(RootDir.FullName, Directory));
							if(SourceDir.Exists)
							{
								Queue.Enqueue(() => FindSourceFiles(SourceDir, SourceFiles, Queue));
							}
						}
					}
					Queue.Wait();
				}

				// Loop through all the source files
				using(ThreadPoolWorkQueue Queue = new ThreadPoolWorkQueue())
				{
					object LogLock = new object();
					foreach(FileReference SourceFile in SourceFiles)
					{
						Queue.Enqueue(() => CheckSourceFile(SourceFile, IdentifierToIndex, LogLock));
					}

					using(LogStatusScope Scope = new LogStatusScope("Checking source files..."))
					{
						while(!Queue.Wait(10 * 1000))
						{
							Scope.SetProgress("{0}/{1}", SourceFiles.Count - Queue.NumRemaining, SourceFiles.Count);
						}
					}
				}
			}
		}

		/// <summary>
		/// Finds all the source files under a given directory
		/// </summary>
		/// <param name="BaseDir">Directory to search</param>
		/// <param name="SourceFiles">List to receive the files found. A lock will be taken on this object to ensure multiple threads do not add to it simultaneously.</param>
		/// <param name="Queue">Queue for additional tasks to be added to</param>
		void FindSourceFiles(DirectoryInfo BaseDir, List<FileReference> SourceFiles, ThreadPoolWorkQueue Queue)
		{
			foreach(DirectoryInfo SubDir in BaseDir.EnumerateDirectories())
			{
				if(!SubDir.Name.Equals("Intermediate", StringComparison.OrdinalIgnoreCase))
				{
					Queue.Enqueue(() => FindSourceFiles(SubDir, SourceFiles, Queue));
				}
			}

			foreach(FileInfo File in BaseDir.EnumerateFiles())
			{
				if(File.Name.EndsWith(".h", StringComparison.OrdinalIgnoreCase) || File.Name.EndsWith(".cpp", StringComparison.OrdinalIgnoreCase))
				{
					if(!IgnoreFileNames.Contains(File.Name))
					{
						lock(SourceFiles)
						{
							SourceFiles.Add(new FileReference(File));
						}
					}
				}
			}
		}

		/// <summary>
		/// Checks whether macros in the given source file are matched
		/// </summary>
		/// <param name="SourceFile"></param>
		/// <param name="IdentifierToIndex">Map of macro identifier to bit index. The complement of an index is used to indicate the end of the pair.</param>
		/// <param name="LogLock">Object used to marshal access to the global log instance</param>
		void CheckSourceFile(FileReference SourceFile, Dictionary<string, List<int>> IdentifierToIndex, object LogLock)
		{
			// Read the text
			string Text = FileReference.ReadAllText(SourceFile);

			// Scan through the file token by token. Each bit in the Flags array indicates an index into the MacroPairs array that is currently active.
			int Flags = 0;
			bool LoctextNamespaceOpen = false;
			for(int Idx = 0; Idx < Text.Length; )
			{
				int StartIdx = Idx++;
				if((Text[StartIdx] >= 'a' && Text[StartIdx] <= 'z') || (Text[StartIdx] >= 'A' && Text[StartIdx] <= 'Z') || Text[StartIdx] == '_')
				{
					// Identifier
					while(Idx < Text.Length && ((Text[Idx] >= 'a' && Text[Idx] <= 'z') || (Text[Idx] >= 'A' && Text[Idx] <= 'Z') || (Text[Idx] >= '0' && Text[Idx] <= '9') || Text[Idx] == '_'))
					{
						Idx++;
					}

					// Extract the identifier
					string Identifier = Text.Substring(StartIdx, Idx - StartIdx);

					// Find the matching flag
					List<int> Index;
					if(IdentifierToIndex.TryGetValue(Identifier, out Index))
					{
						if(Index[0] >= 0)
						{
							// Set the flag (should not already be set)
							int Flag = 1 << Index[0];
							if((Flags & Flag) != 0)
							{
								EpicGames.Core.Log.TraceWarningTask(SourceFile, GetLineNumber(Text, StartIdx), "{0} macro appears a second time without matching {1} macro", Identifier, MacroPairs[Index[0], 1]);
							}
							Flags |= Flag;
						}
						else
						{
							bool bMatched = false;
							// Check for any flag. We clear the first we find when validating, even if that's not technically the correct match. TODO: This means we may report the wrong tag as left over where tags are nested and there's a missing end tag.
							foreach (int SubIndex in Index)
							{
								int Flag = 1 << ~SubIndex;
								// Clear the flag (should already be set)
								if((Flags & Flag) != 0)
								{
									Flags &= ~Flag;
									bMatched = true;
									break;
								}
							}
							
							if (!bMatched)
							{
								string MissingMatching = "";
								foreach (int SubIndex in Index)
								{
									MissingMatching += (MissingMatching.Length > 0 ? ", " : "") +  MacroPairs[~SubIndex, 0];
								}
								EpicGames.Core.Log.TraceWarningTask(SourceFile, GetLineNumber(Text, StartIdx), "{0} macro appears without matching {1} macro", Identifier, MissingMatching);
							}
						}
					}
				}
				else if(Text[StartIdx] == '/' && Idx < Text.Length)
				{
					if(Text[Idx] == '/')
					{
						// Single-line comment
						while(Idx < Text.Length && Text[Idx] != '\n')
						{
							Idx++;
						}
					}
					else if(Text[Idx] == '*')
					{
						// Multi-line comment
						Idx++;
						for(; Idx < Text.Length; Idx++)
						{
							if(Idx + 2 < Text.Length && Text[Idx] == '*' && Text[Idx + 1] == '/')
							{
								Idx += 2;
								break;
							}
						}
					}
				}
				else if(Text[StartIdx] == '"')
				{
					// String
					for(; Idx < Text.Length; Idx++)
					{
						if(Text[Idx] == '"')
						{
							Idx++;
							break;
						}
						if(Text[Idx] == '\\')
						{
							Idx++;
						}
					}
				}
				else if(Text[StartIdx] == '\'')
				{
					// Escaped character (e.g. \n, \', \xAB, \xFFFF)
					if(Text[StartIdx + 1] == '\\')
					{
						Idx += 2;
						for (; Idx < Text.Length; Idx++)
						{
							if (Text[Idx] == '\'')
							{
								Idx++;
								break;
							}
						}
					}
					// Standard single character
					else if(Text[StartIdx + 2] == '\'')
					{
						Idx += 2;
					}
					// Otherwise this is probably a numeric separator and we're going to ignore it
				}
				else if(Text[StartIdx] == '#')
				{
					// Do detection of LOCTEXT_NAMESPACE directives being properly closed
					// It is theoretically valid to redefine LOCTEXT_NAMESPACE, so this is simply
					// ensuring there is a closing #undef 

					// Peek ahead to try and reduce the number of regex comparisons we make
					if(!LoctextNamespaceOpen && Text[StartIdx + 1] == 'd') 
					{
						Match m = OpenLoctextNamespaceRegex.Match(Text, StartIdx);
						if (m.Success && m.Index == StartIdx)
						{
							LoctextNamespaceOpen = true;
						}
					}
					else if(LoctextNamespaceOpen && Text[StartIdx + 1] == 'u')
					{
						Match m = CloseLoctextNamespaceRegex.Match(Text, StartIdx);
						if (m.Success && m.Index == StartIdx)
						{
							LoctextNamespaceOpen = false;
						}
					}

					// Preprocessor directive (eg. #define)
					for(; Idx < Text.Length && Text[Idx] != '\n'; Idx++)
					{
						if(Text[Idx] == '\\')
						{
							Idx++;
						}
					}
				}
			}

			// Check if there's anything left over
			if(Flags != 0)
			{
				for(int Idx = 0; Idx < MacroPairs.GetLength(0); Idx++)
				{
					if((Flags & (1 << Idx)) != 0)
					{
						EpicGames.Core.Log.TraceWarningTask(SourceFile, "{0} macro does not have matching {1} macro", MacroPairs[Idx, 0], MacroPairs[Idx, 1]);
					}
				}
			}
			if(LoctextNamespaceOpen)
			{
				EpicGames.Core.Log.TraceWarningTask(SourceFile, "#define NAMESPACE_LOCTEXT preprocessor directive is missing matching #undef NAMESPACE_LOCTEXT directive");
			}
		}

		/// <summary>
		/// Converts an offset within a text buffer into a line number
		/// </summary>
		/// <param name="Text">Text to search</param>
		/// <param name="Offset">Offset within the text</param>
		/// <returns>Line number corresponding to the given offset. Starts from one.</returns>
		int GetLineNumber(string Text, int Offset)
		{
			int LineNumber = 1;
			for(int Idx = 0; Idx < Offset; Idx++)
			{
				if(Text[Idx] == '\n')
				{
					LineNumber++;
				}
			}
			return LineNumber;
		}
	}
}
