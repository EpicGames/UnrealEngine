// Copyright Epic Games, Inc. All Rights Reserved.
using System;
using System.IO;
using System.Linq;
using System.Text.RegularExpressions;
using AutomationTool;
using EpicGames.Core;
using UnrealBuildBase;
using Microsoft.Extensions.Logging;

using static AutomationTool.CommandUtils;
using System.Collections.Generic;

class CheckCsprojDotNetVersion : BuildCommand
{
	public override void ExecuteBuild()
	{
		// just calling this DesiredTargetVersion because TargetVersion and TargetedVersion get super confusing.
		string DesiredTargetVersionParam = this.ParseParamValue("TargetVersion", null);
		if (string.IsNullOrEmpty(DesiredTargetVersionParam))
		{
			throw new AutomationException("-TargetVersion was not specified.");
		}

		HashSet<string> DesiredTargetVersions = new HashSet<string>(DesiredTargetVersionParam.Split('+', StringSplitOptions.RemoveEmptyEntries | StringSplitOptions.TrimEntries));

		Logger.LogInformation("Scanning for all csproj's...");
		// Check for all csproj's in the engine dir
		DirectoryReference EngineDir = Unreal.EngineDirectory;

		// grab the targeted version.,
		Regex FrameworkRegex = new Regex("<TargetFrameworkVersion>v(\\d\\.\\d\\.?\\d?)<\\/TargetFrameworkVersion>");
		Regex PossibleAppConfigRegex = new Regex("<TargetFrameworkProfile>(.+)<\\/TargetFrameworkProfile>");
		Regex AppConfigRegex = new Regex("<supportedRuntime version=\"v(\\d\\.\\d\\.?\\d?)\" sku=\"\\.NETFramework,Version=v(\\d\\.\\d\\.?\\d?),Profile=(.+)\"\\/>");
		Regex NetCoreRegex = new Regex("<TargetFramework>(.*)<\\/TargetFramework>");
		foreach (FileReference CsProj in DirectoryReference.EnumerateFiles(EngineDir, "*.csproj", SearchOption.AllDirectories))
		{
			if (CsProj.ContainsName("ThirdParty", EngineDir) ||
				(CsProj.ContainsName("UE4TemplateProject", EngineDir) && CsProj.GetFileName().Equals("ProjectTemplate.csproj")) ||
				CsProj.GetFileNameWithoutExtension().ToLower().Contains("_mono") ||
				CsProj.ContainsName("UnrealVS", EngineDir) ||
				CsProj.ContainsName("DatasmithRevitExporter", EngineDir) ||
				CsProj.ContainsName("DatasmithNavisworksExporter", EngineDir) ||
				CsProj.ContainsName("CSVTools", EngineDir) ||
				(CsProj.ContainsName("CustomActions", EngineDir) && CsProj.GetFileName().Equals("InstallerCustomActions.csproj")))
			{
				continue;
			}

			// read in the file
			string Contents = File.ReadAllText(CsProj.FullName);

			// Check if we're a _NETCore app
			Match Match = NetCoreRegex.Match(Contents);
			if (Match.Success)
			{
				string TargetedVersion = Regex.Replace(Match.Groups[1].Value, "-.*$", "");
				if (!DesiredTargetVersions.Contains(TargetedVersion))
				{
					Logger.LogWarning("Targeted Framework version for project: {CsProj} was not {Arg1}! Targeted Version: {TargetedVersion}", CsProj, String.Join("/", DesiredTargetVersions), TargetedVersion);
				}
				continue;
			}

			Match = FrameworkRegex.Match(Contents);
			if (Match.Success)
			{
				string TargetedVersion = Match.Groups[1].Value;
				// make sure we match, throw warning otherwise
				if (!DesiredTargetVersions.Any(DesiredTargetVersion => DesiredTargetVersion.Equals(TargetedVersion, StringComparison.InvariantCultureIgnoreCase)))
				{
					Logger.LogWarning("Targeted Framework version for project: {CsProj} was not {Arg1}! Targeted Version: {TargetedVersion}", CsProj, String.Join("/", DesiredTargetVersions), TargetedVersion);
				}
			}
			// if we don't have a TargetFrameworkVersion, check for the existence of TargetFrameworkProfile.
			else
			{
				Match = PossibleAppConfigRegex.Match(Contents);
				if (!Match.Success)
				{
					Logger.LogInformation("No TargetFrameworkVersion or TargetFrameworkProfile found for project {CsProj}, does it compile properly?", CsProj);
					continue;
				}

				// look for the app config
				FileReference AppConfigFile = FileReference.Combine(CsProj.Directory, "app.config");
				string Profile = Match.Groups[1].Value;
				if (!FileReference.Exists(AppConfigFile))
				{
					Logger.LogInformation("Found TargetFrameworkProfile but no associated app.config containing the version for project {CsProj}.", CsProj);
					continue;
				}

				// read in the app config
				Contents = File.ReadAllText(AppConfigFile.FullName);
				Match = AppConfigRegex.Match(Contents);
				if (!Match.Success)
				{
					Logger.LogInformation("Couldn't find a supportedRuntime match for the version in the app.config for project {CsProj}.", CsProj);
					continue;
				}

				// Version1 is the one that appears right after supportedRuntime
				// Version2 is the one in the sku
				// ProfileString should match the TargetFrameworkProfile from the csproj
				string Version1String = Match.Groups[1].Value;
				string Version2String = Match.Groups[2].Value;
				string ProfileString = Match.Groups[3].Value;

				// not sure how this is possible, but check for it anyway
				if (!ProfileString.Equals(Profile, StringComparison.InvariantCultureIgnoreCase))
				{
					Logger.LogWarning("The TargetFrameworkProfile in csproj {CsProj} ({Profile}) doesn't match the sku in it's app.config ({ProfileString}).", CsProj, Profile, ProfileString);
					continue;
				}

				// if the version numbers don't match the app.config is probably corrupt.
				if (!Version1String.Equals(Version2String, StringComparison.InvariantCultureIgnoreCase))
				{
					Logger.LogWarning("The supportedRunTimeVersion ({Version1String}) and the sku version ({Version2String}) in the app.config for project {CsProj} don't match.", Version1String, Version2String, CsProj);
					continue;
				}

				// make sure the versions match
				if (!(DesiredTargetVersions.Any(DesiredTargetVersion => DesiredTargetVersion.Equals(Version1String, StringComparison.InvariantCultureIgnoreCase))))
				{
					Logger.LogWarning("Targeted Framework version for project: {CsProj} was not {Arg1}! Targeted Version: {Version1String}", CsProj, String.Join("/", DesiredTargetVersions), Version1String);
				}
			}
		}
	}
}
