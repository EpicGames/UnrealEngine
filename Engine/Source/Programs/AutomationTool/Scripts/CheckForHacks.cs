// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.IO;
using System.Linq;
using System.Text;
using System.Text.RegularExpressions;
using System.Threading.Tasks;
using EpicGames.Core;
using UnrealBuildBase;
using Microsoft.Extensions.Logging;

namespace AutomationTool
{
	[Help("Audits the current branch for comments denoting a hack that was not meant to leave another branch, following a given format (\"BEGIN XXXX HACK\", where XXXX is one or more tags separated by spaces).")]
	[Help("Allowed tags may be specified manually on the command line. At least one must match, otherwise it will print a warning.")]
	[Help("The current branch name and fragments of the branch path will also be added by default, so running from //UE5/Main will add \"//UE5/Main\", \"UE5\", and \"Main\".")]
	[Help("-Allow", "Specifies additional tags which are allowed in the BEGIN ... HACK tag list")]
	class CheckForHacks : BuildCommand
	{
		/// <summary>
		/// List of file extensions to consider text, and search for hack lines
		/// </summary>
		static readonly HashSet<string> TextExtensions = new HashSet<string>(StringComparer.InvariantCultureIgnoreCase)
		{
			".cpp",
			".c",
			".h",
			".inl",
			".m",
			".mm",
			".java",
			".pl",
			".pm",
			".cs",
			".sh",
			".bat",
			".xml",
			".htm",
			".html",
			".xhtml",
			".css",
			".asp",
			".aspx",
			".js",
			".py",
		};

		/// <summary>
		/// The pattern that should match hack comments (and captures a list of tags). Ignore anything with a semicolon between BEGIN and HACK to avoid matching statements with "HACK" as a comment (seen in LLVM).
		/// </summary>
		static readonly Regex CompiledRegex = new Regex("(?<!\\w)(?:BEGIN|START)\\s([^;]*)\\sHACK(?!\\w)", RegexOptions.IgnoreCase | RegexOptions.Compiled);

		/// <summary>
		/// Executes the command
		/// </summary>
		public override void ExecuteBuild()
		{
			// Build a list of all the allowed tags
			HashSet<string> AllowTags = new HashSet<string>(StringComparer.InvariantCultureIgnoreCase);

			foreach (string AllowTag in ParseParamValues("Allow"))
			{
				AllowTags.Add(AllowTag);
			}

			if(P4Enabled)
			{
				AllowTags.Add(P4Env.Branch);
				AllowTags.Add(P4Env.Branch.Trim('/'));
				AllowTags.UnionWith(P4Env.Branch.Split(new char[] { '/' }, StringSplitOptions.RemoveEmptyEntries));
			}

			// Enumerate all the files in the workspace
			Logger.LogInformation("Finding files in workspace...");

			List<FileInfo> FilesToCheck = new List<FileInfo>();
			using (ThreadPoolWorkQueue Queue = new ThreadPoolWorkQueue())
			{
				DirectoryInfo BaseDir = new DirectoryInfo(Unreal.EngineDirectory.FullName);
				Queue.Enqueue(() => FindAllFiles(Queue, BaseDir, FilesToCheck));
				Queue.Wait();
			}

			// Scan all of the files for invalid comments
			Logger.LogInformation("Scanning files...");
			using (ThreadPoolWorkQueue Queue = new ThreadPoolWorkQueue())
			{
				foreach(FileInfo File in FilesToCheck)
				{
					FileInfo FileCapture = File;
					Queue.Enqueue(() => ScanSourceFile(FileCapture, AllowTags));
				}
				while(!Queue.Wait(5 * 1000))
				{
					lock(this)
					{
						Logger.LogInformation("Scanning files... [{Arg0}/{Arg1}]", FilesToCheck.Count - Queue.NumRemaining, FilesToCheck.Count);
					}
				}
			}
		}

		/// <summary>
		/// Enumerates all files under the given directory
		/// </summary>
		/// <param name="Queue">Queue to add additional subfolders to search to</param>
		/// <param name="BaseDir">Directory to search</param>
		/// <param name="FilesToCheck">Output list of files to check. Will be locked before adding items.</param>
		void FindAllFiles(ThreadPoolWorkQueue Queue, DirectoryInfo BaseDir, List<FileInfo> FilesToCheck)
		{
			foreach(DirectoryInfo SubDir in BaseDir.EnumerateDirectories())
			{
				DirectoryInfo SubDirCapture = SubDir;
				Queue.Enqueue(() => FindAllFiles(Queue, SubDirCapture, FilesToCheck));
			}
			foreach(FileInfo FileToCheck in BaseDir.EnumerateFiles())
			{
				if(TextExtensions.Contains(FileToCheck.Extension))
				{
					lock(FilesToCheck)
					{
						FilesToCheck.Add(FileToCheck);
					}
				}
			}
		}

		/// <summary>
		/// Scans an individual source file for hack comments
		/// </summary>
		/// <param name="FileToCheck">The file to be checked</param>
		/// <param name="AllowTags">Set of tags which are allowed to appear in hack comments</param>
		void ScanSourceFile(FileInfo FileToCheck, HashSet<string> AllowTags)
		{
			// Ignore the current file. We have comments that reference the desired format for hacks. :)
			if(!String.Equals(FileToCheck.Name, "CheckForHacks.cs", StringComparison.InvariantCultureIgnoreCase))
			{
				using (FileStream Stream = FileToCheck.Open(FileMode.Open, FileAccess.Read, FileShare.ReadWrite))
				{
					using (StreamReader Reader = new StreamReader(Stream, true))
					{
						for (int LineNumber = 1; ; LineNumber++)
						{
							string Line = Reader.ReadLine();
							if (Line == null)
							{
								break;
							}

							Match Result = CompiledRegex.Match(Line);
							if(Result.Success)
							{
								string[] Tags = Result.Groups[1].Value.Split(new char[] { ' ', '\t' }, StringSplitOptions.RemoveEmptyEntries);
								if (!Tags.Any(Tag => AllowTags.Contains(Tag)))
								{
									lock (this)
									{
										EpicGames.Core.Log.WriteLine(EpicGames.Core.LogEventType.Warning, EpicGames.Core.LogFormatOptions.NoSeverityPrefix, "{0}({1}): warning: Code should not be in this branch: '{2}'", FileToCheck.FullName, LineNumber, Line.Trim());
									}
								}
							}
						}
					}
				}
			}
		}
	}
}
