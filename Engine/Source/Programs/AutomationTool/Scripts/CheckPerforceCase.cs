// Copyright Epic Games, Inc. All Rights Reserved.

using AutomationTool;
using System;
using System.Collections.Generic;
using System.Linq;
using System.Text;
using System.Threading.Tasks;
using Microsoft.Extensions.Logging;
using EpicGames.Core;

namespace BuildScripts.Automation
{
	[RequireP4]
	[DoesNotNeedP4CL]
	[Help("Checks that the casing of files within a path on a case-insensitive Perforce server is correct.")]
	[Help("Path", "The path to query")]
	class CheckPerforceCase : BuildCommand
	{
		class TreeNode
		{
			public string Path;
			public List<string> Files = new List<string>();
			public Dictionary<string, TreeNode> ChildNodes = new Dictionary<string, TreeNode>(StringComparer.Ordinal);
			public int Count;

			public TreeNode(string Path)
			{
				this.Path = Path;
			}

			public void Add(string DepotPath, int Idx)
			{
				int NextIdx = DepotPath.IndexOf('/', Idx);
				if(NextIdx == -1)
				{
					Files.Add(DepotPath);
				}
				else
				{
					string Fragment = DepotPath.Substring(Idx, NextIdx - Idx);

					TreeNode ChildNode;
					if (!ChildNodes.TryGetValue(Fragment, out ChildNode))
					{
						ChildNode = new TreeNode(DepotPath.Substring(0, NextIdx));
						ChildNodes.Add(Fragment, ChildNode);
					}
					ChildNode.Add(DepotPath, NextIdx + 1);
				}
				Count++;
			}

			public IEnumerable<string> EnumerateFiles()
			{
				foreach (string File in Files)
				{
					yield return File;
				}
				foreach (TreeNode ChildNode in ChildNodes.Values)
				{
					foreach (string ChildFile in ChildNode.EnumerateFiles())
					{
						yield return ChildFile;
					}
				}
			}

			public int PrintIssues(ILogger Logger)
			{
				int NumIssues = 0;
				foreach (IGrouping<string, KeyValuePair<string, TreeNode>> Group in ChildNodes.GroupBy(x => x.Key, StringComparer.OrdinalIgnoreCase))
				{
					if (Group.Count() > 1)
					{
						Logger.LogWarning(KnownLogEvents.AutomationTool_PerforceCase, "Inconsistent casing for {File}:", Group.First().Value.Path);

						foreach (KeyValuePair<string, TreeNode> Pair in Group)
						{
							Logger.LogWarning(KnownLogEvents.AutomationTool_PerforceCase, "  Could be '{Rendering}':", Pair.Key);

							int NumFiles = 0;
							foreach (string File in Pair.Value.EnumerateFiles())
							{
								Logger.LogWarning(KnownLogEvents.AutomationTool_PerforceCase, "    {DepotFile}", new LogValue(LogValueType.DepotPath, File));
								if (++NumFiles >= 10)
								{
									break;
								}
							}

							Logger.LogWarning(KnownLogEvents.AutomationTool_PerforceCase, "    ({NumFile} file(s))", Pair.Value.Count);
						}

						NumIssues++;
					}

					foreach (KeyValuePair<string, TreeNode> Pair in Group)
					{
						NumIssues += Pair.Value.PrintIssues(Logger);
					}
				}
				return NumIssues;
			}
		}

		public override void ExecuteBuild()
		{
			// Parse the path
			string Filter = ParseRequiredStringParam("Filter");
			if(!Filter.StartsWith("//"))
			{
				throw new AutomationException("Filter is not a depot path");
			}

			// Find all the matching files
			Logger.LogInformation("Finding files matching {Filter}", Filter);
			List<string> Files = P4.Files(String.Format("-e {0}", Filter));
			Logger.LogInformation("Found {Arg0} files", Files.Count);

			// Build a tree of all the conflicting paths and print them out
			TreeNode RootNode = new TreeNode("//");
			foreach (string File in Files)
			{
				if (!File.StartsWith("//", StringComparison.Ordinal))
				{
					Logger.LogWarning("File '{File}' does not start with '//'", File);
					continue;
				}
				RootNode.Add(File, 2);
			}
			RootNode.PrintIssues(Logger);
		}
	}
}
