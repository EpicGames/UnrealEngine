// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.IO;
using System.Linq;
using System.Text;
using System.Threading.Tasks;
using EpicGames.Core;
using UnrealBuildBase;
using UnrealBuildTool;
using Microsoft.Extensions.Logging;

using static AutomationTool.CommandUtils;

namespace AutomationTool
{
	[Help("Checks a directory for folders which should not be distributed")]
	[Help("BaseDir=<Path>", "Path to the base directory containing files to check")]
	[Help("Allow=<Name1>+<Name2>...", "Specify names of folders which should be excluded from the list")]
	class CheckRestrictedFolders : BuildCommand
	{
		public override ExitCode Execute()
		{
			// Get the base directory
			DirectoryReference BaseDir = new DirectoryReference(ParseParamValue("BaseDir"));
			if(!DirectoryReference.Exists(BaseDir))
			{
				throw new AutomationException("Base directory '{0}' does not exist", BaseDir);
			}

			// Find a list of restricted folders, and remove any names which are explicitly allowed
			HashSet<string> RestrictedNames = new HashSet<string>(RestrictedFolder.GetNames(), StringComparer.InvariantCultureIgnoreCase);
			foreach (string AllowParam in ParseParamValues("Allow"))
			{
				RestrictedNames.ExceptWith(AllowParam.Split('+'));
			}

			// Find all the folders which are problematic
			Logger.LogInformation("Searching for folders under {BaseDir} named {Arg1}...", BaseDir, String.Join(", ", RestrictedNames));
			List<DirectoryInfo> ProblemFolders = new List<DirectoryInfo>();
			FindRestrictedFolders(new DirectoryInfo(BaseDir.FullName), RestrictedNames, ProblemFolders);

			// Print out all the restricted folders
			if(ProblemFolders.Count > 0)
			{
				Logger.LogError("Found {Arg0} {Arg1} which should not be distributed:", ProblemFolders.Count, (ProblemFolders.Count == 1) ? "folder" : "folders");
				foreach(DirectoryInfo ProblemFolder in ProblemFolders)
				{
					Logger.LogError("    {Arg0}{Arg1}...", new DirectoryReference(ProblemFolder).MakeRelativeTo(BaseDir), Path.DirectorySeparatorChar);
				}
				return ExitCode.Error_Unknown;
			}

			// Otherwise return success
			Logger.LogInformation("No restricted folders found under {BaseDir}", BaseDir);
			return ExitCode.Success;
		}

		void FindRestrictedFolders(DirectoryInfo CurrentDir, HashSet<string> RestrictedNames, List<DirectoryInfo> ProblemFolders)
		{
			foreach (DirectoryInfo SubDir in CurrentDir.EnumerateDirectories("*", SearchOption.TopDirectoryOnly))
			{
				if(RestrictedNames.Contains(SubDir.Name))
				{
					ProblemFolders.Add(SubDir);
				}
				else
				{
					FindRestrictedFolders(SubDir, RestrictedNames, ProblemFolders);
				}
			}
		}
	}
}
