// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.IO;
using System.Linq;
using System.Text;
using System.Threading.Tasks;
using EpicGames.Core;
using UnrealBuildTool;
using Microsoft.Extensions.Logging;

using static AutomationTool.CommandUtils;

namespace AutomationTool
{
	/// <summary>
	/// Commandlet to clean up all afolders under a temp storage root that are older than a given number of days
	/// </summary>
	[Help("Removes folders in an automation report directory that are older than a certain time.")]
	[Help("ReportDir=<Directory>", "Path to the root report directory")]
	[Help("Days=<N>", "Number of days to keep reports for")]
	[Help("Depth=<N>", "How many subdirectories deep to clean, defaults to 0 (top level cleaning).")]
	class CleanAutomationReports : BuildCommand
	{
		/// <summary>
		/// Entry point for the commandlet
		/// </summary>
		public override void ExecuteBuild()
		{
			string ReportDir = ParseRequiredStringParam("ReportDir");
			string Days = ParseRequiredStringParam("Days");

			double DaysValue;
			if(!Double.TryParse(Days, out DaysValue))
			{
				throw new AutomationException("'{0}' is not a valid value for the -Days parameter", Days);
			}

			string Depth = ParseOptionalStringParam("Depth");
			int TargetDepth;
			if (!int.TryParse(Depth, out TargetDepth))
			{
				TargetDepth = 0;
			}

			DirectoryInfo ReportDirInfo = new DirectoryInfo(ReportDir);
			if (!ReportDirInfo.Exists)
			{
				throw new AutomationException("Report directory '{0}' does not exists.", ReportDirInfo.FullName);
			}

			DateTime RetainTime = DateTime.UtcNow - TimeSpan.FromDays(DaysValue);
			List<DirectoryInfo> DirectoriesToDelete = new List<DirectoryInfo>();
			int DirsScanned = CleanDirectories(ReportDirInfo, RetainTime, DirectoriesToDelete, TargetDepth);

			// Delete old folders.
			Logger.LogInformation("Found {DirsScanned} builds; {Arg1} to delete.", DirsScanned, DirectoriesToDelete.Count);
			for(int Idx = 0; Idx < DirectoriesToDelete.Count; Idx++)
			{
				try
				{
					Logger.LogInformation("[{Arg0}/{Arg1}] Deleting {Arg2}...", Idx + 1, DirectoriesToDelete.Count, DirectoriesToDelete[Idx].FullName);
					DirectoriesToDelete[Idx].Delete(true);
				}
				catch(Exception Ex)
				{
					Logger.LogWarning("Failed to delete folder; will try one file at a time: {Ex}", Ex);
					CommandUtils.DeleteDirectory_NoExceptions(true, DirectoriesToDelete[Idx].FullName);
				}
			}
		}

		private int CleanDirectories(DirectoryInfo Directory, DateTime RetainTime, List<DirectoryInfo> DirectoriesToDelete, int TargetDepth, int CurrentDepth = 0)
		{
			// Go deeper if we need to.
			if(TargetDepth > CurrentDepth)
			{
				int DirsFound = 0;
				foreach(DirectoryInfo SubDirectory in Directory.EnumerateDirectories())
				{
					// If we find a DO_NOT_CLEANUP.txt file before we reach the target depth we want to ignore that entire portion of the directory structure.
					if (File.Exists(Path.Combine(SubDirectory.FullName, "DO_NOT_CLEANUP.txt")))
					{
						Logger.LogInformation("Found DO_NOT_CLEANUP.txt file in {SubDirectory} before target depth, excluding it from cleanup process.", SubDirectory);
						continue;
					}
					DirsFound += CleanDirectories(SubDirectory, RetainTime, DirectoriesToDelete, TargetDepth, CurrentDepth + 1);
				}
				return DirsFound;
			}
			else
			{
				Logger.LogInformation("Scanning {Directory}...", Directory);
				IEnumerable<DirectoryInfo> DirsToScan = Directory.EnumerateDirectories();
				foreach(DirectoryInfo BuildDirectory in DirsToScan)
				{
					try
					{
						// If we find a DO_NOT_CLEANUP.txt file recursively anywhere in a build folder, leave that build alone.
						if (BuildDirectory.EnumerateFiles("*", SearchOption.AllDirectories).Any(x => x.Name.ToLower() == "do_not_cleanup.txt"))
						{
							Logger.LogInformation("Found DO_NOT_CLEANUP.txt file in {BuildDirectory}, skipping it!", BuildDirectory);
							continue;
						}
						if(!BuildDirectory.EnumerateFiles("*", SearchOption.AllDirectories).Any(x => x.LastWriteTimeUtc > RetainTime))
						{
							DirectoriesToDelete.Add(BuildDirectory);
						}
					}
					catch(Exception Ex)
					{
						Logger.LogWarning("Unable to enumerate {Arg0}: {Arg1}", BuildDirectory.FullName, Ex.ToString());
					}
				}
				return DirsToScan.Count();
			}
		}
	}
}
