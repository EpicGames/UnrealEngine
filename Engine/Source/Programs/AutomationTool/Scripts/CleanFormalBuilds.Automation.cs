// Copyright Epic Games, Inc. All Rights Reserved.

using System.IO;

namespace AutomationTool
{
	/// <summary>
	/// Commandlet to clean up all builds under a root directory that follow a given pattern and are older than a given number of days
	/// </summary>
	[Help("Removes folders matching a pattern in a given directory that are older than a certain time.")]
	[Help("ParentDir=<Directory>", "Path to the root directory")]
	[Help("SearchPattern=<Pattern>", "Pattern to match against")]
	[Help("Days=<N>", "Number of days to keep in temp storage (optional - defaults to 4)")]
	class CleanFormalBuilds : BuildCommand
	{
		/// <summary>
		/// Entry point for the commandlet
		/// </summary>
		public override void ExecuteBuild()
		{
			string ParentDir = ParseParamValue("ParentDir", null);
			if (ParentDir == null)
			{
				throw new AutomationException("Missing -ParentDir parameter");
			}
			ParentDir = Path.GetFullPath(ParentDir);

			string SearchPattern = ParseParamValue("SearchPattern", null);
			if (SearchPattern == null)
			{
				throw new AutomationException("Missing -SearchPattern parameter");
			}

			string Days = ParseParamValue("Days", null);
			if (Days == null)
			{
				CommandUtils.CleanFormalBuilds(ParentDir, SearchPattern);
			}
			else
			{
				double DaysValue;
				if (!double.TryParse(Days, out DaysValue))
				{
					throw new AutomationException("'{0}' is not a valid value for the -Days parameter", Days);
				}
				CommandUtils.CleanFormalBuilds(ParentDir, SearchPattern, DaysValue);
			}
		}
	}
}
