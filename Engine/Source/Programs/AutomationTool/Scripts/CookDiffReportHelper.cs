// Copyright Epic Games, Inc. All Rights Reserved.

using EpicGames.Core;
using Microsoft.Extensions.Logging;
using System;
using System.Collections.Generic;
using System.IO;
using System.Linq;
using System.Text.Json;
using System.Threading.Tasks;

namespace AutomationTool
{
	[Help("CookDiffReportHelper analyses the json files created by -run=cook -diffonly -cookdiffjson=path/to/my.json ... ")]
	[Help("Source", "Path to the cookdiffjson file or folder")]
	[Help("MaxResults", "Maximum number of results to display (default is all results)")]
	[Help("MaxFiles", "Maximum number of files to log for a particular diff (default 10. use 0 to show all files)")]
	public class CookDiffReportHelper : BuildCommand
	{
		public override void ExecuteBuild()
		{
			string Source = ParseRequiredStringParam("Source");
			int MaxResults = ParseParamInt("MaxResults", int.MaxValue);
			int MaxFiles = ParseParamInt("MaxFiles", 10);
			if (MaxFiles == 0)
			{
				MaxFiles = int.MaxValue;
			}

			DetailRoot Detail = ReadJson(Source);
			GenerateReport(Detail, MaxResults, MaxFiles);
		}

		private void GenerateReport(DetailRoot Detail, int MaxResults, int MaxFiles )
		{
			Logger.LogInformation("************************** COOK DIFF REPORT **************************");
			Logger.LogInformation("Project Name:    {ProjectName}", Detail.ProjectName);
			Logger.LogInformation("Build Version:   {BuildVersion}", Detail.BuildVersion);
			Logger.LogInformation("");

			int NumResults = MaxResults;
			HashSet<Diff> UsedDiffs = [];

			PropertyNamePriorityReport.GenerateReport(Detail, ref NumResults, UsedDiffs, MaxFiles);
			CallstackPriorityReport.GenerateReport(Detail, ref NumResults, UsedDiffs, MaxFiles);
			PackagePriorityReport.GenerateReport(Detail, ref NumResults, UsedDiffs);

			Logger.LogInformation("**********************************************************************");
		}



		private sealed class DetailRoot
		{
			public string ProjectName { get; set; }
			public string BuildVersion { get; set; }
			public List<Package> Packages { get; set; } = [];
		};

		private sealed class Package
		{
			public string Filename { get; set; }
			public string ClassName { get; set; }
			public Section Header { get; set; }
			public Section Exports { get; set; }
			public string Diagnostics { get; set; }
			public IEnumerable<Section> Sections => new[] {Header, Exports}.Where( S => S != null );
		};

		private sealed class Section
		{
			public string Filename { get; set; }
			public UInt64 SourceSize { get; set; } = 0;
			public UInt64 DestSize  { get; set; } = 0;
			public bool UndiagnosedDiff  { get; set; } = false;
			public int UnreportedDiffs { get; set; } = 0;
			public List<Diff> Diffs { get; set; } = [];
			public List<TableDiff> TableDiffs { get; set; } = [];
			public bool SizeMismatch => (SourceSize != DestSize);
		};

		private sealed class Diff
		{
			public Int64 LocalOffset { get; set; } = 0;
			public int Size { get; set; } = 0;
			public int Count { get; set; }	= 1;
			public string ObjectName { get; set; }
			public string PropertyName { get; set; }
			public string Callstack { get; set; }
		};

		private sealed class TableDiff
		{
			public string ItemName { get; set; }
			public string Detail { get; set; }
		};


		private IEnumerable<FileReference> FindSourceJsons(string Source)
		{
			// source is a folder
			DirectoryReference SourceDirectory = new(Source);
			if (DirectoryReference.Exists(SourceDirectory))
			{
				return DirectoryReference.EnumerateFiles( SourceDirectory, "*.json" );
			}
			if (Source.EndsWith('/') || Source.EndsWith('\\'))
			{
				return [];
			}


			// source is a file
			FileReference JsonFile = new(Source);

			// for multiprocess cook, additional json files may have been created alongside this one, myfile-1.json, myfile-2.json etc
			string BaseFileName = JsonFile.GetFileNameWithoutExtension();
			string SubFileNameFilter = BaseFileName + "-*" + JsonFile.GetExtension();
			List<FileReference> JsonFiles = DirectoryReference.EnumerateFiles(JsonFile.Directory, SubFileNameFilter).ToList();

			if (FileReference.Exists(JsonFile))
			{
				JsonFiles.Add(JsonFile);
			}

			return JsonFiles;
		}

		private async Task<DetailRoot> ParseAndMergeJsonAsync(IEnumerable<FileReference> SourceJsons)
		{
			// read all json files (these could be big so go wide)
			IEnumerable<DetailRoot> Reports = (await Task.WhenAll(SourceJsons.Select(
				async SourceJson =>
				{
					try
					{
						using FileStream JsonFile = File.OpenRead(SourceJson.FullName);
						return await JsonSerializer.DeserializeAsync<DetailRoot>(JsonFile);
					}
					catch(Exception e)
					{
						Logger.LogError("Failed to read json file {file} : {msg}", SourceJson.FullName, e.Message);
						return null;
					}
				}))).Where( R => R != null );
			if (!Reports.Any())
			{
				throw new AutomationException("Could not parse any json files");
			}

			// sanity check
			if (!Reports.All( R => R.ProjectName == Reports.First().ProjectName))
			{
				throw new AutomationException("Mismatched ProjectName. Expected {0} in all json files", Reports.First().ProjectName);
			}
			if (!Reports.All( R => R.BuildVersion == Reports.First().BuildVersion))
			{
				throw new AutomationException("Mismatched BuildVersion. Expected {0} in all json files", Reports.First().BuildVersion);
			}

			// merge the result
			DetailRoot Result = new()
			{
				ProjectName = Reports.First().ProjectName,
				BuildVersion = Reports.First().BuildVersion,
				Packages = [.. Reports.SelectMany(R => R.Packages)]
			};

			return Result;

		}

		private DetailRoot ReadJson(string Source)
		{
			// find the json files in the source folder
			IEnumerable<FileReference> SourceJsons = FindSourceJsons(Source);
			if (!SourceJsons.Any())
			{
				throw new AutomationException("No json files found in -source={0}", Source);
			}
			Logger.LogInformation("Found {num} json files:", SourceJsons.Count());
			foreach (FileReference SourceJson in SourceJsons)
			{
				Logger.LogInformation("\t{json}", SourceJson);
			}

			// load and merge all json files
			DetailRoot Detail = ParseAndMergeJsonAsync(SourceJsons).GetAwaiter().GetResult();
			return Detail;
		}


		private static string GetCleanCallstack(string Callstack)
		{
			if (string.IsNullOrEmpty(Callstack))
			{
				return string.Empty;
			}

			List<string> Lines = Callstack.Split('\n', StringSplitOptions.RemoveEmptyEntries | StringSplitOptions.TrimEntries).ToList();

			// remove lines that may contain instance-specific data
			string[] ExcludePrefixes = ["Serialized Object:"/*, "Serialized Property:"*/ ];
			Lines.RemoveAll( L => ExcludePrefixes.Any( P => L.StartsWith(P, StringComparison.OrdinalIgnoreCase) ) );

			return string.Join('\n', Lines);
		}

		private static string GetCleanPackageFilename(string Filename)
		{
			// trim off most of the path
			string TrimString = "Saved/Cooked/";
			int CropIndex = Filename.IndexOf(TrimString);
			if (CropIndex != -1)
			{
				CropIndex = Filename.IndexOf('/', CropIndex + TrimString.Length + 1);
				if (CropIndex != -1)
				{
					Filename = Filename.Substring(CropIndex+1);
				}
			}

			return Filename;
		}


		private class PropertyNamePriorityReport
		{
			class CallstackInfo
			{
				public string Callstack;
				public HashSet<string> Filenames = [];
			};

			class PropertyNameInfo
			{
				public Dictionary<int,CallstackInfo> CallstackMap = []; // key = callstack hash
				public HashSet<string> Filenames = [];
				public Dictionary<string,int> ClassCounts = [];
				public List<Diff> Diffs = [];
				public int DiffBytes = 0;
				public int SizeMismatches = 0;
			};

			public static void GenerateReport(DetailRoot Detail, ref int MaxItems, HashSet<Diff> UsedDiffs, int MaxFiles)
			{
				Dictionary<string,PropertyNameInfo> InfoMap = [];

				foreach (Package Package in Detail.Packages)
				{
					foreach (Section Section in Package.Sections)
					{
						foreach (Diff Diff in Section.Diffs)
						{
							if (string.IsNullOrEmpty(Diff.PropertyName))
							{
								continue;
							}

							if (UsedDiffs.Contains(Diff))
							{
								continue;
							}

							string Callstack = GetCleanCallstack(Diff.Callstack);
							if (string.IsNullOrEmpty(Callstack))
							{
								continue;
							}

							if (!InfoMap.TryGetValue(Diff.PropertyName, out PropertyNameInfo Info))
							{
								Info = new();
								InfoMap.Add(Diff.PropertyName, Info );
							}

							int CallstackHash = Callstack.GetHashCode();
							if (!Info.CallstackMap.TryGetValue(CallstackHash, out CallstackInfo CallstackInfo))
							{
								CallstackInfo = new();
								CallstackInfo.Callstack = Callstack;
								Info.CallstackMap.Add(CallstackHash, CallstackInfo);
							}
							CallstackInfo.Filenames.Add(Section.Filename ?? Package.Filename);

							Info.DiffBytes += (Diff.Count * Diff.Size);
							Info.Filenames.Add(Section.Filename ?? Package.Filename);
							Info.Diffs.Add(Diff);

							Info.ClassCounts.TryAdd(Package.ClassName, 0);
							Info.ClassCounts[Package.ClassName]++;

							if (Section.SizeMismatch)
							{
								Info.SizeMismatches++;
							}
						}
					}
				}

				int NumItems = Math.Min(MaxItems, InfoMap.Count);
				MaxItems -= NumItems;

				IOrderedEnumerable<KeyValuePair<string,PropertyNameInfo>> OrderedInfoMap = InfoMap.OrderByDescending( Pair => Pair.Value.Filenames.Count ); // @todo: may need heuristic here
				foreach (KeyValuePair<string,PropertyNameInfo> InfoPair in OrderedInfoMap.Take(NumItems))
				{
					PropertyNameInfo Info = InfoPair.Value;
					IOrderedEnumerable<KeyValuePair<int,CallstackInfo>> OrderedCallstackMap = Info.CallstackMap.OrderByDescending( Pair => Pair.Value.Filenames.Count ); // @todo: may need heuristic here
					CallstackInfo CallstackInfo = OrderedCallstackMap.First().Value; 
					int NumFiles = Info.Filenames.Count;
					Logger.LogWarning("Serialization property affecting {files} files, totalling at least {bytes} bytes of nondeterminism{sizemsg}", NumFiles, Info.DiffBytes, (Info.SizeMismatches > 0) ? $" - {Info.SizeMismatches} files have nondeterministic sizes" : "" );
					Logger.LogInformation("\tSerialized Property: {property}", InfoPair.Key);
					Logger.LogInformation("\tHighest-Impact Callstack Affecting {count} files ({other} other callstacks):", CallstackInfo.Filenames.Count, Info.CallstackMap.Count-1);
					foreach (string CallstackLine in CallstackInfo.Callstack.EnumerateLines())
					{
						if (!CallstackLine.StartsWith("Serialized Property:")) // don't show this as it's already been shown
						{
							Logger.LogInformation("\t\t{line}", CallstackLine);
						}
					}
					Logger.LogInformation("\tAffected classes:");
					foreach (KeyValuePair<string,int> ClassCountPair in Info.ClassCounts)
					{
						Logger.LogInformation("\t\t{class} x {count}", ClassCountPair.Key, ClassCountPair.Value);
					}
					
					if (NumFiles > MaxFiles)
					{
						Logger.LogInformation("\tAffected files (showing first {max}):", MaxFiles);
						NumFiles = MaxFiles;
					}
					else
					{
						Logger.LogInformation("\tAffected files:");
					}
					foreach (string Filename in Info.Filenames.Take(NumFiles))
					{
						Logger.LogInformation("\t\t{filename}", GetCleanPackageFilename(Filename));
					}
					Logger.LogInformation("");

					foreach (Diff Diff in Info.Diffs)
					{
						UsedDiffs.Add(Diff);
					}
				}
			}
		}



		private class CallstackPriorityReport
		{
			class CallstackInfo
			{
				public string Callstack;
				public HashSet<string> Filenames = [];
				public Dictionary<string,int> ClassCounts = [];
				public List<Diff> Diffs = [];
				public int DiffBytes = 0;
				public int SizeMismatches = 0;
			};

			public static void GenerateReport(DetailRoot Detail, ref int MaxItems, HashSet<Diff> UsedDiffs, int MaxFiles)
			{
				Dictionary<int,CallstackInfo> InfoMap = []; // key = callstack hash

				foreach (Package Package in Detail.Packages)
				{
					foreach (Section Section in Package.Sections)
					{
						foreach (Diff Diff in Section.Diffs)
						{
							if (UsedDiffs.Contains(Diff))
							{
								continue;
							}

							string Callstack = GetCleanCallstack(Diff.Callstack);
							if (string.IsNullOrEmpty(Callstack))
							{
								continue;
							}

							int CallstackHash = Callstack.GetHashCode();
							if (!InfoMap.TryGetValue(CallstackHash, out CallstackInfo Info))
							{
								Info = new();
								Info.Callstack = Callstack;
								InfoMap.Add(CallstackHash, Info);
							}

							Info.DiffBytes += (Diff.Count * Diff.Size);
							Info.Filenames.Add(Section.Filename ?? Package.Filename);
							Info.Diffs.Add(Diff);

							Info.ClassCounts.TryAdd(Package.ClassName, 0);
							Info.ClassCounts[Package.ClassName]++;

							if (Section.SizeMismatch)
							{
								Info.SizeMismatches++;
							}
						}
					}
				}

				int NumItems = Math.Min(MaxItems, InfoMap.Count);
				MaxItems -= NumItems;

				

				IOrderedEnumerable<KeyValuePair<int,CallstackInfo>> OrderedInfoMap = InfoMap.OrderByDescending( Pair => Pair.Value.Filenames.Count ); // @todo: may need heuristic here
				foreach (KeyValuePair<int,CallstackInfo> InfoPair in OrderedInfoMap.Take(NumItems))
				{
					CallstackInfo Info = InfoPair.Value;
					int NumFiles = Info.Filenames.Count;
					Logger.LogWarning("Serialization callstack affecting {files} files, totalling at least {bytes} bytes of nondeterminism{sizemsg}", NumFiles, Info.DiffBytes, (Info.SizeMismatches > 0) ? $" - {Info.SizeMismatches} files have nondeterministic sizes" : "" );
					Logger.LogInformation("\t{callstack}", Info.Callstack.Replace("\n","\n\t"));
					Logger.LogInformation("\tAffected classes:");
					foreach (KeyValuePair<string,int> ClassCountPair in Info.ClassCounts)
					{
						Logger.LogInformation("\t\t{class} x {count}", ClassCountPair.Key, ClassCountPair.Value);
					}
					
					if (NumFiles > MaxFiles)
					{
						Logger.LogInformation("\tAffected files (showing first {max}):", MaxFiles);
						NumFiles = MaxFiles;
					}
					else
					{
						Logger.LogInformation("\tAffected files:");
					}
					foreach (string Filename in Info.Filenames.Take(NumFiles))
					{
						Logger.LogInformation("\t\t{filename}", GetCleanPackageFilename(Filename));
					}
					Logger.LogInformation("");

					foreach (Diff Diff in Info.Diffs)
					{
						UsedDiffs.Add(Diff);
					}
				}
			}
		}




		private class PackagePriorityReport
		{
			public static void GenerateReport(DetailRoot Detail, ref int MaxItems, HashSet<Diff> UsedDiffs)
			{
				IOrderedEnumerable<Package> OrderedPackages = Detail.Packages.OrderByDescending( P => CalculateImportance(P, UsedDiffs) );
				foreach (Package Package in OrderedPackages)
				{
					MaxItems--;
					if (MaxItems < 0)
					{
						break;
					}

					IEnumerable<Diff> HeaderDiffs = Package.Header?.Diffs.Where( D => !UsedDiffs.Contains(D));    // }  
					IEnumerable<Diff> ExportDiffs = Package.Exports?.Diffs.Where( D => !UsedDiffs.Contains(D));   // } only reporting diffs we've not reported elsewhere
					bool bHeaderHasDiffs = SectionHasDiffs(Package.Header, HeaderDiffs);
					bool bExportHasDiffs = SectionHasDiffs(Package.Exports, ExportDiffs);

					if (bHeaderHasDiffs || bExportHasDiffs)
					{
						int NumHeaderDiffBytes = HeaderDiffs?.Sum( D => D.Count * D.Size) ?? 0;
						int NumExportDiffBytes = ExportDiffs?.Sum( D => D.Count * D.Size) ?? 0;
						if (NumHeaderDiffBytes > 0 && NumExportDiffBytes > 0)
						{
							Logger.LogWarning("Package header with {header} byte(s) of nondeterminism, and exports with {exports} byte(s) of nondeterminism", NumHeaderDiffBytes, NumExportDiffBytes);
						}
						else if (NumHeaderDiffBytes > 0)
						{
							Logger.LogWarning("Package header with {header} byte(s) of nondeterminism", NumHeaderDiffBytes);
						}
						else if (NumExportDiffBytes > 0)
						{
							Logger.LogWarning("Package exports with {exports} byte(s) of nondeterminism", NumExportDiffBytes);
						}
						else
						{
							Logger.LogWarning("Nondeterministic package");
						}

						Logger.LogInformation("Package Name: {PackageName}", GetCleanPackageFilename(Package.Filename));
						Logger.LogInformation("Class Name:   {ClassName}", Package.ClassName);
			
						if (bHeaderHasDiffs)
						{
							Logger.LogInformation("Header:");
							GenerateReport(Package.Header, HeaderDiffs, UsedDiffs);
						}

						if (bExportHasDiffs)
						{
							Logger.LogInformation("Exports:");
							GenerateReport(Package.Exports, ExportDiffs, UsedDiffs);
						}

						Logger.LogInformation("");			
					}
				}
			}

			private static bool SectionHasDiffs( Section Section, IEnumerable<Diff> SectionDiffs )
			{
				return Section != null && (Section.SizeMismatch || Section.UndiagnosedDiff || Section.UnreportedDiffs > 0 || Section.TableDiffs.Count > 0 || (SectionDiffs?.Any() ?? false));
			}

			private static void GenerateReport(Section Section, IEnumerable<Diff> SectionDiffs, HashSet<Diff> UsedDiffs)
			{
				int NumPreviousReported = Section.Diffs.Intersect(UsedDiffs).Count();
				if (!string.IsNullOrEmpty(Section.Filename))
				{
					Logger.LogInformation("\tFile name: {filename}", GetCleanPackageFilename(Section.Filename));
				}
				if (Section.SizeMismatch)
				{
					Logger.LogInformation("\tSize mismatch: {sourcesize} vs {destsize}", Section.SourceSize, Section.DestSize);
				}
				if (Section.UndiagnosedDiff)
				{
					Logger.LogInformation("\tUndiagnosed diff (DumpPackageHeaderDiffs does not yet implement describing the difference)");
				}
				if (SectionDiffs.Any())
				{
					Logger.LogInformation("\tDiffs:");
					foreach (Diff Diff in SectionDiffs)
					{
						if (Diff.Count > 1)
						{
							Logger.LogInformation("\t\t{count} x {size} byte(s), starting at local offset {offset}", Diff.Count, Diff.Size, Diff.LocalOffset);
						}
						else
						{
							Logger.LogInformation("\t\t{size} byte(s), starting at local offset {offset}", Diff.Size, Diff.LocalOffset);
						}
						if (!string.IsNullOrEmpty(Diff.ObjectName))
						{
							Logger.LogInformation("\t\t\tObject name: {name}", Diff.ObjectName);
						}
						if (!string.IsNullOrEmpty(Diff.PropertyName))
						{
							Logger.LogInformation("\t\t\tProperty name: {name}", Diff.PropertyName);
						}
						if (!string.IsNullOrEmpty(Diff.Callstack))
						{
							Logger.LogInformation("\t\t\tCallstack:");
							PrintLinesIndented("\t\t\t\t", Diff.Callstack);
						}

						UsedDiffs.Add(Diff);
					}
				}
				if (Section.UnreportedDiffs > 0)
				{
					Logger.LogInformation("\t\t+{count} more diffs", Section.UnreportedDiffs);
				}
				if (NumPreviousReported > 0)
				{
					Logger.LogInformation("\t\t+{count} previously reported diffs", NumPreviousReported);
				}

				if (Section.TableDiffs.Count > 0)
				{
					foreach (TableDiff TableDiff in Section.TableDiffs)
					{
						Logger.LogInformation("\tTable Diffs for {item}", TableDiff.ItemName);
						PrintLinesIndented("\t\t", TableDiff.Detail);
					}
				}
			}

			private static void PrintLinesIndented( string Indent, string Lines)
			{
				foreach (string Line in Lines.EnumerateLines())
				{
					Logger.LogInformation("{indent}{line}", Indent, Line );
				}
			}


			private static int CalculateImportance(Package Package, HashSet<Diff> UsedDiffs)
			{
				// @todo: proper heuristics!!
				int HeaderImportance = CalculateImportance(Package.Header, UsedDiffs);
				int ExportsImportance = CalculateImportance(Package.Exports, UsedDiffs);
				return HeaderImportance += (ExportsImportance * 100);
			}

			private static int CalculateImportance(Section Section, HashSet<Diff> UsedDiffs)
			{
				// @todo: proper heuristics!!
				int Importance = 0;
				if (Section != null)
				{		
					if (Section.SizeMismatch)
					{
						Importance += 10;
					}
					if (Section.UndiagnosedDiff)
					{
						Importance += 10;
					}
					Importance += Section.UnreportedDiffs * 10;
					Importance += CalculateImportance(Section.Diffs, UsedDiffs);
					Importance += CalculateImportance(Section.TableDiffs);
				}

				return Importance;
			}

			private static int CalculateImportance(IEnumerable<Diff> Diffs, HashSet<Diff> UsedDiffs)
			{
				// @todo: proper heuristics!!
				int Importance = 0;
				foreach (Diff Diff in Diffs)
				{
					Importance += (Diff.Count * Diff.Size) * (UsedDiffs.Contains(Diff) ? 1 : 4); // lower priority if we've prevously reported it
				}

				return Importance;
			}

			private static int CalculateImportance(IEnumerable<TableDiff> TableDiffs)
			{
				return TableDiffs.Count() * 10;
			}
		}
	}
}

