// Copyright Epic Games, Inc. All Rights Reserved.
using System;
using System.Collections.Generic;
using System.Text;
using System.IO;
using System.Linq;
using AutomationTool;
using UnrealBuildTool;
using EpicGames.Core;
using Microsoft.Extensions.Logging;

class CopySharedCookedBuild : BuildCommand
{
	[Help("Copies the current shared cooked build from the network to the local PC")]
	[RequireP4]
	[DoesNotNeedP4CL]

	FileReference ProjectFile
	{
		get
		{
			FileReference ProjectFullPath = null;
			var OriginalProjectName = ParseParamValue("project", "");

			if (string.IsNullOrEmpty(OriginalProjectName))
			{
				throw new AutomationException("No project file specified. Use -project=<project>.");
			}

			var ProjectName = OriginalProjectName;
			ProjectName = ProjectName.Trim(new char[] { '\"' });
			if (ProjectName.IndexOfAny(new char[] { '\\', '/' }) < 0)
			{
				ProjectName = CombinePaths(CmdEnv.LocalRoot, ProjectName, ProjectName + ".uproject");
			}
			else if (!FileExists_NoExceptions(ProjectName))
			{
				ProjectName = CombinePaths(CmdEnv.LocalRoot, ProjectName);
			}
			if (FileExists_NoExceptions(ProjectName))
			{
				ProjectFullPath = new FileReference(ProjectName);
			}
			else
			{
				var Branch = new BranchInfo();
				var GameProj = Branch.FindGame(OriginalProjectName);
				if (GameProj != null)
				{
					ProjectFullPath = GameProj.FilePath;
				}
				if (ProjectFullPath == null || !FileExists_NoExceptions(ProjectFullPath.FullName))
				{
					throw new AutomationException("Could not find a project file {0}.", ProjectName);
				}
			}
			return ProjectFullPath;
		}
	}

	public override void ExecuteBuild()
	{
		Logger.LogInformation("************************* CopySharedCookedBuild");

		// Parse the project filename (as a local path)
		

		string CmdLinePlatform = ParseParamValue("Platform", null);

		bool bOnlyCopyAssetRegistry = ParseParam("onlycopyassetregistry");

		string SharedBuildCL = ParseParamValue("buildcl", "any");

		SharedCookedBuild.SharedCookType BuildType = (SharedCookedBuild.SharedCookType)Enum.Parse(typeof(SharedCookedBuild.SharedCookType), SharedBuildCL, true);

		List<UnrealTargetPlatform> TargetPlatforms = new List<UnrealTargetPlatform>();
		var PlatformNames = new List<string>(CmdLinePlatform.Split('+'));
		foreach (var PlatformName in PlatformNames)
		{
			// Look for dependent platforms, Source_1.Dependent_1+Source_2.Dependent_2+Standalone_3
			var SubPlatformNames = new List<string>(PlatformName.Split('.'));

			foreach (var SubPlatformName in SubPlatformNames)
			{
				TargetPlatforms.Add(UnrealTargetPlatform.Parse(SubPlatformName));
			}
		}

		new SharedCookedBuild(ProjectFile, TargetPlatforms.Select(x => x.ToString()), BuildType).CopySharedCookedBuilds();


		
		/*
				// Build the list of paths that need syncing
				List<string> SyncPaths = new List<string>();
				if(bIsProjectFile)
				{
					SyncPaths.Add(CommandUtils.CombinePaths(PathSeparator.Slash, BranchRoot, "*"));
					SyncPaths.Add(CommandUtils.CombinePaths(PathSeparator.Slash, BranchRoot, "Engine", "..."));
					SyncPaths.Add(CommandUtils.CombinePaths(PathSeparator.Slash, CommandUtils.GetDirectoryName(ProjectFileRecord.DepotFile), "..."));
				}
				else
				{
					SyncPaths.Add(CommandUtils.CombinePaths(PathSeparator.Slash, CommandUtils.GetDirectoryName(ProjectFileRecord.DepotFile), "..."));
				}

				// Sync them down
				foreach(string SyncPath in SyncPaths)
				{
					Log("Syncing {0}@{1}", SyncPath, CL);
					P4.Sync(String.Format("{0}@{1}", SyncPath, CL));
				}

				// Get the name of the editor target
				string EditorTargetName = "UnrealEditor";
				if(bIsProjectFile)
				{
					string SourceDirectoryName = Path.Combine(Path.GetDirectoryName(ProjectFileName), "Source");
					if(Directory.Exists(SourceDirectoryName))
					{
						foreach(string EditorTargetFileName in Directory.EnumerateFiles(SourceDirectoryName, "*Editor.Target.cs"))
						{
							EditorTargetName = Path.GetFileNameWithoutExtension(Path.GetFileNameWithoutExtension(EditorTargetFileName));
							break;
						}
					}
				}

				// Build everything
				UnrealTargetPlatform CurrentPlatform = HostPlatform.Current.HostEditorPlatform;

				UnrealBuild.BuildAgenda Agenda = new UnrealBuild.BuildAgenda();
				Agenda.AddTarget(EditorTargetName, CurrentPlatform, UnrealTargetConfiguration.Development, ProjectFileName.EndsWith(".uproject", StringComparison.InvariantCultureIgnoreCase)? new FileReference(ProjectFileName) : null);
				Agenda.AddTarget("ShaderCompileWorker", CurrentPlatform, UnrealTargetConfiguration.Development);
				Agenda.AddTarget("UnrealLightmass", CurrentPlatform, UnrealTargetConfiguration.Development);
				Agenda.AddTarget("InterchangeWorker", CurrentPlatform, UnrealTargetConfiguration.Development);
				Agenda.AddTarget("CrashReportClient", CurrentPlatform, UnrealTargetConfiguration.Shipping);

				UnrealBuild Build = new UnrealBuild(this);
				Build.UpdateVersionFiles(ActuallyUpdateVersionFiles: true, ChangelistNumberOverride: CL);
				Build.Build(Agenda, InUpdateVersionFiles: false);*/
	}
}
