// Copyright Epic Games, Inc. All Rights Reserved.

using AutomationTool;
using EpicGames.Core;
using System;
using System.Collections.Generic;
using System.IO;
using System.IO.Compression;
using System.Linq;
using System.Text;
using System.Text.RegularExpressions;
using UnrealBuildBase;
using Microsoft.Extensions.Logging;

namespace Rocket.Automation
{
	public class CreateComponentZips : BuildCommand
	{
		class Category
		{
			public string Name;
			public FileFilter Filter = new FileFilter();
		}

		public override void ExecuteBuild()
		{
			DirectoryReference SourceDir = ParseRequiredDirectoryReferenceParam("SourceDir");
			DirectoryReference OutputDir = ParseRequiredDirectoryReferenceParam("OutputDir");
			DirectoryReference.CreateDirectory(OutputDir);

			FileReference Location = FileReference.Combine(Unreal.EngineDirectory, "Restricted", "NotForLicensees", "Build", "LauncherAttributes.ini");

			string[] Lines = FileReference.ReadAllLines(Location);

			Category CurrentCategory = null;
			List<Category> Categories = new List<Category>();
			foreach (string Line in Lines)
			{
				string TrimLine = Line.Trim();
				if (TrimLine.Length > 0)
				{
					if (TrimLine[0] == '[')
					{
						Match Match = Regex.Match(TrimLine, @"^\[LauncherAttribute\.tag:([A-Za-z0-9_]+)\]$");
						if (Match.Success)
						{
							CurrentCategory = new Category { Name = Match.Groups[1].Value };
							Categories.Add(CurrentCategory);
						}
						else
						{
							CurrentCategory = null;
						}
					}
					else
					{
						if (CurrentCategory != null)
						{
							CurrentCategory.Filter.AddRule(TrimLine);
						}
					}
				}
			}

			List<FileInfo> SourceFiles = SourceDir.ToDirectoryInfo().EnumerateFiles("*", SearchOption.AllDirectories).ToList();
			Logger.LogInformation("Found {Arg0} source files", SourceFiles.Count);

			FileReference MainOutputFile = FileReference.Combine(OutputDir, $"UnrealEngine.zip");
			CreateZip(SourceDir, SourceFiles, x => !Categories.Any(y => y.Filter.Matches(x)), MainOutputFile);

			foreach (Category Category in Categories)
			{
				FileReference OutputFile = FileReference.Combine(OutputDir, $"UnrealEngine_{Category.Name}.zip");
				CreateZip(SourceDir, SourceFiles, x => Category.Filter.Matches(x), OutputFile);
			}
		}

		static void CreateZip(DirectoryReference SourceDir, List<FileInfo> SourceFiles, Predicate<string> Predicate, FileReference OutputFile)
		{
			long TotalSize = 0;
			List<FileReference> SourceFileRefs = new List<FileReference>();
			foreach (FileInfo SourceFile in SourceFiles)
			{
				FileReference SourceFileRef = new FileReference(SourceFile);
				if (Predicate(SourceFileRef.MakeRelativeTo(SourceDir)))
				{
					SourceFileRefs.Add(SourceFileRef);
					TotalSize += SourceFile.Length;
				}
			}

			Logger.LogInformation("Creating {OutputFile} ({1:n0} files, {2:n1}mb)", OutputFile, SourceFileRefs.Count, TotalSize / (1024.0 * 1024.0));
			CommandUtils.ZipFiles(OutputFile, SourceDir, SourceFileRefs);
		}
	}
}
