// Copyright Epic Games, Inc. All Rights Reserved.
using System;
using System.Collections.Generic;
using System.Text;
using System.IO;
using AutomationTool;
using UnrealBuildTool;
using EpicGames.Core;
using UnrealBuildBase;
using System.Linq;
using Microsoft.Extensions.Logging;

[Help("Create stub code for platform extension")]
[Help("Source", "Path to source .uplugin, .build.cs or .target.cs, or a source folder to search")]
[Help("Platform", "Platform(s) or Platform Groups to generate for")]
[Help("Project", "Optional path to project (only required if not creating code for Engine modules/plugins")]
[Help("SkipPluginModules", "Do not generate platform extension module files when generating a platform extension plugin")]
[Help("AllowOverwrite", "If target files already exist they'll be overwritten rather than skipped")]
[Help("AllowUnknownPlatforms", "Allow platform & platform groups that are not known, for example when generating code for extensions we do not have access to")]
[Help("AllowPlatformExtensionsAsParents", "When creating a platform extension from another platform extension, use the source platform as the parent")]
[Help("P4", "Create a changelist for the new files")]
[Help("CL", "Override the changelist #")]
public class CreatePlatformExtension : BuildCommand
{
	readonly List<ModuleHostType> ModuleTypeDenyList = new List<ModuleHostType>
	{
		ModuleHostType.Developer,
		ModuleHostType.Editor, 
		ModuleHostType.EditorNoCommandlet,
		ModuleHostType.EditorAndProgram,
		ModuleHostType.Program,
	};

	ConfigHierarchy GameIni;
	DirectoryReference ProjectDir;
	DirectoryReference SourceDir;
	public List<string> NewFiles = new List<string>();
	public List<string> ModifiedFiles = new List<string>();
	public List<string> WritableFiles = new List<string>(); // for testing only
	
	string[] Platforms;
	public bool bSkipPluginModules = false;
	public bool bOverwriteExistingFile = false;
	public bool bIsTest = false;
	public bool bAllowPlatformExtensionsAsParents = false;
	public int CL = -1;

	public override void ExecuteBuild()
	{
		// Parse the parameters
		string[] SrcPlatforms = ParseParamValue("Platform", "").Split('+', StringSplitOptions.RemoveEmptyEntries);
		string Source = ParseParamValue("Source", "");
		string Project = ParseParamValue("Project", "");
		bSkipPluginModules = ParseParam("SkipPluginModules");
		bOverwriteExistingFile = ParseParam("AllowOverwrite");
		CL = ParseParamInt("CL", -1 );

		// make sure we have somewhere to look
		if (string.IsNullOrEmpty(Source))
		{
			Logger.LogError("No -Source= directory/file specified");
			return;
		}

		// Sanity check platforms list
		SrcPlatforms = VerifyPlatforms(SrcPlatforms);
		if (SrcPlatforms.Length == 0)
		{
			Logger.LogError("Please specify at least one platform or platform group");
			return;
		}

		// cannot have both -P4 and -DebugTest
		bIsTest = ParseParam("DebugTest") && System.Diagnostics.Debugger.IsAttached; //NB. -DebugTest is for debugging this program only
		if (CommandUtils.P4Enabled && bIsTest)
		{
			Logger.LogError("Cannot specify both -P4 and -DebugTest");
			return;
		}

		// cannot have -CL without -P4
		if (!CommandUtils.P4Enabled && CL >= 0)
		{
			Logger.LogError("-CL requires -P4");
			return;
		}

		// sanity check changelist
		if (CommandUtils.P4Enabled && CL >= 0)
		{
			bool bPending;
			if (!P4.ChangeExists(CL, out bPending) || !bPending)
			{
				Logger.LogError("Changelist {CL} cannot be used or is not valid", CL);
				return;
			}
		}

		// Generate the code
		try
		{
			GeneratePlatformExtension(Project, Source, SrcPlatforms);

			// check the generated files
			if (NewFiles.Count > 0 || ModifiedFiles.Count > 0)
			{
				// display final report
				Logger.LogInformation("{Text}", System.Environment.NewLine);
				Logger.LogInformation("{Text}", System.Environment.NewLine);
				Logger.LogInformation("{Text}", System.Environment.NewLine);
				Logger.LogInformation("{Text}", "The following files should have been added or edited" + ((CL > 0) ? $" in changelist {CL}:" : ":") );
				foreach (string NewFile in NewFiles)
				{
					Logger.LogInformation("\t{NewFile} (added)", NewFile);
				}
				foreach (string ModifiedFile in ModifiedFiles)
				{
					Logger.LogInformation("\t{ModifiedFile} (edit)", ModifiedFile);
				}
				Logger.LogInformation("{Text}", System.Environment.NewLine);
				Logger.LogInformation("{Text}", System.Environment.NewLine);
				Logger.LogInformation("{Text}", System.Environment.NewLine);
				Logger.LogWarning("*** It is strongly recommended that each file is manually verified! ***");
				Logger.LogInformation("{Text}", System.Environment.NewLine);
				Logger.LogInformation("{Text}", System.Environment.NewLine);
				Logger.LogInformation("{Text}", System.Environment.NewLine);

				// remove everything if requested (for debugging etc)
				if (bIsTest)
				{
					Logger.LogInformation("Deleting all the files because this is just a test...");
					foreach (string NewFile in NewFiles)
					{
						File.Delete(NewFile);
					}
					foreach (string WritableFile in WritableFiles)
					{
						File.Delete(WritableFile);
						File.Move(WritableFile + ".tmp.bak", WritableFile);
						File.SetAttributes( WritableFile, File.GetAttributes(WritableFile) | FileAttributes.ReadOnly );
					}
				}
			}
		}
		catch(Exception)
		{
			// something went wrong - clean up anything we've created so far
			foreach (string NewFile in NewFiles)
			{
				Logger.LogInformation("Removing partial file ${NewFile} due to error", NewFile);
				File.Delete(NewFile);
			}
			foreach (string WritableFile in WritableFiles)
			{
				Logger.LogInformation("Restoring read-only file ${WritableFile} due to error", WritableFile);
				File.Delete(WritableFile);
				File.Move(WritableFile + ".tmp.bak", WritableFile);
				File.SetAttributes( WritableFile, File.GetAttributes(WritableFile) | FileAttributes.ReadOnly );
			}


			// try to safely clean up the perforce changelist too, if it was not specified on the command line
			try
			{
				if (CL > 0 && CommandUtils.P4Enabled && ParseParamInt("CL", -1) != CL)
				{
					Logger.LogInformation("Removing partial changelist ${CL} due to error", CL);
					P4.DeleteChange(CL,true);
				}
			}
			catch(Exception e)
			{
				Logger.LogError("{Text}", e.Message);
			}


			throw;
		}
	}


	/// <summary>
	/// Generates platform extensions from the given source
	/// </summary>
	public void GeneratePlatformExtension( string InProject, string InSource, string[] InPlatforms )
	{
		FileReference ProjectFile = string.IsNullOrEmpty(InProject) ? null : ProjectUtils.FindProjectFileFromName(InProject);

		// Prepare values
		ProjectDir = ProjectFile?.Directory;
		GameIni = ConfigCache.ReadHierarchy(ConfigHierarchyType.Game, ProjectDir, BuildHostPlatform.Current.Platform);
		Platforms = InPlatforms;

		if (Directory.Exists(InSource))
		{
			SourceDir = new DirectoryReference(InSource);

			// check the directory for plugins first, because the plugins will automatically generate the modules too
			List<string> Plugins = Directory.EnumerateFiles(InSource, "*.uplugin", SearchOption.AllDirectories).ToList();
			if (Plugins.Count > 0)
			{
				foreach (string Plugin in Plugins)
				{
					GeneratePluginPlatformExtension(new FileReference(Plugin));
				}
			}
			else
			{
				// there were no plugins found, so search for module & target rules instead
				List<string> ModuleRules = Directory.EnumerateFiles(InSource, "*.build.cs", SearchOption.AllDirectories).ToList();
				ModuleRules.AddRange(Directory.EnumerateFiles(InSource, "*.target.cs", SearchOption.AllDirectories));
				if (ModuleRules.Count > 0)
				{
					foreach (string ModuleRule in ModuleRules)
					{
						GenerateModulePlatformExtension(new FileReference(ModuleRule), Platforms);
					}
				}
				else
				{
					Logger.LogError("Cannot find any supported files in {InSource}", InSource);
				}
			}
		}
		else if (File.Exists(InSource))
		{
			FileReference Source = new FileReference(InSource);
			SourceDir = Source.Directory;
			GeneratePlatformExtensionFromFile(Source);
		}
		else
		{
			Logger.LogError("Invalid path or file name {InSource}", InSource);
		}
	}



	/// <summary>
	/// Create the platform extension plugin files of the given plugin, for the given platforms
	/// </summary>
	private void GeneratePluginPlatformExtension(FileReference PluginPath)
	{
		// sanity check plugin path
		if (!File.Exists(PluginPath.FullName))
		{
			Logger.LogError("File not found: {PluginPath}", PluginPath);
			return;
		}

		DirectoryReference PluginDir = PluginPath.Directory;
		if (ProjectDir == null && !PluginDir.IsUnderDirectory(Unreal.EngineDirectory))
		{
			Logger.LogError("{PluginPath} is not under the Engine directory, and no -project= has been specified", PluginPath);
			return;
		}

		DirectoryReference RootDir = ProjectDir ?? Unreal.EngineDirectory;
		if (!PluginDir.IsUnderDirectory(RootDir))
		{
			Logger.LogError("{PluginPath} is not under {RootDir}", PluginPath, RootDir);
			return;
		}

		// load the plugin & find suitable modules, if required
		PluginDescriptor ParentPlugin = PluginDescriptor.FromFile(PluginPath); //NOTE: if the PluginPath is itself a child plugin, not all allow list, deny list & supported platform information will be available.
		List<PluginReferenceDescriptor> ParentPluginDescs = new List<PluginReferenceDescriptor>();
		Dictionary<ModuleDescriptor, FileReference> ParentModuleRules = new Dictionary<ModuleDescriptor, FileReference>();
		if (!bSkipPluginModules && ParentPlugin.Modules != null)
		{
			// find all module rules that are listed in the plugin
			DirectoryReference ModuleRulesPath = DirectoryReference.Combine( PluginDir, "Source" );
			if (DirectoryReference.Exists(ModuleRulesPath))
			{
				var ModuleRules = DirectoryReference.EnumerateFiles(ModuleRulesPath, "*.build.cs", SearchOption.AllDirectories);
				foreach (FileReference ModuleRule in ModuleRules)
				{
					string ModuleRuleName = GetPlatformExtensionBaseNameFromPath(ModuleRule.FullName);
					ModuleDescriptor ModuleDesc = ParentPlugin.Modules.Find(ParentModuleDesc => ParentModuleDesc.Name.Equals(ModuleRuleName, StringComparison.InvariantCultureIgnoreCase));
					if (ModuleDesc != null)
					{
						ParentModuleRules.Add(ModuleDesc, ModuleRule);
					}
				}
			}
		}

		bool bParentPluginDirty = false;

		// generate the platform extension files
		string BasePluginName = GetPlatformExtensionBaseNameFromPath(PluginPath.FullName);
		foreach (string PlatformName in Platforms)
		{
			// verify final file name
			string FinalFileName = MakePlatformExtensionPathFromSource(RootDir, PluginDir, PlatformName, BasePluginName + "_" + PlatformName + ".uplugin");
			if (File.Exists(FinalFileName) && !(bOverwriteExistingFile && EditFile(FinalFileName)))
			{
				Logger.LogWarning("Skipping {FinalFileName} as it already exists", FinalFileName);
				continue;
			}

			// create the child plugin
			Directory.CreateDirectory(Path.GetDirectoryName(FinalFileName));
			using (JsonWriter ChildPlugin = new JsonWriter(FinalFileName))
			{
				UnrealTargetPlatform Platform;
				bool bHasPlatform = UnrealTargetPlatform.TryParse(PlatformName, out Platform);

				// a platform reference is needed if there are already platforms listed in the parent, or the parent requires an explicit platform list
				bool NeedsPlatformReference<T>( List<T> ParentPlatforms, bool bHasExplicitPlatforms )
				{
					return (bHasPlatform && ((ParentPlatforms != null && ParentPlatforms.Count > 0) || bHasExplicitPlatforms));
				}

				// create the plugin definition
				ChildPlugin.WriteObjectStart();
				ChildPlugin.WriteValue("FileVersion", (int)PluginDescriptorVersion.ProjectPluginUnification ); // this is the version that this code has been tested against
				ChildPlugin.WriteValue("bIsPluginExtension", true );
				if (NeedsPlatformReference(ParentPlugin.SupportedTargetPlatforms, ParentPlugin.bHasExplicitPlatforms))
				{
					ChildPlugin.WriteStringArrayField("SupportedTargetPlatforms", new string[]{ Platform.ToString() } );
				}

				// select all modules that need child module references for this platform
				IEnumerable<ModuleDescriptor> ModuleDescs = ParentPlugin.Modules?.Where( ModuleDesc => ShouldCreateChildReferenceForModule(ModuleDesc, bHasPlatform, Platform));
				if (ModuleDescs != null && ModuleDescs.Any() )
				{
					ChildPlugin.WriteArrayStart("Modules");
					foreach (ModuleDescriptor ParentModuleDesc in ModuleDescs)
					{
						// create the child module reference
						ChildPlugin.WriteObjectStart();
						ChildPlugin.WriteValue("Name", ParentModuleDesc.Name);
						ChildPlugin.WriteValue("Type", ParentModuleDesc.Type.ToString());
						if (NeedsPlatformReference(ParentModuleDesc.PlatformAllowList, ParentModuleDesc.bHasExplicitPlatforms))
						{
							ChildPlugin.WriteStringArrayField("PlatformAllowList", new string[] { Platform.ToString() } );
						}
						else if (NeedsPlatformReference(ParentModuleDesc.PlatformDenyList, ParentModuleDesc.bHasExplicitPlatforms))
						{
							ChildPlugin.WriteStringArrayField("PlatformDenyList", new string[] { Platform.ToString() } );
						}
						if (ParentModuleDesc.PlatformArchitectureAllowList != null && ParentModuleDesc.PlatformArchitectureAllowList.ContainsKey(Platform))
						{
							ChildPlugin.WriteStringArrayField("PlatformArchitectureAllowList", ParentModuleDesc.PlatformArchitectureAllowList.SelectMany( Pair => Pair.Value.Select( Arch => $"{Pair.Key}:{Arch}" ) ));
						}
						if (ParentModuleDesc.PlatformArchitectureDenyList != null && ParentModuleDesc.PlatformArchitectureDenyList.ContainsKey(Platform))
						{
							ChildPlugin.WriteStringArrayField("PlatformArchitectureDenyList", ParentModuleDesc.PlatformArchitectureDenyList.SelectMany( Pair => Pair.Value.Select( Arch => $"{Pair.Key}:{Arch}" ) ));
						}
						ChildPlugin.WriteObjectEnd();

						// see if there is a module rule file too & generate the rules file for this platform
						FileReference ParentModuleRule;
						if (ParentModuleRules.TryGetValue(ParentModuleDesc, out ParentModuleRule))
						{
							GenerateModulePlatformExtension(ParentModuleRule, new string[] { PlatformName });
						}

						// remove platform from parent plugin references
						if (bHasPlatform && ParentModuleDesc.PlatformAllowList != null && ParentModuleDesc.PlatformAllowList.Contains(Platform) )
						{
							ParentModuleDesc.PlatformAllowList.Remove(Platform);
							ParentModuleDesc.bHasExplicitPlatforms |= (ParentModuleDesc.PlatformAllowList.Count == 0); // an empty list is interpreted as 'all platforms are allowed' otherwise
							bParentPluginDirty = true;
						}
						if (bHasPlatform && ParentModuleDesc.PlatformArchitectureAllowList != null)
						{
							bParentPluginDirty |= ParentModuleDesc.PlatformArchitectureAllowList.Remove(Platform);
						}
						if (bHasPlatform && ParentModuleDesc.PlatformArchitectureDenyList != null)
						{
							bParentPluginDirty |= ParentModuleDesc.PlatformArchitectureDenyList.Remove(Platform);
						}
						if (bHasPlatform && ParentModuleDesc.PlatformDenyList != null)
						{
							bParentPluginDirty |= ParentModuleDesc.PlatformDenyList.Remove(Platform);
						}
					}
					ChildPlugin.WriteArrayEnd();
				}

				// select all plugins that need child plugin references for this platform
				IEnumerable<PluginReferenceDescriptor> PluginDescs = ParentPlugin.Plugins?.Where( PluginDesc => ShouldCreateChildReferenceForDependentPlugin(PluginDesc, bHasPlatform, Platform ) );
				if (PluginDescs != null && PluginDescs.Any() )
				{
					ChildPlugin.WriteArrayStart("Plugins");
					foreach (PluginReferenceDescriptor ParentPluginDesc in PluginDescs)
					{
						// create the child plugin reference
						ChildPlugin.WriteObjectStart();
						ChildPlugin.WriteValue("Name", ParentPluginDesc.Name );
						ChildPlugin.WriteValue("Enabled", ParentPluginDesc.bEnabled);
						if (NeedsPlatformReference(ParentPluginDesc.PlatformAllowList?.ToList(), ParentPluginDesc.bHasExplicitPlatforms))
						{
							ChildPlugin.WriteStringArrayField("PlatformAllowList", new string[] { Platform.ToString() } );
						}
						if (NeedsPlatformReference(ParentPluginDesc.PlatformDenyList?.ToList(), ParentPluginDesc.bHasExplicitPlatforms))
						{
							ChildPlugin.WriteStringArrayField("PlatformDenyList", new string[] { Platform.ToString() } );
						}
						ChildPlugin.WriteObjectEnd();

						// remove platform from parent plugin references
						if (bHasPlatform && PlatformArrayContainsPlatform( ParentPluginDesc.PlatformAllowList, Platform ) )
						{
							ParentPluginDesc.PlatformAllowList = ParentPluginDesc.PlatformAllowList.Where( X => !X.Equals(Platform.ToString() ) ).ToArray();
							ParentPluginDesc.bHasExplicitPlatforms |= (ParentPluginDesc.PlatformAllowList.Length == 0); // an empty list is interpreted as "all platforms" otherwise
							bParentPluginDirty = true;
						}
						if (bHasPlatform && PlatformArrayContainsPlatform( ParentPluginDesc.PlatformDenyList, Platform ) )
						{
							ParentPluginDesc.PlatformDenyList = ParentPluginDesc.PlatformDenyList.Where( X => !X.Equals(Platform.ToString() ) ).ToArray();
							bParentPluginDirty = true;
						}
					}

					ChildPlugin.WriteArrayEnd();
				}
				ChildPlugin.WriteObjectEnd();

				// remove platform from parent plugin, if necessary
				if (bHasPlatform && ParentPlugin.SupportedTargetPlatforms != null && ParentPlugin.SupportedTargetPlatforms.Contains(Platform) )
				{
					ParentPlugin.SupportedTargetPlatforms.Remove(Platform);
					ParentPlugin.bHasExplicitPlatforms |= (ParentPlugin.SupportedTargetPlatforms.Count == 0); // an empty list is interpreted as "all platforms" otherwise
					bParentPluginDirty = true;
				}
			}
			AddNewFile(FinalFileName);
		}

		// save parent plugin, if necessary
		if (bParentPluginDirty && EditFile(PluginPath.FullName) )
		{
			ParentPlugin.Save(PluginPath.FullName);
		}
	}


	/// <summary>
	/// Creates the platform extension child class files of the given module, for the given platforms
	/// </summary>
	private void GenerateModulePlatformExtension(FileReference ModulePath, string[] PlatformNames)
	{
		// sanity check module path
		if (!File.Exists(ModulePath.FullName))
		{
			Logger.LogError("File not found: {ModulePath}", ModulePath);
			return;
		}

		DirectoryReference ModuleDir = ModulePath.Directory;
		if (ProjectDir == null && !ModuleDir.IsUnderDirectory(Unreal.EngineDirectory))
		{
			Logger.LogError("{ModulePath} is not under the Engine directory, and no -project= has been specified", ModulePath);
			return;
		}

		DirectoryReference RootDir = ProjectDir ?? Unreal.EngineDirectory;
		if (!ModuleDir.IsUnderDirectory(RootDir))
		{
			Logger.LogError("{ModulePath} is not under {RootDir}", ModulePath, RootDir);
			return;
		}

		// sanity check module file name
		string ModuleFilename = ModulePath.GetFileName();
		string ModuleExtension = ModuleFilename.Substring( ModuleFilename.IndexOf('.') );
		ModuleFilename = ModuleFilename.Substring( 0, ModuleFilename.Length - ModuleExtension.Length );
		if (!ModuleExtension.Equals(".build.cs", System.StringComparison.InvariantCultureIgnoreCase ) && !ModuleExtension.Equals(".target.cs", System.StringComparison.InvariantCultureIgnoreCase))
		{
			Logger.LogError("{ModulePath} is a module/rules file. Expecting .build.cs or .target.cs", ModulePath);
			return;
		}

		// load module file & find module class name, and optional class namespace
		char[] ClassNameSeparators = new char[] { ' ', '\t', ':' };
		const string ClassDeclaration = "public class ";
		const string NamespaceDeclaration = "namespace ";
		string[] ModuleContents = File.ReadAllLines(ModulePath.FullName);
		string ModuleClassDeclaration = ModuleContents.FirstOrDefault( L => L.Trim().StartsWith(ClassDeclaration) );
		string ModuleNamespaceDeclaration = ModuleContents.FirstOrDefault( L => L.Trim().StartsWith(NamespaceDeclaration) );
		if (string.IsNullOrEmpty(ModuleClassDeclaration))
		{
			Logger.LogError("Cannot find class declaration in ${ModulePath}", ModulePath);
			return;
		}
		string ParentModuleName = ModuleClassDeclaration.Trim().Remove(0, ClassDeclaration.Length).Split(ClassNameSeparators, StringSplitOptions.None).Last();
		if (bAllowPlatformExtensionsAsParents || ParentModuleName.Equals("ModuleRules") || ParentModuleName.Equals("TargetRules") )
		{
			ParentModuleName = ModuleClassDeclaration.Trim().Remove(0, ClassDeclaration.Length ).Split(ClassNameSeparators, StringSplitOptions.None ).First();
		}
		if (string.IsNullOrEmpty(ParentModuleName))
		{
			Logger.LogError("Cannot parse class declaration in ${ModulePath}", ModulePath);
			return;
		}
		string ParentNamespace = string.IsNullOrEmpty(ModuleNamespaceDeclaration) ? "" : (ModuleNamespaceDeclaration.Trim().Remove(0, NamespaceDeclaration.Length ).Split(' ', StringSplitOptions.None ).First() + ".");
		string BaseModuleName = ParentModuleName;
		int Index = BaseModuleName.IndexOf('_'); //trim off _[platform] suffix
		if (Index != -1)
		{
			BaseModuleName = BaseModuleName.Substring(0, Index);
		}
		BaseModuleName = BaseModuleName.TrimEnd( new char[] {':'} ).Trim(); // trim off any : suffix

		// ignore if it is the default namespace for build rules
		if (ParentNamespace == "UnrealBuildTool.Rules." && ModuleExtension.Equals(".build.cs", System.StringComparison.InvariantCultureIgnoreCase ))
		{
			ParentNamespace = "";
		}

		// load template and generate the platform extension files
		string BaseModuleFileName = GetPlatformExtensionBaseNameFromPath( ModulePath.FullName );
		string CopyrightLine = MakeCopyrightLine();
		string Template = LoadTemplate($"PlatformExtension{ModuleExtension}.template");
		foreach (string PlatformName in PlatformNames)
		{
			// verify the final file name
			string FinalFileName = MakePlatformExtensionPathFromSource(RootDir, ModuleDir, PlatformName, BaseModuleFileName + "_" + PlatformName + ModuleExtension );
			if (File.Exists(FinalFileName) && !(bOverwriteExistingFile && EditFile(FinalFileName) ))
			{
				Logger.LogWarning("Skipping {FinalFileName} as it already exists", FinalFileName);
				continue;
			}

			// generate final code from the template
			string FinalOutput = Template;
			FinalOutput = FinalOutput.Replace("%COPYRIGHT_LINE%",     CopyrightLine,                     StringComparison.InvariantCultureIgnoreCase );
			FinalOutput = FinalOutput.Replace("%PARENT_MODULE_NAME%", ParentNamespace+ParentModuleName,  StringComparison.InvariantCultureIgnoreCase );
			FinalOutput = FinalOutput.Replace("%BASE_MODULE_NAME%",   BaseModuleName,                    StringComparison.InvariantCultureIgnoreCase );
			FinalOutput = FinalOutput.Replace("%PLATFORM_NAME%",      PlatformName,                      StringComparison.InvariantCultureIgnoreCase );

			// save the child .cs file
			Directory.CreateDirectory(Path.GetDirectoryName(FinalFileName));
			File.WriteAllText(FinalFileName, FinalOutput);
			AddNewFile(FinalFileName);
		}
	}



	/// <summary>
	/// Generates platform extension files based on the given source file name
	/// </summary>
	private void GeneratePlatformExtensionFromFile(FileReference Source)
	{
		if (Source.FullName.ToLower().EndsWith(".uplugin") )
		{
			GeneratePluginPlatformExtension(Source);
		}
		else if (Source.FullName.ToLower().EndsWith(".build.cs") || Source.FullName.ToLower().EndsWith(".target.cs"))
		{
			GenerateModulePlatformExtension(Source, Platforms);
		}
		else
		{
			Logger.LogError("unsupported file type {Source}", Source);
		}
	}



	#region boilerplate & helpers


	/// <summary>
	/// Determines whether we should attempt to add a child module reference for the given plugin module
	/// </summary>
	/// <param name="ModuleDesc"></param>
	/// <param name="bHasPlatform"></param>
	/// <param name="Platform"></param>
	/// <returns></returns>
	private bool ShouldCreateChildReferenceForModule( ModuleDescriptor ModuleDesc, bool bHasPlatform, UnrealTargetPlatform Platform )
	{
		// make sure it's a type that is usually associated with platform extensions
		if (ModuleTypeDenyList.Contains(ModuleDesc.Type))
		{
			return false;
		}

		// this module must have supported platforms explicitly listed so we must create a child reference
		if (ModuleDesc.bHasExplicitPlatforms)
		{
			return true;
		}

		// the module has a non-empty platform allow list so we must create a child reference
		if (ModuleDesc.PlatformAllowList != null && ModuleDesc.PlatformAllowList.Count > 0)
		{
			return true;
		}

		// the module has a non-empty platform deny list that explicitly mentions this platform
		if (bHasPlatform && ModuleDesc.PlatformDenyList != null && ModuleDesc.PlatformDenyList.Contains(Platform))
		{
			return true;
		}

		// the module has an empty platform allow list so no explicit platform reference is needed
		return false;
	}

	/// <summary>
	/// Determines whether we should attempt to add this dependent plugin module to the child plugin references
	/// </summary>
	/// <param name="PluginDesc"></param>
	/// <param name="bHasPlatform"></param>
	/// <param name="Platform"></param>
	/// <returns></returns>
	private bool ShouldCreateChildReferenceForDependentPlugin( PluginReferenceDescriptor PluginDesc, bool bHasPlatform, UnrealTargetPlatform Platform )
	{
		// this plugin reference must have supported platforms explicitly listed so we must create a child reference
		if (PluginDesc.bHasExplicitPlatforms)
		{
			return true;
		}

		// the plugin reference has a non-empty platform allow list so we must create a child reference
		if (PluginDesc.PlatformAllowList != null && PluginDesc.PlatformAllowList.Length > 0)
		{
			return true;
		}

		// the plugin reference has a non-empty platform deny list that explicitly mentions this platform
		if (bHasPlatform && PlatformArrayContainsPlatform( PluginDesc.PlatformDenyList, Platform ) )
		{
			return true;
		}

		// the plugin reference has an empty platform allow list so no explicit platform reference is needed
		return false;
	}



	/// <summary>
	/// Generates the final platform extension file path for the given source directory, platform and filename
	/// </summary>
	private string MakePlatformExtensionPathFromSource( DirectoryReference RootDir, DirectoryReference SourceDir, string PlatformName, string Filename )
	{
		string BaseDir = SourceDir.MakeRelativeTo(RootDir)
			.Replace(Path.AltDirectorySeparatorChar, Path.DirectorySeparatorChar);

		// handle Restricted folders first - need to keep the restricted folder at the start of the path
		string OptionalRestrictedDir = "";
		if (BaseDir.StartsWith("Restricted" + Path.DirectorySeparatorChar))
		{
			string[] RestrictedFragments = BaseDir.Split(Path.DirectorySeparatorChar, 3); // 3 = Restricted/NotForLicensees/<remainder...>
			if (RestrictedFragments.Length > 0)
			{
				OptionalRestrictedDir = Path.Combine( RestrictedFragments.SkipLast(1).ToArray() ); // Restricted/NotForLicensees
				BaseDir = RestrictedFragments.Last(); // remainder
			}
		}

		// handle Platform folders next - trim off the source platform
		if (BaseDir.StartsWith("Platforms" + Path.DirectorySeparatorChar))
		{
			string[] PlatformFragments = BaseDir.Split(Path.DirectorySeparatorChar, 3); // 3 = Platforms/<platform>/<remainder...>
			if (PlatformFragments.Length > 0)
			{
				BaseDir = PlatformFragments.Last(); //remainder
			}
		}

		// build the final path
		return Path.Combine( RootDir.FullName, OptionalRestrictedDir, "Platforms", PlatformName, BaseDir, Filename );
	}


	/// <summary>
	/// Given a full path to a plugin or module file, returns the raw file name - trimming off any _[platform] suffix too
	/// </summary>
	private string GetPlatformExtensionBaseNameFromPath( string FileName )
	{
		// trim off path
		string BaseName = Path.GetFileName(FileName);

		// trim off any extensions
		string Extensions = BaseName.Substring( BaseName.IndexOf('.') );
		BaseName = BaseName.Substring( 0, BaseName.Length - Extensions.Length );

		// trim off any platform suffix
		int Idx = BaseName.IndexOf('_');
		if (Idx != -1)
		{
			BaseName = BaseName.Substring(0,Idx);
		}

		return BaseName;
	}


	
	/// <summary>
	/// Load the given file from the engine templates folder
	/// </summary>
	private string LoadTemplate( string FileName )
	{
		string TemplatePath = Path.Combine( Unreal.EngineDirectory.FullName, "Content", "Editor", "Templates", FileName );
		return File.ReadAllText(TemplatePath);
	}



	/// <summary>
	/// Look up the project/engine specific copyright string
	/// </summary>
	private string MakeCopyrightLine()
	{
		string CopyrightNotice = "";
		GameIni.GetString("/Script/EngineSettings.GeneralProjectSettings", "CopyrightNotice", out CopyrightNotice);

		if (!string.IsNullOrEmpty(CopyrightNotice))
		{
			return "// " + CopyrightNotice;
		}
		else
		{
			return "";
		}
	}


	/// <summary>
	/// Returns whether there is a valid changelist for adding files to
	/// </summary>
	/// <returns></returns>
	private bool HasChangelist()
	{
		if (!CommandUtils.P4Enabled)
		{
			return false;
		}

		if (CL == -1)
		{
			// add the files to perforce if that is available
			string Description = $"[AUTO-GENERATED] {string.Join('+', Platforms)} platform extension files from {SourceDir.MakeRelativeTo(Unreal.RootDirectory)}\n\n#nocheckin verify the code has been generated successfully before checking in!";

			CL = P4.CreateChange(P4Env.Client, Description );
		}

		return (CL != -1);
	}

	/// <summary>
	/// Adds the given file to the list of new files, optionally adding to the current changelist if applicable
	/// </summary>
	/// <param name="NewFile"></param>
	private void AddNewFile( string NewFile )
	{
		if (ModifiedFiles.Contains(NewFile) || NewFiles.Contains(NewFile))
		{
			return;
		}

		if (HasChangelist())
		{
			P4.Add(CL, CommandUtils.MakePathSafeToUseWithCommandLine(NewFile) );
		}

		NewFiles.Add(NewFile);
	}

	/// <summary>
	/// Attempts to edit the given file, optionally checking it out if applicable. If this is just a test, read-only files are backed up and made writable
	/// </summary>
	/// <param name="ExistingFile"></param>
	/// <returns></returns>
	private bool EditFile( string ExistingFile )
	{
		if (ModifiedFiles.Contains(ExistingFile) || NewFiles.Contains(ExistingFile))
		{
			return true;
		}

		if (HasChangelist())
		{
			P4.Edit(CL, CommandUtils.MakePathSafeToUseWithCommandLine(ExistingFile) );
		}


		if ((File.GetAttributes(ExistingFile) & FileAttributes.ReadOnly) == FileAttributes.ReadOnly)
		{
			if (bIsTest)
			{
				File.Copy(ExistingFile, ExistingFile + ".tmp.bak");

				// make the file writable if this is a test
				WritableFiles.Add(ExistingFile);
				File.SetAttributes( ExistingFile, File.GetAttributes(ExistingFile) & ~FileAttributes.ReadOnly );
			}
			else
			{
				Logger.LogWarning("Cannot edit {ExistingFile} because it is read-only", ExistingFile);
				return false;
			}
		}

		ModifiedFiles.Add(ExistingFile);
		return true;
	}


	/// <summary>
	/// Heler function to see if the given platform name array contains the given platform
	/// </summary>
	/// <param name="PlatformNames"></param>
	/// <param name="Platform"></param>
	/// <returns></returns>
	private bool PlatformArrayContainsPlatform( string[] PlatformNames, UnrealTargetPlatform Platform )
	{
		return PlatformNames != null && PlatformNames.Any( PlatformName => PlatformName.Equals(Platform.ToString(), StringComparison.InvariantCultureIgnoreCase) );
	}


	/// <summary>
	/// Returns a list of validated and case-corrected platform and platform groups
	/// </summary>
	private string[] VerifyPlatforms(string[] Platforms)
	{
		bool bAllowUnknownPlatforms = ParseParam("AllowUnknownPlatforms");

		List<string> Result = new List<string>();
		foreach (string PlatformName in Platforms)
		{
			// see if this is a platform
			UnrealTargetPlatform Platform;
			if (UnrealTargetPlatform.TryParse(PlatformName, out Platform))
			{
				Result.Add(Platform.ToString());
				continue;
			}

			// see if this is a platform group
			UnrealPlatformGroup PlatformGroup;
			if (UnrealPlatformGroup.TryParse(PlatformName, out PlatformGroup))
			{
				Result.Add(PlatformGroup.ToString());
				continue;
			}

			// this is an unknown item - see if we will accept it anyway...
			if (bAllowUnknownPlatforms)
			{
				Logger.LogWarning("{PlatformName} is not a known Platform or Platform Group. The code will still be generated but you may not be able to test it locally", PlatformName);
				Result.Add(PlatformName);
			}
			else
			{
				Logger.LogWarning("{PlatformName} is not a known Platform or Platform Group and so it will be ignored. Specify -AllowUnknownPlatforms to allow it anyway", PlatformName);
			}
		}

		return Result.ToArray();
		
	}
	#endregion
}
