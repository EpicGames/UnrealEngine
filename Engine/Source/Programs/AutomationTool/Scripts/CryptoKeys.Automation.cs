// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.IO;
using System.Linq;
using System.Text;
using System.Threading.Tasks;
using EpicGames.Core;
using UnrealBuildTool;
using Microsoft.Extensions.Logging;

namespace AutomationTool
{
	class CryptoKeys : BuildCommand
	{
		public override void ExecuteBuild()
		{
			var Params = new ProjectParams
			(
				Command: this,
				// Shared
				RawProjectPath: ProjectPath
			);

			Logger.LogInformation("********** CRYPTOKEYS COMMAND STARTED **********");

			string UEEditorExe = HostPlatform.Current.GetUnrealExePath(Params.UnrealExe);
			if (!FileExists(UEEditorExe))
			{
				throw new AutomationException("Missing " + UEEditorExe + " executable. Needs to be built first.");
			}

			bool bCycleAllKeys = ParseParam("updateallkeys");
			bool bCycleEncryptionKey = bCycleAllKeys || ParseParam("updateencryptionkey");
			bool bCycleSigningKey = bCycleAllKeys || ParseParam("updatesigningkey");

			if (!bCycleAllKeys && !bCycleEncryptionKey && !bCycleSigningKey)
			{
				throw new Exception("A target for key cycling must be specified when using the cryptokeys automation script\n\t-updateallkeys: Update all keys\n\t-updateencryptionkey: Update encryption key\n\t-updatesigningkey: Update signing key");
			}

			FileReference OutputFile = FileReference.Combine(ProjectPath.Directory, "Config", "DefaultCrypto.ini");
			FileReference NoRedistOutputFile = FileReference.Combine(ProjectPath.Directory, "Restricted", "NoRedist", "Config", "DefaultCrypto.ini");
			FileReference DestinationFile = OutputFile;

			// If the project has a DefaultCrypto.ini in a NoRedist folder, we want to copy the newly generated file into that location
			if (FileReference.Exists(NoRedistOutputFile))
			{
				DestinationFile = NoRedistOutputFile;
			}

			string ChangeDescription = "Automated update of ";
			if (bCycleEncryptionKey)
			{
				ChangeDescription += "encryption";
			}

			if (bCycleSigningKey)
			{
				if (bCycleEncryptionKey)
				{
					ChangeDescription += " and ";
				}
				ChangeDescription += "signing";
			}

			ChangeDescription += " key";

			if (bCycleEncryptionKey && bCycleSigningKey)
			{
				ChangeDescription += "s";
			}

			ChangeDescription += " for project " + Params.ShortProjectName;

			P4Connection SubmitP4 = null;
			int NewCL = 0;
			if (CommandUtils.P4Enabled)
			{
				SubmitP4 = CommandUtils.P4;

				NewCL = SubmitP4.CreateChange(Description: ChangeDescription);
				SubmitP4.Revert(String.Format("-k \"{0}\"", DestinationFile.FullName));
				SubmitP4.Sync(String.Format("-k \"{0}\"", DestinationFile.FullName), AllowSpew: false);
				SubmitP4.Add(NewCL, String.Format("\"{0}\"", DestinationFile.FullName));
				SubmitP4.Edit(NewCL, String.Format("\"{0}\"", DestinationFile.FullName));
			}
			else
			{
				Logger.LogInformation("{Text}", ChangeDescription);
				FileReference.MakeWriteable(OutputFile);
			}

			string CommandletParams = "";
			if (bCycleAllKeys) CommandletParams = "-updateallkeys";
			else if (bCycleEncryptionKey) CommandletParams = "-updateencryptionkey";
			else if (bCycleSigningKey) CommandletParams = "-updatesigningkey";

			RunCommandlet(ProjectPath, UEEditorExe, "CryptoKeys", CommandletParams);

			if (DestinationFile != OutputFile)
			{
				File.Delete(DestinationFile.FullName);
				FileReference.Move(OutputFile, DestinationFile);
			}

			if (SubmitP4 != null)
			{
				int ActualCL;
				SubmitP4.Submit(NewCL, out ActualCL);
			}
		}

		public bool MakeNoRedist { get { return true; } }

		private FileReference ProjectFullPath;
		public virtual FileReference ProjectPath
		{
			get
			{
				if (ProjectFullPath == null)
				{
					ProjectFullPath = ParseProjectParam();
				}
				return ProjectFullPath;
			}
		}
	}
}
