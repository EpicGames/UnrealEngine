// Copyright Epic Games, Inc. All Rights Reserved.

using EpicGames.Core;
using Microsoft.Extensions.Logging;
using System;
using System.Collections.Generic;
using System.IO;
using System.Linq;
using System.Text.Json;
using System.Threading;
using UnrealBuildBase;

namespace AutomationTool
{
	/// <summary>
	/// Commandlet to remove duplicate files in the UAT script directories
	/// </summary>
	[Help("Removes duplicate binaries in UAT script directories that exist in AutomationTool or AutomationUtils")]
	class DedupeAutomationScripts : BuildCommand
	{
		/// <inheritdoc/>
		public override void ExecuteBuild()
		{
			DirectoryReference scriptModuleDir = DirectoryReference.Combine(Unreal.EngineDirectory, "Intermediate", "ScriptModules");

			Log.Logger.LogInformation("Scanning for files shared with AutomationTool or AutomationUtils...");
			ParallelQuery<CsProjBuildRecord> records = DirectoryReference.EnumerateFiles(scriptModuleDir, "*.Automation.json")
				.AsParallel()
				.Select(async x => JsonSerializer.Deserialize<CsProjBuildRecord>(await FileReference.ReadAllTextAsync(x)))
				.Select(x => x.Result);

			DirectoryReference uatDir = DirectoryReference.Combine(Unreal.EngineDirectory, "Binaries", "DotNET", "AutomationTool");
			HashSet<string> uatFiles = [
				.. DirectoryReference.EnumerateFiles(uatDir).Select(x => x.MakeRelativeTo(uatDir)),
				.. DirectoryReference.EnumerateFiles(DirectoryReference.Combine(uatDir, "runtimes"), "*", SearchOption.AllDirectories).Select(x => x.MakeRelativeTo(uatDir)),
			];

			DirectoryReference utilsDir = DirectoryReference.Combine(Unreal.EngineDirectory, "Binaries", "DotNET", "AutomationTool", "AutomationUtils");
			HashSet<string> utilsFiles = [.. DirectoryReference.EnumerateFiles(utilsDir).Select(x => x.MakeRelativeTo(utilsDir))];
			if (DirectoryReference.Exists(DirectoryReference.Combine(utilsDir, "runtimes")))
			{
				utilsFiles.UnionWith(DirectoryReference.EnumerateFiles(DirectoryReference.Combine(utilsDir, "runtimes"), "*", SearchOption.AllDirectories).Select(x => x.MakeRelativeTo(utilsDir)));
			}

			ParallelQuery<FileReference> fileToDelete = records.AsParallel().SelectMany(record =>
			{
				FileReference projectPath = FileReference.Combine(scriptModuleDir, record.ProjectPath);
				FileReference targetPath = FileReference.Combine(projectPath.Directory, record.TargetPath);
				IEnumerable<FileReference> targetFiles = DirectoryReference.EnumerateFiles(targetPath.Directory, "*", SearchOption.AllDirectories);

				if (targetPath.Directory != utilsDir)
				{
					return [
						.. targetFiles.Where(x => uatFiles.Contains(x.MakeRelativeTo(targetPath.Directory))),
						.. targetFiles.Where(x => utilsFiles.Contains(x.MakeRelativeTo(targetPath.Directory)))
					];
				}
				return targetFiles.Where(x => uatFiles.Contains(x.MakeRelativeTo(targetPath.Directory)));
			}).Distinct();

			ParallelQuery<DirectoryReference> possiblyEmptyFolders = records.AsParallel().SelectMany(record =>
			{
				FileReference projectPath = FileReference.Combine(scriptModuleDir, record.ProjectPath);
				FileReference targetPath = FileReference.Combine(projectPath.Directory, record.TargetPath);
				return DirectoryReference.EnumerateDirectories(targetPath.Directory, "*", SearchOption.AllDirectories);
			}).Distinct();

			int deletedFiles = 0;
			int deletedFolders = 0;
			long totalBytes = 0;
			foreach (FileReference file in fileToDelete.Order())
			{
				long bytes = file.ToFileInfo().Length;
				Interlocked.Add(ref totalBytes, bytes);
				Interlocked.Increment(ref deletedFiles);
				FileReference.Delete(file);
				Log.Logger.LogDebug("Deleted {File} ({Size})", file, StringUtils.FormatBytesString(bytes));
			}

			foreach (DirectoryReference folder in possiblyEmptyFolders.Order().Reverse())
			{
				IEnumerable<FileSystemReference> entries = [
					.. DirectoryReference.EnumerateFiles(folder),
					.. DirectoryReference.EnumerateDirectories(folder)
				];
				if (!entries.Any())
				{
					Interlocked.Increment(ref deletedFolders);
					DirectoryReference.Delete(folder);
					Log.Logger.LogDebug("Deleted empty directory {Directory}", folder);
				}
			}

			Log.Logger.LogInformation("Deleted {FileCount} files(s) + {FolderCount} empty folder(s) ({TotalSize})", deletedFiles, deletedFolders,  StringUtils.FormatBytesString(totalBytes));
		}
	}
}
