// Copyright Epic Games, Inc. All Rights Reserved.
using System;
using System.Collections.Generic;
using System.IO;
using System.Threading;
using System.Reflection;
using AutomationTool;
using UnrealBuildTool;
using Microsoft.Extensions.Logging;
using System.Linq;

namespace AutomationScripts
{
	public partial class Project : CommandUtils
	{
		public static void Deploy(ProjectParams Params)
		{
			Params.ValidateAndLog();
			if (!Params.Deploy)
			{
				return;
			}

			Logger.LogInformation("********** DEPLOY COMMAND STARTED **********");
			var StartTime = DateTime.UtcNow;

			if (!Params.NoClient)
			{
				var DeployContextList = CreateDeploymentContext(Params, false, false);
				foreach (var SC in DeployContextList)
				{
					if (SC.StageTargetPlatform.DeployViaUFE)
					{
						string ClientCmdLine = "-run=Deploy ";
						ClientCmdLine += "-Device=" + string.Join("+", Params.Devices) + " ";
						ClientCmdLine += "-Targetplatform=" + SC.StageTargetPlatform.PlatformType.ToString() + " ";
						ClientCmdLine += "-SourceDir=\"" + CombinePaths(Params.BaseStageDirectory, SC.StageTargetPlatform.PlatformType.ToString()) + "\" ";
						string ClientApp = CombinePaths(CmdEnv.LocalRoot, "Engine/Binaries/Win64/UnrealFrontend.exe");

						Logger.LogInformation("Deploying via UFE:");
						Logger.LogInformation("\tClientCmdLine: " + ClientCmdLine + "");

						//@todo UAT: Consolidate running of external applications like UFE (See 'RunProjectCommand' for other instances)
						PushDir(Path.GetDirectoryName(ClientApp));
						// Always start client process and don't wait for exit.
						IProcessResult ClientProcess = Run(ClientApp, ClientCmdLine, null, ERunOptions.NoWaitForExit);
						PopDir();
						if (ClientProcess != null)
						{
							do
							{
								Thread.Sleep(100);
							}
							while (ClientProcess.HasExited == false);
						}
					}
					else
					{
						bool bSuccess = SC.CustomStagingHandlers.Any( H => H.TryDeploy(Params, SC) );
						if (!bSuccess )
						{
							if (SC.CustomDeployment == null || !SC.CustomDeployment.Deploy(Params, SC))
							{
								SC.StageTargetPlatform.Deploy(Params, SC);
							}
						}
					}
				}
			}
			if (Params.DedicatedServer)
			{
				ProjectParams ServerParams = new ProjectParams(Params);
				ServerParams.Devices = new ParamList<string>(ServerParams.ServerDevice);
				var DeployContextList = CreateDeploymentContext(ServerParams, true, false);
				foreach (var SC in DeployContextList)
				{
					SC.StageTargetPlatform.Deploy(ServerParams, SC);
				}
			}

			Logger.LogInformation("Deploy command time: {0:0.00} s", (DateTime.UtcNow - StartTime).TotalMilliseconds / 1000);
			Logger.LogInformation("********** DEPLOY COMMAND COMPLETED **********");
		}
	}
}
