// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.IO;
using System.Text;
using System.Text.Json;
using EpicGames.Core;
using Microsoft.Extensions.Logging;

namespace AutomationTool
{
	public class DiffCDOFolders : BuildCommand
	{
		public override void ExecuteBuild()
		{
			if (ParseParam("Help"))
			{
				Logger.LogInformation("Usage: DiffCDOFolders -UnrealProjectPath=\"Path to the unreal project\" -FirstRunFolderName=\"Name of the first run\" -SecondRunFolderName=\"Name of the second run\" -NoGenerate (optional to not generate the CDO folders, and use the existing ones)");
				return;
			}

			if (!ParseParam("NoGenerate")) {
				GenerateCDOFolders(new FileReference(ParseParamValue("UnrealProjectPath")), ParseParamValue("FirstRunFolderName"), ParseParamValue("SecondRunFolderName"));
			}

			string firstRunPath = Path.Combine(Path.GetDirectoryName(ParseParamValue("UnrealProjectPath")),"Saved", "Temp", ParseParamValue("FirstRunFolderName"));
			string secondRunPath = Path.Combine(Path.GetDirectoryName(ParseParamValue("UnrealProjectPath")),"Saved", "Temp", ParseParamValue("SecondRunFolderName"));

			if (string.IsNullOrEmpty(firstRunPath) || string.IsNullOrEmpty(secondRunPath))
			{
				throw new AutomationException("Both paths must exist. FirstRunFolder: " + firstRunPath + " SecondRunFolder: " + secondRunPath);
			}

			string[] firstRunFiles = Directory.GetFiles(firstRunPath, "*", SearchOption.AllDirectories);
			string[] secondRunFiles = Directory.GetFiles(secondRunPath, "*", SearchOption.AllDirectories);

			StringBuilder textDiff = new StringBuilder();

			for (int i = 0; i < firstRunFiles.Length; i++)
			{
				string firstFileText = File.ReadAllText(firstRunFiles[i]);
				string secondFileText = File.ReadAllText(secondRunFiles[i]);

				if (!string.Equals(firstFileText, secondFileText, StringComparison.Ordinal))
				{
					using (JsonDocument firstFileJson = JsonDocument.Parse(firstFileText))
					{
						using (JsonDocument secondFileJson = JsonDocument.Parse(secondFileText))
						{
							foreach (JsonProperty property in firstFileJson.RootElement.EnumerateObject())
							{
								JsonElement secondFileValue = secondFileJson.RootElement.GetProperty(property.Name);
								if (!string.Equals(property.Value.ToString(), secondFileValue.ToString(), StringComparison.Ordinal))
								{
									textDiff.AppendLine("Diff at: " + Path.GetRelativePath(firstRunPath, firstRunFiles[i]) + " Property: " + property.Name);
									StringBuilder propertyDiff = new StringBuilder();
									string[] firstLines = property.Value.ToString().Split(new[] { "\n" }, StringSplitOptions.None);
									string[] secondLines = secondFileValue.ToString().Split(new[] { "\n" }, StringSplitOptions.None);

									for (int lineIdx = 0; lineIdx < firstLines.Length; lineIdx++)
									{
										string firstLine = firstLines[lineIdx];
										string secondLine = secondLines[lineIdx];

										if (!string.Equals(firstLine, secondLine, StringComparison.Ordinal))
										{
											propertyDiff.AppendLine("iter 1: >>> " + firstLine);
											propertyDiff.AppendLine("iter 2: <<< " + secondLine);
										}
										else
										{
											propertyDiff.AppendLine(firstLine);
										}
									}
									textDiff.Append(propertyDiff);
									textDiff.AppendLine("--------------------------------");
								}
							}
						}
					}
				}
			}
			if (textDiff.Length > 0)
			{
				throw new AutomationException("Diff was detected.\n" + textDiff.ToString() + "\n To run locally, run .\\runUAT.bat DiffCDOFolders -UnrealProjectPath=\"Path to the unreal project\" -FirstRunFolderName=\"Name of the first run\" -SecondRunFolderName=\"Name of the second run\"");
			} 
			else
			{
				Logger.LogInformation("No diff was detected.");
			}
		}

		private void GenerateCDOFolders(FileReference unrealProjectPath, string firstRunFolderName, string secondRunFolderName)
		{
			Logger.LogInformation("Generating CDO folders...");
			string editorPath = Path.Combine(CmdEnv.LocalRoot, "Engine", "Binaries", "Win64", "UnrealEditor.exe");

			RunCommandlet(unrealProjectPath, editorPath, "DumpArchetypeInfo", firstRunFolderName);
			RunCommandlet(unrealProjectPath, editorPath, "DumpArchetypeInfo", secondRunFolderName);
		}
	}
}
