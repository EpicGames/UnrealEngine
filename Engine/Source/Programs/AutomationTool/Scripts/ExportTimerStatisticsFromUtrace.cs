// Copyright Epic Games, Inc. All Rights Reserved.

using AutomationTool;
using EpicGames.Core;
using System.Collections.Generic;
using System.IO;
using System;
using UnrealBuildBase;
using System.Linq;
using System.Text;
using UnrealBuildTool;
using Polly;
using Microsoft.Extensions.Logging;

namespace ExportTimerStatisticsFromUtrace.Automation
{
	[Help("Runs UnrealInsights to export timers from a utrace file to CSV file.")]
	[ParamHelp("TraceFile", "Input UnrealInsights .utrace file to open", ParamType = typeof(string), Required = true)]
	[ParamHelp("CSVFile", "Output CSV file to write", ParamType = typeof(string), Required = true)]
	[ParamHelp("Threads", "Threads to export", ParamType = typeof(string), Required = false)]
	[ParamHelp("TimerRegion", "TimerRegion to export", ParamType = typeof(string), Required = true)]
	[ParamHelp("MaxTimerCount", "Number of top timers to export", ParamType = typeof(string), Required = true)]
	class ExportTimerStatisticsFromUtrace : BuildCommand
	{
		public override void ExecuteBuild()
		{
			string TraceFile = ParseRequiredStringParam("TraceFile");
			string CSVFile = ParseRequiredStringParam("CSVFile");
			string TimerRegion = ParseRequiredStringParam("TimerRegion");
			string NumTimers = ParseRequiredStringParam("MaxTimerCount");

			string Threads = ParseOptionalStringParam("Threads");
			string ThreadArgument = "";
			if (!string.IsNullOrWhiteSpace(Threads))
			{
				ThreadArgument = string.Format(" -threads={0}", Threads);
			}

			string Arguments = string.Format("-OpenTraceFile=\"{0}\" -unattended -autoquit -noui -nullrhi -ExecOnAnalysisCompleteCmd=\"TimingInsights.ExportTimerStatistics {1} -region={2} -maxtimercount={3} {4} -sortby=totalinclusivetime -sortorder=descending\"", 
				TraceFile,
				CSVFile,
				TimerRegion,
				NumTimers,
				ThreadArgument
				);
			Logger.LogInformation("About to run Insights with arguments '{Arguments}'", Arguments);
			int ReturnCode = RunInsights(Arguments);
			if (ReturnCode != 0)
			{
				throw new AutomationException("UnrealInsights invocation with arguments '{0}' failed with the return code {1}",
					Arguments, ReturnCode);
			}
		}

		/** Runs Insights with the given args and returns its error code. */
		protected static int RunInsights(string Args, UnrealTargetConfiguration InConfiguration = UnrealTargetConfiguration.Development)
		{
			UnrealTargetPlatform ThisHostPlaform = HostPlatform.Platform;
			string HostName = "UnrealInsights";
			string Extension;
			string BuildPath = Path.Combine(Unreal.EngineDirectory.FullName,"Binaries", ThisHostPlaform.ToString());
			string BuildExecutable;
			if (ThisHostPlaform != UnrealTargetPlatform.Mac)
			{
				Extension = ThisHostPlaform == UnrealTargetPlatform.Linux ? string.Empty : ".exe";
				BuildExecutable = Path.Combine(BuildPath, InConfiguration == UnrealTargetConfiguration.Development ? $"{HostName}{Extension}" : $"{HostName}-{ThisHostPlaform}-{InConfiguration}{Extension}");
			}
			else 
			{
				Extension = ".app";
				BuildExecutable = Path.Combine(BuildPath, "Contents", "MacOS",InConfiguration == UnrealTargetConfiguration.Development ? $"{HostName}" : $"{HostName}-{ThisHostPlaform}-{InConfiguration}");
			}

			if (!File.Exists(BuildExecutable))
			{
				throw new AutomationException("Was not able to find UnrealInsights binary at '{0}'. Make sure it is build in {1} configuration", 
					BuildExecutable, InConfiguration.ToString());
			}

			Logger.LogInformation("Running binary '{BuildExecutable}' with arguments '{Args}'", BuildExecutable, Args);
			return UnrealBuildTool.Utils.RunLocalProcessAndLogOutput(BuildExecutable, Args, EpicGames.Core.Log.Logger);
		}
	}
}
