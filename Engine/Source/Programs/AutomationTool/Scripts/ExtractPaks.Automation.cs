// Copyright Epic Games, Inc. All Rights Reserved.
using System;
using System.Collections.Generic;
using System.Text;
using System.Text.RegularExpressions;
using System.IO;
using System.Threading;
using System.Reflection;
using AutomationTool;
using UnrealBuildTool;
using EpicGames.Core;
using System.Linq;
using Microsoft.Extensions.Logging;

public class ExtractPaks : BuildCommand
{
	public override void ExecuteBuild()
	{
		var bLayered = ParseParam("layered");
		var SourceDirectory = ParseParamValue("sourcedirectory", null);
		var TargetDirectory = ParseParamValue("targetdirectory", null);
		var CryptoKeysFile = ParseParamValue("cryptokeysjson", null);
		var Compressor = ParseParamValue("customcompressor", null);
		var Project = ParseParamValue("project", null);

		string ExtraArgs = "";

		if (!string.IsNullOrEmpty(Compressor))
		{
			ExtraArgs += "-customcompressor=" + Compressor;
		}

		if (Params.Contains("passargs"))
		{
			IEnumerable<string> ExtraArgArray = Params.Where((arg, index) => index > Array.IndexOf(Params, "passargs"));

			if (ExtraArgArray.Count() > 0)
			{
				ExtraArgs += " -" + string.Join(" -", ExtraArgArray);
			}
		}

		if (String.IsNullOrEmpty(SourceDirectory))
		{
			throw new AutomationException("SourceDirectory is a required parameter");
		}
		if (String.IsNullOrEmpty(TargetDirectory))
		{
			throw new AutomationException("TargetDirectory is a required parameter");
		}
		if (String.IsNullOrEmpty(CryptoKeysFile))
		{
			throw new AutomationException("CryptoKeysJson is a required parameter");
		}

		DirectoryInfo SourceDirectoryInfo = new DirectoryInfo(SourceDirectory);

		Logger.LogInformation("Extracting paks from {SourceDirectory} to {TargetDirectory}", SourceDirectory, TargetDirectory);

		FileReference ProjectFile = null;
		if (!string.IsNullOrEmpty(Project))
		{
			ProjectFile = ProjectUtils.FindProjectFileFromName(Project);

			if (ProjectFile == null)
			{
				throw new AutomationException("Could not find project file based on {0}", Project);
			}
		}

		PackageUtils.ExtractPakFiles(SourceDirectoryInfo, TargetDirectory, CryptoKeysFile, ExtraArgs, bLayered, ProjectFile);
	}
}
