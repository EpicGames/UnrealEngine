// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.IO;
using System.Linq;
using EpicGames.Core;
using UnrealBuildTool;

namespace AutomationTool
{
	/// <summary>
	/// Commandlet to finalize the creation of an installed build - creating an InstalledBuild.txt file and writing
	/// out installed platform entries for all platforms/configurations where a UnrealGame .target file can be found
	/// </summary>
	[Help("Command to perform additional steps to prepare an installed build.")]
	[Help("OutputDir=<RootDirectory>", "Root Directory of the installed build data (required)")]
	[Help("ContentOnlyPlatforms=<PlatformNameList>", "List of platforms that should be marked as only supporting content projects (optional)")]
	[Help("Platforms=<PlatformList>)", "List of platforms to add")]
	[Help("<Platform>Architectures=<ArchitectureList>)", "List of architectures that are used for a given platform (optional)")]
	[Help("<Platform>GPUArchitectures=<GPUArchitectureList>)", "List of GPU architectures that are used for a given platform (optional)")]
	[Help("AnalyticsTypeOverride=<TypeName>", "Name to give this build for analytics purposes (optional)")]
	[Help("BuildId=<BuildIdentifier>", "Unique identifier for this build (optional)")]
	class FinalizeInstalledBuild : BuildCommand
	{
		/// <summary>
		/// Entry point for the commandlet
		/// </summary>
		public override void ExecuteBuild()
		{
			string OutputDir = ParseRequiredStringParam("OutputDir");
			List<UnrealTargetPlatform> Platforms = ParsePlatformsParamValue("Platforms");
			List<UnrealTargetPlatform> ContentOnlyPlatforms = ParsePlatformsParamValue("ContentOnlyPlatforms");
			string AnalyticsTypeOverride = ParseParamValue("AnalyticsTypeOverride");
			string BuildId = ParseOptionalStringParam("BuildId");

			// Write InstalledBuild.txt to indicate Engine is installed
			string InstalledBuildFile = CommandUtils.CombinePaths(OutputDir, "Engine/Build/InstalledBuild.txt");
			CommandUtils.CreateDirectory(CommandUtils.GetDirectoryName(InstalledBuildFile));

			// Write build Identifier into InstalledBuild.txt
			if(String.IsNullOrWhiteSpace(BuildId))
			{
				BuildId = Guid.NewGuid().ToString().ToUpper();
			}
			CommandUtils.WriteAllText(InstalledBuildFile, BuildId);

			// Write InstalledBuild.txt to indicate Engine is installed
			string Project = ParseParamValue("Project");
			if(Project != null)
			{
				string InstalledProjectBuildFile = CommandUtils.CombinePaths(OutputDir, "Engine/Build/InstalledProjectBuild.txt");
				CommandUtils.CreateDirectory(CommandUtils.GetDirectoryName(InstalledProjectBuildFile));
				CommandUtils.WriteAllText(InstalledProjectBuildFile, new FileReference(Project).MakeRelativeTo(new DirectoryReference(OutputDir)));
			}

			string OutputEnginePath = Path.Combine(OutputDir, "Engine");
			string OutputBaseEnginePath = Path.Combine(OutputEnginePath, "Config", "BaseEngine.ini");
			FileAttributes OutputAttributes = FileAttributes.ReadOnly;
			List<String> IniLines = new List<String>();

			// Should always exist but if not, we don't need extra line
			if (File.Exists(OutputBaseEnginePath))
			{
				OutputAttributes = File.GetAttributes(OutputBaseEnginePath);
				IniLines.Add("");
			}
			else
			{
				CommandUtils.CreateDirectory(CommandUtils.GetDirectoryName(OutputBaseEnginePath));
				CommandUtils.WriteAllText(OutputBaseEnginePath, "");
				OutputAttributes = File.GetAttributes(OutputBaseEnginePath) | OutputAttributes;
			}

			// Create list of platform configurations installed in a Rocket build
			List<InstalledPlatformInfo.InstalledPlatformConfiguration> InstalledConfigs = new List<InstalledPlatformInfo.InstalledPlatformConfiguration>();

			// Add the editor platform/architecture(s) for unspecified target, otherwise we'll never be able to run UAT
			UnrealArchitectures EditorArchitectures = GetArchitecturesForPlatform(HostPlatform.Current.HostEditorPlatform);
			foreach (UnrealArch EditorArchitecture in EditorArchitectures.Architectures)
			{
				InstalledConfigs.Add(new InstalledPlatformInfo.InstalledPlatformConfiguration(UnrealTargetConfiguration.Development, HostPlatform.Current.HostEditorPlatform, TargetRules.TargetType.Editor, EditorArchitecture, "", EProjectType.Unknown, false));
				InstalledConfigs.Add(new InstalledPlatformInfo.InstalledPlatformConfiguration(UnrealTargetConfiguration.DebugGame, HostPlatform.Current.HostEditorPlatform, TargetRules.TargetType.Editor, EditorArchitecture, "", EProjectType.Unknown, false));
			}

			foreach (UnrealTargetPlatform CodeTargetPlatform in Platforms)
			{
				// Build a list of pre-compiled architecture combinations for this platform if any
				UnrealArchitectures AllArchitectures = GetArchitecturesForPlatform(CodeTargetPlatform);

				// Check whether this platform should only be used for content based projects
				EProjectType ProjectType = ContentOnlyPlatforms.Contains(CodeTargetPlatform) ? EProjectType.Content : EProjectType.Any;

				// Allow Content only platforms to be shown as options in all projects
				bool bCanBeDisplayed = ProjectType == EProjectType.Content;
				foreach (UnrealTargetConfiguration CodeTargetConfiguration in Enum.GetValues(typeof(UnrealTargetConfiguration)))
				{
					Dictionary<String, TargetType> Targets = new Dictionary<string, TargetType>() {
						{ "UnrealGame", TargetType.Game },
						{ "UnrealClient", TargetType.Client },
						{ "UnrealServer", TargetType.Server }
					};
					foreach (KeyValuePair<string, TargetType> Target in Targets)
					{
						string CurrentTargetName = Target.Key;
						TargetType CurrentTargetType = Target.Value;

						// Need to check for development receipt as we use that for the Engine code in DebugGame
						UnrealTargetConfiguration EngineConfiguration = (CodeTargetConfiguration == UnrealTargetConfiguration.DebugGame) ? UnrealTargetConfiguration.Development : CodeTargetConfiguration;

						// if the platform doesn't split up multi-arch into multiple targets (which equate to receipt files) then make multiple configs from one receipt
						if (UnrealArchitectureConfig.ForPlatform(CodeTargetPlatform).Mode != UnrealArchitectureMode.OneTargetPerArchitecture)
						{
							FileReference ReceiptFileName = TargetReceipt.GetDefaultPath(new DirectoryReference(OutputEnginePath), CurrentTargetName, CodeTargetPlatform, EngineConfiguration, AllArchitectures);
							if (FileReference.Exists(ReceiptFileName))
							{
								// Strip the output folder so that this can be used on any machine
								string RelativeReceiptFileName = ReceiptFileName.MakeRelativeTo(new DirectoryReference(OutputDir));

								// Blindly append all of the architecture names
								foreach (UnrealArch Arch in AllArchitectures.Architectures)
								{
									InstalledConfigs.Add(new InstalledPlatformInfo.InstalledPlatformConfiguration(CodeTargetConfiguration, CodeTargetPlatform, CurrentTargetType, Arch, RelativeReceiptFileName, ProjectType, bCanBeDisplayed));
								}
							}
						}
						// otherwise, look for receipts for each specified architecture
						else
						{
							foreach (UnrealArch Arch in AllArchitectures.Architectures)
							{
								FileReference ReceiptFileName = TargetReceipt.GetDefaultPath(new DirectoryReference(OutputEnginePath), CurrentTargetName, CodeTargetPlatform, EngineConfiguration, new(Arch.ToString(), CodeTargetPlatform));
								if (FileReference.Exists(ReceiptFileName))
								{
									string RelativeReceiptFileName = ReceiptFileName.MakeRelativeTo(new DirectoryReference(OutputDir));
									InstalledConfigs.Add(new InstalledPlatformInfo.InstalledPlatformConfiguration(CodeTargetConfiguration, CodeTargetPlatform, CurrentTargetType, Arch, RelativeReceiptFileName, ProjectType, bCanBeDisplayed));
								}
							}
						}
					}
				}
			}

			UnrealBuildTool.InstalledPlatformInfo.WriteConfigFileEntries(InstalledConfigs, IniLines);

			if (!String.IsNullOrEmpty(AnalyticsTypeOverride))
			{
				// Write Custom Analytics type setting
				IniLines.Add("");
				IniLines.Add("[Analytics]");
				IniLines.Add(String.Format("UE4TypeOverride=\"{0}\"", AnalyticsTypeOverride));
			}

			// Make sure we can write to the the config file
			File.SetAttributes(OutputBaseEnginePath, OutputAttributes & ~FileAttributes.ReadOnly);
			File.AppendAllLines(OutputBaseEnginePath, IniLines);
			File.SetAttributes(OutputBaseEnginePath, OutputAttributes);
		}

		/// <summary>
		/// Parse an argument containing a list of platforms
		/// </summary>
		/// <param name="Name">Name of the argument</param>
		/// <returns>List of platforms</returns>
		List<UnrealTargetPlatform> ParsePlatformsParamValue(string Name)
		{
			string PlatformsString = ParseParamValue(Name);
			if (String.IsNullOrWhiteSpace(PlatformsString))
			{
				return new List<UnrealTargetPlatform>();
			}
			else
			{
				return PlatformsString.Split(new char[] { ';' }, StringSplitOptions.RemoveEmptyEntries).Select(x => UnrealTargetPlatform.Parse(x)).ToList();
			}
		}

		/// <summary>
		/// Look for a platform's architecture(s) on the commandline, and if not, then ask the platform's config for active architectures
		/// </summary>
		/// <param name="Platform">Platform to look for architectures</param>
		/// <returns>One or more architectures</returns>
		private UnrealArchitectures GetArchitecturesForPlatform(UnrealTargetPlatform Platform)
		{
			// Try to parse additional Architectures from the command line
			string Architectures = ParseParamValue(Platform.ToString() + "Architectures");

			// Wwas a platform specified on commandline?
			if (!string.IsNullOrWhiteSpace(Architectures))
			{
				return UnrealArchitectures.FromString(Architectures, Platform);
			}
			// if there aren't any, use the default
			else
			{
				// get the architectures for the target platform, for unspecified target
				UnrealArchitectureConfig PlatformArchConfig = UnrealArchitectureConfig.ForPlatform(Platform);
				return PlatformArchConfig.ActiveArchitectures(null, null);
			}
		}
	}
}
