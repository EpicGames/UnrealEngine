// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Linq;
using System.Text;
using System.Threading.Tasks;
using Microsoft.Extensions.Logging;

namespace AutomationTool
{
	[RequireP4]
	[DoesNotNeedP4CL]
	[Help("Fixes the case of files on a case-insensitive Perforce server by removing and re-adding them.")]
	[Help("Files", "Pattern for files to match. Should be a full depot path with the correct case. May end with a wildcard.")]
	class FixPerforceCase : BuildCommand
	{
		const string BoilerplateText = "\n\n#rb none\n#rnx\n#preflight skip\n#jira none\n#submittool FixPerforceCase\n#okforgithub ignore";

		/// <summary>
		/// Main entry point for the command
		/// </summary>
		public override void ExecuteBuild()
		{
			string FileSpec = ParseRequiredStringParam("Files");

			// Make sure the patterns are a valid syntax
			if (!FileSpec.StartsWith("//"))
			{
				throw new AutomationException("Files must be specified as full depot paths");
			}

			// Pick out the source and target prefixes
			string Prefix;
			if (FileSpec.EndsWith("*"))
			{
				Prefix = FileSpec.Substring(0, FileSpec.Length - 1);
			}
			else if (FileSpec.EndsWith("..."))
			{
				Prefix = FileSpec.Substring(0, FileSpec.Length - 3);
			}
			else
			{
				Prefix = FileSpec;
			}

			// Make sure there aren't any other wildcards in the pattern
			if (Prefix.Contains("?") || Prefix.Contains("*") || Prefix.Contains("..."))
			{
				throw new AutomationException("Wildcards are only permitted at the end of filespecs");
			}

			// Find all the source files
			List<string> SourceFiles = P4.Files(String.Format("-e {0}", FileSpec));
			if (SourceFiles.Count == 0)
			{
				throw new AutomationException("No files found matching {0}", FileSpec);
			}
			SourceFiles.RemoveAll(x => x.StartsWith(Prefix, StringComparison.Ordinal));

			// Error if we didn't find anything
			if (SourceFiles.Count == 0)
			{
				throw new AutomationException("No files found matching spec");
			}

			// Find all the target files
			List<string> TargetFiles = new List<string>(SourceFiles.Count);
			foreach (string SourceFile in SourceFiles)
			{
				if (SourceFile.StartsWith(Prefix, StringComparison.OrdinalIgnoreCase))
				{
					TargetFiles.Add(Prefix + SourceFile.Substring(Prefix.Length));
				}
				else
				{
					throw new AutomationException("Source file '{0}' does not start with '{1}'", SourceFile, Prefix);
				}
			}

			// Print what we're going to do
			Logger.LogInformation("Ready to rename {Arg0} files:", SourceFiles.Count);
			for (int Idx = 0; Idx < SourceFiles.Count; Idx++)
			{
				Logger.LogInformation("{0,3}: {Arg1}", Idx, SourceFiles[Idx]);
				Logger.LogInformation("{0,3}  {Arg1}", "", TargetFiles[Idx]);
			}

			// If we're not going through with it, print the renames
			if (!AllowSubmit)
			{
				Logger.LogWarning("Skipping due to no -Submit option");
				return;
			}

			// Force sync all the old files 
			foreach (string OldFile in SourceFiles)
			{
				P4.LogP4("", String.Format("sync -f {0}", OldFile));
			}

			// Delete all the old files 
			int DeleteChangeNumber = P4.CreateChange(Description: String.Format("Fixing case of {0} (1/2){1}", FileSpec, BoilerplateText));
			foreach (string OldFile in SourceFiles)
			{
				P4.LogP4("", String.Format("delete -k -c {0} {1}", DeleteChangeNumber, OldFile));
			}
			P4.Submit(DeleteChangeNumber);

			// Re-add all the files in the new location
			int AddChangeNumber = P4.CreateChange(Description: String.Format("Fixing case of {0} (2/2){1}", FileSpec, BoilerplateText));
			foreach (string NewFile in TargetFiles)
			{
				P4.LogP4("", String.Format("add -f -c {0} {1}", AddChangeNumber, NewFile));
			}
			P4.Submit(AddChangeNumber);
		}
	}
}
