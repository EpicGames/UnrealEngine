// Copyright Epic Games, Inc. All Rights Reserved.
using System;
using System.Collections.Generic;
using System.Text;
using System.IO;
using AutomationTool;
using UnrealBuildTool;
using Microsoft.Extensions.Logging;

class FixupRedirects : BuildCommand
{
	public override void ExecuteBuild()
	{
		var ProjectName = ParseParamValue("project", "");

		var EditorExe = CombinePaths(CmdEnv.LocalRoot, @"Engine/Binaries/Win64/UnrealEditor-Cmd.exe");
		Logger.LogInformation("********** Running FixupRedirects: {EditorExe} -run=ResavePackages -unattended -nopause -buildmachine -fixupredirects -autocheckout -autocheckin -projectonly", EditorExe);
		RunCommandlet(GetCommandletProjectFile(ProjectName), EditorExe, "ResavePackages", "-unattended -nopause -buildmachine -fixupredirects -autocheckout -autocheckin -projectonly");
	}
}
