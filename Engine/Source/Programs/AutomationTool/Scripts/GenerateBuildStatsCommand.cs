// Copyright Epic Games, Inc. All Rights Reserved.

using Microsoft.Extensions.Logging;
using Microsoft.Extensions.FileSystemGlobbing;
using System;
using System.Collections.Generic;
using System.IO;
using System.Linq;
using System.Text;
using System.Text.Json;

namespace AutomationTool
{
	[Help("Generates file size statistics for a directory tree.")]
	[ParamHelp("RootDir", "<path> \t\t\t\t Base directory for scanning", ParamType = typeof(string), Required = false)]
	[ParamHelp("OutputPath", "<path> \t\t\t\t Path where the stats file will be saved", ParamType = typeof(string), Required = false)]
	[ParamHelp("OutputFormat", "<csv or json> \t\t\t Output format for the stats file", ParamType = typeof(string), Required = false)]
	[ParamHelp("Include", "<path1;path2;...> \t\t\t Semicolon-separated list of relative paths or patterns (prefix, Dir/*.ext, Dir/**/*.ext)", ParamType = typeof(string), Required = true)]
	[ParamHelp("Exclude", "<path1;path2;...> \t\t\t Semicolon-separated list of relative paths or patterns (prefix, Dir/*.ext, Dir/**/*.ext)", ParamType = typeof(string), Required = false)]
	[ParamHelp("MinFileSize", "<bytes> \t\t\t\t Files smaller than this value are treated as small", ParamType = typeof(long), Required = false)]
	[ParamHelp("SmallFileHandling", "<include or ignore or aggregate> \t Behaviour strategy for files smaller than MinFileSize", ParamType = typeof(string), Required = false)]

	public class GenerateBuildStats : BuildCommand
	{
		public override void ExecuteBuild()
		{
			Logger.LogInformation("=== GenerateBuildStats: starting ===");

			BuildStatsOptions BuildStatsOptions = ParseCommandLineParam();

			LogBuildStatsOptions(BuildStatsOptions);

			VerifyOrCreateDirectory(BuildStatsOptions.OutputDir);

			BuildStatsDocument BuildStatsDocument;
			try
			{
				BuildStatsDocument = GenerateBuildStatsDocument(BuildStatsOptions);
			}
			catch (Exception)
			{
				Logger.LogError("GenerateBuildStats: Can't generate build stats.");
				throw;
			}

			string OutputPath = Path.Combine(BuildStatsOptions.OutputDir, BuildStatsOptions.Filename);
			SaveStats(BuildStatsDocument, OutputPath, BuildStatsOptions.OutputFormat, BuildStatsOptions.SmallFileHandlingMode);

			Logger.LogInformation("=== GenerateBuildStats: completed successfully ===");
		}

		private BuildStatsOptions ParseCommandLineParam()
		{
			string RootDirParam = ParseParamValue("RootDir");
			string RootDir = string.IsNullOrWhiteSpace(RootDirParam)
				? Directory.GetCurrentDirectory()
				: RootDirParam;

			string OutputPathParam = ParseParamValue("OutputPath");
			string OutputDir = GetOutputDir(OutputPathParam);
			string Filename = GetFilename(OutputPathParam);

			string OutputFormatStr = ParseParamValue("OutputFormat") ?? "csv";
			OutputFormat OutputFormat = ParseOutputFormat(OutputFormatStr);

			string Extension = OutputFormat == OutputFormat.Csv ? ".csv" : ".json";
			if (!Path.HasExtension(Filename))
			{
				Filename = Filename + Extension;
			}
			
			string[] Exclude = ParsePathsList(ParseParamValue("Exclude"));
			string IncludeCLIStr = ParseParamValue("Include");
			string[] Include = ParsePathsList(IncludeCLIStr);
			if (Include.Length == 0)
			{
				throw new AutomationException("GenerateBuildStats: -Include param is required. You must include at least 1 filepath to include. You have provided {0}", IncludeCLIStr);
			}

			long MinFileSizeBytes = 0;
			string MinFileSizeStr = ParseParamValue("MinFileSize");
			if (!string.IsNullOrWhiteSpace(MinFileSizeStr))
			{
				if (!long.TryParse(MinFileSizeStr, out MinFileSizeBytes) || MinFileSizeBytes < 0)
				{
					throw new AutomationException("GenerateBuildStats: Invalid MinFileSize Value: {0}", MinFileSizeStr);
				}
			}

			string SmallFileHandlingModeStr = ParseParamValue("SmallFileHandling") ?? "include";
			SmallFileHandlingMode SmallFileHandlingMode = ParseSmallFileHandlingMode(SmallFileHandlingModeStr);

			return new BuildStatsOptions
			{
				RootDir = RootDir,
				OutputDir = OutputDir,
				Filename = Filename,
				OutputFormat = OutputFormat,
				Include = Include,
				Exclude = Exclude,
				MinFileSizeBytes = MinFileSizeBytes,
				SmallFileHandlingMode = SmallFileHandlingMode
			};
		}

		private static void LogBuildStatsOptions(BuildStatsOptions Options)
		{
			Logger.LogInformation("RootDir           : {0}", Options.RootDir);
			Logger.LogInformation("OutputPath        : {0}", Path.Combine(Options.OutputDir, Options.Filename));
			Logger.LogInformation("OutputFormat      : {0}", Options.OutputFormat);
			Logger.LogInformation("Include           : {0}", Options.Include.Length > 0 ? string.Join(", ", Options.Include) : "None");
			Logger.LogInformation("Exclude           : {0}", Options.Exclude.Length > 0 ? string.Join(", ", Options.Exclude) : "None");
			Logger.LogInformation("MinFileSize       : {0} bytes", Options.MinFileSizeBytes);
			Logger.LogInformation("SmallFileHandling : {0}", Options.SmallFileHandlingMode);
		}

		private static void VerifyOrCreateDirectory(string OutputDir)
		{
			try
			{
				Directory.CreateDirectory(OutputDir);
			}
			catch (Exception)
			{
				Logger.LogError("GenerateBuildStats: Can't create output directory.");
				throw;
			}
		}

		private static BuildStatsDocument GenerateBuildStatsDocument(BuildStatsOptions Options)
		{
			if (!Directory.Exists(Options.RootDir))
			{
				throw new DirectoryNotFoundException($"Root directory does not exist: {Options.RootDir}");
			}

			List<FileEntity> FilteredFiles = [];
			Matcher Matcher = new Matcher();
			Matcher.AddIncludePatterns(Options.Include);
			Matcher.AddExcludePatterns(Options.Exclude);
			foreach (string FullPath in Matcher.GetResultsInFullPath(Options.RootDir))
			{
				FileInfo FileInfo = new FileInfo(FullPath);
				string RelativePath = Path.GetRelativePath(Options.RootDir, FullPath).Replace('\\', '/');
				string Extension = Path.GetExtension(FullPath);
				if (string.IsNullOrWhiteSpace(Extension))
				{
					Extension = "None";
				}

				FilteredFiles.Add(new FileEntity
				{
					RelativePath = RelativePath,
					Bytes = FileInfo.Length,
					Extension = Extension,
				});
			}

			BuildStatsDocument SeparatedFiles = ProcessFiles(
				FilteredFiles,
				Options.MinFileSizeBytes,
				Options.SmallFileHandlingMode);

			return SeparatedFiles;
		}

		private static void SaveStats(BuildStatsDocument BuildStatsDocument, string OutputPath, OutputFormat OutputFormat, SmallFileHandlingMode Mode)
		{
			IStatsWriter Writer = new CsvStatsWriter();
			if (OutputFormat == OutputFormat.Json)
			{
				Writer = new JsonStatsWriter();
			}

			if (Mode == SmallFileHandlingMode.Aggregate)
			{
				string SmallAggregatedStatsFilePath = GenerateAggregatedStatsFilePath(OutputPath);
				Writer.Write(BuildStatsDocument.Files, OutputPath);
				Writer.Write(BuildStatsDocument.SmallAggregatedFiles, SmallAggregatedStatsFilePath);
				Logger.LogInformation("GenerateBuildStats: processed {0} large files and {1} small files.", BuildStatsDocument.Files.Count(), BuildStatsDocument.SmallAggregatedFiles.Count());
				Logger.LogInformation("Large files: {0}", OutputPath);
				Logger.LogInformation("Small files: {0}", SmallAggregatedStatsFilePath);
			}
			else
			{
				Writer.Write(BuildStatsDocument.Files, OutputPath);
				Logger.LogInformation("GenerateBuildStats: processed {0} files.", BuildStatsDocument.Files.Count());
				Logger.LogInformation("Output: {0}", OutputPath);
			}
		}

		// Helpers
		private static BuildStatsDocument ProcessFiles(IEnumerable<FileEntity> Files, long MinFileSizeBytes, SmallFileHandlingMode Mode)
		{
			if (Mode == SmallFileHandlingMode.Include || MinFileSizeBytes <= 0)
			{
				return new BuildStatsDocument
				{
					Files = Files,
					SmallAggregatedFiles = []
				};
			}

			IEnumerable<FileEntity> SmallFiles = [];
			IEnumerable<FileEntity> LargeFiles = Files.Where(f => (f.Bytes >= MinFileSizeBytes));
			if (Mode == SmallFileHandlingMode.Aggregate)
			{
				SmallFiles = Files.Where(f => (f.Bytes < MinFileSizeBytes));
			}

			return new BuildStatsDocument
			{
				Files = LargeFiles,
				SmallAggregatedFiles = SmallFiles
			};
		}

		private static string GenerateAggregatedStatsFilePath(string MainStatFilePath)
		{
			string Extension = Path.GetExtension(MainStatFilePath);
			return MainStatFilePath.Replace(Extension, "_Small" + Extension);
		}

		private string[] ParsePathsList(string RawPatterns)
		{
			if (string.IsNullOrWhiteSpace(RawPatterns))
			{
				return [];
			}

			return RawPatterns
				.Split([';'], StringSplitOptions.RemoveEmptyEntries)
				.Select(p => p.Replace('\\', '/'))
				.ToArray();
		}

		private static OutputFormat ParseOutputFormat(string Value)
		{
			switch (Value.ToLowerInvariant())
			{
				case "csv": return OutputFormat.Csv;
				case "json": return OutputFormat.Json;
				default:
					throw new AutomationException(
						"GenerateBuildStats: Unknown OutputFormat '{0}' (expected csv or json).", Value);
			}
		}

		private static SmallFileHandlingMode ParseSmallFileHandlingMode(string Value)
		{
			switch (Value.ToLowerInvariant())
			{
				case "include": return SmallFileHandlingMode.Include;
				case "ignore": return SmallFileHandlingMode.Ignore;
				case "aggregate": return SmallFileHandlingMode.Aggregate;
				default:
					throw new AutomationException(
						"GenerateBuildStats: Unknown SmallFileHandling '{0}' (expected include|ignore|aggregate).", Value);
			}
		}

		private static string GetOutputDir(string OutputPathParam)
		{
			if (string.IsNullOrWhiteSpace(OutputPathParam))
			{
				return CommandUtils.CmdEnv.LocalRoot;
			}
			if (!Path.HasExtension(OutputPathParam))
			{
				return OutputPathParam;
			}

			string DirectoryPath = Path.GetDirectoryName(OutputPathParam);
			if (string.IsNullOrWhiteSpace(DirectoryPath))
			{
				return Directory.GetCurrentDirectory();
			}

			return DirectoryPath;
		}

		private static string GetFilename(string OutputPathParam)
		{
			if (string.IsNullOrWhiteSpace(OutputPathParam) || !Path.HasExtension(OutputPathParam))
			{
				return "GenerateBuildStatsOutput";
			}

			return Path.GetFileName(OutputPathParam);
		}

		// StatsWriter
		private interface IStatsWriter
		{
			void Write(
				IEnumerable<FileEntity> Files,
				string OutputFilePath);
		}

		private class JsonStatsWriter : IStatsWriter
		{
			public void Write(IEnumerable<FileEntity> Files, string OutputFilePath)
			{
				var Options = new JsonSerializerOptions
				{
					WriteIndented = true
				};
				var Json = JsonSerializer.Serialize(Files, Options);
				File.WriteAllText(OutputFilePath, Json, new UTF8Encoding(false));
			}
		}

		private class CsvStatsWriter : IStatsWriter
		{
			public void Write(IEnumerable<FileEntity> Files, string OutputFilePath)
			{
				using var Writer = new StreamWriter(OutputFilePath, false, new UTF8Encoding(false));
				Writer.WriteLine("File,Bytes,Extension");
				foreach (FileEntity File in Files)
				{
					Writer.WriteLine($"{File.RelativePath},{File.Bytes},{File.Extension}");
				}
			}
		}

		// Data models
		private enum SmallFileHandlingMode
		{
			Include,
			Ignore,
			Aggregate
		}

		private enum OutputFormat
		{
			Csv,
			Json
		}

		private class FileEntity
		{
			public string RelativePath { get; init; }
			public long Bytes { get; init; }
			public string Extension { get; init; }
		}

		private class BuildStatsOptions
		{
			public string RootDir { get; init; }
			public string OutputDir { get; init; }
			public string Filename { get; init; }
			public OutputFormat OutputFormat { get; init; }
			public string[] Include { get; init; }
			public string[] Exclude { get; init; }
			public long MinFileSizeBytes { get; init; }
			public SmallFileHandlingMode SmallFileHandlingMode { get; init; }
		}

		private class BuildStatsDocument
		{
			public IEnumerable<FileEntity> Files { get; init; }
			public IEnumerable<FileEntity> SmallAggregatedFiles { get; init; }
		}
	}
}
