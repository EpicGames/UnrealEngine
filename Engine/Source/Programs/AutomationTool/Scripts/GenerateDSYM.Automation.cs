// Copyright Epic Games, Inc. All Rights Reserved.
using System;
using AutomationTool;
using UnrealBuildBase;
using UnrealBuildTool;
using System.Linq;
using EpicGames.Core;
using System.IO;
using System.Collections.Generic;
using Microsoft.Extensions.Logging;

[Help(@"Generates IOS debug symbols for a remote project.")]
[Help("project=Name", @"Project name (required), i.e: -project=QAGame")]
[Help("config=Configuration", @"Project configuration (required), i.e: -config=Development")]
public class GenerateDSYM : BuildCommand
{
	public override void ExecuteBuild()
	{
		if (OperatingSystem.IsMacOS())
		{
			bool bSaveFlat = ParseParamBool("flat");

			List<string> Binaries = new List<string>();
			string FileList = ParseParamValue("file=");
			if (!string.IsNullOrEmpty(FileList))
			{
				Binaries.AddRange(FileList.Split(','));
			}
			else
			{
				string PlatformName = ParseParamValue("platform");
				FileReference ProjectFile = ParseProjectParam();
				string TargetName = ParseOptionalStringParam("target") ?? System.IO.Path.GetFileNameWithoutExtension(ProjectFile?.FullName ?? "");
				UnrealTargetConfiguration Configuration = ParseOptionalEnumParam<UnrealTargetConfiguration>("config") ?? UnrealTargetConfiguration.Development;

				if (ProjectFile == null || PlatformName == null || TargetName == null)
				{
					Logger.LogError("Must specify a file(s) with -file=, or other parameters to find the binaries:-project=<Path or name of project>\n" +
						"-platform=<Mac|IOS|TVOS>\n" + 
						"-target=<TargetName> (Optional, defaults to the ProjectName)\n" + 
						"-config=<Debug|DebugGame|Development|Test|Shipping> (Optional - defaults to Development)\n" +
						"Ex: -project=EngineTest -platform=Mac -target=EngineTestEditor -config=Test\n\n"+
						"Other options:\n" +
						"-flat (Options, if specified, the .dSYMs will be flat files that are easier to copy around between computers/servers/etc\n\n");
					return;
				}

				UnrealTargetPlatform Platform;
				if (!UnrealTargetPlatform.TryParse(PlatformName, out Platform) || !Platform.IsInGroup(UnrealPlatformGroup.Apple))
				{
					Logger.LogError("Platform must be one of Mac, IOS, TVOS");
					return;
				}


				// Apple platforms don't need the architecture pass in because they RequiresArchitectureFilenames returns false, but if that ever changes, we pass in the active architecture anyway
				UnrealArchitectures Architectures = UnrealArchitectureConfig.ForPlatform(Platform).ActiveArchitectures(ProjectFile, TargetName);

				FileReference ReceiptFile = TargetReceipt.GetDefaultPath(DirectoryReference.FromFile(ProjectFile) ?? Unreal.EngineDirectory, TargetName, Platform, Configuration, Architectures);
				TargetReceipt Receipt = TargetReceipt.Read(ReceiptFile);
				// get the products that we can dsym (duylibs and executables)
				IEnumerable<BuildProduct> Products = Receipt.BuildProducts.Where(x => x.Type == BuildProductType.Executable || x.Type == BuildProductType.DynamicLibrary);
				Binaries.AddRange(Products.Select(x => x.Path.FullName));
			}

			// sort binaries by size so we start with large ones first to try to not have a single long tent pole hanging out at the end
			// in one test, doing this sped up some tests by around 30%
			if (Binaries.Count > 5)
			{
				Logger.LogInformation("Sorting binaries by size...");
				Binaries.Sort((A, B) => new FileInfo(B).Length.CompareTo(new FileInfo(A).Length));
			}


			DateTime Start = DateTime.Now;
			Dictionary<string, TimeSpan> Timers = new Dictionary<string, TimeSpan>();
			Int64 TotalAmountProcessed = 0;
			int Counter = 1;
			Logger.LogInformation("Starting dSYM generation...");
			System.Threading.Tasks.Parallel.ForEach(Binaries, (Binary) =>
			{
				int Index;
				long Filesize = new FileInfo(Binary).Length;
				lock (Timers)
				{
					Index = Counter++;
				}

				//Log.TraceInformation("  [{0}/{1}] {2}, source is {3:N2} mb", Index, Binaries.Count, Path.GetFileName(Binary), Filesize / 1024.0 / 1024.0);

				// put dSYM next to the binary
				string dSYM = Path.ChangeExtension(Binary, ".dSYM");
				if (Binary.Contains(".app/"))
				{
					// or the .app if the binary is inside one
					dSYM = Path.Combine(Path.GetDirectoryName(Binary.Substring(0, Binary.LastIndexOf(".app"))), Path.GetFileName(dSYM));
				}

				Logger.LogInformation("  Generating for {Binary} -> {dSYM}", Binary, dSYM);

				string Command;
				if (bSaveFlat)
				{
					Command = string.Format("-c 'rm -rf \"{0}\"; dsymutil -o \"{0}\" {2} \"{1}\"'", dSYM, Binary, bSaveFlat ? "--flat" : "");
				}
				else 
				{ 
					Command = $"-c '{Unreal.EngineDirectory}/Build/BatchFiles/Mac/GenerateUniversalDSYM.sh \"{Binary}\" \"{dSYM}\"'";
				}	

				DateTime RunStart = DateTime.Now;
				Run("bash", Command, Options: ERunOptions.NoLoggingOfRunCommand);
				TimeSpan Diff = DateTime.Now - RunStart;
				lock (Timers)
				{
					Timers.Add(dSYM, Diff);
					TotalAmountProcessed += Filesize;
				}

				Logger.LogInformation("  [{Index}/{Arg1}] {Arg2} took {3:g}, source is {4:N2} mb", Index, Binaries.Count, Path.GetFileName(dSYM), Diff, Filesize / 1024.0 / 1024.0);

			});

			Logger.LogInformation("\n\n-------------------------------------------------------------------");
			Logger.LogInformation("GeneratedSYM took {Arg0}, processed {1:N2} mb of binary data.", DateTime.Now - Start, (double)TotalAmountProcessed / 1024.0 / 1024.0);
			if (Binaries.Count > 5)
			{
				Logger.LogInformation("Slowest dSYMS were:");
	
				List<string> SlowDSYMs = Timers.Keys.ToList();
				SlowDSYMs.Sort((A, B) => Timers[B].CompareTo(Timers[A]));
				Logger.LogInformation("{Text}", string.Join("\n", SlowDSYMs.Take(10).Select(x => string.Format("{0} took {1}", Path.GetFileName(x), Timers[x]))));
			}
		}
		else
		{
			var ProjectName = ParseParamValue("project");
			var Config = ParseParamValue("config");

			var IPPExe = CombinePaths(CmdEnv.LocalRoot, "Engine/Binaries/DotNET/IOS/IPhonePackager.exe");

			RunAndLog(CmdEnv, IPPExe, "RPC " + ProjectName + " -config " + Config + " GenDSYM");

		}
	}
}