// Copyright Epic Games, Inc. All Rights Reserved.
using EpicGames.Core;
using System.IO;
using System.Collections.Generic;
using System.Diagnostics;
using UnrealBuildTool;
using System;
using System.Text;
using System.Linq;
using Microsoft.Extensions.Logging;

namespace AutomationTool
{
	[Help("Generates a report about all platforms and their configuration settings.")]
	[Help("Platforms", "Limit report to these platforms. e.g. Windows+iOS. Default is all platforms")]
	[Help("CsvOut", "<filename> - Generate a CSV file instead of an html file")]
	[Help("DDPISection", "ShaderPlatform,DataDrivenPlatformInfo - which config section to check")]
	[Help("UseFieldNames", "Use field names instead of friendly names")]
	[Help("OpenReport", "Opens the html report once it has been generated")]
	class GeneratePlatformReport : BuildCommand
	{
		#nullable enable

		class PlatformReportRow
		{
			public List<string> Columns = new List<string>();
		};

		class PlatformReport
		{
			public string ReportName = "";
			public string ReportDescription = "";
			public List<PlatformReportRow> Rows = new List<PlatformReportRow>();
		};

		public override void ExecuteBuild()
		{
			// Parse parameters
			string[] Platforms = ParseParamValue("Platforms", "").Split('+', StringSplitOptions.RemoveEmptyEntries);
			string? CsvFile = ParseParamValue("CsvOut");
			string DDPISection = ParseParamValue("DDPISection","DataDrivenPlatformInfo");
			bool bFriendlyNames = !ParseParam("UseFieldNames");
			bool bOpenReport = ParseParam("OpenReport");

			// Generate the report
			PlatformReport Report = GenerateDDPIReport(Platforms, DDPISection);
			if (Report.Rows.Count <= 1)
			{
				Logger.LogError("cannot generate report");
				return;
			}

			// Apply friendly names
			if (bFriendlyNames)
			{
				for (int ColumnIndex = 0; ColumnIndex < Report.Rows[0].Columns.Count; ColumnIndex++)
				{
					Report.Rows[0].Columns[ColumnIndex] = MakeFriendlyName(Report.Rows[0].Columns[ColumnIndex]);
				}
			}

			// Save the report
			if (CsvFile != null)
			{
				WriteReportToCSV( Report, CsvFile );
			}
			else
			{
				string HtmlFile = Path.Combine( Directory.GetCurrentDirectory(), $"{Report.ReportName}_Report.html" );
				WriteReportToHTML( Report, HtmlFile );

				if (bOpenReport)
				{
					ProcessStartInfo ProcessInfo = new ProcessStartInfo(HtmlFile);
					ProcessInfo.UseShellExecute = true;
					Process.Start(ProcessInfo);
				}
			}
		}

		private PlatformReport GenerateDDPIReport(string[] Platforms, string DDPISection)
		{
			// Collect the list of DDPIs to use
			IReadOnlyDictionary<string, DataDrivenPlatformInfo.ConfigDataDrivenPlatformInfo> PlatformInfos; 
			if (Platforms == null || Platforms.Length == 0)
			{
				PlatformInfos = DataDrivenPlatformInfo.GetAllPlatformInfos();
			}
			else
			{
				Dictionary<string, DataDrivenPlatformInfo.ConfigDataDrivenPlatformInfo> PlatformInfosDict = new Dictionary<string, DataDrivenPlatformInfo.ConfigDataDrivenPlatformInfo>();
				foreach (string PlatformName in Platforms)
				{
					DataDrivenPlatformInfo.ConfigDataDrivenPlatformInfo? Config = DataDrivenPlatformInfo.GetDataDrivenInfoForPlatform(PlatformName);
					if (Config != null)
					{
						PlatformInfosDict.Add(PlatformName, Config);
					}
				}
				PlatformInfos = PlatformInfosDict;
			} 	

			// collect all possible config values
			SortedSet<string> ColumnNames = new SortedSet<string>();
			Dictionary<string,ConfigHierarchySection> AllConfigSections = new Dictionary<string, ConfigHierarchySection>();
			foreach (var PlatformInfoPair in PlatformInfos)
			{
				// get DDPI and make sure there is config
				DataDrivenPlatformInfo.ConfigDataDrivenPlatformInfo DDPI = PlatformInfoPair.Value;
				if (DDPI.PlatformConfig == null)
				{
					continue;
				}

				// find all config sections that match the report type (this helps to find multiple ShaderPlatform config sections, for example)
				string PlatformName = PlatformInfoPair.Key;
				foreach (string SectionName in DDPI.PlatformConfig.SectionNames.Where( s => s.StartsWith(DDPISection, StringComparison.InvariantCultureIgnoreCase) ) )
				{
					ConfigFileSection? Section = null;
					if (DDPI.PlatformConfig.TryGetSection(SectionName, out Section))
					{
						string PlatformSectionName = PlatformName + " " + SectionName;
						ConfigHierarchySection PlatformSection = new ConfigHierarchySection(new List<ConfigFileSection>() { Section });

						AllConfigSections.Add( PlatformSectionName, PlatformSection );

						ColumnNames.UnionWith( PlatformSection.KeyNames );
					}
				}
			}


			// build the report and add the header row
			PlatformReport Report = new PlatformReport();
			Report.ReportName = $"DDPI_{DDPISection}";
			Report.ReportDescription = $"Data-Driven Platform Info - {DDPISection}";
			
			PlatformReportRow HeaderRow = new PlatformReportRow();
			HeaderRow.Columns.Add(DDPISection);
			HeaderRow.Columns.AddRange(ColumnNames);
			Report.Rows.Add(HeaderRow);

			// write all platform rows
			foreach ( var ConfigSectionPair in AllConfigSections )
			{
				// first column is the row name
				PlatformReportRow PlatformRow = new PlatformReportRow();
				PlatformRow.Columns.Add(ConfigSectionPair.Key);

				// look up key values for subsequent columns
				foreach (string Column in ColumnNames)
				{
					string? Value = null;
					ConfigSectionPair.Value.TryGetValue(Column, out Value );
					PlatformRow.Columns.Add( Value ?? "" );
				}

				Report.Rows.Add(PlatformRow);
			}

			return Report;
		}



		// bMyVariable -> My Variable (note that this does not handle all possible cases, just enough to make the generated column names more readable)
		private string MakeFriendlyName( string PropertyName )
		{
			PropertyName = PropertyName.Replace(":b",":").Replace("_b","_");

			StringBuilder FriendlyName = new StringBuilder();

			char[] Chars = PropertyName.ToCharArray();
			bool bWasUpperCase = char.IsUpper(Chars[0]);
			bool bWasDigit = char.IsDigit(Chars[0]);
			foreach( char Char in Chars )
			{
				// skip leading b
				if (FriendlyName.Length == 0 && Char == 'b' )
				{
					continue;
				}
				
				// insert space if the case/digit change
				bool bIsUpperCase = char.IsUpper(Char);
				bool bIsDigit = char.IsDigit(Char);
				if (FriendlyName.Length > 0 && bIsUpperCase && !bWasUpperCase || bIsDigit && !bWasDigit)
				{
					FriendlyName.Append(' ');
				}
				bWasUpperCase = bIsUpperCase;
				bWasDigit = bIsDigit;

				// add the character
				FriendlyName.Append((Char == '_') ? ' ' : Char);
			}
					
			return FriendlyName.ToString();
		}



		private void WriteReportToCSV(PlatformReport Report, string CSVFile)
		{
			StringBuilder Builder = new StringBuilder();
			foreach (PlatformReportRow Row in Report.Rows)
			{
				Builder.AppendLine( string.Join(',', Row.Columns ) );
			}
			System.IO.File.WriteAllText(CSVFile, Builder.ToString());
			Console.WriteLine($"Report written to {CSVFile}");
		}

		private void WriteReportToHTML(PlatformReport Report, string HTMLFile)
		{
			StringBuilder Builder = new StringBuilder();

			// write header
			Builder.AppendLine("<!DOCTYPE html>");
			Builder.AppendLine("<html>");
			Builder.AppendLine($"<head><title>{Report.ReportDescription}</title></head>");

			// write styles
			int cellPadding = 2;
			Builder.AppendLine("<style type='text/css' id='pageStyle'>");
			Builder.AppendLine("p {  font-family: 'Verdana', Times, serif; font-size: 12px }");
			Builder.AppendLine("table, th, td { border: 2px solid black; border-collapse: collapse; z-index:0; padding: " + cellPadding + "px; vertical-align: center; font-family: 'Verdana', Times, serif; font-size: 11px;}");
			Builder.AppendLine("th:first-child, td:first-child { border-left: 2px solid black; position: sticky; left: 0; z-index:1; background-color: #e2e2e2; }");
			Builder.AppendLine("tr:nth-child(odd) {background-color: #e2e2e2;}");
			Builder.AppendLine("tr:nth-child(even) {background-color: #ffffff;}");
			Builder.AppendLine("tr:first-child {background-color: #e2e2e2;}");
			Builder.AppendLine("</style>");

			// write body
			Builder.AppendLine("<body>");
			Builder.AppendLine("<table id='mainTable'>");
			string RowTag = "th";
			foreach (PlatformReportRow Row in Report.Rows)
			{
				Builder.AppendLine("<tr>");
				foreach (string Column in Row.Columns)
				{
					Builder.Append($"<{RowTag}>");
					Builder.Append(Column);
					Builder.AppendLine($"</{RowTag}>");
				}
				Builder.AppendLine("</tr>");
				RowTag = "td";
			}
			Builder.AppendLine("</table>");
			Builder.AppendLine("</body>");
			Builder.AppendLine("</html>");

			// save html
			System.IO.File.WriteAllText(HTMLFile, Builder.ToString());
			Console.WriteLine($"Report written to {HTMLFile}");
		}
	}
}
