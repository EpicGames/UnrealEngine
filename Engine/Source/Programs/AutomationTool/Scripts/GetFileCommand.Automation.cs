// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.IO;
using System.Threading;
using System.Reflection;
using AutomationTool;
using UnrealBuildTool;
using EpicGames.Core;
using Microsoft.Extensions.Logging;

namespace AutomationScripts
{
	public partial class Project : CommandUtils
	{
		public static void GetFile(ProjectParams Params)
		{
			Params.ValidateAndLog();
			if (string.IsNullOrEmpty(Params.GetFile))
			{
				return;
			}

			Logger.LogInformation("********** GETFILE COMMAND STARTED **********");
			var StartTime = DateTime.UtcNow;

			var FileName = Path.GetFileName(Params.GetFile);
			var LocalFile = CombinePaths(CmdEnv.EngineSavedFolder, FileName);

			var SC = CreateDeploymentContext(Params, false);
			if (SC.Count == 0)
			{
				throw new AutomationException("Failed to create deployment context");
			}

			SC[0].StageTargetPlatform.GetTargetFile(Params.GetFile, LocalFile, Params);

			Logger.LogInformation("GetFile command time: {0:0.00} s", (DateTime.UtcNow - StartTime).TotalMilliseconds / 1000);
			Logger.LogInformation("********** GETFILE COMMAND COMPLETED **********");
		}
	}
}
