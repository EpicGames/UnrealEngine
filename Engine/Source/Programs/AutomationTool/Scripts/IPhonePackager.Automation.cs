// Copyright Epic Games, Inc. All Rights Reserved.
using System;
using System.Collections.Generic;
using System.Text;
using System.IO;
using AutomationTool;
using UnrealBuildTool;
using EpicGames.Core;
using System.Runtime.ConstrainedExecution;
using Microsoft.Extensions.Logging;

[Help("UAT command to call into the integrated IPhonePackager code")]
class IPhonePackager : BuildCommand
{
	public override void ExecuteBuild()
	{

		string ProjectFile = ParseParamValue("project");
		string BundleIdentifier = ParseParamValue("bundleid");
		bool bIsTVOS = ParseParam("tvos");

		CodeSigningConfig.Initialize(new FileReference(ProjectFile), bIsTVOS:bIsTVOS);
		List<string> Certs = AppleCodeSign.FindCertificates();
		List<FileReference> Provisions = AppleCodeSign.FindProvisions(BundleIdentifier, bForDistribution:false, out _);


		Logger.LogInformation("Certs:");
		foreach (string Cert in Certs)
		{
			Logger.LogInformation($"  {Cert}");
		}
		Logger.LogInformation("Provisions:");
		foreach (FileReference Provision in Provisions)
		{
			Logger.LogInformation($"  {Provision}");
		}


		FileReference MobileProvisionFile;
		string SigningCertificate;
		if (AppleCodeSign.FindCertAndProvision(BundleIdentifier, out MobileProvisionFile, out SigningCertificate))
		{
			Logger.LogInformation($"Matched: {MobileProvisionFile}- {SigningCertificate}");
		}


		Platform IOS = Platform.GetPlatform(UnrealTargetPlatform.IOS);
		string Command = ParseParamValue("cmd", "");

		// check the return value
		int ReturnValue = IOS.RunCommand("IPP:" + Command);
		if (ReturnValue != 0)
		{
			throw new AutomationException("Internal IPP returned {0}", ReturnValue);
		}
	}
}
