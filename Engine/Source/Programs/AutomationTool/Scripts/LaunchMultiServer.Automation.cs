// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.IO;
using UnrealBuildBase;
using UnrealBuildTool;
using EpicGames.Core;
using System.Linq;
using Microsoft.Extensions.Logging;
using System.Collections.Generic;
using System.Diagnostics;
using System.Runtime.InteropServices;

using Microsoft.VisualStudio.OLE.Interop;
using System.Runtime.Versioning;
using System.Security.Cryptography;
using System.Text;

namespace AutomationTool
{
	class ProcessDebugger
	{
		[SupportedOSPlatform("windows")]
		[DllImport("ole32.dll")]
		public static extern int CreateBindCtx(int reserved, out IBindCtx ppbc);

		[SupportedOSPlatform("windows")]
		[DllImport("ole32.dll")]
		public static extern int GetRunningObjectTable(int reserved, out IRunningObjectTable prot);

		public static void DebugProcess(int ProcessID, BuildCommand Command, ProjectParams Params)
		{
			if (OperatingSystem.IsWindows())
			{
				if (CommandUtils.ParseParam(Command.Params, "Rider"))
				{
					bool bUsesUProject = CommandUtils.ParseParam(Command.Params,"RiderUProject");
					DebugProcessWithRider(ProcessID, Params.RawProjectPath, bUsesUProject);
				}
				else
				{
					DebugProcessWithVisualStudio(ProcessID);
				}
			}
		}

		public static void DebugProcessWithRider(int ProcessId, FileReference ProjectPath, bool bUsesUProject)
		{
			// Rider allows to attach to any process using commandline arguments. If you already have a rider instance opened with the provided project path, that
			// running instance will be used instead of starting a new one.

			string FinalSolutionOrProjectPath = bUsesUProject ? ProjectPath.ToString() : null;
			if (!bUsesUProject)
			{
				DirectoryReference CurrentDirectory = ProjectPath.Directory;
				do
				{
					string PathToEvaluate = CurrentDirectory + "\\UE5.sln";
					if (File.Exists(PathToEvaluate))
					{
						FinalSolutionOrProjectPath = PathToEvaluate;
						break;
					}

					CurrentDirectory = CurrentDirectory.ParentDirectory;
				} 
				while(CurrentDirectory != null && !CurrentDirectory.IsRootDirectory());
			}

			if (FinalSolutionOrProjectPath == null)
			{
				Log.Logger.LogInformation($"Failed to find a project solution file path. We cannot attach using Rider.");
				return;
			}

			string RiderPath = Environment.GetEnvironmentVariable("RIDERINSTALLDIR", EnvironmentVariableTarget.Machine);

			if (RiderPath == null)
			{
				Log.Logger.LogError($"Failed to find Rider's binary path. Is Rider executable location added to the RIDERINSTALLDIR system environment variable?.");
				return;
			}

			var RiderProcess = new Process();
			
			RiderProcess.StartInfo.FileName = RiderPath + "/Rider64.exe";
			RiderProcess.StartInfo.UseShellExecute = false;
			RiderProcess.StartInfo.Arguments = $"attach-to-process Native {ProcessId} {FinalSolutionOrProjectPath}";
			if (!RiderProcess.Start())
			{
				Log.Logger.LogError($"Failed to start or connect to Rider. We cannot attach to the selected process.");
			}
		}
		
		public static void DebugProcessWithVisualStudio(int ProcessID)
		{
			if (OperatingSystem.IsWindows())
			{
				EnvDTE._DTE visualStudioInstance = GetVisualStudioInstance();

				if (visualStudioInstance != null)
				{
					AttachVisualStudioToPID(visualStudioInstance, ProcessID);
				}
				else
				{
					Log.Logger.LogInformation($"Failed to find a Visual Studio Instance.");
				}
			}
		}

		[SupportedOSPlatform("windows")]
		private static EnvDTE._DTE GetVisualStudioInstance()
		{
			EnvDTE._DTE visualStudioInstance = null;

			uint numFetched = 0;
			IRunningObjectTable runningObjectTable = null;
			IEnumMoniker monikerEnumerator;
			IMoniker[] monikers = new IMoniker[1];

			GetRunningObjectTable(0, out runningObjectTable);
			runningObjectTable.EnumRunning(out monikerEnumerator);
			monikerEnumerator.Reset();

			while (monikerEnumerator.Next(1, monikers, out numFetched) == 0)
			{
				IBindCtx ctx;
				CreateBindCtx(0, out ctx);

				string runningObjectName;
				monikers[0].GetDisplayName(ctx, null, out runningObjectName);

				object runningObjectVal;
				runningObjectTable.GetObject(monikers[0], out runningObjectVal);

				if (!(runningObjectVal is EnvDTE._DTE) || !runningObjectName.StartsWith("!VisualStudio"))
				{
					continue;
				}

				return (EnvDTE._DTE)runningObjectVal;
			}

			return visualStudioInstance;
		}

		[SupportedOSPlatform("windows")]
		private static void AttachVisualStudioToPID(EnvDTE._DTE visualStudioInstance, int processID)
		{
			int retryCount = 0;
			while (true)
			{
				try
				{
					var processToAttachTo = visualStudioInstance.Debugger.LocalProcesses.Cast<EnvDTE.Process>().FirstOrDefault(process => process.ProcessID == processID);

					if (processToAttachTo == null)
					{
						Log.Logger.LogInformation("Failed to find running Process matching provided Process Name {0}", processID);
						continue;
					}
					else
					{
						processToAttachTo.Attach();
					}

					break;
				}
				catch (COMException e)
				{
					if ((uint)e.ErrorCode == 0x8001010a || (uint)e.ErrorCode == 0x80010001)
					{
						if (++retryCount < 15)
						{
							Log.Logger.LogInformation("Attach Debugger - Got RPC Retry Later exception. Will try again ");
							System.Threading.Thread.Sleep(20);
							continue;
						}
					}
					Log.Logger.LogInformation("Failed to attach debugger. COMException was thrown: " + e.ToString());
					break;
				}
				catch (Exception e)
				{
					Log.Logger.LogInformation("Failed to attach debugger. Exception was thrown: " + e.ToString());
					break;
				}
			}
		}

		[SupportedOSPlatform("windows")]
		private static bool IsDebuggerAttached(EnvDTE._DTE VisualStudio, string processID)
		{
			bool DebuggerAttached = false;

			if (VisualStudio.Debugger.DebuggedProcesses.Count != 0)
			{
				foreach (EnvDTE.Process debuggedProcess in VisualStudio.Debugger.DebuggedProcesses)
				{
					if (debuggedProcess.Name.Contains(processID))
					{
						DebuggerAttached = true;
						break;
					}
				}
			}
			return DebuggerAttached;
		}

		[SupportedOSPlatform("windows")]
		private static bool IsDebuggerAttachedToPID(EnvDTE._DTE VisualStudio, int processID)
		{
			bool DebuggerAttached = false;

			if (VisualStudio.Debugger.DebuggedProcesses.Count != 0)
			{
				foreach (EnvDTE.Process debuggedProcess in VisualStudio.Debugger.DebuggedProcesses)
				{
					if (debuggedProcess.ProcessID == processID)
					{
						DebuggerAttached = true;
						break;
					}
				}
			}
			return DebuggerAttached;
		}
	};

	[Help(@"Launches multiple server processes for a project using the MultiServerReplication plugin.

Example running 2 hosts locally with a single proxy server & client:
	-cook -notimeouts -numservers=2 -client=proxy -proxyclientcount=1 -proxycycleprimary

Example running 4 clients locally connecting to single proxy server w/ a non-local cluster of 3:
	-cook -notimeouts -client=proxy -proxyclientcount=4 -proxycycleprimary -nonlocalservers=172.27.63.109:9001,172.27.63.109:9002,172.27.63.109:9003
	")]
	[Help("Project=<project>", "Project to open. Will search current path and paths in ueprojectdirs. Defaults to current project (in uShell).")]
	[Help("Map=<MapName>", "Map to load on startup.")]
	public class LaunchMultiServer : BuildCommand, IProjectParamsHelpers
	{
		// We will use a convention where ServerId 1 will start at BaseGameListenPort + 1;
		public int BaseGameListenPort = 9000;
		public int ProxyServerBasePort = -1;

		// Stats port counter: each process gets a unique -statsPort starting at this value
		private int NextStatsPort = 10100;

		[CommandLine, Help("CustomConfig=<Section>", "Read a different section of the Project's GameConfig for default values")]
		public string CustomConfig = "";

		[CommandLine, Help("NumServers=##", "Set the fleet size of servers (default: 2)")]
		public int NumServers = -1;

		[CommandLine(ListSeparator = ','), Help("NonLocalServers=Address1:Port1,Address2:Port2,etc.", "Specify servers that are not hosted locally but belong to the same fleet")]
		public List<string> NonLocalServers = new List<string>();

		[CommandLine("-Client", ListSeparator = ','), Help("Client=<Type>", "Specify the client type to connect with (proxy or direct or proxy,direct)")]
		public List<string> ClientType = new List<string>();

		[CommandLine, Help("DirectClientCount=##", "Specify how many clients to launch in client=direct mode.  If 0 (default), will launch one for each server.")]
		public int DirectClientCount = 0;

		[CommandLine, Help("BotCount=##", "Specify how many client bots to launch. Client bots are launched with -bot it is up to the game to specify the implementation.")]
		public int BotCount = 0;

		[CommandLine, Help("ProxyServerCount=##", "Specify how many proxy servers to launch in client=proxy mode")]
		public int ProxyServerCount = 1;

		[CommandLine, Help("ProxyClientCount=##", "Specify how many clients to launch PER SERVER in client=proxy mode")]
		public int ProxyClientCount = 1;

		[CommandLine, Help("ProxyBotCount=##", "Specify how many bots to launch PER SERVER in client=proxy mode. Prefer -BotCount=XX.")]
		public int ProxyBotCount = 0;

		[CommandLine, Help("ProxyClientPrimary=##", "Specify which game server to prefer as the initial proxy connection, as an integer or the string \"random\"")]
		public string ProxyClientPrimary = "";

		[CommandLine, Help("ProxyCyclePrimary", "Specify that each subsequent proxy client should choose a different server")]
		public bool ProxyCyclePrimary = false;

		[CommandLine, Help("NoTimeouts", "Disable timeouts (recommended for debugging)")]
		public bool NoTimeouts = false;

		[CommandLine, Help("NoConsole", "Disable launching a console (not recommended, but may improve performance)")]
		public bool NoConsole = false;

		[CommandLine, Help("NoNewConsole", "Disable launching the new style console")]
		public bool NoNewConsole = false;

		[CommandLine, Help("AttachToServers", "Force Visual Studio to attach to the servers when launched (requires VS be launched or it will hang)")]
		public bool AttachToServers = false;

		[CommandLine, Help("AttachToProxies", "Force Visual Studio to attach to the proxies when launched (requires VS be launched or it will hang)")]
		public bool AttachToProxies = false;

		[CommandLine, Help("AttachToClients", "Force Visual Studio to attach to the clients when launched (requires VS be launched or it will hang)")]
		public bool AttachToClients = false;

		[CommandLine, Help("CommonArgs", "Specify arguments that are run by all processes")]
		public string CommonArgs = "";

		[CommandLine, Help("EnableStatsPort", "Enable the -statsPort argument on all launched processes")]
		public bool EnableStatsPort = false;

		[CommandLine, Help("Matchmake", "Project will use a matchmaking service to connect servers instead of passing the -MultiServerPeers and -ProxyGameServers options to servers.")]
		public bool Matchmake = false;

		// We should only parse the commandline arguments once, since doing so multiple times will add entries to the List<> args.
		bool bParsedCommandLineArguments = false;

		public string GenerateRandomSessionString(int Length)
		{
			const string Chars = "ABCDEFGHIJKLMNOPQRSTUVWXYZabcdefghijklmnopqrstuvwxyz0123456789";
			string RandString = RandomNumberGenerator.GetString(Chars, Length);
			return RandString;
		}

		protected enum ProcessLaunchType
		{
			Server,
			DirectClient,
			ProxyServer,
			ProxyClient,
			BotClient,
		}

		protected virtual void ParseCommandLineArguments()
		{
			if (!bParsedCommandLineArguments)
			{
				// We need to prepend the '-' back on to all of the Parameters for ParseArguments to work
				UnrealBuildTool.CommandLine.ParseArguments(Params.Select(x => $"-{x}"), this);
				bParsedCommandLineArguments = true;
			}
		}

		/// <summary>
		/// The entry point for the build command
		/// </summary>
		public override ExitCode Execute()
		{
			Logger.LogInformation("********** RUN MULTISERVER COMMAND STARTED **********");
			var StartTime = DateTime.UtcNow;

			ParseCommandLineArguments();
			ReadConfiguration();
			int TotalNumServers = NumServers;

			// CommonArgs are used for all instances spawned
			CommonArgs += (CustomConfig?.Length > 0) ? $" -CustomConfig={CustomConfig}" : "";

			// These are servers not on our local host
			string[] NonLocalServerAddresses = NonLocalServers.ToArray();
			if (NonLocalServerAddresses.Length > TotalNumServers)
			{
				Logger.LogWarning($"Specified more NonLocalServers=({NonLocalServerAddresses.Length}) than NumServers={TotalNumServers}. Inferring that we want to run with no local servers and only non-local servers. Setting NumServers={NonLocalServerAddresses.Length}.");
				TotalNumServers = NonLocalServerAddresses.Length;
			}
			int NumLocalServers = TotalNumServers - NonLocalServerAddresses.Length;

			// Start-up the servers first
			Logger.LogInformation($"Launching {NumLocalServers} Local Servers for {TotalNumServers} Total Servers");
			string[] LocalGameServerAddresses = StartProcessesForDedicatedServers(NumLocalServers, NonLocalServerAddresses, CommonArgs);

			// Generate all of the GameServerAddresses by merging the local & non-local servers
			string[] AllGameServerAddresses = LocalGameServerAddresses.Concat(NonLocalServerAddresses).ToArray();

			// Now start-up the clients
			foreach (string ClientValue in ClientType)
			{
				if (ClientValue.ToLower() == "direct")
				{
					int MaxDirectClientCount = 0;
					foreach (var GameServerAddressWithPortRange in AllGameServerAddresses)
					{
						string GameServerBaseAddress;
						int StartPort, EndPort;
						bool bValidAddress = ParseAddressAndPortRange(out GameServerBaseAddress, out StartPort, out EndPort, GameServerAddressWithPortRange);
						if (bValidAddress)
						{
							MaxDirectClientCount += (EndPort - StartPort) + 1;
						}
					}

					if (DirectClientCount != 0)
					{
						if (DirectClientCount > MaxDirectClientCount)
						{
							Logger.LogWarning($"Specified DirectClientCount=({DirectClientCount}) but only a maximum of {MaxDirectClientCount} addresses to connect to.");
							DirectClientCount = MaxDirectClientCount;
						}
					}
					else
					{
						DirectClientCount = MaxDirectClientCount;
					}
                    Logger.LogInformation($"Starting {DirectClientCount} direct client instances connecting to MultiServer instances");
					StartProcessesForClients(ProcessLaunchType.DirectClient, AllGameServerAddresses, CommonArgs, 1, DirectClientCount);

					if (BotCount > 0)
					{
						// NumBotsPerServer = Ceil(BotCount / ServerCount);
						int ServerCount = CountServers(AllGameServerAddresses);
						int NumBotsPerServer = (BotCount + ServerCount - 1) / ServerCount;

						Logger.LogInformation($"Starting {BotCount} bots connecting to {string.Join(',', AllGameServerAddresses)}. Roughly {NumBotsPerServer} per server.");
						StartProcessesForClients(ProcessLaunchType.BotClient, AllGameServerAddresses, $"{CommonArgs} -nullrhi -bot -nosound -unattended -nosplash", NumBotsPerServer, BotCount);
					}
				}
				else if (ClientValue.ToLower() == "proxy")
				{
					Logger.LogInformation($"Starting {ProxyServerCount} Proxy Servers each with {ProxyClientCount} Clients");

					string[] LocalProxyAddresses = GenerateLocalServerAddressRange(ProxyServerCount, ProxyServerBasePort);
					StartProcessesForProxyServer(LocalProxyAddresses, AllGameServerAddresses, CommonArgs);
					StartProcessesForClients(ProcessLaunchType.ProxyClient, LocalProxyAddresses, CommonArgs, ProxyClientCount);

					if (ProxyBotCount > 0)
					{
						Logger.LogInformation($"Starting {ProxyBotCount} bots for each proxy address: {string.Join(',', LocalProxyAddresses)}.");
						StartProcessesForClients(ProcessLaunchType.BotClient, LocalProxyAddresses, $"{CommonArgs} -nullrhi -bot -nosound -unattended -nosplash", ProxyBotCount);
					}

					if (BotCount > 0)
					{
						// NumBotsPerProxy = Ceil(BotCount / ProxyServerCount);
						int NumBotsPerProxy = (BotCount + ProxyServerCount - 1) / ProxyServerCount;
						Logger.LogInformation($"Starting {BotCount} bots connecting to {string.Join(',', LocalProxyAddresses)}. Roughly {NumBotsPerProxy} per proxy.");
						StartProcessesForClients(ProcessLaunchType.BotClient, LocalProxyAddresses, $"{CommonArgs} -nullrhi -bot -nosound -unattended -nosplash", NumBotsPerProxy, BotCount);
					}
				}
				else
				{
					Logger.LogWarning($"Unknown client type specified: {ClientValue}");
				}
			}

			Logger.LogInformation("Run command time: {0:0.00} s", (DateTime.UtcNow - StartTime).TotalMilliseconds / 1000);
			Logger.LogInformation("********** RUN MULTISERVER COMMAND COMPLETED **********");

			return ExitCode.Success;
		}

		/// <summary>
		/// Reads the configuration from the CustomConfig specified.  If no CustomConfig was specified, then rely solely on the command-line parameters.
		/// The command-line parameters should always take precedence.
		/// </summary>
		protected void ReadConfiguration()
		{
			// Parse server configuration from ini files
			ConfigHierarchy ProjectGameConfig = ConfigCache.ReadHierarchy(ConfigHierarchyType.Game, DirectoryReference.FromFile(ProjectPath), UnrealTargetPlatform.Win64, CustomConfig);

			const String ServerDefConfigSection = "/Script/MultiServerConfiguration.MultiServerSettings";
			const String ProxyConfigSection = "/Script/MultiServerConfiguration.Proxy";

			if (ProxyServerBasePort > 0)
			{
				Logger.LogInformation($"Using specified ProxyServerBasePort {ProxyServerBasePort}");
			}
			else if (ProjectGameConfig.TryGetValue(ProxyConfigSection, "ListenPort", out ProxyServerBasePort))
			{
				Logger.LogInformation($"Found {ProxyConfigSection}.ListenPort setting ProxyServerBasePort to {ProxyServerBasePort}");
			}
			else
			{
				ProxyServerBasePort = BaseGameListenPort + 2000;
				Logger.LogWarning($"Could not find {ProxyConfigSection}.ListenPort in Game Config File {ProjectGameConfig} Defaulting ProxyServerBasePort to {ProxyServerBasePort}");
			}

			int TotalNumServers = NumServers;
			if (TotalNumServers < 0)
			{
				if (!ProjectGameConfig.TryGetValue(ServerDefConfigSection, "TotalNumServers", out TotalNumServers))
				{
					TotalNumServers = 2;
					Logger.LogWarning($"Could not find {ServerDefConfigSection}.TotalNumServers in Game Config File {ProjectGameConfig} Defaulting to {TotalNumServers}");
				}
			}
			NumServers = TotalNumServers;
		}

		/// <summary>
		/// Given the number of local servers and the base port, generate the array of address:port for that range of ports
		/// </summary>
		/// <param name="NumLocalServers">The number of servers in the localhost range</param>
		/// <param name="BasePort">The port to start enumerating at</param>
		/// <returns>The address range of servers: ["127.0.0.1:BasePort+1", ..., "127.0.0.1::BasePort+NumLocalServers"]</returns>
		string[] GenerateLocalServerAddressRange(int NumLocalServers, int BasePort)
		{
			string[] LocalServers = new string[NumLocalServers];
			for (int i = 0; i < NumLocalServers; ++i)
			{
				// It's important to note that Unreal does not handle "localhost" properly, it needs to be the loopback address
				LocalServers[i] = $"127.0.0.1:{BasePort + i + 1}";
			}

			return LocalServers;
		}

		/// <summary>
		///	Starts all of the server processes that should execute on the local host.
		/// </summary>
		/// <param name="NumLocalServers">The number of local servers to spin-up as part of the total server cluster(the total cluster consistent of these local servers + NonLocalServers).</param>
		/// <param name="NonLocalServers">Specify servers that are not running on the local machine but contribute to the Metaverse</param>
		/// <param name="AdditionalServerArguments">These are extra parameters that the server processes should put on the command-line(shared between all servers)</param>
		/// <returns>The Addresses of all of the Game Servers spun-up.</returns>
		virtual protected string[] StartProcessesForDedicatedServers(int NumLocalServers, string[] NonLocalServers, string AdditionalServerArguments)
		{
			if (NumLocalServers < 1)
			{
				return new string[] { };
			}

			int NumTotalServers = NumLocalServers + NonLocalServers.Length;
			string[] LocalGameServerAddresses = new string[1];

			// This is optional to the multi-server process, these are in addition to the MultiServerNumPeers argument
			string ServerCommonArgs = $" -server {AdditionalServerArguments} -NODEBUGOUTPUT";
			ServerCommonArgs += NoTimeouts ? " -notimeouts" : "";
			ServerCommonArgs += IsAttachingDebugger(ProcessLaunchType.Server) ? " -WaitForDebuggerNoBreak" : "";
			ServerCommonArgs += $" -MultiServerNumServers={NumTotalServers}";

			if (!Matchmake)
			{
				ServerCommonArgs += (NonLocalServers.Length > 0) ? $" -MultiServerPeers={string.Join(',', NonLocalServers)}" : "";
				ServerCommonArgs += " -MultiServerLocalHost";
			}

			for (int ServerId = 1; ServerId <= NumLocalServers; ++ServerId)
			{
				int ServerGamePort = BaseGameListenPort + ServerId;

				Logger.LogInformation("Starting Dedicated MultiServer for Game Port {0}", ServerGamePort);

				string ServerArgs = ServerCommonArgs.Replace("{ServerId}", ServerId.ToString(), StringComparison.OrdinalIgnoreCase);
				ServerArgs += $" -port={ServerGamePort} -log={ProjectFullPath.GetFileNameWithoutAnyExtensions()}-Server-{ServerId}.log";

				string AppTitle = $"Server ID {ServerId}";
				ExecuteApp(ProcessLaunchType.Server, AppTitle, ServerArgs);
			}

			// If there's a range of addresses, give us a range
			if (NumLocalServers > 1)
				LocalGameServerAddresses[0] = $"127.0.0.1:{BaseGameListenPort + 1}-{BaseGameListenPort + NumLocalServers}";
			else
				LocalGameServerAddresses[0] = $"127.0.0.1:{BaseGameListenPort + 1}";

			return LocalGameServerAddresses;
		}

        /// <summary>
        /// Start the Client processes
        /// </summary>
        /// <param name="ClientType">What type of Client we're launching which will help set the titles properly</param>
        /// <param name="GameServerAddresses">The game servers (or proxies) that a client will connect to.  The addresses may contain a port range (e.g. IpAddress:StartPort-EndPort).</param>
        /// <param name="AdditionalClientArguments">What additional parameters we launch the client with</param>
        /// <param name="ClientsPerServer">The number of clients to launch per-GameServerAddress entry</param>
        /// <param name="MaxClientCount">The maximum number of clients to launch.  Zero for no limit (one for each server address or proxy server port)</param>
        virtual protected void StartProcessesForClients(ProcessLaunchType ClientType, string[] GameServerAddresses, string AdditionalClientArguments, int ClientsPerServer = 1, int MaxClientCount = 0)
		{
			int TotalClientCount = 0;
			for (int ServerAddressIdx= 0; ServerAddressIdx < GameServerAddresses.Length; ++ServerAddressIdx)
			{
				string GameServerAddressWithPortRange = GameServerAddresses[ServerAddressIdx];
				string GameServerBaseAddress;
				int StartPort, EndPort;

				bool bValidAddress = ParseAddressAndPortRange(out GameServerBaseAddress, out StartPort, out EndPort, GameServerAddressWithPortRange);
				if (!bValidAddress)
				{
					Logger.LogError("Invalid Game Server Address specified {GameServerAddress} should be in the format of IpAddress:Port or IpAddress:StartPort-EndPort", GameServerAddressWithPortRange);
					return;
				}

				for (int Port = StartPort; Port <= EndPort; ++Port)
				{
					string ClientArgs = $" -game {AdditionalClientArguments}";
					ClientArgs += NoTimeouts ? " -notimeouts" : "";
					ClientArgs += $" {GameServerBaseAddress}:{Port}";
					ClientArgs += $" -windowed";

					// Potentially launch multiple clients per server address
					for (int ClientInstanceNum = 0; ClientInstanceNum < ClientsPerServer; ++ClientInstanceNum)
					{
						int ClientId = (ServerAddressIdx + 1 + ClientInstanceNum);

						++TotalClientCount;
						string InstanceArgs = ClientArgs + $" -log={ProjectFullPath.GetFileNameWithoutAnyExtensions()}-{ClientType}-{TotalClientCount}.log";
						InstanceArgs += $" -SaveWinPos={TotalClientCount}";

						string InstanceDetails = (ClientsPerServer > 1) ? $"(Srv {ServerAddressIdx + 1})" : string.Empty;
						string AppTitle = $"{ClientType} {ClientId} {InstanceDetails}";
						ExecuteApp(ClientType, AppTitle, InstanceArgs);

						if (MaxClientCount != 0 && TotalClientCount >= MaxClientCount)
						{
							return;
						}
					}
				}
			}
		}

		virtual protected void StartProcessesForProxyServer(string[] LocalProxyServerAddresses, string[] AllGameServerAddresses, string AdditionalProxyArguments)
		{
			string CommonProxyServerArgs = AdditionalProxyArguments;
			CommonProxyServerArgs += " -ini:Engine:[ConsoleVariables]:net.proxy.Enabled=1";
			CommonProxyServerArgs += NoTimeouts ? " -notimeouts" : "";
			CommonProxyServerArgs += IsAttachingDebugger(ProcessLaunchType.ProxyServer) ? " -WaitForDebuggerNoBreak" : "";
			CommonProxyServerArgs += " -port={ProxyServerPort}"; // will be Replaced() below.

			if (!Matchmake && AllGameServerAddresses.Length > 0)
			{
				CommonProxyServerArgs += $" -ProxyGameServers={string.Join(',', AllGameServerAddresses).TrimEnd(',')}";
			}

			if (ProxyClientPrimary.Length > 0)
			{
				CommonProxyServerArgs += $" -ProxyClientPrimaryGameServer={ProxyClientPrimary}";
			}

			if (ProxyCyclePrimary)
			{
				CommonProxyServerArgs += " -ProxyCyclePrimaryGameServer";
			}

			for (int ProxyIdx = 1; ProxyIdx <= LocalProxyServerAddresses.Length; ++ProxyIdx)
			{
				// Split off the port
				string LocalProxyServerAddress = LocalProxyServerAddresses[ProxyIdx - 1];
				string[] AddressParts = LocalProxyServerAddress.Split(':');

				// Parse the port to use for the -port argument
				int ProxyServerPort;
				if (!int.TryParse(AddressParts.LastOrDefault(), out ProxyServerPort))
				{
					Logger.LogError($"Error Starting LocalProxyServer with Address {LocalProxyServerAddress} had invalid Port Specifier: {AddressParts.LastOrDefault()}");
					continue;
				}

				string ProxyServerArgs = CommonProxyServerArgs
					.Replace("{ProxyId}", ProxyIdx.ToString(), StringComparison.OrdinalIgnoreCase)
					.Replace("{ProxyServerPort}", ProxyServerPort.ToString(), StringComparison.OrdinalIgnoreCase);

				ProxyServerArgs += $" -log={ProjectFullPath.GetFileNameWithoutAnyExtensions()}-Proxy-{ProxyIdx}.log";

				string AppTitle = $"ProxyServer {ProxyIdx}";
				ExecuteApp(ProcessLaunchType.ProxyServer, AppTitle, ProxyServerArgs);
			}
		}


		/// <summary>
		/// Execute the specified LaunchType app with a particular window AppTitle and arguments
		/// </summary>
		/// <param name="LaunchType">The type of app to launch</param>
		/// <param name="AppTitle">The title of the window (we will try to name the console windows & game windows this)</param>
		/// <param name="AdditionalArguments">Any additional arguments to pass on the command-line</param>
		/// <exception cref="AutomationException"></exception>
		protected void ExecuteApp(ProcessLaunchType LaunchType, string AppTitle, string AdditionalArguments)
		{
			const int PauseBetweenProcessMS = 100;

			var Params = new ProjectParams
			(
				Command: this,
				RawProjectPath: ProjectPath,
				DedicatedServer: LaunchType == ProcessLaunchType.Server || LaunchType == ProcessLaunchType.ProxyServer,
				Client: LaunchType == ProcessLaunchType.DirectClient || LaunchType == ProcessLaunchType.ProxyClient || LaunchType == ProcessLaunchType.BotClient
			);

			var DeployContexts = AutomationScripts.Project.CreateDeploymentContext(Params, Params.DedicatedServer);
			if (DeployContexts.Count == 0)
			{
				throw new AutomationException("No DeployContexts for launching a process.");
			}

			var DeployContext = DeployContexts[0];

			var App = CombinePaths(CmdEnv.LocalRoot, "Engine/Binaries/Win64/UnrealEditor.exe");
			if (Params.Cook)
			{
				List<FileReference> Exes = DeployContext.StageTargetPlatform.GetExecutableNames(DeployContext);
				App = Exes[0].FullName;
			}

			string Args = Params.Client ? Params.ClientCommandline : $"{Params.MapToRun} {Params.ServerCommandline}";
			Args += " -messaging";
			Args += (Params.Cook ? "" : " " + DeployContext.ProjectArgForCommandLines);

			// Use this option when logging with VeryVerbose to avoid unusable debugging windows (too much spew)
			// The logging will still go to the output files (and you can use a text editor like Notepad++ to auto-reload them)
			Args += NoConsole ? " -nodebugoutput -noconsole" : " -log";
			Args += NoNewConsole ? "" : " -newconsole ";

			Args += String.Format(" -ConsoleTitle=\"{0} {1}\" -SessionName=\"{0} {1}\" ", Params.ShortProjectName, AppTitle);
			Args += AdditionalArguments;
			if (EnableStatsPort)
			{
				Args += $" -statsPort={NextStatsPort}";
				NextStatsPort++;
			}
			Args = Args.Trim();

			PushDir(Path.GetDirectoryName(App));
			try
			{
				var NewProcess = Run(App, Args, null, ERunOptions.Default | ERunOptions.NoWaitForExit | ERunOptions.NoStdOutRedirect);
				if (NewProcess != null)
				{
					// Remove started process so it won't be killed on UAT exit.
					// Essentially forces the -NoKill command-line option behavior for these.
					ProcessManager.RemoveProcess(NewProcess);
				}

				// Pause between starting processes to enforce startup determinism.
				System.Threading.Thread.Sleep(PauseBetweenProcessMS);

				if (IsAttachingDebugger(LaunchType))
				{
					ProcessDebugger.DebugProcess(NewProcess.ProcessObject.Id, this, Params);
				}
			}
			catch
			{
				throw;
			}
			finally
			{
				PopDir();
			}
		}

		/// <summary>
		/// Helper function allows us to break-out a potential input of "address", "address:port", or "address:startPort-endPort".
		/// </summary>
		/// <param name="IpAddress">The base ip address of the passed-in addressToParse.</param>
		/// <param name="StartPort">The start port of the port range in addressToParse. If no port specified, zero.</param>
		/// <param name="EndPort">The end port of the port range in addressToParse. If none specified, return startPort.</param>
		/// <param name="AddressToParse">The input address to parse of the form "address", "address:port", or "address:startPort-endPort"</param>
		/// <returns>True if a successful parse is made.  False if it is somehow malformed.</returns>
		protected bool ParseAddressAndPortRange(out string IpAddress, out int StartPort, out int EndPort, string AddressToParse)
		{
			IpAddress = string.Empty;
			StartPort = 0;
			EndPort = 0;

			if (string.IsNullOrEmpty(AddressToParse))
				return false;

			string[] AddressParts = AddressToParse.Split(':');
			IpAddress = AddressParts[0];

			// See if we've also specified a port (if not we're done, ports stay at zero)
			if (AddressParts.Length < 2)
				return true;

			// Check for a port range (e.g. :9001-9005) or a single port (:9001)
			string[] PortRange = AddressParts[1].Split('-');
			if (!int.TryParse(PortRange[0], out StartPort))
				return false;

			// If we specified a port range (9001-9005) then parse the endPort
			if (PortRange.Length > 1)
			{
				if (!int.TryParse(PortRange[1], out EndPort))
				{
					EndPort = StartPort;
					return false;
				}

				return (EndPort > StartPort);
			}
			else
			{
				EndPort = StartPort;
			}

			return true;
		}

		/// <summary>
		/// Counts the number of servers specified in the passed-in string array. We account for port ranges.
		/// </summary>
		/// <param name="ServerAddresses">The server addresses (IpAddress:StartPort-EndPort)</param>
		/// <returns>The number of servers specified (taking into account port ranges)</returns>
		protected int CountServers(string[] ServerAddresses)
		{
			int TotalCount = 0;
			foreach (string AddressToParse in ServerAddresses)
			{
				string BaseAddress;
				int StartPort, EndPort;
				ParseAddressAndPortRange(out BaseAddress, out StartPort, out EndPort, AddressToParse);

				TotalCount += EndPort - StartPort + 1;
			}

			return TotalCount;
		}

		/// <summary>
		/// Are we requesting a debugger be attached to this particular process?
		/// </summary>
		/// <param name="LaunchType">The type of process we plan to launch</param>
		/// <returns>true if the commandlet parameters have specified we attach a debugger to this process</returns>
		protected bool IsAttachingDebugger(ProcessLaunchType LaunchType)
		{
			return (AttachToServers && (LaunchType == ProcessLaunchType.Server)) ||
				 (AttachToProxies && (LaunchType == ProcessLaunchType.ProxyServer)) ||
				(AttachToClients && (LaunchType == ProcessLaunchType.DirectClient || LaunchType == ProcessLaunchType.ProxyClient));
		}

		private FileReference ProjectFullPath;
		public virtual FileReference ProjectPath
		{
			get
			{
				if (ProjectFullPath == null)
				{
					ProjectFullPath = ParseProjectParam();

					if (ProjectFullPath == null)
					{
						throw new AutomationException("No project file specified. Use -project=<project>.");
					}
				}

				return ProjectFullPath;
			}
		}
	}
}
