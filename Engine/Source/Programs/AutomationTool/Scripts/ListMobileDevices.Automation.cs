// Copyright Epic Games, Inc. All Rights Reserved.

using System.Collections.Generic;
using AutomationTool;
using UnrealBuildTool;
using EpicGames.Core;
using Microsoft.Extensions.Logging;

namespace AutomationScripts.Automation
{
	public class ListMobileDevices : BuildCommand
	{
		public override void ExecuteBuild()
		{
			Logger.LogInformation("======= ListMobileDevices - Start =======");

			var GlobalParams = new ProjectParams(
				Command: this,
				RawProjectPath: new FileReference(@"D:\UE\Samples\Games\TappyChicken\TappyChicken.uproject")
				);

			if (ParseParam("android"))
			{
				GetConnectedDevices(GlobalParams, Platform.GetPlatform(UnrealTargetPlatform.Android));
			}

			if (ParseParam("ios"))
			{
				throw new AutomationException("iOS is not yet implemented.");
			}

			Logger.LogInformation("======= ListMobileDevices - Done ========");
		}

		private static void GetConnectedDevices(ProjectParams Params, Platform TargetPlatform)
		{
			var PlatformName = TargetPlatform.PlatformType.ToString();
			List<string> ConnectedDevices;
			TargetPlatform.GetConnectedDevices(Params, out ConnectedDevices);

			try
			{
				foreach (var DeviceName in ConnectedDevices)
				{
					Logger.LogInformation("Device:{PlatformName}:{DeviceName}", PlatformName, DeviceName);
				}
			}
			catch
			{
				throw new AutomationException("No {0} devices", PlatformName);
			}		
		}
	}
}

