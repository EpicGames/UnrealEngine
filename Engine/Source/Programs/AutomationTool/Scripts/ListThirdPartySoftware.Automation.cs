// Copyright Epic Games, Inc. All Rights Reserved.
using System;
using System.Collections.Generic;
using System.Text;
using System.IO;
using AutomationTool;
using UnrealBuildTool;
using System.Linq;
using System.Text.RegularExpressions;
using EpicGames.Core;
using UnrealBuildBase;
using Microsoft.Extensions.Logging;

using static AutomationTool.CommandUtils;

[Help("Lists TPS files associated with any source used to build a specified target(s). Grabs TPS files associated with source modules, content, and engine shaders.")]
[Help("Target", "One or more UBT command lines to enumerate associated TPS files for (eg. UnrealGame Win64 Development).")]
class ListThirdPartySoftware : BuildCommand
{
	public override void ExecuteBuild()
	{
		Logger.LogInformation("************************* List Third Party Software");

		string ProjectPath = ParseParamValue("Project", String.Empty);

		//Add quotes to avoid issues with spaces in project path
		if (ProjectPath != String.Empty)
			ProjectPath = "\"" + ProjectPath + "\"";

		// Parse the list of targets to list TPS for. Each target is specified by -Target="Name|Configuration|Platform" on the command line.
		HashSet<FileReference> TpsFiles = new HashSet<FileReference>();
		foreach(string Target in ParseParamValues(Params, "Target"))
		{
			// Get the path to store the exported JSON target data
			FileReference OutputFile = FileReference.Combine(Unreal.EngineDirectory, "Intermediate", "Build", "ThirdParty.json");

			IProcessResult Result;

			Result = Run(Unreal.DotnetPath.FullName, $"\"{UnrealBuild.UnrealBuildToolDll}\" {Target.Replace('|', ' ')} {ProjectPath} -Mode=JsonExport -OutputFile=\"{OutputFile.FullName}\"", Options: ERunOptions.Default);

			if (Result.ExitCode != 0)
			{
				throw new AutomationException("Failed to run UBT");
			}

			// Read the exported target info back in
			JsonObject Object = JsonObject.Read(OutputFile);

			// Get the project file if there is one
			FileReference ProjectFile = null;
			string ProjectFileName;
			if(Object.TryGetStringField("ProjectFile", out ProjectFileName))
			{
				ProjectFile = new FileReference(ProjectFileName);
			}

			// Get the default paths to search
			HashSet<DirectoryReference> DirectoriesToScan = new HashSet<DirectoryReference>();
			DirectoriesToScan.Add(DirectoryReference.Combine(Unreal.EngineDirectory, "Shaders"));
			DirectoriesToScan.Add(DirectoryReference.Combine(Unreal.EngineDirectory, "Content"));
			if(ProjectFile != null)
			{
				DirectoriesToScan.Add(DirectoryReference.Combine(ProjectFile.Directory, "Content"));
			}

			// Add all the paths for each module, and its runtime dependencies
			JsonObject Modules = Object.GetObjectField("Modules");
			foreach(string ModuleName in Modules.KeyNames)
			{
				JsonObject Module = Modules.GetObjectField(ModuleName);
				DirectoriesToScan.Add(new DirectoryReference(Module.GetStringField("Directory")));
				
				JsonObject[] RuntimeDependencies;
				if (Module.TryGetObjectArrayField("RuntimeDependencies", out RuntimeDependencies))
				{
					foreach (JsonObject RuntimeDependency in RuntimeDependencies)
					{
						string RuntimeDependencyPath;
						if (RuntimeDependency.TryGetStringField("SourcePath", out RuntimeDependencyPath) || RuntimeDependency.TryGetStringField("Path", out RuntimeDependencyPath))
						{
							List<FileReference> Files = FileFilter.ResolveWildcard(DirectoryReference.Combine(Unreal.EngineDirectory, "Source"), RuntimeDependencyPath);
							DirectoriesToScan.UnionWith(Files.Select(x => x.Directory));
						}
					}
				}
			}

			// Remove any directories that are under other directories, and sort the output list
			List<DirectoryReference> SortedDirectoriesToScan = new List<DirectoryReference>();
			foreach(DirectoryReference DirectoryToScan in DirectoriesToScan.OrderBy(x => x.FullName))
			{
				if(SortedDirectoriesToScan.Count == 0 || !DirectoryToScan.IsUnderDirectory(SortedDirectoriesToScan[SortedDirectoriesToScan.Count - 1]))
				{
					SortedDirectoriesToScan.Add(DirectoryToScan);
				}
			}

			// Get the platforms to exclude
			List<UnrealTargetPlatform> SupportedPlatforms = new List<UnrealTargetPlatform> { UnrealTargetPlatform.Parse(Object.GetStringField("Platform")) };
			string[] ExcludePlatformNames = Utils.MakeListOfUnsupportedPlatforms(SupportedPlatforms, bIncludeUnbuildablePlatforms: true, Log.Logger).ToArray();

			// Find all the TPS files under the engine directory which match
			foreach(DirectoryReference DirectoryToScan in SortedDirectoriesToScan)
			{
				foreach(FileReference TpsFile in DirectoryReference.EnumerateFiles(DirectoryToScan, "*.tps", SearchOption.AllDirectories))
				{
					if(!TpsFile.ContainsAnyNames(ExcludePlatformNames, DirectoryToScan))
					{
						TpsFiles.Add(TpsFile);
					}
				}
			}
		}

		// Also add any redirects
		List<string> OutputMessages = new List<string>();
		foreach(FileReference TpsFile in TpsFiles)
		{
			string Message = TpsFile.FullName;
			try
			{
				string[] Lines = FileReference.ReadAllLines(TpsFile);
				foreach(string Line in Lines)
				{
					const string RedirectPrefix = "Redirect:";

					int Idx = Line.IndexOf(RedirectPrefix, StringComparison.InvariantCultureIgnoreCase);
					if(Idx >= 0)
					{
						FileReference RedirectTpsFile = FileReference.Combine(TpsFile.Directory, Line.Substring(Idx + RedirectPrefix.Length).Trim());
						Message = String.Format("{0} (redirect from {1})", RedirectTpsFile.FullName, TpsFile.FullName);
						break;
					}
				}
			}
			catch (Exception Ex)
			{
				ExceptionUtils.AddContext(Ex, "while processing {0}", TpsFile);
				throw;
			}
			OutputMessages.Add(Message);
		}
		OutputMessages.Sort();

		// Print them all out
		foreach(string OutputMessage in OutputMessages)
		{
			Logger.LogInformation("{Text}", OutputMessage);
		}
	}
}
