// Copyright Epic Games, Inc. All Rights Reserved.
using System;
using System.Collections.Generic;
using System.Linq;
using System.Text;
using System.IO;
using System.Security.Cryptography;
using AutomationTool;
using UnrealBuildTool;
using EpicGames.Localization;
using EpicGames.Core;
using System.Threading.Tasks;
using Microsoft.Extensions.Logging;
using System.Diagnostics;
using System.IO.Enumeration;




[Help("Updates the external localization data using the arguments provided.")]
[Help("UEProjectRoot", "Optional root-path to the project we're gathering for (defaults to CmdEnv.LocalRoot if unset).")]
[Help("UEProjectDirectory", "Sub-path to the project we're gathering for (relative to UEProjectRoot).")]
[Help("UEProjectName", "Optional name of the project we're gathering for (should match its .uproject file, eg QAGame).")]
[Help("LocalizationProjectNames", "Comma separated list of the projects to gather text from.")]
[Help("LocalizationBranch", "Optional suffix to use when uploading the new data to the localization provider.")]
[Help("LocalizationProvider", "Optional localization provide override.")]
[Help("LocalizationSteps", "Optional comma separated list of localization steps to perform [Download, Gather, Import, Export, Compile, GenerateReports, Upload] (default is all). Only valid for projects using a modular config.")]
[Help("IncludePlugins", "Optional flag to include plugins from within the given UEProjectDirectory as part of the gather. This may optionally specify a comma separated list of the specific plugins to gather (otherwise all plugins will be gathered).")]
[Help("IncludePluginsDirectory", "Optional parameter that is a list of relative paths to a directory under UEProjectDirectory separated by a ';' character. All plugins under this directory will be gathered from (if not excluded) E.g -IncludePluginsDirectory=\"Plugins/A;Plugins/B;Plugins/C\"")]
[Help("ExcludePlugins", "Optional comma separated list of plugins to exclude from the gather.")]
[Help("ExcludePluginsDirectory", "Optional list of relative paths to a directory under UEProjectDirectory separated by the ';' character. All plugins under this directory will be excluded from gather. E.g -ExcludePluginsDirectory=\"Plugins/A;Plugins/B;Plugins/C\"")]
[Help("EnableIncludedPlugins", "Optional flag that passes all included plugins that aren't excluded to the -EnablePlugins editor argument to ensure content and metadata for plugins are loaded for gathering.")]
[Help("IncludePlatforms", "Optional flag to include platforms from within the given UEProjectDirectory as part of the gather.")]
[Help("AdditionalCommandletArguments", "Optional arguments to pass to the gather process.")]
[Help("ParallelGather", "Run the gather processes for a single batch in parallel rather than sequence.")]
[Help("Preview", "Run the localization command in preview mode. This passes the -Preview flag along to all commandlets as an additional argument and deletes all temporary files generated by the commandlets in preview mode. Primarily used for build farm automation where localization warnings from localization gathers can be previewed without checking out any files under SCC.")]
[Help("TemplateLocalizationConfigFiles", "An optional list of paths to localization config files to be used as templates for Auto localization config file generation. If the paths provided are relative, they should be relative to UEProjectDirectory. Else the path should be absolute. All paths should be separated by a ;.")]
class Localize : BuildCommand
{
	private class LocalizationBatch
	{
		public LocalizationBatch(string InUEProjectDirectory, string InLocalizationTargetDirectory, string InRemoteFilenamePrefix, IReadOnlyList<string> InLocalizationProjectNames)
		{
			UEProjectDirectory = InUEProjectDirectory;
			LocalizationTargetDirectory = InLocalizationTargetDirectory;
			RemoteFilenamePrefix = InRemoteFilenamePrefix;
			LocalizationProjectNames = InLocalizationProjectNames;
		}

		public string UEProjectDirectory { get; private set; }
		public string LocalizationTargetDirectory { get; private set; }
		public string RemoteFilenamePrefix { get; private set; }
		public IReadOnlyList<string> LocalizationProjectNames { get; private set; }
	};

	private class LocalizationTask
	{
		public LocalizationTask(LocalizationBatch InBatch, string InUEProjectRoot, string InLocalizationProviderName, int InPendingChangeList, BuildCommand InCommand)
		{
			Batch = InBatch;
			RootWorkingDirectory = CombinePaths(InUEProjectRoot, Batch.UEProjectDirectory);
			RootLocalizationTargetDirectory = CombinePaths(InUEProjectRoot, Batch.LocalizationTargetDirectory);

			//Try and find our localization provider
			{
				LocalizationProvider.LocalizationProviderArgs LocProviderArgs;
				LocProviderArgs.RootWorkingDirectory = RootWorkingDirectory;
				LocProviderArgs.RootLocalizationTargetDirectory = RootLocalizationTargetDirectory;
				LocProviderArgs.RemoteFilenamePrefix = Batch.RemoteFilenamePrefix;
				LocProviderArgs.Command = InCommand;
				LocProviderArgs.PendingChangeList = InPendingChangeList;
				LocProvider = LocalizationProvider.GetLocalizationProvider(InLocalizationProviderName, LocProviderArgs);
			}
		}

		public LocalizationBatch Batch;
		public string RootWorkingDirectory;
		public string RootLocalizationTargetDirectory;
		public LocalizationProvider LocProvider = null;
		public List<ProjectInfo> ProjectInfos = new List<ProjectInfo>();
		public List<IProcessResult> GatherProcessResults = new List<IProcessResult>();
	};

	private abstract class IGatherTextCommandletLauncherStrategy
	{
		public class Args
		{
			public string AbsoluteEditorExePath { get; set; } = "";
			public ERunOptions CommandletRunOptions { get; set; } = ERunOptions.Default;
			public string AbsoluteUEProjectDirectoryPath { get; set; } = "";
			public string UEProjectName { get; set; } = "";
			public string EditorArgs { get; set; } = "";
			public List<string> LocalizationStepNames { get; set; } = new();

			public string GetAbsoluteUEProjectPath()
			{
				// This implies that we're gathering for the Engine 
				if (String.IsNullOrEmpty(UEProjectName))
				{
					return "";
				}
				return Path.Combine(AbsoluteUEProjectDirectoryPath, $"{UEProjectName}.uproject");
			}
		}

		public IGatherTextCommandletLauncherStrategy(IGatherTextCommandletLauncherStrategy.Args InArgs)
		{
			LauncherArgs = InArgs;
		}
		public abstract void LaunchCommandlet(List<LocalizationTask> LocalizationTasks);

		protected readonly IGatherTextCommandletLauncherStrategy.Args LauncherArgs;
	}

	private class BatchedGatherTextCommandletLauncherStrategy : IGatherTextCommandletLauncherStrategy
	{
		public BatchedGatherTextCommandletLauncherStrategy(IGatherTextCommandletLauncherStrategy.Args InArgs) : base(InArgs)
		{

		}
		
		public override void LaunchCommandlet(List<LocalizationTask> LocalizationTasks)
		{
			foreach (var LocalizationTask in LocalizationTasks)
			{
				foreach (var ProjectInfo in LocalizationTask.ProjectInfos)
				{
					List<string> LocalizationConfigFiles = ProjectInfo.GetConfigFilesToRun(LauncherArgs.LocalizationStepNames);
					if (LocalizationConfigFiles.Count > 0)
					{
						string ConcatenatedConfigFiles = String.Join(";", LocalizationConfigFiles);
						string CommandLine = $"\"{LauncherArgs.GetAbsoluteUEProjectPath()}\" -run=GatherText -config=\"{ConcatenatedConfigFiles}\" {LauncherArgs.EditorArgs}";
						Logger.LogInformation("Running localization commandlet for '{Arg0}': {Arguments}", ProjectInfo.ProjectName, CommandLine);
						LocalizationTask.GatherProcessResults.Add(Run(LauncherArgs.AbsoluteEditorExePath, CommandLine, null,	LauncherArgs.CommandletRunOptions));
					}
					else
					{
						Logger.LogInformation($"Localization target '{ProjectInfo.ProjectName}' has no valid config files associated with the specified localization steps. The localization target will not be gathered.");
						LocalizationTask.GatherProcessResults.Add(null);
					}
				}
			}
		}
	}

	private class ConsolidatedGatherTextCommandletLauncherStrategy : IGatherTextCommandletLauncherStrategy
	{
		public ConsolidatedGatherTextCommandletLauncherStrategy(IGatherTextCommandletLauncherStrategy.Args InArgs) : base(InArgs)
		{

		}
		public override void LaunchCommandlet(List<LocalizationTask> LocalizationTasks)
		{
			List<string> ConsolidatedConfigFiles = new List<string>();
			List<string> ProjectsToGather = new List<string>();
			foreach (var LocalizationTask in LocalizationTasks)
			{
				foreach (var ProjectInfo in LocalizationTask.ProjectInfos)
				{
					List<string> LocalizationConfigFiles = ProjectInfo.GetConfigFilesToRun(LauncherArgs.LocalizationStepNames);
					if (LocalizationConfigFiles.Count > 0)
					{
						ConsolidatedConfigFiles.AddRange(ProjectInfo.GetConfigFilesToRun(LauncherArgs.LocalizationStepNames));
						ProjectsToGather.Add(ProjectInfo.ProjectName);
					}
					else
					{
						Logger.LogInformation($"Localization target {ProjectInfo.ProjectName} will not be gathered because it does not have localization config files that can be run.");
					}
					
				}
			}

			// Now that we've consolidated all the config files, we write them out to the Saved Directory 
			string SaveDirectory = Path.Combine(LauncherArgs.AbsoluteUEProjectDirectoryPath, "Saved", "Localization");
			if (!Directory.Exists(SaveDirectory))
			{
				Directory.CreateDirectory(SaveDirectory);
			}
			// we append a GUID to the file name to ensure that parallel gathers can still work without stomping on the file and multiple runs of the Localize command can still result in unique files for debugging 
			string SaveFile = Path.Combine(SaveDirectory, $"ConfigList_{Guid.NewGuid()}.txt");
			File.WriteAllLines(SaveFile, ConsolidatedConfigFiles);

			// now construct the command line and run 
			string CommandLine = $"\"{LauncherArgs.GetAbsoluteUEProjectPath()}\" -run=GatherText -ConfigList=\"{SaveFile}\" {LauncherArgs.EditorArgs}";
			string ConcatenatedProjectsString = String.Join(",", ProjectsToGather);
			Logger.LogInformation($"Consolidating localization gather for following localization targets: \"{ConcatenatedProjectsString}\"");
			Logger.LogInformation($"Running consolidated gather text commandlet - {CommandLine}");
			IProcessResult ConsolidatedProcessResult = Run(LauncherArgs.AbsoluteEditorExePath, CommandLine, null, LauncherArgs.CommandletRunOptions);

			// Go through all tasks and set the gather process result  accordingly for the various projects
			// We need to do this otherwise projects won't be uploaded successfully 
			foreach (var LocalizationTask in LocalizationTasks)
			{
				foreach (var ProjectInfo in LocalizationTask.ProjectInfos)
				{
					List<string> LocalizationConfigFiles = ProjectInfo.GetConfigFilesToRun(LauncherArgs.LocalizationStepNames);
					if (LocalizationConfigFiles.Count > 0)
					{
						LocalizationTask.GatherProcessResults.Add(ConsolidatedProcessResult);
					}
					else
					{
						LocalizationTask.GatherProcessResults.Add(null);
					}
				}
			}
		}
	}

	private string UEProjectRoot = CmdEnv.LocalRoot;
	private string UEProjectDirectory = "";
	private string UEProjectName = "";
	private List<string> LocalizationProjectNames = new();
	private string LocalizationProviderName = "";
	private List<string> LocalizationStepNames = new();
	private bool bShouldGatherPlugins = false;
	private bool bShouldEnableIncludedPlugins = false;
	private List<string> IncludePlugins = new();
	private List<string> ExcludePlugins = new();
	private bool bShouldGatherPlatforms = false;
	private string AdditionalCommandletArguments = "";
	private List<LocalizationConfigFile> TemplateLocalizationConfigFiles = new List<LocalizationConfigFile>();
	private bool bEnableParallelGather = false;
	private bool bIsRunningInPreview = false;
	private bool bPreserveAutoGeneratedResources = false;
	private bool bConsolidateConfigFiles = false;
	private int PendingChangeList = -1;

	HashSet<string> AutoGeneratedFiles = new();
	HashSet<string> AutoGeneratedDirectories = new();

	public override void ExecuteBuild()
	{
		if (!ParseCommandLine())
		{
			Logger.LogError($"Errors detected in command line. Please fix the parameters and arguments for the command line and re-run the command..");
			return;
		}

		var StartTime = DateTime.UtcNow;

		var LocalizationBatches = new List<LocalizationBatch>();
		// Add the static set of localization projects as a batch
		if (LocalizationProjectNames.Count > 0)
		{
			AddStaticLocalizationBatches(LocalizationBatches);
		}

		// Build up any additional batches needed for platforms
		if (bShouldGatherPlatforms)
		{
			AddPlatformLocalizationBatches(LocalizationBatches);
		}

		// Build up any additional batches needed for plugins
		if (bShouldGatherPlugins)
		{
			HashSet<string> PluginsAdded = new();
			HashSet<string> AdditionalPluginsToExclude = new HashSet<string>();
			AddPluginLocalizationBatches(LocalizationBatches, PluginsAdded, AdditionalPluginsToExclude);
			if (PluginsAdded.Count > 0)
			{
				AutoGeneratePluginLocalizationFiles(PluginsAdded);
			}
			if (AdditionalPluginsToExclude.Count > 0)
			{
				// We now remove these plugins from the include plugins list so that we don't enable them unnecessarily down the line 
				Logger.LogInformation($"Removing {AdditionalPluginsToExclude.Count} additional plugins that do not need to be localized.");
				// Technically we could just set the include plugins to be the added plugisn, but we leave this exclusion explicit to make things clearer
				// Further, additional logic can be included next time to modify what plugisn should be removed 
				IncludePlugins = IncludePlugins.Except(AdditionalPluginsToExclude.ToList()).ToList();
			}
		}

		if (LocalizationBatches.Count == 0)
		{
			Logger.LogWarning("No localization batches found from input parameters provided. Nothing will be localized.");
			return;
		}

		// Create a single changelist to use for all changes
		if (P4Enabled && !bIsRunningInPreview)
		{
			var ChangeListCommitMessage = String.Format("Localization Automation using CL {0}", P4Env.Changelist);
			if (File.Exists(CombinePaths(CmdEnv.LocalRoot, @"Engine/Restricted/NotForLicensees/Build/EpicInternal.txt")))
			{
				ChangeListCommitMessage += "\n#okforgithub ignore";
			}

			PendingChangeList = P4.CreateChange(P4Env.Client, ChangeListCommitMessage);
		}

		// Prepare to process each localization batch
		var LocalizationTasks = new List<LocalizationTask>();
		foreach (var LocalizationBatch in LocalizationBatches)
		{
			var LocalizationTask = new LocalizationTask(LocalizationBatch, UEProjectRoot, LocalizationProviderName, PendingChangeList, this);
			LocalizationTasks.Add(LocalizationTask);

			// Make sure the Localization configs and content is up-to-date to ensure we don't get errors later on
			// we still run this in preview mode to bring all the gather configs up to date
			if (P4Enabled)
			{
				Logger.LogInformation("Sync necessary content to head revision");
				P4.Sync(P4Env.Branch + "/" + LocalizationTask.Batch.LocalizationTargetDirectory + "/Config/Localization/...");
				P4.Sync(P4Env.Branch + "/" + LocalizationTask.Batch.LocalizationTargetDirectory + "/Content/Localization/...");
			}

			// Generate the info we need to gather for each project
			foreach (var ProjectName in LocalizationTask.Batch.LocalizationProjectNames)
			{
				LocalizationTask.ProjectInfos.Add(GenerateProjectInfo(LocalizationTask.RootLocalizationTargetDirectory, ProjectName, LocalizationStepNames));
			}
		}

		// Hash the current PO files on disk so we can work out whether they actually change
		Dictionary<string, byte[]> InitalPOFileHashes = null;
		if (P4Enabled && !bIsRunningInPreview)
		{
			InitalPOFileHashes = GetPOFileHashes(LocalizationBatches, UEProjectRoot);
		}

		InitializeLocalizationProvider(LocalizationTasks);

		// Download the latest translations from our localization provider
		if (LocalizationStepNames.Contains("Download"))
		{
			DownloadFilesFromLocalizationProvider(LocalizationTasks);
		}

		// Begin the gather command for each task
		// These can run in parallel when ParallelGather is enabled
		StartGatherCommands(LocalizationTasks);

		// Wait for each commandlet process to finish and report the result.
		// This runs even for non-parallel execution to log the exit state of the process.
		WaitForCommandletResults(LocalizationTasks);

		// If we are running in preview, we can go ahead and delete all generated preview files after the gather step is complete
		if (bIsRunningInPreview)
        {
			CleanUpGeneratedPreviewFiles(LocalizationBatches);
		}

		if (!bPreserveAutoGeneratedResources)
		{
			CleanUpAutoGeneratedFiles();
		}

		// Upload the latest sources to our localization provider
		if (LocalizationStepNames.Contains("Upload"))
		{
			UploadFilesToLocalizationProvider(LocalizationTasks);
		}

		// Clean-up the changelist so it only contains the changed files, and then submit it (if we were asked to)
		if (P4Enabled && !bIsRunningInPreview)
		{
			// Revert any PO files that haven't changed aside from their header, 
			// and ensure that any that have are added/edited in source control
			FinalizeSourceControlState(LocalizationBatches, InitalPOFileHashes);

			// Submit that single changelist now
			if (AllowSubmit)
			{
				int SubmittedChangeList;
				P4.Submit(PendingChangeList, out SubmittedChangeList);
			}
		}

		var RunDuration = (DateTime.UtcNow - StartTime).TotalMilliseconds;
		Logger.LogInformation("Localize command finished in {Arg0} seconds", RunDuration / 1000);
	}

	private void ParseIncludeExcludePlugins(in String Parameter, ref List<String> ParseResult)
	{
		var ParameterStr = ParseParamValue(Parameter);
		if (!string.IsNullOrEmpty(ParameterStr))
		{
			//first we extract all plugin name that do not contains wild card
			List<string> WildCharIncludes = new List<string>();
			foreach (var PluginName in ParameterStr.Split(','))
			{
				if (PluginName.Contains("*") || PluginName.Contains("&"))
				{
					WildCharIncludes.Add(PluginName.Trim());
				}
				else
				{
					ParseResult.Add(PluginName.Trim());
				}
			}
			//then we extract all plugin name that correspond to a  wild card include
			if (WildCharIncludes.Count() > 0)
			{
				var PluginsRootDirectory = new DirectoryReference(CombinePaths(UEProjectRoot, UEProjectDirectory));
				IReadOnlyList<PluginInfo> AllPlugins = Plugins.ReadPluginsFromDirectory(PluginsRootDirectory, "Plugins", UEProjectName.Length == 0 ? PluginType.Engine : PluginType.Project);
				foreach (var PluginInfo in AllPlugins)
				{
					if (WildCharIncludes.Exists(x => FileSystemName.MatchesSimpleExpression(x, PluginInfo.Name)))
					{
						ParseResult.Add(PluginInfo.Name);
					}
				}
			}
		}
	}
	private bool ParseCommandLine()
	{
		UEProjectRoot = ParseParamValue("UEProjectRoot", Default: CmdEnv.LocalRoot);
		UEProjectDirectory = ParseParamValue("UEProjectDirectory");
		if (UEProjectDirectory == null)
		{
			throw new AutomationException("Missing required command line argument: 'UEProjectDirectory'");
		}

		UEProjectName = ParseParamValue("UEProjectName", Default: "");
		
		{
			var LocalizationProjectNamesStr = ParseParamValue("LocalizationProjectNames");
			if (LocalizationProjectNamesStr != null)
			{
				foreach (var ProjectName in LocalizationProjectNamesStr.Split(','))
				{
					LocalizationProjectNames.Add(ProjectName.Trim());
				}
			}
		}

		LocalizationProviderName = ParseParamValue("LocalizationProvider", Default: "");

		{
			var LocalizationStepNamesStr = ParseParamValue("LocalizationSteps");
			if (LocalizationStepNamesStr == null)
			{
				LocalizationStepNames.AddRange([ "Download", "Gather", "Import", "Export", "Compile", "Upload" ]);
			}
			else
			{
				foreach (var StepName in LocalizationStepNamesStr.Split(','))
				{
					LocalizationStepNames.Add(StepName.Trim());
				}
			}
			if (LocalizationStepNames.Contains("Gather"))
			{
				// Gather runs a superset of the GenerateReports step, so if we're running Gather, we can skip GenerateReports
				LocalizationStepNames.Remove("GenerateReports");
			}
			LocalizationStepNames.Add("Monolithic"); // Always allow the monolithic scripts to run as we don't know which steps they do
		}

		bShouldGatherPlugins = ParseParam("IncludePlugins");
		bShouldEnableIncludedPlugins = ParseParam("EnableIncludedPlugins");

		string PluginsRootPath = CombinePaths(UEProjectRoot, UEProjectDirectory);

		//First we gather the plugin from the command and extract the wild char plugin
		if (bShouldGatherPlugins)
		{
			ParseIncludeExcludePlugins("IncludePlugins", ref IncludePlugins);
		}

		string IncludePluginsUnderDirectoryStr = ParseParamValue("IncludePluginsDirectory");
		if (!string.IsNullOrEmpty(IncludePluginsUnderDirectoryStr))
		{
			bShouldGatherPlugins = true;
			foreach (string IncludePluginDirectory in IncludePluginsUnderDirectoryStr.Split(";"))
			{
				string AbsolutePathToIncludePluginsDirectory = Path.Combine(PluginsRootPath, IncludePluginDirectory.Trim());
				IncludePlugins.AddRange(LocalizationUtilities.GetPluginNamesUnderDirectory(AbsolutePathToIncludePluginsDirectory, PluginsRootPath, UEProjectName.Length == 0 ? PluginType.Engine : PluginType.Project));
			}
		}

		string ExcludePluginsUnderDirectoryStr = ParseParamValue("ExcludePluginsDirectory");
		if (!string.IsNullOrEmpty(ExcludePluginsUnderDirectoryStr))
		{
			foreach (string ExcludePluginDirectory in ExcludePluginsUnderDirectoryStr.Split(";"))
			{
				string AbsolutePathToExcludePluginsDirectory = Path.Combine(PluginsRootPath, ExcludePluginDirectory.Trim());
				ExcludePlugins.AddRange(LocalizationUtilities.GetPluginNamesUnderDirectory(AbsolutePathToExcludePluginsDirectory, PluginsRootPath, UEProjectName.Length == 0 ? PluginType.Engine : PluginType.Project));
			}
		}

		if (bShouldGatherPlugins)
		{
			ParseIncludeExcludePlugins("ExcludePlugins", ref ExcludePlugins);
		}

		bShouldGatherPlatforms = ParseParam("IncludePlatforms");

		AdditionalCommandletArguments = ParseParamValue("AdditionalCommandletArguments", Default: "");
		// We remove any leading or trailing quotes from AdditionalCommandletArguments
		if (!String.IsNullOrEmpty(AdditionalCommandletArguments))
		{
			AdditionalCommandletArguments = AdditionalCommandletArguments.Trim();
			if (AdditionalCommandletArguments.StartsWith("\"") && AdditionalCommandletArguments.EndsWith("\""))
			{
				// We subtract 2 to remove the last " character
				AdditionalCommandletArguments = AdditionalCommandletArguments[1..^1];
			}
		}

		bEnableParallelGather = ParseParam("ParallelGather");
		if (bEnableParallelGather)
		{
			Logger.LogInformation("Parallel gather enabled. Multiple instances of the editor will be used to gather each individual localization batch.");
		}

		bIsRunningInPreview = ParseParam("Preview");
		// We pass the preview switch along to have the gather text commandlets exhibit different behaviors. See UGatherTextCommandlet
		if (bIsRunningInPreview)
		{
			Logger.LogInformation("Running in preview mode. Preview switch will be passed along to all localization commandlets to be run.");
			AdditionalCommandletArguments += " -Preview";
		}

		bPreserveAutoGeneratedResources = ParseParam("PreserveAutoGeneratedResources");
		if (bPreserveAutoGeneratedResources)
		{
			Logger.LogInformation("Preserving auto-generated content. Auto-generated files and folders will not be automatically cleaned up at the end of this command.");
		}

		bConsolidateConfigFiles = ParseParam("ConsolidateConfigFiles");
		if (bConsolidateConfigFiles)
		{
			Logger.LogInformation("Consolidating config files for various localization targets. A single instance of the Editor will be run to perform all localization gather steps.");
		}

		return ParseTemplateLocalizationConfigFiles();
	}

	private bool ParseTemplateLocalizationConfigFiles()
	{
		string TemplateLocalizationConfigFilesString = ParseParamValue("TemplateLocalizationConfigFiles");
		if (String.IsNullOrEmpty(TemplateLocalizationConfigFilesString))
		{
			Logger.LogInformation("No template localization config files provided. Default localization config values will be used for Auto localization config generation.");
			// No template parameter exists, it's still considered a success 
			return true;
		}

		List<string> LocalizationConfigFileStrings = new List<string>();
		bool bShouldEarlyOut = false;
		foreach (string TemplateLocalizationConfigFile in TemplateLocalizationConfigFilesString.Split(';'))
		{
			string absoluteTemplateFilePath = Path.IsPathRooted(TemplateLocalizationConfigFile) 
					? TemplateLocalizationConfigFile
					: Path.Combine(UEProjectRoot, UEProjectDirectory, TemplateLocalizationConfigFile);
			if (!File.Exists(absoluteTemplateFilePath))
			{ 
				Logger.LogError($"Provided relative template file path '{TemplateLocalizationConfigFile}' expands to absolute file path '{absoluteTemplateFilePath}' which cannot be found. Please check the template file paths provided are correct.");
				bShouldEarlyOut = true; 
				continue;
			}
			LocalizationConfigFileStrings.Add(absoluteTemplateFilePath);
		}

		if (bShouldEarlyOut)
		{
			Logger.LogError("1 or more provided template localization config files could not be found on disk. Please check the paths are correct and relative to the provided UEProjectDirectory.");
			return false;
		}

		// We assume if more than 1 template file is provided that this is intended for modular localization config files 
		if (LocalizationConfigFileStrings.Count > 1)
		{
			foreach (string TemplateLocalizationConfigFile in LocalizationConfigFileStrings)
			{
				if (!LocalizationUtilities.IsModularLocalizationConfigFile(TemplateLocalizationConfigFile))
				{
					Logger.LogError($"Provided template localization config file '{TemplateLocalizationConfigFile}' is not a Monolithic localization config file.");
					bShouldEarlyOut = true;
					continue;
				}
			}

			if (bShouldEarlyOut)
			{
				Logger.LogError("Provided template localization config files all expected to be Monolithic localization config files. 1 or more provided template files are not Monolithic.");
				return false;
			}
		}
		// @TODOLocalization: Handle the monolithic template loc config file that's currently not supported 
		// At this point we know all of the files exist and we have verified whether or not the files are all valid/if there are copies etc
		// We load the localization config files up front here because the template files should really only be loaded once instead of every single time for every plugin 
		foreach (string TemplateLocalizationConfigFile in LocalizationConfigFileStrings)
		{
			LocalizationConfigFile template = LocalizationConfigFile.Load(TemplateLocalizationConfigFile);
			if (template == null)
			{
				Logger.LogError($"Failed to load template localization config file '{TemplateLocalizationConfigFile}'. Verify that the provided template localization config file is correct and properly formatted.");
				bShouldEarlyOut = true;
				continue;
			}
			Logger.LogInformation($"Template localization config file '{TemplateLocalizationConfigFile}' successfully loaded and will be be used for Auto localization config generation.");
			TemplateLocalizationConfigFiles.Add(template);
		}

		if (bShouldEarlyOut)
		{
			Logger.LogError("1 or more provided template localization config files failed to be loaded. Verify that all provided template localization config files are valid and correctly formatted.");
			return false;
		}
		return true;
	}

	private void AddStaticLocalizationBatches(List<LocalizationBatch> LocalizationBatches)
	{
		LocalizationBatches.Add(new LocalizationBatch(UEProjectDirectory, UEProjectDirectory, "", LocalizationProjectNames));
	}

	private void AddPlatformLocalizationBatches(List<LocalizationBatch> LocalizationBatches)
	{
		var PlatformsRootDirectory = new DirectoryReference(CombinePaths(UEProjectRoot, UEProjectDirectory, "Platforms"));
		if (DirectoryReference.Exists(PlatformsRootDirectory))
		{
			foreach (DirectoryReference PlatformDirectory in DirectoryReference.EnumerateDirectories(PlatformsRootDirectory))
			{
				// Find the localization targets defined for this platform
				var PlatformTargetNames = GetLocalizationTargetsFromDirectory(new DirectoryReference(CombinePaths(PlatformDirectory.FullName, "Config", "Localization")));
				if (PlatformTargetNames.Count > 0)
				{
					var RootRelativePluginPath = PlatformDirectory.MakeRelativeTo(new DirectoryReference(UEProjectRoot));
					RootRelativePluginPath = RootRelativePluginPath.Replace('\\', '/'); // Make sure we use / as these paths are used with P4

					LocalizationBatches.Add(new LocalizationBatch(UEProjectDirectory, RootRelativePluginPath, "", PlatformTargetNames));
				}
			}
		}
	}

	private void AddPluginLocalizationBatches(List<LocalizationBatch> LocalizationBatches, HashSet<string> PluginsAdded, HashSet<string> AdditionalPluginsToExclude)
	{
		var PluginsRootDirectory = new DirectoryReference(CombinePaths(UEProjectRoot, UEProjectDirectory));
		IReadOnlyList<PluginInfo> AllPlugins = Plugins.ReadPluginsFromDirectory(PluginsRootDirectory, "Plugins", UEProjectName.Length == 0 ? PluginType.Engine : PluginType.Project);

		// Add a batch for each plugin that meets our criteria
		var AvailablePluginNames = new HashSet<string>();
		foreach (var PluginInfo in AllPlugins)
		{
			AvailablePluginNames.Add(PluginInfo.Name);

			bool bShouldIncludePlugin = (IncludePlugins.Count == 0 || IncludePlugins.Contains(PluginInfo.Name)) && !ExcludePlugins.Contains(PluginInfo.Name);
			bool bPluginHasLocalizationTarget = PluginInfo.Descriptor.LocalizationTargets != null && PluginInfo.Descriptor.LocalizationTargets.Length > 0;
			if (bShouldIncludePlugin && bPluginHasLocalizationTarget)
			{
				var RootRelativePluginPath = PluginInfo.Directory.MakeRelativeTo(new DirectoryReference(UEProjectRoot));
				RootRelativePluginPath = RootRelativePluginPath.Replace('\\', '/'); // Make sure we use / as these paths are used with P4

				var PluginTargetNames = new List<string>();
				foreach (var LocalizationTarget in PluginInfo.Descriptor.LocalizationTargets)
				{
					// we only add the plugin target if the config generation policy is auto or user, i.e it should actually participate in the localization gather 
					if (LocalizationTarget.ConfigGenerationPolicy != LocalizationConfigGenerationPolicy.Never)
					{
						PluginTargetNames.Add(LocalizationTarget.Name);
					}
					
				}

				// If the plugin has no valid loc targets to gather, we don't create the batch 
				if (PluginTargetNames.Count > 0)
				{
					LocalizationBatches.Add(new LocalizationBatch(UEProjectDirectory, RootRelativePluginPath, PluginInfo.Name, PluginTargetNames));
					PluginsAdded.Add(PluginInfo.Name);
				}
				else
				{
					// The plugin doesn't actually have to be included. 
					AdditionalPluginsToExclude.Add(PluginInfo.Name);
				}
			}
		}

		// If we had an explicit list of plugins to include, warn if any were missing
		foreach (string PluginName in IncludePlugins)
		{
			if (!AvailablePluginNames.Contains(PluginName))
			{
				Logger.LogWarning("The plugin '{PluginName}' specified by -IncludePlugins wasn't found and will be skipped.", PluginName);
			}
			else if (!PluginsAdded.Contains(PluginName))
			{
				// If the plugin was in the include list of plugins but they weren't added, this means that those plugisn failed the filter. 
				// Those plugisn should now be removed from the include list to prevent them from being enabled for no reason 
				AdditionalPluginsToExclude.Add(PluginName);
			}
		}

		foreach (string AdditionalPluginToExclude in AdditionalPluginsToExclude)
		{
			Logger.LogInformation($"The plugin '{AdditionalPluginToExclude}' has no localization targets that needs localizing and will be skipped.");
		}
	}

	private string BuildEditorArguments()
	{
		var EditorArguments = P4Enabled
				? String.Format("-SCCProvider=Perforce -P4Port={0} -P4User={1} -P4Client={2} -P4Passwd={3} -P4Changelist={4} -EnableSCC -DisableSCCSubmit", P4Env.ServerAndPort, P4Env.User, P4Env.Client, P4.GetAuthenticationToken(), PendingChangeList)
				: "-SCCProvider=None";
		if (IsBuildMachine)
		{
			EditorArguments += " -BuildMachine";
		}
		EditorArguments += " -Unattended";
		EditorArguments += " -NoShaderCompile";
		//EditorArguments += " -LogLocalizationConflicts";
		if (bEnableParallelGather)
		{
			EditorArguments += " -multiprocess";
		}

		AppendEnableIncludedPluginsArgument();

		if (!String.IsNullOrEmpty(AdditionalCommandletArguments))
		{
			EditorArguments += " " + AdditionalCommandletArguments;
		}
		return EditorArguments;
	}

	private void AppendEnableIncludedPluginsArgument()
	{
		if (!bShouldEnableIncludedPlugins)
		{
			Logger.LogInformation("Included plugins will not be auto enabled in commandlets.");
			return;
		}

		Logger.LogInformation($"Included plugins requested to be enabled. All specified included plugins that are not excluded and pass filtering will be enabled.");
		// We append all the included plugins to -EnablePlugins if -EnableIncludedPlugins is enabled. This wil ensure that the plugin content and metadata will be loaded.
		// @TODOLocalization: Ideally the enabling of plugins should be per batch, otherwise each instance of the editor is enabling a bunch of plugins it doesn't need 
		HashSet<string> PluginsToEnableSet = IncludePlugins.Except(ExcludePlugins).ToHashSet();
		string EnablePluginsToken = "-EnablePlugins=";
		if (!string.IsNullOrEmpty(AdditionalCommandletArguments))
		{
			// It's possible that there are already specified values for -EnabledPlugins, we willneed to try and parse them first.

			int EnablePluginsTokenIndex = AdditionalCommandletArguments.IndexOf(EnablePluginsToken);
			string EnablePluginsOldValue = "";
			if (EnablePluginsTokenIndex > -1)
			{
				// -EnablePlugins token exists in the additional commandlet args. We need to process it 
				Logger.LogInformation("-EnablePlugins already provided as an argument. Adding plugins provided to the full ist of plugins to enable.");
				int EnablePluginsValueStartIndex = EnablePluginsTokenIndex + EnablePluginsToken.Length;
				// We try and find the end of the string where it's separated by a space  between the next token. IndexOf() returns -1 if it can't find the space.
				// We decrement by 1 as we need the index of the last character not the index of the string 
				int EnablePluginsValueEndIndex = AdditionalCommandletArguments.IndexOf(" ", EnablePluginsValueStartIndex) - 1;
				// We can't find a next space. This means we're the last parameter in AdditionalCommandletArguments. The end index will be the length of the string - 1
				if (EnablePluginsValueEndIndex < 0)
				{
					EnablePluginsValueEndIndex = AdditionalCommandletArguments.Length - 1;
				}

				// Isolate the value of -EnablePlugins and add them to our list of plugins to enable 
				EnablePluginsOldValue = AdditionalCommandletArguments.Substring(EnablePluginsValueStartIndex, EnablePluginsValueEndIndex - EnablePluginsValueStartIndex + 1);
				foreach (string Plugin in EnablePluginsOldValue.Split(','))
				{
					PluginsToEnableSet.Add(Plugin.Trim());
				}
				// Now that we have parsed the values, we remove the field from the command line to add it in below 
				AdditionalCommandletArguments = AdditionalCommandletArguments.Remove(EnablePluginsTokenIndex, EnablePluginsValueEndIndex - EnablePluginsTokenIndex + 1);
				Logger.LogInformation($"Removed '{EnablePluginsToken}' argument from provided AdditionalCommandletArguments. Remaining value is '-AdditionalCommandletArgument={AdditionalCommandletArguments}'");
			}
		}

		if (PluginsToEnableSet.Count == 0)
		{
			Logger.LogInformation("No included plugins to enable.");
			return;
		}

		Logger.LogInformation("Filtering out GFP plugins from being enabled.");
		string GameFeaturesPluginDirectory = "GameFeatures";
		// We filter out GFPs from being auto-enabled in commandlets as we want the release versions of plugisn to be respected
		// The GFP system has ultimate authority of whether a GFP can be loaded. Thus force loading plugins can fail if the GFP system determines that it should not be loaded and cause errors. We bypass that by filtering GFPs out. 
		List<string> PluginsToRemove = new();
		foreach (string Plugin in PluginsToEnableSet)
		{
			PluginInfo Info = Plugins.GetPlugin(Plugin);
			if (Info is null)
			{
				PluginsToRemove.Add(Plugin);
				continue;
			}

			if (Info.Directory.ContainsName(GameFeaturesPluginDirectory, 0))
			{
				PluginsToRemove.Add(Plugin);
			}
		}
		if (PluginsToRemove.Count > 0)
		{
			Logger.LogInformation($"Removing {PluginsToRemove.Count} game feature plugins from plugins to be enabled.");
			foreach (string Plugin in PluginsToRemove)
			{
				Logger.LogInformation($"Removing GFP '{Plugin}' from being enabled.");
				PluginsToEnableSet.Remove(Plugin);
			}
		}
		
		if (PluginsToEnableSet.Count == 0)
		{
			Logger.LogInformation("All requested included plugins to be enabled have been filtered out. No specified included plugins will be enabled.");
			return;
		}
		string EnablePluginsNewValue = String.Join(',', PluginsToEnableSet.ToArray());
		Logger.LogInformation($"Appending following plugins to be enabled: {EnablePluginsNewValue}");

		// The token was either previously removed or never existed, we'll add it to the end 
		AdditionalCommandletArguments += " " + EnablePluginsToken + EnablePluginsNewValue;
		Logger.LogInformation($"AdditionalCommandletArguments='{AdditionalCommandletArguments}'");	
	}

	private void InitializeLocalizationProvider(List<LocalizationTask> LocalizationTasks)
	{
		foreach (var LocalizationTask in LocalizationTasks)
		{
			if (LocalizationTask.LocProvider != null)
			{
				foreach (var ProjectInfo in LocalizationTask.ProjectInfos)
				{
					Task SetupTask = LocalizationTask.LocProvider.InitializeProjectWithLocalizationProvider(ProjectInfo.ProjectName, ProjectInfo.ImportInfo);
					SetupTask.Wait();
				}
			}
		}
	}

	private void DownloadFilesFromLocalizationProvider(List<LocalizationTask> LocalizationTasks)
	{
		foreach (var LocalizationTask in LocalizationTasks)
		{
			if (LocalizationTask.LocProvider != null)
			{
				foreach (var ProjectInfo in LocalizationTask.ProjectInfos)
				{
					Task DownloadTask = LocalizationTask.LocProvider.DownloadProjectFromLocalizationProvider(ProjectInfo.ProjectName, ProjectInfo.ImportInfo);
					DownloadTask.Wait();
				}
			}
		}
	}

	private void UploadFilesToLocalizationProvider(List<LocalizationTask> LocalizationTasks)
	{
		foreach (var LocalizationTask in LocalizationTasks)
		{
			if (LocalizationTask.LocProvider != null)
			{
				// Upload all text to our localization provider
				for (int ProjectIndex = 0; ProjectIndex < LocalizationTask.ProjectInfos.Count; ++ProjectIndex)
				{
					var ProjectInfo = LocalizationTask.ProjectInfos[ProjectIndex];
					var RunResult = LocalizationTask.GatherProcessResults[ProjectIndex];

					if (RunResult != null && RunResult.ExitCode == 0)
					{
						// Recalculate the split platform paths before doing the upload, as the export may have changed them
						ProjectInfo.ExportInfo.CalculateSplitPlatformNames(LocalizationTask.RootLocalizationTargetDirectory);
						Task UploadTask = LocalizationTask.LocProvider.UploadProjectToLocalizationProvider(ProjectInfo.ProjectName, ProjectInfo.ExportInfo);
						UploadTask.Wait();
					}
					else
					{
						Logger.LogWarning("Skipping upload to the localization provider for '{Arg0}' due to an earlier commandlet failure.", ProjectInfo.ProjectName);
					}
				}
			}
		}
	}

	private string GetEditorExePath()
	{
		var EditorExe = HostPlatform.Current.GetUnrealExePath("UnrealEditor-Cmd.exe");
		Logger.LogInformation($"Looking for editor executable: {EditorExe}");
		if (!File.Exists(EditorExe))
		{
			// Try using the debug executable instead
			// (which is supposed to have a different name depending on the platform)
			var CurrentOS = RuntimePlatform.Current;
			if (CurrentOS == RuntimePlatform.Type.Mac)
			{
				EditorExe = "UnrealEditor-Mac-Debug-Cmd";
			}
			else if (CurrentOS == RuntimePlatform.Type.Linux)
			{
				EditorExe = "UnrealEditor-Linux-Debug-Cmd";
			}
			else
			{
				EditorExe = "UnrealEditor-Win64-Debug-Cmd.exe";
			}
			Logger.LogInformation($"Editor executable not found, now looking for: {EditorExe}");
			EditorExe = HostPlatform.Current.GetUnrealExePath(EditorExe);
		}
		return EditorExe;
	}

	private ERunOptions GetCommandletRunOptions()
	{
		var CommandletRunOptions = ERunOptions.Default | ERunOptions.NoLoggingOfRunCommand; // Disable logging of the run command as it will print the exit code which GUBP can pick up as an error (we do that ourselves later)
		if (bEnableParallelGather)
		{
			CommandletRunOptions |= ERunOptions.NoWaitForExit;
		}
		return CommandletRunOptions;
	}

	private void StartGatherCommands(List<LocalizationTask> LocalizationTasks)
	{
		IGatherTextCommandletLauncherStrategy.Args Args = new();
		Args.AbsoluteEditorExePath = GetEditorExePath();

		// Set the common basic editor arguments
		Args.EditorArgs = BuildEditorArguments();

		// Set the common process run options
		Args.CommandletRunOptions = GetCommandletRunOptions();

		Args.UEProjectName = UEProjectName;
		Args.AbsoluteUEProjectDirectoryPath = CombinePaths(UEProjectRoot, UEProjectDirectory);
		Args.LocalizationStepNames = LocalizationStepNames;

		IGatherTextCommandletLauncherStrategy Launcher;
		if (bConsolidateConfigFiles)
		{
			Launcher = new ConsolidatedGatherTextCommandletLauncherStrategy(Args);
		}
		else
		{
			Launcher = new BatchedGatherTextCommandletLauncherStrategy(Args);
		}
		Launcher.LaunchCommandlet(LocalizationTasks);
	}

	private void WaitForCommandletResults(List<LocalizationTask> LocalizationTasks)
	{
		foreach (var LocalizationTask in LocalizationTasks)
		{
			for (int ProjectIndex = 0; ProjectIndex < LocalizationTask.ProjectInfos.Count; ++ProjectIndex)
			{
				var ProjectInfo = LocalizationTask.ProjectInfos[ProjectIndex];
				var RunResult = LocalizationTask.GatherProcessResults[ProjectIndex];

				if (RunResult != null)
				{
					RunResult.WaitForExit();
					RunResult.OnProcessExited();
					RunResult.DisposeProcess();

					if (RunResult.ExitCode == 0)
					{
						Logger.LogInformation("The localization commandlet for '{Arg0}' exited with code 0.", ProjectInfo.ProjectName);
					}
					else
					{
						Logger.LogWarning("The localization commandlet for '{Arg0}' exited with code {Arg1} which likely indicates a crash.", ProjectInfo.ProjectName, RunResult.ExitCode);
					}
				}
			}
		}
	}

	private void CleanUpGeneratedPreviewFiles(List<LocalizationBatch> LocalizationBatches)
	{
		var PreviewManifestFiles = GetPreviewManifestFilesToDelete(LocalizationBatches, UEProjectRoot);
		foreach (var PreviewManifestFile in PreviewManifestFiles)
		{
			Logger.LogInformation("Deleting preview manifest file {PreviewManifestFile}.", PreviewManifestFile);
			try
			{
				File.Delete(PreviewManifestFile);
			}
			catch (Exception Ex)
			{
				Logger.LogInformation("[FAILED] Deleting preview file: '{PreviewManifestFile}' - {Ex}", PreviewManifestFile, Ex);
			}
		}
	}

	private void FinalizeSourceControlState(List<LocalizationBatch> LocalizationBatches, Dictionary<string, Byte[]> InitalPOFileHashes)
	{
		if (!P4Enabled || bIsRunningInPreview)
		{
			return;
		}

		{
			var POFilesToAdd = new List<string>();
			var POFilesToEdit = new List<string>();
			var POFilesToRevert = new List<string>();

			var CurrentPOFileHashes = GetPOFileHashes(LocalizationBatches, UEProjectRoot);
			foreach (var CurrentPOFileHashPair in CurrentPOFileHashes)
			{
				if (P4.FileExistsInDepot(CurrentPOFileHashPair.Key))
				{
					byte[] InitialPOFileHash;
					if (InitalPOFileHashes.TryGetValue(CurrentPOFileHashPair.Key, out InitialPOFileHash) && InitialPOFileHash.SequenceEqual(CurrentPOFileHashPair.Value))
					{
						POFilesToRevert.Add(CurrentPOFileHashPair.Key);
					}
					else
					{
						POFilesToEdit.Add(CurrentPOFileHashPair.Key);
					}
				}
				else
				{
					POFilesToAdd.Add(CurrentPOFileHashPair.Key);
				}
			}

			if (POFilesToAdd.Count > 0)
			{
				P4.Add(PendingChangeList, POFilesToAdd);
			}

			if (POFilesToEdit.Count > 0)
			{
				P4.Edit(PendingChangeList, POFilesToEdit);
			}

			if (POFilesToRevert.Count > 0)
			{
				P4.Revert(PendingChangeList, POFilesToRevert);
			}
		}

		// Revert any other unchanged files
		P4.RevertUnchanged(PendingChangeList);
	}

	private void AutoGeneratePluginLocalizationFiles(HashSet<string> PluginNames)
	{
		Stopwatch stopWatch = Stopwatch.StartNew();
		string PluginsRootDirectory = Path.Combine(UEProjectRoot, UEProjectDirectory);
		DirectoryReference PluginsRootDirectoryReference = new DirectoryReference(PluginsRootDirectory);

		foreach (string PluginName in PluginNames)
		{
			PluginInfo Info = Plugins.GetPlugin(PluginName);
			if (Info is null)
			{
				continue;
			}

			foreach (LocalizationTargetDescriptor Target in Info.Descriptor.LocalizationTargets)
			{
				if (Target.ConfigGenerationPolicy == LocalizationConfigGenerationPolicy.Auto)
				{
					string PluginConfigLocalizationPath = CombinePaths(Info.Directory.FullName, "Config", "Localization");
					if (!Directory.Exists(PluginConfigLocalizationPath))
					{
						Logger.LogInformation($"Plugin '{Info.Name}' does not have a Config Localization folder. Creating '{PluginConfigLocalizationPath}'.");
						Directory.CreateDirectory(PluginConfigLocalizationPath);
						AutoGeneratedDirectories.Add(PluginConfigLocalizationPath);
					}
					LocalizationConfigFileGenerator Generator = LocalizationConfigFileGenerator.GetGeneratorForFileFormat(LocalizationConfigFileFormat.Latest);
					LocalizationConfigFileGeneratorParams GeneratorParams = new();
					GeneratorParams.LoadingPolicy = Target.LoadingPolicy;
					GeneratorParams.LocalizationTargetName = Target.Name;
					GeneratorParams.LocalizationTargetRootDirectory = Info.Directory.MakeRelativeTo(PluginsRootDirectoryReference);
					List<LocalizationConfigFile> PluginLocalizationConfigFiles;
					// If we parsed template localization config files to use, we generate with templates 
					if (TemplateLocalizationConfigFiles.Count > 0)
					{
						PluginLocalizationConfigFiles = Generator.GenerateConfigFilesFromTemplate(GeneratorParams, TemplateLocalizationConfigFiles);
					}
					else
					{
						PluginLocalizationConfigFiles = Generator.GenerateDefaultSettingsConfigFiles(GeneratorParams);
					}
					// We only count as an error when generation of template localization config files fail 
					if ((TemplateLocalizationConfigFiles.Count > 0) && (PluginLocalizationConfigFiles.Count != TemplateLocalizationConfigFiles.Count))
					{
						Logger.LogError($"1 or more localization config files failed to be auto generated for localization target '{GeneratorParams.LocalizationTargetName}'");
						continue;
// We don't early out because we want all generated localization config files to still be cleaned up and not let loose on disk 
					}
					foreach (LocalizationConfigFile PluginLocalizationConfigFile in PluginLocalizationConfigFiles)
					{
						string SaveFilePath = Path.Combine(PluginConfigLocalizationPath, PluginLocalizationConfigFile.Name);
						if (File.Exists(SaveFilePath))
						{
							Logger.LogInformation($"Plugin localization target %{Target.Name} in {Info.Name} plugin already contains localization config file {SaveFilePath}. The file will not be auto-generated.");
							continue;
						}
						Logger.LogInformation($"Auto generating localization config file '{SaveFilePath}' for localization target '{Target.Name}' in plugin {Info.Name}.");
						AutoGeneratedFiles.Add(SaveFilePath);
						if (!bIsRunningInPreview)
						{
							PluginLocalizationConfigFile.Write(new FileReference(SaveFilePath));
						}
					}
				}
			}
		}
		stopWatch.Stop();
		Logger.LogInformation($"Generating localization config files for {PluginNames.Count} plugins took {stopWatch.ElapsedMilliseconds}ms.");
	}

	private void CleanUpAutoGeneratedFiles()
	{
		if (AutoGeneratedFiles.Count > 0)
		{
			Logger.LogInformation("Cleaning up auto-generated files.");
			foreach (string AutoGeneratedFile in AutoGeneratedFiles)
			{
				if (File.Exists(AutoGeneratedFile))
				{
					Logger.LogInformation($"Deleting auto-generated file '{AutoGeneratedFile}'");
					if (!bIsRunningInPreview)
					{
						File.Delete(AutoGeneratedFile);
					}
				}
			}
			Logger.LogInformation("Finished cleaning up all auto-generated files.");
		}

		if (AutoGeneratedDirectories.Count > 0)
		{
			Logger.LogInformation("Cleaning up auto-generated directories.");
			foreach (string AutoGeneratedDirectory in AutoGeneratedDirectories)
			{
				if (Directory.Exists(AutoGeneratedDirectory))
				{
					Logger.LogInformation($"Deleting auto-generated directory '{AutoGeneratedDirectory}'.");
					if (!bIsRunningInPreview)
					{
						Directory.Delete(AutoGeneratedDirectory);
					}
				}
			}
			Logger.LogInformation("Finished cleaning up auto-generated directories.");
		}
	}

	private ProjectInfo GenerateProjectInfo(string RootWorkingDirectory, string ProjectName, IReadOnlyList<string> LocalizationStepNames)
	{
		var LocalizationSteps = new List<ProjectStepInfo>();
		ProjectImportExportInfo ImportInfo = null;
		ProjectImportExportInfo ExportInfo = null;

		// Projects generated by the localization dashboard will use multiple config files that must be run in a specific order
		// Older projects (such as the Engine) would use a single config file containing all the steps
		// Work out which kind of project we're dealing with...
		var MonolithicConfigFile = CombinePaths(RootWorkingDirectory, String.Format(@"Config/Localization/{0}.ini", ProjectName));
		if (File.Exists(MonolithicConfigFile))
		{
			LocalizationSteps.Add(new ProjectStepInfo("Monolithic", MonolithicConfigFile));

			ImportInfo = GenerateProjectImportExportInfo(RootWorkingDirectory, MonolithicConfigFile);
			ExportInfo = ImportInfo;
		}
		else
		{
			var FileSuffixes = new[] { 
				new { Suffix = "Gather", Required = LocalizationStepNames.Contains("Gather") }, 
				new { Suffix = "Import", Required = LocalizationStepNames.Contains("Import") || LocalizationStepNames.Contains("Download") },	// Downloading needs the parsed ImportInfo
				new { Suffix = "Export", Required = LocalizationStepNames.Contains("Gather") || LocalizationStepNames.Contains("Upload")},		// Uploading needs the parsed ExportInfo
				new { Suffix = "Compile", Required = LocalizationStepNames.Contains("Compile") }, 
				new { Suffix = "GenerateReports", Required = false } 
			};

			foreach (var FileSuffix in FileSuffixes)
			{
				var ModularConfigFile = CombinePaths(RootWorkingDirectory, String.Format(@"Config/Localization/{0}_{1}.ini", ProjectName, FileSuffix.Suffix));

				if (File.Exists(ModularConfigFile))
				{
					LocalizationSteps.Add(new ProjectStepInfo(FileSuffix.Suffix, ModularConfigFile));

					if (FileSuffix.Suffix == "Import")
					{
						ImportInfo = GenerateProjectImportExportInfo(RootWorkingDirectory, ModularConfigFile);
					}
					else if (FileSuffix.Suffix == "Export")
					{
						ExportInfo = GenerateProjectImportExportInfo(RootWorkingDirectory, ModularConfigFile);
					}
				}
				else if (FileSuffix.Required)
				{
					throw new AutomationException("Failed to find a required config file! '{0}'", ModularConfigFile);
				}
			}
		}

		return new ProjectInfo(ProjectName, LocalizationSteps, ImportInfo, ExportInfo);
	}

	private ProjectImportExportInfo GenerateProjectImportExportInfo(string RootWorkingDirectory, string LocalizationConfigFile)
	{
		ConfigFile File = new ConfigFile(new FileReference(LocalizationConfigFile), ConfigLineAction.Add);
		var LocalizationConfig = new ConfigHierarchy(new ConfigFile[] { File });

		string DestinationPath;
		if (!LocalizationConfig.GetString("CommonSettings", "DestinationPath", out DestinationPath))
		{
			throw new AutomationException("Failed to find a required config key! Section: 'CommonSettings', Key: 'DestinationPath', File: '{0}'", LocalizationConfigFile);
		}

		string ManifestName;
		if (!LocalizationConfig.GetString("CommonSettings", "ManifestName", out ManifestName))
		{
			throw new AutomationException("Failed to find a required config key! Section: 'CommonSettings', Key: 'ManifestName', File: '{0}'", LocalizationConfigFile);
		}

		string ArchiveName;
		if (!LocalizationConfig.GetString("CommonSettings", "ArchiveName", out ArchiveName))
		{
			throw new AutomationException("Failed to find a required config key! Section: 'CommonSettings', Key: 'ArchiveName', File: '{0}'", LocalizationConfigFile);
		}

		string PortableObjectName;
		if (!LocalizationConfig.GetString("CommonSettings", "PortableObjectName", out PortableObjectName))
		{
			throw new AutomationException("Failed to find a required config key! Section: 'CommonSettings', Key: 'PortableObjectName', File: '{0}'", LocalizationConfigFile);
		}

		string NativeCulture;
		if (!LocalizationConfig.GetString("CommonSettings", "NativeCulture", out NativeCulture))
		{
			throw new AutomationException("Failed to find a required config key! Section: 'CommonSettings', Key: 'NativeCulture', File: '{0}'", LocalizationConfigFile);
		}

		List<string> CulturesToGenerate;
		if (!LocalizationConfig.GetArray("CommonSettings", "CulturesToGenerate", out CulturesToGenerate))
		{
			throw new AutomationException("Failed to find a required config key! Section: 'CommonSettings', Key: 'CulturesToGenerate', File: '{0}'", LocalizationConfigFile);
		}

		bool bUseCultureDirectory;
		if (!LocalizationConfig.GetBool("CommonSettings", "bUseCultureDirectory", out bUseCultureDirectory))
		{
			// bUseCultureDirectory is optional, default is true
			bUseCultureDirectory = true;
		}

		var ProjectImportExportInfo = new ProjectImportExportInfo(DestinationPath, ManifestName, ArchiveName, PortableObjectName, NativeCulture, CulturesToGenerate, bUseCultureDirectory);
		ProjectImportExportInfo.CalculateSplitPlatformNames(RootWorkingDirectory);
		return ProjectImportExportInfo;
	}

	private List<string> GetLocalizationTargetsFromDirectory(DirectoryReference ConfigDirectory)
	{
		var LocalizationTargets = new List<string>();

		if (DirectoryReference.Exists(ConfigDirectory))
		{
			var FileSuffixes = new[] {
				"_Gather",
				"_Import",
				"_Export",
				"_Compile",
				"_GenerateReports",
			};

			foreach (FileReference ConfigFile in DirectoryReference.EnumerateFiles(ConfigDirectory))
			{
				string LocalizationTarget = ConfigFile.GetFileNameWithoutExtension();
				foreach (var FileSuffix in FileSuffixes)
				{
					if (LocalizationTarget.EndsWith(FileSuffix))
					{
						LocalizationTarget = LocalizationTarget.Remove(LocalizationTarget.Length - FileSuffix.Length);
					}
				}
				if (!LocalizationTargets.Contains(LocalizationTarget))
				{
					LocalizationTargets.Add(LocalizationTarget);
				}
			}
		}

		return LocalizationTargets;
	}

	private Dictionary<string, byte[]> GetPOFileHashes(IReadOnlyList<LocalizationBatch> LocalizationBatches, string UEProjectRoot)
	{
		var AllFiles = new Dictionary<string, byte[]>();

		foreach (var LocalizationBatch in LocalizationBatches)
		{
			var LocalizationPath = CombinePaths(UEProjectRoot, LocalizationBatch.LocalizationTargetDirectory, "Content", "Localization");
			if (!Directory.Exists(LocalizationPath))
			{
				continue;
			}

			string[] POFileNames = Directory.GetFiles(LocalizationPath, "*.po", SearchOption.AllDirectories);
			foreach (var POFileName in POFileNames)
			{
				using (StreamReader POFileReader = File.OpenText(POFileName))
				{
					// Don't include the PO header (everything up to the first empty line) in the hash as it contains transient information (like timestamps) that we don't care about
					bool bHasParsedHeader = false;
					var POFileHash = MD5.Create();

					string POFileLine;
					while ((POFileLine = POFileReader.ReadLine()) != null)
					{
						if (!bHasParsedHeader)
						{
							bHasParsedHeader = POFileLine.Length == 0;
							continue;
						}

						var POFileLineBytes = Encoding.UTF8.GetBytes(POFileLine);
						POFileHash.TransformBlock(POFileLineBytes, 0, POFileLineBytes.Length, null, 0);
					}

					POFileHash.TransformFinalBlock(new byte[0], 0, 0);

					AllFiles.Add(POFileName, POFileHash.Hash);
				}
			}
		}

		return AllFiles;
	}

	private List<string> GetPreviewManifestFilesToDelete(IReadOnlyList<LocalizationBatch> LocalizationBatches, string UEProjectRoot)
	{
		var AllPreviewManifestFiles = new List<string>();
		foreach (var LocalizationBatch in LocalizationBatches)
		{
			var LocalizationPath = CombinePaths(UEProjectRoot, LocalizationBatch.LocalizationTargetDirectory, "Content", "Localization");
			if (!Directory.Exists(LocalizationPath))
			{
				continue;
			}
			string[] PreviewManifestFilenames= Directory.GetFiles(LocalizationPath, "*_Preview.manifest", SearchOption.AllDirectories);
			foreach (var ManifestFilename in PreviewManifestFilenames)
			{
				AllPreviewManifestFiles.Add(ManifestFilename);
			}
		}
		return AllPreviewManifestFiles;
	}

}

[Help("OnlyLoc", "Optional.  Only submit generated loc files, do not submit any other generated file.")]
[Help("NoRobomerge", "Optional.  Do not include the markup in the CL description to allow robomerging to other branches.")]
public class ExportMcpTemplates : BuildCommand
{
	public static string GetGameBackendFolder(FileReference ProjectFile)
	{
		return Path.Combine(ProjectFile.Directory.FullName, "Content", "Backend");
	}

	public static void RunExportTemplates(FileReference ProjectFile, bool bCheckoutAndSubmit, bool bOnlyLoc, bool bbNoRobomerge, string CommandletOverride)
	{
		string EditorExe = "UnrealEditor.exe";
		EditorExe = HostPlatform.Current.GetUnrealExePath(EditorExe);

		string GameBackendFolder = GetGameBackendFolder(ProjectFile);
		if (!DirectoryExists_NoExceptions(GameBackendFolder))
		{
			throw new AutomationException("Error: RunExportTemplates failure. GameBackendFolder not found. {0}", GameBackendFolder);
		}

		string FolderToGenerateIn = GameBackendFolder;

		string Parameters = "-GenerateLoc";

		int WorkingCL = -1;
		if (bCheckoutAndSubmit)
		{
			if (!CommandUtils.P4Enabled)
			{
				throw new AutomationException("Error: RunExportTemplates failure. bCheckoutAndSubmit used without access to P4");
			}

			// Check whether all templates in folder are latest.  If not skip exporting.
			List<string> FilesPreviewSynced;
			CommandUtils.P4.PreviewSync(out FilesPreviewSynced, FolderToGenerateIn + "/...");
			if (FilesPreviewSynced.Count() > 0)
			{
				Logger.LogInformation("Some files in folder {FolderToGenerateIn} are not latest, which means that these files might have already been updated by an earlier exporting job.  Skip this one.", FolderToGenerateIn);
				return;
			}

			String CLDescription = String.Format("RunExportTemplates Updated mcp templates using CL {0}", P4Env.Changelist);
			if (bOnlyLoc)
			{
				CLDescription += " [OnlyLoc]";
			}
			if (!bbNoRobomerge)
			{
				CLDescription += "\r\n#robomerge[ALL] #DisregardExcludedAuthors";
			}

			WorkingCL = CommandUtils.P4.CreateChange(CommandUtils.P4Env.Client, CLDescription);
			CommandUtils.P4.Edit(WorkingCL, FolderToGenerateIn + "/...");
		}

		string Commandlet = string.IsNullOrWhiteSpace(CommandletOverride) ? "ExportTemplatesCommandlet" : CommandletOverride;
		CommandUtils.RunCommandlet(ProjectFile, EditorExe, Commandlet, Parameters);

		if (WorkingCL > 0)
		{
			CommandUtils.P4.RevertUnchanged(WorkingCL);

			if (bOnlyLoc)
			{
				// Revert all folders and files except GeneratedLoc.json
				foreach (string DirPath in Directory.GetDirectories(FolderToGenerateIn))
				{
					DirectoryInfo Dir = new DirectoryInfo(DirPath);
					CommandUtils.P4.Revert(WorkingCL, FolderToGenerateIn + "/" + Dir.Name + "/...");
				}

				foreach (string FilePath in Directory.GetFiles(FolderToGenerateIn))
				{
					FileInfo File = new FileInfo(FilePath);
					if (File.Name != "GeneratedLoc.json")
					{
						CommandUtils.P4.Revert(WorkingCL, FolderToGenerateIn + "/" + File.Name);
					}
				}
			}

			// If the CL is empty after the RevertUnchanged, the submit call will just delete the CL and return cleanly
			int SubmittedCL;
			CommandUtils.P4.Submit(WorkingCL, out SubmittedCL, false, true);
		}
	}

	public override void ExecuteBuild()
	{
		string ProjectName = ParseParamValue("ProjectName", null);
		if (string.IsNullOrWhiteSpace(ProjectName))
		{
			throw new AutomationException("Error: ExportMcpTemplates failure. No ProjectName defined!");
		}

		FileReference ProjectFile = new FileReference(CombinePaths(CmdEnv.LocalRoot, ProjectName, String.Format("{0}.uproject", ProjectName)));
		bool bOnlyLoc = ParseParam("OnlyLoc");
		bool bNoRobomerge = ParseParam("NoRobomerge");
		string CommandletOverride = ParseParamValue("Commandlet", null);
		RunExportTemplates(ProjectFile, true, bOnlyLoc, bNoRobomerge, CommandletOverride);
	}
}

// Legacy alias
class Localise : Localize
{
};
