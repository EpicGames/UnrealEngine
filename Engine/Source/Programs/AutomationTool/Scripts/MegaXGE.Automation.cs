// Copyright Epic Games, Inc. All Rights Reserved.
using System;
using System.Collections.Generic;
using System.Text;
using System.IO;
using AutomationTool;
using UnrealBuildTool;
using EpicGames.Core;
using Microsoft.Extensions.Logging;

[Help("Compiles a bunch of stuff together with megaxge: Example arguments: -ubtargs=\"-nopdb\" -Target1=\"PlatformerGame win32|ios debug|development\"")]
[Help(typeof(UnrealBuild))]
[Help("ubtargs", "-args -for -ubt")]
[Help("clean", "Cleans targets before building")]
[Help("progress", "Reports the current steps to the log")]
[Help("Target1", "target1[|target2...] platform1[|platform2...] config1[|config2...]")]
[Help("Target2", "target1[|target2...] platform1[|platform2...] config1[|config2...]")]

class MegaXGE : BuildCommand
{
	public override void ExecuteBuild()
	{
		int WorkingCL = -1;
		if (P4Enabled && AllowSubmit)
		{
			string CmdLine = "";
			foreach (var Arg in Params)
			{
				CmdLine += Arg.ToString() + " ";
			}
			WorkingCL = P4.CreateChange(P4Env.Client, String.Format("MegaXGE build from changelist {0} - Params: {1}", P4Env.Changelist, CmdLine));
		}

		string UbtArgs = ParseParamValue("ubtargs", "");
		Logger.LogInformation("************************* MegaXGE");

		bool Clean = ParseParam("Clean");
		string CleanToolLocation = CombinePaths(CmdEnv.LocalRoot, "Engine", "Build", "Batchfiles", "Clean.bat");

		bool ShowProgress = ParseParam("Progress");

		var UnrealBuild = new UnrealBuild(this);

		var Agenda = new UnrealBuild.BuildAgenda();

		Logger.LogInformation("*************************");
		for (int Arg = 1; Arg < 100; Arg++)
		{
			string Parm = String.Format("Target{0}", Arg);
			string Target = ParseParamValue(Parm, "");
			if (String.IsNullOrEmpty(Target))
			{
				break;
			}

			FileReference ProjectFile = null;

			string ProjectFileParam = ParseParamValue(String.Format("Project{0}", Arg), null);
			if(ProjectFileParam != null)
			{
				ProjectFile = new FileReference(ProjectFileParam);
				if(!FileReference.Exists(ProjectFile))
				{
					throw new AutomationException("Project file '{0}' could not be found");
				}
			}

			var Parts = Target.Split(' ');

			string JustTarget = Parts[0];
			if (String.IsNullOrEmpty(JustTarget))
			{
				throw new AutomationException("BUILD FAILED target option '{0}' not parsed.", Target);
			}
			var Targets = JustTarget.Split('|');
			if (Targets.Length < 1)
			{
				throw new AutomationException("BUILD FAILED target option '{0}' not parsed.", Target);
			}

			var Platforms = new List<UnrealTargetPlatform>();
			var Configurations = new List<UnrealTargetConfiguration>();

			for (int Part = 1; Part < Parts.Length; Part++)
			{
				if (!String.IsNullOrEmpty(Parts[Part]))
				{
					var SubParts = Parts[Part].Split('|');

					foreach (var SubPart in SubParts)
					{
						UnrealTargetPlatform Platform;
						if (UnrealTargetPlatform.TryParse(SubPart, out Platform))
						{
							Platforms.Add(Platform);
						}
						else
						{
							switch (SubPart.ToUpperInvariant())
							{
								case "DEBUG":
									Configurations.Add(UnrealTargetConfiguration.Debug);
									break;
								case "DEBUGGAME":
									Configurations.Add(UnrealTargetConfiguration.DebugGame);
									break;
								case "DEVELOPMENT":
									Configurations.Add(UnrealTargetConfiguration.Development);
									break;
								case "SHIPPING":
									Configurations.Add(UnrealTargetConfiguration.Shipping);
									break;
								case "TEST":
									Configurations.Add(UnrealTargetConfiguration.Test);
									break;
								default:
									throw new AutomationException("BUILD FAILED target option {0} not recognized.", SubPart);
							}
						}

					}
				}
			}
			if (Platforms.Count < 1)
			{
				Platforms.Add(UnrealTargetPlatform.Win64);
			}
			if (Configurations.Count < 1)
			{
				Configurations.Add(UnrealTargetConfiguration.Development);
			}
			foreach (var Platform in Platforms)
			{
				foreach (var CurTarget in Targets)
				{
					foreach (var Configuration in Configurations)
					{
						Agenda.AddTargets(new string[] { CurTarget }, Platform, Configuration, ProjectFile, UbtArgs);
						Logger.LogInformation("Target {CurTarget} {Arg1} {Arg2}", CurTarget, Platform.ToString(), Configuration.ToString());
						if (Clean)
						{
							string Args = String.Format("{0} {1} {2}", CurTarget, Platform.ToString(), Configuration.ToString());
							RunAndLog(CmdEnv, CleanToolLocation, Args);
						}
					}
				}
			}
		}
		Logger.LogInformation("*************************");

		UnrealBuild.Build(Agenda, InUpdateVersionFiles: IsBuildMachine);

		// 		if (WorkingCL > 0) // only move UAT files if we intend to check in some build products
		// 		{
		// 			UnrealBuild.CopyUATFilesAndAddToBuildProducts();
		// 		}

		UnrealBuild.CheckBuildProducts(UnrealBuild.BuildProductFiles);

		if (WorkingCL > 0)
		{
			// Sign everything we built
			CodeSign.SignMultipleIfEXEOrDLL(this, UnrealBuild.BuildProductFiles);

			// Open files for add or edit
			UnrealBuild.AddBuildProductsToChangelist(WorkingCL, UnrealBuild.BuildProductFiles);

			int SubmittedCL;
			P4.Submit(WorkingCL, out SubmittedCL, true, true);
		}

		PrintRunTime();

	}
}
