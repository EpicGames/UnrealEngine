// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.IO;
using System.Collections.Generic;
using System.Linq;
using System.Text;
using AutomationTool;
using EpicGames.Core;
using UnrealBuildBase;
using Microsoft.Extensions.Logging;


[Help("MemreportToHelper is used to take a memreport file, extract the CSV sections and generate a GSheet from them (i.e:RunUAT.bat MemReportToGSheet -report=myFile.memreport -name=SheetName (optional)")]
public class MemreportHelper : BuildCommand
{

	class MRCsvBlock
	{

		public int LineIndex;
		public string Title;
		public string SizeKey;
		public List<Dictionary<string, string>> SortedRows = new List<Dictionary<string, string>>();
		public StringBuilder sb = new StringBuilder();

	}

	private string[] InputFileAllLines;
	private List<MRCsvBlock> CsvBlocks = new List<MRCsvBlock>();
	private bool ReadCsvListAndAdd(string ListHeader, int LineOffset, string SizeKey, bool SkipCsvRequired=false)
	{
		bool FoundCsv = false;
		int StartIndex = InputFileAllLines.FindIndex (s => s.StartsWith(ListHeader));

		if(StartIndex != -1)
		{
			if (InputFileAllLines[StartIndex].Contains("-csv") || SkipCsvRequired)
			{
				// hackily remove duped list (these come through as alphasort + resourcesizesort)
				if (InputFileAllLines[StartIndex].Contains("-alphasort") == false)
				{
					int HeaderIndex = StartIndex + LineOffset;
					int DataIndex = HeaderIndex + 1;
					string[] HeaderValues = InputFileAllLines[HeaderIndex].Split(',');

					if (HeaderValues.Length > 1)
					{
						List<Dictionary<string, string>> Rows = new List<Dictionary<string, string>>();
						MRCsvBlock block = new MRCsvBlock();

						block.LineIndex = StartIndex;
						block.Title = InputFileAllLines[StartIndex];
						block.SizeKey = SizeKey;

						// Parse the rest of the data
						while (DataIndex < InputFileAllLines.Length)
						{
							Dictionary<string, string> OneRow = new Dictionary<string, string>();
							string Line = InputFileAllLines[DataIndex];
							DataIndex++;

							string[] Values = Line.Split(',');
							if (Values.Length >= HeaderValues.Length)
							{
								// used to remove the starting blank column on obj lists
								int startindex = SkipCsvRequired ? 0 : 1;
								for (int i = startindex; i < HeaderValues.Length; i++)
								{
									OneRow[HeaderValues[i].Trim()] = Values[i].Trim();
								}
								Rows.Add(OneRow);
							}
							else
							{
								break;
							}
						}
						if (Rows.Count > 0)
						{
							block.SortedRows = Rows.OrderByDescending(x => float.Parse(x[SizeKey])).ToList();
						}

						CsvBlocks.Add(block);
					}
				}

				// replace the line so subsequent calls do not find the same array
				InputFileAllLines[StartIndex] = InputFileAllLines[StartIndex].Replace(ListHeader, "ListReplaced");

				FoundCsv = true;
			}
		}

		return FoundCsv;
	}

	public override ExitCode Execute()
	{
		Logger.LogInformation("************************ STARTING MemreportHelper ************************");

		if (Params.Length < 1)
		{
			Logger.LogError("Invalid number of arguments {Arg0}", Params.Length);
			return ExitCode.Error_Arguments;
		}

		string MemreportFileorPath = ParseParamValue("report", "");
		string InOutputDir = ParseParamValue("outcsvdir", "");
		string MaxOutputString = ParseParamValue("maxoutputcount", "");
		string MinKBLim = ParseParamValue("minkblim", "");
		int MaxOutputCount = 75;
		float MinOutputKBLimit = 1 * 1024;
		
		if (string.IsNullOrWhiteSpace(MemreportFileorPath))
		{
			Logger.LogError("No memreport file or directory specified");
			return ExitCode.Error_Arguments;
		}

		

		if (string.IsNullOrEmpty(MaxOutputString) == false)
		{
			MaxOutputCount = int.Parse(MaxOutputString);
		}

		if (string.IsNullOrEmpty(MinKBLim) == false)
		{
			MinOutputKBLimit = int.Parse(MinKBLim);
		}

		float MeshMin = MinOutputKBLimit / 2;
		float SkelMeshMin = MinOutputKBLimit / 2;

		

		List<string> FilesToProcess = new List<string>();

		if (Directory.Exists(MemreportFileorPath))
		{
			var FilesReturned = Directory.EnumerateFiles(MemreportFileorPath, "*.memreport");

			foreach(string file in FilesReturned)
			{
				FilesToProcess.Add(file);
			}
		}
		else
		{
			if (File.Exists(MemreportFileorPath) == false)
			{
				Logger.LogError("{Text}", "Memreport file not found: " + MemreportFileorPath);
				return ExitCode.Error_Arguments;
			}

			FilesToProcess.Add(MemreportFileorPath);
		}

		foreach (string MRFile in FilesToProcess)
		{
			string OutputDir = InOutputDir;

			if (string.IsNullOrEmpty(OutputDir))
			{
				OutputDir = Path.Combine(Path.GetDirectoryName(MRFile), Path.GetFileName(MRFile).Replace('.', '_'));
			}

			// Pull the file into a buffer of lines (this will be stomped by the process)
			InputFileAllLines = File.ReadAllLines(MRFile);
			CsvBlocks.Clear();

			bool CsvFound = ReadCsvListAndAdd("Obj List:", 3, "ResExcKB");

			if (CsvFound)
			{
				// look for other 'obj list' entries only if we found the main Obj List

				while (CsvFound)
				{
					CsvFound = ReadCsvListAndAdd("Obj List:", 3, "ResExcKB");
				}

				// search for 'Listing NONVT textures.' and its ilk, these are ad-hoc and not explicitly set to -csv 
				// order here is important for latest badness filtering to ensure badness from "all" is unique from NONVT/UNCOMP list
				ReadCsvListAndAdd("Listing NONVT textures", 1, "Current Size (KB)", true);
				ReadCsvListAndAdd("Listing uncompressed textures.", 1, "Current Size (KB)", true);
				ReadCsvListAndAdd("Listing all textures.", 1, "Current Size (KB)", true);

				foreach (var csvBlock in CsvBlocks)
				{

					csvBlock.sb.AppendLine(csvBlock.Title);

					// write out headers
					if (csvBlock.Title.Contains("-csv"))
					{
						foreach (var key in csvBlock.SortedRows[0].Keys)
						{
							csvBlock.sb.Append(key + ",");
						}
						csvBlock.sb.Append(Environment.NewLine);
						foreach (var row in csvBlock.SortedRows)
						{
							foreach (var ele in row.Values)
							{
								csvBlock.sb.Append(ele + ",");
							}
							csvBlock.sb.Append(Environment.NewLine);
						}
						csvBlock.sb.Append(Environment.NewLine);
						csvBlock.sb.Append(Environment.NewLine);
					}
					else
					{
						// Max Height  Max Size(KB)   Bias Authored   Current Width   Current Height  Current Size(KB)   Format LODGroup    Name Streaming   UnknownRef VT  Usage Count NumMips Uncompressed
						// These are the explicit textures lists, slightly different formating

						var headers = new[]
						{
						"Name",
						"Format",
						"Current Size (KB)",
						"Current Width",
						"Current Height",
						"Max Size (KB)",
						"Max Width",
						"Max Height",
						"LODGroup",
						"Uncompressed",
						"VT",
						"Streaming",
						"UnknownRef",
						"Usage Count",
						"NumMips",
					};

						csvBlock.sb.AppendLine(string.Join(",", headers));

						foreach (var row in csvBlock.SortedRows)
						{
							csvBlock.sb.AppendLine(string.Join(",", headers.Select(h => row[h])));
						}

						csvBlock.sb.AppendLine();
						csvBlock.sb.AppendLine();
					}

				}

				if (!DirectoryExists(OutputDir))
				{
					CreateDirectory(OutputDir);
				}

				string OutputRootFileName = Path.Combine(OutputDir, Path.GetFileName(MRFile));

				File.WriteAllText(OutputRootFileName + ".csv", "");

				// output the WHOLE set to 1 file + each set to individual files
				foreach (var csvBlock in CsvBlocks)
				{
					File.AppendAllText(OutputRootFileName + ".csv", csvBlock.sb.ToString());

					string SanitizedFileName = OutputRootFileName + "." + csvBlock.Title.Replace("-csv", "").Replace(' ', '_').Replace('-', '_').Replace(':', '_');

					File.WriteAllText(SanitizedFileName + ".csv", csvBlock.sb.ToString());
				}

				{
					// Now output a 'badness' file, this is heavily tailored to workflows in frosty right now (9/20/21 andrew.firth)

					List<string> PreOutput = new List<string>();

					StringBuilder sb = new StringBuilder();

					// Output "bad" textures
					{
						var TextureHeaders = new[]
						{
						"Name",
						"Format",
						"Current Size (KB)",
						"Current Width",
						"Current Height",
						"Uncompressed",
						"VT",
						"Streaming",
					};

						sb.AppendLine("Type," + string.Join(",", TextureHeaders));

						foreach (var csvBlock in CsvBlocks)
						{
							// limit this to just "all textures" 
							//if (csvBlock.Title.Contains("Listing NONVT textures") || csvBlock.Title.Contains("Listing uncompressed textures") || csvBlock.Title.Contains("Listing all textures."))
							if (csvBlock.Title.Contains("Listing all textures."))
							{
								sb.AppendLine(csvBlock.Title);
								int OutputCount = 0;
								// output top 10 enties, limited info
								foreach (var row in csvBlock.SortedRows)
								{
									if (PreOutput.Contains(row["Name"]) == false)
									{
										sb.AppendLine("," + string.Join(",", TextureHeaders.Select(h => row[h])));

										PreOutput.Add(row["Name"]);

										// handle limiting the badness output based on user options
										OutputCount++;
										if (OutputCount > MaxOutputCount)
										{
											break;
										}

										if (float.Parse(row["Current Size (KB)"]) < MinOutputKBLimit)
										{
											break;
										}
									}
								}
								sb.AppendLine("");
							}
						}
					}

					// Output "bad" meshes
					{
						var MeshHeaders = new[]
						{
						"Object",
						"ResExcKB"
					};

						sb.AppendLine("Type," + string.Join(",", MeshHeaders));

						foreach (var csvBlock in CsvBlocks)
						{
							if (csvBlock.Title.Contains("Obj List: class=StaticMesh -alphasort"))
							{
								sb.AppendLine(csvBlock.Title);
								int OutputCount = 0;
								// output top 10 enties, limited info
								foreach (var row in csvBlock.SortedRows)
								{
									if (PreOutput.Contains(row["Object"]) == false)
									{
										sb.AppendLine("," + string.Join(",", MeshHeaders.Select(h => row[h])));

										PreOutput.Add(row["Object"]);

										// handle limiting the badness output based on user options
										OutputCount++;
										if (OutputCount > MaxOutputCount)
										{
											break;
										}

										if (float.Parse(row["ResExcKB"]) < MeshMin)
										{
											break;
										}
									}
								}
								sb.AppendLine("");
							}
						}
					}

					// Output "bad" skelmesh
					{
						var MeshHeaders = new[]
						{
						"Object",
						"ResExcKB"
					};

						sb.AppendLine("Type," + string.Join(",", MeshHeaders));

						foreach (var csvBlock in CsvBlocks)
						{
							if (csvBlock.Title.Contains("Obj List: class=SkeletalMesh"))
							{
								sb.AppendLine(csvBlock.Title);
								int OutputCount = 0;
								// output top 10 enties, limited info
								foreach (var row in csvBlock.SortedRows)
								{
									if (PreOutput.Contains(row["Object"]) == false)
									{
										sb.AppendLine("," + string.Join(",", MeshHeaders.Select(h => row[h])));

										PreOutput.Add(row["Object"]);

										// handle limiting the badness output based on user options
										OutputCount++;
										if (OutputCount > MaxOutputCount)
										{
											break;
										}

										if (float.Parse(row["ResExcKB"]) < SkelMeshMin)
										{
											break;
										}
									}
								}
								sb.AppendLine("");
							}
						}
					}


					File.WriteAllText(OutputRootFileName + ".badness.csv", sb.ToString());
				}
			}
		}

		Logger.LogInformation("************************ MemreportHelper WORK COMPLETED ************************");

		return ExitCode.Success;
	}

}



