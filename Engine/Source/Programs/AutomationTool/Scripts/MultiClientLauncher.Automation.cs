// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Diagnostics;
using System.IO;
using System.Threading;
using Amazon;
using AutomationTool;
using EpicGames.Core;
using Microsoft.CodeAnalysis.CSharp.Syntax;
using Microsoft.Extensions.Logging;
using UnrealBuildBase;
using UnrealBuildTool;

namespace MultiClientLauncher.Automation
{
	[Help("Run many game clients, a server, and connect them")]
	[ParamHelp("ClientExe", "Absolute path to the client to run", ParamType = typeof(FileReference))]
	[ParamHelp("ClientCount", "How many bot clients to run, must consecutively follow FirstClientNumber", ParamType = typeof(int))]
	[ParamHelp("BuildIdOverride", "Parameter for -buildidoverride switch, often used to narrow down matchmaking to a particular server (optional)", ParamType = typeof(int))]
	[ParamHelp("ClientArgsFile", "Absolute path to a file containing the client arguments (Engine/Build/AutomationWorkflows/ManyBotClientsDefault.txt by default)", ParamType = typeof(FileReference))]
	[ParamHelp("ClientLogDir", "Absolute path to the directory with client log files (relative to the exe by default)", ParamType = typeof(FileReference))]
	[ParamHelp("FirstClientNumber", "The number of the first LoadBot client (0 by default)", ParamType = typeof(int))]
	[ParamHelp("NullRHI", "Pass -nullrhi to the clients, defaults to false", ParamType = typeof(bool))]
	[ParamHelp("GridLayout", "If clients aren't nullrhi, lay them out in 320x240 fashion, defaults to true", ParamType = typeof(bool))]
	[ParamHelp("NoTimeouts", "Disable timeouts in this script (defaults to false)", ParamType = typeof(bool))]
	[ParamHelp("SleepTimeBetweenLaunches", "How long to sleep between running clients in milliseconds (could prevent race conditions), 100 by default", ParamType = typeof(int))]
	[ParamHelp("MaxRunAttemptsPerClient", "Maximum number of attempts to run a client which crashes or fails to connect to the server, defaults to 3", ParamType = typeof(int))]
	[ParamHelp("ClientSessionCompleted", "Log message indicating that a client has completed a game session and may be terminated", ParamType = typeof(string))]
	[ParamHelp("ClientFailed", "Log message indicating that a client failed to connect to the server", ParamType = typeof(string))]
	[ParamHelp("ClientConnected", "Log message indicating that a client connected to the server", ParamType = typeof(string))]
	[ParamHelp("DeleteExistingLogs", "Whether to clear the log directory before launching clients, defaults to true", ParamType = typeof(bool))]
	public class MultiClientLauncher : BuildCommand
	{
		private bool CancelClientProcesses = false;
		
		private string ClientExe;
		private string ClientLogDir;
		private string ClientArgs;
		private string ClientLogFilenameGuess;

		private int FirstClientNumber;
		private int ClientCount;
		private int BuildIdOverride;
		private bool NullRHI = false;
		private bool GridLayout = true;
		private bool NoTimeouts = false;
		private bool DeleteExistingLogs = true;

		private int SleepTimeBetweenLaunches = 100;
		private int MaxRunAttemptsPerClient = 3;

		private const int SleepTimeBetweenChecksForClientRelaunches = 1000;

		protected ClientLogIndicators ClientIndicators;
		
		private void ParseCommandLine()
		{
			FileReference ClientExeFile = ParseRequiredFileReferenceParam("ClientExe");
			ClientExe = ClientExeFile.ToString();
			ClientCount = int.Parse(ParseRequiredStringParam("ClientCount"));
			BuildIdOverride = ParseParamInt("BuildIdOverride", -1);

			// guess the log filename from the binary, e.g. FooClient-Linux-Shipping -> FooGame
			ClientLogFilenameGuess = ClientExeFile.GetFileNameWithoutAnyExtensions();
			if (ClientLogFilenameGuess.Contains("-"))
			{
				ClientLogFilenameGuess = ClientLogFilenameGuess.Split('-')[0];
			}
			if (ClientLogFilenameGuess.Contains("Client"))
			{
				ClientLogFilenameGuess = ClientLogFilenameGuess.Replace("Client", "Game");
			}

			// file comm sucks and is unreliable, but better than nothing
			ClientLogDir = ParseParamValue("ClientLogDir", "");
			if (string.IsNullOrEmpty(ClientLogDir))
			{
				// figure out from ClientExe path
				ClientLogDir = Utils.CollapseRelativeDirectories(CommandUtils.CombinePaths(ClientExeFile.Directory.ToString(), "../../Saved/Logs"));
			}

			FileReference ClientArgsFileRef = new FileReference(ParseParamValue("ClientArgsFile", GetDefaultArgsFile()));
			if (!FileReference.Exists(ClientArgsFileRef))
			{
				throw new BuildException("ClientArgs file {0} does not exist (override with -ClientArgsFile=...)", ClientArgsFileRef);
			}
			ClientArgs = FileReference.ReadAllText(ClientArgsFileRef);
			if (BuildIdOverride != -1)
			{
				ClientArgs += string.Format(" -buildidoverride={0} ", BuildIdOverride);
			}

			FirstClientNumber = int.Parse(ParseParamValue("FirstClientNumber", "0"));

			SleepTimeBetweenLaunches = ParseParamInt("SleepTimeBetweenLaunches", -1);
			MaxRunAttemptsPerClient = ParseParamInt("MaxRunAttemptsPerClient", 3);
			NullRHI = ParseParamBool("NullRHI", NullRHI);
			GridLayout = ParseParamBool("GridLayout", GridLayout);
			NoTimeouts = ParseParamBool("NoTimeouts", NoTimeouts);
			DeleteExistingLogs = ParseParamBool("DeleteExistingLogs", DeleteExistingLogs);

			// disable grid layout for nullrhi
			if (NullRHI || ClientArgs.Contains("-nullrhi"))
			{
				GridLayout = false;
			}

			if (NullRHI)
			{
				ClientArgs += " -nullrhi ";
			}

			InitializeClientLogIndicators();
		}

		protected virtual string GetDefaultArgsFile()
		{
			return "Engine/Build/AutomationWorkflows/ManyBotClientsDefault.txt";
		}

		// Derived commands may hardcode these log indicators
		protected virtual void InitializeClientLogIndicators()
		{
			ClientIndicators.FinishedGameAndDisconnected = ParseRequiredStringParam("ClientSessionCompleted");
			ClientIndicators.FailedToConnectToServer = ParseRequiredStringParam("ClientFailed");
			ClientIndicators.ConnectedToServer = ParseRequiredStringParam("ClientConnected");
		}
		
		public override ExitCode Execute()
		{
			ParseCommandLine();
			
			List<ClientProcess> ClientProcesses = new List<ClientProcess>();
			
			// Allow ctrl C to terminate all client processes
			Console.CancelKeyPress += delegate
			{
				CancelClientProcesses = true;
				KillProcesses(ClientProcesses);
			};

			if (DeleteExistingLogs)
			{
				// Delete all previous log files
				Console.WriteLine("Deleting all existing log files in the log directory {0}", ClientLogDir);
				string[] LogFiles = Directory.GetFiles(ClientLogDir);
				foreach (string Filename in LogFiles)
				{
					if (Filename.EndsWith(".log"))
					{
						File.Delete(Filename);
					}
				}
			}

			try
			{
				// Run all client processes
				Console.WriteLine("Spawning clients.");
				for (int ClientIdx = 0; ClientIdx < ClientCount; ++ClientIdx)
				{
					if (CancelClientProcesses)
					{
						break;
					}

					Console.WriteLine("Spawning client {0}...", ClientIdx);

					string ClientNumber = (FirstClientNumber + ClientIdx).ToString("00000.##");
					string CurrentClientArgs = ClientArgs.Replace("#REPLACED_WITH_FIVE_DIGIT_CLIENT_ID#", ClientNumber);

					string CurrentClientLog = CommandUtils.CombinePaths(ClientLogDir, ClientLogFilenameGuess);
					if (ClientIdx > 0)
					{
						CurrentClientLog += "_" + (ClientIdx + 1);
					}
					CurrentClientLog += ".log";

					CurrentClientArgs += string.Format(" -abslog={0} ", CurrentClientLog);

					if (GridLayout)
					{
						// assume 4k screen, which can fit 90 (10x9) clients running in 384x240. Note - not trying 320x240 as these days client will not use 4:3 aspect ratio
						const int ResX = 384;
						const int ResY = 240;
						const int ClientsPerRow = 3840 / ResX;
						int WinY = ResY * (ClientIdx / ClientsPerRow);
						int WinX = ResX * (ClientIdx % ClientsPerRow);
						CurrentClientArgs += string.Format(" -WinX={0} -WinY={1} -ResX={2} -ResY={3} ", WinX, WinY, ResX, ResY);
					}

					System.Console.WriteLine("Args: {0}", CurrentClientArgs);
					ClientProcesses.Add(SpawnClientProcess(ClientExe, CurrentClientLog, CurrentClientArgs));

					if (SleepTimeBetweenLaunches != -1)
					{
						Console.WriteLine("Sleeping for {0}ms...", SleepTimeBetweenLaunches);
						Thread.Sleep(SleepTimeBetweenLaunches);
					}
				}

				while (!CancelClientProcesses && ClientProcesses.Count > 0)
				{
					// Iterate through clients in reverse order for safe removal
					for (int i = ClientProcesses.Count - 1; i >= 0; i--)
					{
						if (ClientProcesses[i].Stopped())
						{
							bool OutOfTries = !ClientProcesses[i].Start();

							if (OutOfTries)
							{
								ClientProcesses.RemoveAt(i);
							}
						}
					}

					// Todo: Consider replacing sleeps with waiting for LogSkimmer threads to finish reading
					// Todo:   - That may still happen very fast and result in more busywaiting
					Thread.Sleep(SleepTimeBetweenChecksForClientRelaunches);
				}

				return ExitCode.Success;
			}
			catch(Exception)
			{
				KillProcesses(ClientProcesses);
				return ExitCode.Error_Unknown;
			}
		}
		
		private ClientProcess SpawnClientProcess(string ExeFilename, string ClientLogFilename, string ExeArguments)
		{
			ClientProcess ClientProc = new ClientProcess(ExeFilename, ExeArguments, MaxRunAttemptsPerClient, NoTimeouts, ClientLogFilename, ClientIndicators);
			ClientProc.Start();
			return ClientProc;
		}

		private static void KillProcesses(List<ClientProcess> Processes)
		{
			Console.WriteLine("Killing all client processes");
			foreach (ClientProcess CurrentProcess in Processes)
			{
				CurrentProcess.Kill();
			}
		}

		protected struct ClientLogIndicators
		{
			public string FinishedGameAndDisconnected;
			public string FailedToConnectToServer;
			public string ConnectedToServer;
		}

		private class ClientProcess
		{
			private readonly Process Proc;
			private int RemainingRunAttempts;

			private Thread LogSkimmer;
			private const int SleepTimeBetweenLogSkims = 1000;
			private readonly ClientLogIndicators LogIndicators;
			private readonly string LogFilepath;

			private readonly Stopwatch ConnectionTimer;
			private const int MinutesUntilTimeout = 15; // Todo: maybe make this a command line argument
			private bool NoTimeouts = false;

			public ClientProcess(string Exe, string Args, int MaxRunAttempts, bool IgnoreTimeouts, string ClientLog, ClientLogIndicators ClientIndicators)
			{
				Proc = new Process();
				Proc.StartInfo.FileName = Exe;
				Proc.StartInfo.Arguments = Args;
				RemainingRunAttempts = MaxRunAttempts;
				NoTimeouts = IgnoreTimeouts;

				LogFilepath = ClientLog;
				LogIndicators = ClientIndicators;

				ConnectionTimer = new Stopwatch();
			}
			
			public bool Stopped()
			{
				return Proc.HasExited && !LogSkimmer.IsAlive;
			}

			// Returns false if the client is out of attempts to start the process
			public bool Start()
			{
				if (RemainingRunAttempts <= 0)
				{
					return false;
				}
				RemainingRunAttempts--;
				
				bool Success = Proc.Start();
				if (!Success)
				{
					Console.WriteLine("Failed to start process for {0}", Proc.StartInfo.FileName);
					return true;
				}

				ConnectionTimer.Restart();

				LogSkimmer = new Thread(MonitorClient);
				LogSkimmer.Start();

				return true;
			}

			private string GetProcessName()
			{
				return "(" + Proc.ProcessName + ": " + Proc.Id + ")";
			}

			public void Kill()
			{
				Proc.Kill();
			}

			private void MonitorClient()
			{
				// Wait long enough for log files to be created
				double MaxSecondsToWait = 10;
				double WaitedSoFar = 0;
				do
				{
					if (File.Exists(LogFilepath))
					{
						break;
					}

					Thread.Sleep(2000);
					WaitedSoFar += 2;

					if (WaitedSoFar >= MaxSecondsToWait)
					{
						throw new BuildException("Log file {0} was not created after {1} seconds (did process crash on start?)", LogFilepath, WaitedSoFar);
					}
				}
				while (true);

				string AllServerOutput = "";
				using FileStream ProcessLog = File.Open(LogFilepath, FileMode.Open, FileAccess.Read, FileShare.ReadWrite);
				using StreamReader LogReader = new StreamReader(ProcessLog);

				// Read until the process has exited or we found the success text in the log
				while (!Proc.HasExited)
				{
					while (!LogReader.EndOfStream)
					{
						string Output = LogReader.ReadToEnd();
						if (string.IsNullOrEmpty(Output))
						{
							continue;
						}

						AllServerOutput += Output;

						if (ConnectionTimer.IsRunning)
						{
							// If a client connected to the server, there's no more need to check for timing out
							if (AllServerOutput.Contains(LogIndicators.ConnectedToServer))
							{
								Console.WriteLine("Client {0} successfully connected to the server.", GetProcessName());
								ConnectionTimer.Stop();
								break;
							}

							// If client failed to connect, kill process and decrement attempts
							if (AllServerOutput.Contains(LogIndicators.FailedToConnectToServer))
							{
								Console.WriteLine("Client {0} failed to connect to server. Relaunching client. Attempts left: {1}",
									GetProcessName(), RemainingRunAttempts);
								Kill();
								return;
							}
						}
						else
						{
							// If the client completed a session, kill the process
							if (AllServerOutput.Contains(LogIndicators.FinishedGameAndDisconnected))
							{
								RemainingRunAttempts = 0;
								Console.WriteLine("Client {0} completed session.", GetProcessName());
								Kill();
								return;
							}
						}
					}

					// If timed out while attempting to connect, kill process and decrement attempts
					if (!NoTimeouts && ConnectionTimer.IsRunning && ConnectionTimer.Elapsed.Minutes >= MinutesUntilTimeout)
					{
						Console.WriteLine("Client {0} timed out. Attempts left: {1}", GetProcessName(), RemainingRunAttempts);
						Kill();
						return;
					}
					
					// Wait for more logging to occur
					Thread.Sleep(SleepTimeBetweenLogSkims);
				}
			}
		}
	}
}