// Copyright Epic Games, Inc. All Rights Reserved.

using AutomationTool;
using System;
using System.Collections.Generic;
using System.Linq;
using System.IO;
using Microsoft.Extensions.Logging;
using EpicGames.Core;
using UnrealBuildBase;


namespace BuildScripts.Automation
{
	[RequireP4]
	[DoesNotNeedP4CL]
	[Help("Shelves multiple changelists into one (useful for preflighting multiple changes together")]
	[Help("TargetCL=<changelist>", "If not specified, the script will create a new changelist to shelve into, and write out the CL number")]
	[Help("SourceCLs=<cl,cl,...>", "List of changelists to shelve into the TargetCL")]
	class MultiShelve : BuildCommand
	{
		public override void ExecuteBuild()
		{
			string SourceCLsParam = ParseRequiredStringParam("SourceCLs");
			IEnumerable<int> SourceCLs = SourceCLsParam.Split(",", StringSplitOptions.TrimEntries).Select(x => int.Parse(x));
			int TargetCL = ParseParamInt("TargetCL", -1);
			bool bAllowSpew = ParseParam("AllowSpew");
			bool bCleanTarget = ParseParam("CleanTarget");

			P4Connection P4 = new P4Connection(null, null);


			if (TargetCL == -1)
			{
				TargetCL = P4.CreateChange(Description: $"Shelving changelists " + string.Join(", ", SourceCLs));
				Logger.LogInformation("Creating target changelist >> {TargetCL} << to shelve into", TargetCL);
				Logger.LogInformation("");
			}
			// make sure there aren't files already in the target CL
			else
			{
				if (P4.ChangeFiles(TargetCL, out _, AllowSpew: bAllowSpew).Count > 0)
				{
					Logger.LogInformation("");
					Logger.LogInformation("Changelist {TargetCL} already contains files. Note that these files will be reshelved for each source CL.", TargetCL);
				}

				if (bCleanTarget)
				{
					Logger.LogInformation("");
					Logger.LogInformation("Deleting shelved files in target CL {TargetCL}", TargetCL);
					P4.DeleteShelvedFiles(TargetCL, AllowSpew: bAllowSpew);
				}	
			}

			foreach (int CL in SourceCLs)
			{
				List<string> Files = P4.ChangeFiles(CL, out _, AllowSpew: bAllowSpew);

				Logger.LogInformation("");
				if (Files.Count == 0)
				{
					Logger.LogInformation("Skipping CL {CL}, it had no files!", CL);
					continue;
				}
				Logger.LogInformation("Shelving from {CL}:", CL);


				Logger.LogInformation("  Moving {Arg0} files from {CL}", Files.Count, CL);
				P4.Reopen(TargetCL, Files, AllowSpew: bAllowSpew);

				Logger.LogInformation("  Shelving...");
				P4.ShelveNoRevert(TargetCL, AllowSpew: bAllowSpew);

				Logger.LogInformation("  Moving back...");
				P4.Reopen(CL, Files, AllowSpew: bAllowSpew);
			}
			Logger.LogInformation("");
			Logger.LogInformation("All done!");
			Logger.LogInformation("");
			Logger.LogInformation("");
		}
	}
}
