// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.IO;
using System.Linq;
using System.Text.RegularExpressions;
using EpicGames.Core;
using UnrealBuildBase;
using UnrealBuildTool;
using Microsoft.Extensions.Logging;

using static AutomationTool.CommandUtils;

namespace AutomationTool
{

	[Help("Opens the specified project.")]
	[Help("project=<QAGame>", "Project to open. Will search current path and paths in ueprojectdirs. If omitted will open vanilla UnrealEditor")]
	[ParamHelp("Run", "The name of the commandlet to run")]
	public class OpenEditor : BuildCommand
	{
		// exposed as a property so projects can derive and set this directly
		public string ProjectName { get; set; } = string.Empty;

		private string _UnrealEditorApp = "UnrealEditor";
		public string UnrealEditorApp
		{
			get
			{
				return _UnrealEditorApp;
			}
			set
			{
				_UnrealEditorApp = value;
			}
		}

		public bool bNoProject { get; set; }

		public OpenEditor()
		{
			bNoProject = false;
		}

		protected IEnumerable<string> GetParamList()
		{
			string EditorArgs = "";
			ProjectName = (string.IsNullOrEmpty(ProjectName) || !Path.IsPathRooted(ProjectName)) ? ParseParamValue("project", ProjectName) : ProjectName;
			IEnumerable<string> ParamList = null;

			if (bNoProject)
			{
				ParamList = Params;
			}
			else
			{
				if (Path.IsPathRooted(ProjectName) && File.Exists(ProjectName))
				{
					EditorArgs = ProjectName;
				}
				else if (!string.IsNullOrEmpty(ProjectName))
				{
					FileReference ProjectFile = ProjectUtils.FindProjectFileFromName(ProjectName);

					if (ProjectFile == null)
					{
						throw new AutomationException("Unable to find uproject file for {0}", ProjectName);
					}

					EditorArgs = ProjectFile.FullName;
				}

				// filter out any -project argument since we want it to be the first un-prefixed argument to the editor 
				ParamList = this.Params
								.Where(P => P.StartsWith("project=", StringComparison.OrdinalIgnoreCase) == false);
			}
			
			return new[] { EditorArgs }.Concat(ParamList);
		}

		public override ExitCode Execute()
		{
			string EditorPath = string.Empty;
			if (Path.IsPathRooted(UnrealEditorApp) && File.Exists(UnrealEditorApp))
			{
				EditorPath = UnrealEditorApp;
			}
			else
			{
				EditorPath = HostPlatform.Current.GetUnrealExePath(UnrealEditorApp);
			}

			bool bLaunched = RunProceess.RunUntrackedProcess(EditorPath, string.Join(" -", GetParamList()));

			return bLaunched ? ExitCode.Success : ExitCode.Error_UATLaunchFailure;
		}
	}

	public class RunProceess
	{ 
		public static bool RunUntrackedProcess(string BinaryPath, string Args)
		{
			Logger.LogInformation("Running {BinaryPath} {Args}", BinaryPath, Args);

			var NewProcess = HostPlatform.Current.CreateProcess(BinaryPath);
			var Result = new ProcessResult(BinaryPath, NewProcess, false, false);
			System.Diagnostics.Process Proc = Result.ProcessObject;

			Proc.StartInfo.FileName = BinaryPath;
			Proc.StartInfo.Arguments = string.IsNullOrEmpty(Args) ? "" : Args;
			Proc.StartInfo.UseShellExecute = false;
			return Proc.Start();
		}
	}
}
