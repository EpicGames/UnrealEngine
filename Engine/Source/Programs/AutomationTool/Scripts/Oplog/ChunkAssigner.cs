// Copyright Epic Games, Inc. All Rights Reserved.

using AutomationTool;
using EpicGames.Core;
using Microsoft.Extensions.Logging;
using System;
using System.Collections.Generic;
using UnrealBuildTool;
using static AutomationScripts.Oplog.OplogChunkAssigner;

namespace AutomationScripts.Oplog
{
#nullable enable
	/// <summary>
	/// Abstract base class for chunk assignment transforms.
	/// Games can provide a custom implementation by:
	/// <list type="number">
	///   <item>Subclassing <see cref="ChunkAssigner"/>.</item>
	///   <item>Tagging the subclass with <c>[ChunkAssigner("MyAssignerName")]</c>.</item>
	///   <item>Setting <c>[/Script/UnrealEd.ProjectPackagingSettings] ChunkAssigner=MyAssignerName</c>
	///         in DefaultGame.ini.</item>
	/// </list>
	/// If no custom assigner is configured, the engine default (<see cref="DefaultChunkAssigner"/>) is used.
	/// <para>
	/// This mirrors the <c>CustomStageCopyHandler</c> extensibility pattern.
	/// </para>
	/// </summary>
	public abstract class ChunkAssigner
	{
		/// <summary>
		/// Assigns packages in the graph to chunks and returns the structured
		/// <see cref="ChunkAssignments"/> result. OplogEntries can be transformed into a package
		/// graph via <see cref="PackageGraphBuilder.Build"/>. All package metadata is accessible
		/// via <see cref="PackageGraph.GetMetadata"/> and <see cref="PackageGraph.QueryNodes"/>.
		/// </summary>
		/// <param name="OplogEntries">The oplog entries read from the Zen cook store.</param>
		/// <param name="TargetPlatform">The target platform being staged.</param>
		/// <param name="Receipt">The build receipt for the staged target.</param>
		/// <returns>
		/// A <see cref="ChunkAssignments"/> with one <see cref="ContainerChunk"/> per pakchunk,
		/// each carrying its <see cref="ContainerChunkFile"/> list. Files identified by
		/// <see cref="ContainerChunkFile.ChunkId"/> alone may leave PackageName/FilePath unset;
		/// loose files (no IoChunkId) or files whose staged path diverges from the canonical
		/// cooked path must carry PackageName/FilePath explicitly.
		/// </returns>
		public abstract ChunkAssignments AssignChunks(IReadOnlyList<OplogEntry> OplogEntries, UnrealTargetPlatform TargetPlatform, TargetReceipt Receipt);

		/// <summary>
		/// Called by <see cref="CreateFromConfig"/> after instantiation to supply the project file and
		/// game config. Override to receive either or both
		/// (e.g. <see cref="PrimaryAssetChunkAssigner"/> needs <paramref name="gameIni"/>;
		/// a custom game-side assigner may need both).
		/// The default implementation is a no-op.
		/// </summary>
		public virtual void Configure(FileReference projectFile, ConfigHierarchy engineIni, ConfigHierarchy gameIni) { }

		/// <summary>
		/// Creates a <see cref="ChunkAssigner"/> by name, scanning all loaded script assemblies
		/// for a class tagged with <c>[ChunkAssignerAttribute("name")]</c>.
		/// </summary>
		/// <exception cref="BuildException">Thrown if the named assigner is not found or cannot be instantiated.</exception>
		public static ChunkAssigner Create(string name)
		{
			if (s_registry == null)
			{
				s_registry = BuildRegistry([typeof(ChunkAssigner)]);
			}

			if (!s_registry!.TryGetValue(name, out Type? type) || type == null)
			{
				throw new BuildException($"Unknown custom chunk assigner '{name}'. " +
					"Ensure the class is tagged with [ChunkAssigner(\"{name}\")] " +
					"and is in a loaded script assembly.");
			}

			ChunkAssigner? instance = (ChunkAssigner?)Activator.CreateInstance(type);
			if (instance == null)
			{
				throw new BuildException($"Could not instantiate custom chunk assigner '{name}'.");
			}

			return instance;
		}

		/// <summary>
		/// Creates a <see cref="ChunkAssigner"/> by reading the ini key
		/// <c>[/Script/UnrealEd.ProjectPackagingSettings] ChunkAssigner</c>
		/// from the supplied game config. Falls back to <see cref="DefaultChunkAssigner"/>
		/// if the key is absent or the named type is not found.
		/// </summary>
		public static ChunkAssigner CreateFromConfig(FileReference projectFile, ConfigHierarchy engineIni, ConfigHierarchy gameIni)
		{
			gameIni.GetString("/Script/UnrealEd.ProjectPackagingSettings", "ChunkAssigner", out string? assignerName);
			if (!string.IsNullOrEmpty(assignerName))
			{
				Log.Logger.LogInformation("Using custom chunk assigner: {Name}", assignerName);
				try
				{
					ChunkAssigner namedAssigner = Create(assignerName!);
					namedAssigner.Configure(projectFile, engineIni, gameIni);
					return namedAssigner;
				}
				catch (Exception ex)
				{
					Log.Logger.LogWarning(
						"Failed to create custom chunk assigner '{Name}': {Message}. " +
						"Falling back to DefaultChunkAssigner.",
						assignerName, ex.Message);
				}
			}

			Log.Logger.LogInformation("Using default chunk assigner");
			return new DefaultChunkAssigner();
		}

		// ---- Registry (built once, lazily) ----
		internal static Dictionary<string, Type>? s_registry;
	}

	/// <summary>
	/// Attribute used to register a <see cref="ChunkAssigner"/> subclass by name.
	/// </summary>
	[AttributeUsage(AttributeTargets.Class)]
	public sealed class ChunkAssignerAttribute : OplogAttribute
	{
		public ChunkAssignerAttribute(string name) : base(name)
		{
		}
	}
#nullable disable
}
