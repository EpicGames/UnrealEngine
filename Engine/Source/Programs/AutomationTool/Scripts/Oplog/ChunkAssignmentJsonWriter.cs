// Copyright Epic Games, Inc. All Rights Reserved.

using EpicGames.Core;
using EpicGames.Serialization;
using Microsoft.Extensions.Logging;
using System.Collections.Generic;
using System.IO;
using System.Text.Json;

namespace AutomationScripts.Oplog
{
#nullable enable
	/// <summary>
	/// Writes the <see cref="ChunkAssignments"/> result both as a debug JSON file and as a
	/// Zen oplog op. The JSON path is a straight reflection-based serialization of the
	/// ContainerChunks list (full shape, every property emitted). The oplog op applies the
	/// "different or no IoChunkId" filtering rule so the on-wire payload only carries
	/// PackageName/FilePath when they can't be derived from the cook oplog by IoChunkId.
	/// </summary>
	public sealed class ChunkAssignmentJsonWriter
	{
		/// <summary>
		/// Writes <paramref name="assignments"/> to <c>chunk_assignments.json</c> inside
		/// <paramref name="outputDir"/>. Streams one <see cref="ContainerChunk"/> at a time
		/// straight to the file via <see cref="Utf8JsonWriter"/>
		/// </summary>
		public void Write(ChunkAssignments assignments, string outputDir)
		{
			Directory.CreateDirectory(outputDir);
			string chunkAssignmentJsonPath = Path.Combine(outputDir, "chunk_assignments.json");

			JsonSerializerOptions jsonOptions = new JsonSerializerOptions { WriteIndented = true };
			JsonWriterOptions writerOptions = new JsonWriterOptions { Indented = true };

			using (FileStream stream = File.Create(chunkAssignmentJsonPath))
			using (Utf8JsonWriter writer = new Utf8JsonWriter(stream, writerOptions))
			{
				writer.WriteStartArray();
				foreach (ContainerChunk chunk in assignments.ContainerChunks)
				{
					JsonSerializer.Serialize(writer, chunk, jsonOptions);
					// Flush after each chunk so the writer's internal buffer doesn't grow with
					// the whole document — peak memory stays ~one chunk regardless of total size.
					writer.Flush();
				}
				writer.WriteEndArray();
			}

			Log.Logger.LogInformation("Wrote chunk assignment JSON: {Path}", chunkAssignmentJsonPath);
		}

		/// <summary>
		/// Appends a single CbObject to the Zen oplog keyed <c>"ChunkAssignment"</c>. The op
		/// payload uses the same omission rules as the JSON output.
		/// </summary>
		public void WriteOp(ChunkAssignments assignments, OplogWriter oplogWriter)
		{
			var w = new CbWriter();
			w.BeginObject();
			w.BeginArray("chunks", CbFieldType.Object);
			foreach (ContainerChunk chunk in assignments.ContainerChunks)
			{
				w.BeginObject();
				w.WriteInteger("id", chunk.ContainerChunkId);
				if (!string.IsNullOrEmpty(chunk.Name))
				{
					w.WriteString("name", chunk.Name);
				}
				if (!string.IsNullOrEmpty(chunk.EncryptionKeyGuid))
				{
					w.WriteString("encguid", chunk.EncryptionKeyGuid);
				}
				w.BeginArray("files", CbFieldType.Object);
				foreach (ContainerChunkFile file in chunk.Files)
				{
					WriteFileCb(w, file);
				}
				w.EndArray();
				w.EndObject();
			}
			w.EndArray();
			w.EndObject();

			CbObject payload = w.ToObject();
			oplogWriter.AppendOp("ChunkAssignment", payload);
		}

		// ---- Private helpers ----

		private static void WriteFileCb(CbWriter w, ContainerChunkFile file)
		{
			w.BeginObject();

			bool hasChunkId = !file.ChunkId.IsEmpty;
			if (hasChunkId)
			{
				w.WriteHash("id", file.ChunkId.Hash);
			}

			bool pathDiffers = !string.IsNullOrEmpty(file.FilePath) &&
				!string.Equals(file.FilePath, file.UfsPath, System.StringComparison.OrdinalIgnoreCase);
			bool emitIdentity = !hasChunkId || pathDiffers;

			if (emitIdentity)
			{
				if (!string.IsNullOrEmpty(file.PackageName))
				{
					w.WriteString("pkg", file.PackageName);
				}
				if (!string.IsNullOrEmpty(file.FilePath))
				{
					w.WriteString("path", file.FilePath);
				}
			}

			if (file.Size != 0)
			{
				w.WriteInteger("size", file.Size);
			}

			if (file.BulkDataMap.Count > 0)
			{
				w.BeginArray("bulk_data_map", CbFieldType.Object);
				foreach (BulkDataMapEntry e in file.BulkDataMap)
				{
					w.BeginObject();
					w.WriteInteger("offset", e.Offset);
					w.WriteInteger("duplicate_offset", e.DuplicateOffset);
					w.WriteInteger("size", e.Size);
					w.WriteInteger("flags", e.Flags);
					w.WriteInteger("cooked_index", e.CookedIndex);
					w.EndObject();
				}
				w.EndArray();
			}

			if (HasNonZeroSegment(file.BulkDataSegments))
			{
				w.BeginArray("bulk_data_segments", CbFieldType.Object);
				foreach (BulkDataSegments seg in file.BulkDataSegments)
				{
					if (seg.Offset == 0 && seg.Size == 0 && seg.ChunkId.IsEmpty)
					{
						continue;
					}
					w.BeginObject();
					if (!seg.ChunkId.IsEmpty)
					{
						w.WriteHash("id", seg.ChunkId.Hash);
					}
					w.WriteInteger("offset", seg.Offset);
					w.WriteInteger("size", seg.Size);
					w.EndObject();
				}
				w.EndArray();
			}

			w.EndObject();
		}

		private static bool HasNonZeroSegment(IReadOnlyList<BulkDataSegments> segments)
		{
			foreach (BulkDataSegments seg in segments)
			{
				if (seg.Offset != 0 || seg.Size != 0 || !seg.ChunkId.IsEmpty)
				{
					return true;
				}
			}
			return false;
		}
	}
#nullable disable
}
