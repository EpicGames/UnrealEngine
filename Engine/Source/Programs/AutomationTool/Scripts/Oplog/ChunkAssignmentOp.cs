// Copyright Epic Games, Inc. All Rights Reserved.

using EpicGames.Core;
using EpicGames.Serialization;
using Microsoft.Extensions.Logging;
using System.Collections.Generic;

namespace AutomationScripts.Oplog
{
#nullable enable
	/// <summary>
	/// Reader-side counterpart to <c>ChunkAssignmentJsonWriter.WriteOp</c>. The op key is
	/// <c>"ChunkAssignment"</c>; the attachment payload is a CbObject containing a
	/// <c>chunks</c> array of <c>{ id, name?, encguid?, files: [ { id?, pkg?, path?, size? } ] }</c>.
	/// <c>id</c> on the file is the <see cref="IoChunkId"/> from the cook oplog's packagedata
	/// / bulkdata; <c>pkg</c> and <c>path</c> are populated only when the file has no
	/// IoChunkId or the staged path diverges from the canonical cook path.
	/// </summary>
	[OplogEntry("ChunkAssignmentOp")]
	public sealed class ChunkAssignmentOp : OplogEntry
	{
		public ChunkAssignments Assignments { get; private set; } = new ChunkAssignments();

		public override string GetKey() { return "ChunkAssignment"; }

		public override bool ParseData(CbField entryField, string BaseOplogURL)
		{
			CbField attachmentData = entryField["value"];
			CbObject attachment = DownloadAttachment(BaseOplogURL, attachmentData.AsAttachment());
			if (attachment == CbObject.Empty)
			{
				Log.Logger.LogWarning("ChunkAssignmentOp: empty attachment");
				return false;
			}

			foreach (CbField chunkField in attachment["chunks"].AsArray())
			{
				CbObject chunkObj = chunkField.AsObject();
				var container = new ContainerChunk
				{
					ContainerChunkId  = (int)chunkObj["id"].AsInt64(),
					Name              = chunkObj["name"].AsString(),
					EncryptionKeyGuid = chunkObj["encguid"].AsString(),
				};

				foreach (CbField fileField in chunkObj["files"].AsArray())
				{
					CbObject fileObj = fileField.AsObject();

					IoChunkId chunkId = IoChunkId.Empty;
					CbField idField = fileObj.Find("id");
					if (idField.HasValue())
					{
						chunkId = new IoChunkId(idField.AsHash());
					}

					string? pkg  = ReadOptionalString(fileObj, "pkg");
					string? path = ReadOptionalString(fileObj, "path");
					long    size = (long)fileObj.Find("size").AsInt64();

					var file = new ContainerChunkFile
					{
						ChunkId     = chunkId,
						PackageName = pkg,
						FilePath    = path,
						UfsPath     = path,
						Size        = size,
					};

					CbField bulkMap = fileObj.Find("bulk_data_map");
					if (bulkMap.HasValue())
					{
						foreach (CbField entry in bulkMap.AsArray())
						{
							CbObject e = entry.AsObject();
							file.BulkDataMap.Add(new BulkDataMapEntry(
								Offset:          (long)e["offset"].AsInt64(),
								DuplicateOffset: (long)e["duplicate_offset"].AsInt64(),
								Size:            (long)e["size"].AsInt64(),
								Flags:           (uint)e["flags"].AsUInt64(),
								CookedIndex:     (byte)e["cooked_index"].AsUInt64()
							));
						}
					}

					CbField bulkSegs = fileObj.Find("bulk_data_segments");
					if (bulkSegs.HasValue())
					{
						foreach (CbField entry in bulkSegs.AsArray())
						{
							CbObject e = entry.AsObject();
							IoChunkId segId = e.Find("id").HasValue()
								? new IoChunkId(e["id"].AsHash())
								: IoChunkId.Empty;
							file.BulkDataSegments.Add(new BulkDataSegments(
								ChunkId: segId,
								Offset:  (long)e["offset"].AsInt64(),
								Size:    (long)e["size"].AsInt64()
							));
						}
					}

					container.Files.Add(file);
				}

				Assignments.ContainerChunks.Add(container);
			}

			Log.Logger.LogInformation("ChunkAssignmentOp: parsed {Count} chunk(s) from oplog", Assignments.ContainerChunks.Count);
			return true;
		}

		/// <summary>
		/// Fills in missing <see cref="ContainerChunkFile.FilePath"/> values by looking up
		/// each file's <see cref="IoChunkId"/> against the packagedata / bulkdata entries of
		/// all <see cref="PackageOplogEntry"/> instances in <paramref name="entries"/>. Files
		/// with no IoChunkId (already carrying explicit FilePath) are left untouched.
		/// </summary>
		public void ResolveFilenames(IReadOnlyCollection<OplogEntry> entries)
		{
			var idToFile = new Dictionary<IoHash, (string PackageName, string Filename)>();
			foreach (OplogEntry entry in entries)
			{
				if (entry is not PackageOplogEntry pkg)
				{
					continue;
				}
				foreach (PackageFileInfo pf in pkg.PackageFiles)
				{
					idToFile[pf.ChunkId] = (pkg.PackageName, pf.Filename);
				}
				foreach (PackageFileInfo bf in pkg.BulkDataFiles)
				{
					idToFile[bf.ChunkId] = (pkg.PackageName, bf.Filename);
				}
			}

			int resolved = 0, missing = 0;
			foreach (ContainerChunk chunk in Assignments.ContainerChunks)
			{
				foreach (ContainerChunkFile file in chunk.Files)
				{
					if (file.ChunkId.IsEmpty)
					{
						continue;
					}
					if (idToFile.TryGetValue(file.ChunkId.Hash, out var info))
					{
						file.PackageName ??= info.PackageName;
						file.UfsPath     ??= info.Filename;
						file.FilePath    ??= info.Filename;
						resolved++;
					}
					else
					{
						missing++;
					}
				}
			}

			if (missing > 0)
			{
				Log.Logger.LogWarning(
					"ChunkAssignmentOp: failed to resolve {Missing} IoChunkId(s) against packagedata/bulkdata; {Resolved} resolved",
					missing, resolved);
			}
		}

		private static string? ReadOptionalString(CbObject obj, string name)
		{
			CbField f = obj.Find(name);
			if (!f.HasValue())
			{
				return null;
			}
			string s = f.AsString();
			return string.IsNullOrEmpty(s) ? null : s;
		}
	}
#nullable disable
}
