// Copyright Epic Games, Inc. All Rights Reserved.

using EpicGames.Core;
using System;
using System.Collections.Generic;

namespace AutomationScripts.Oplog
{
#nullable enable
	/// <summary>
	/// Alias for IO Store chunk id. The C# automation layer uses <see cref="IoHash"/> for the
	/// 12-byte chunk identifier that maps to a packagedata/bulkdata entry in the cook oplog;
	/// this struct keeps the canonical C++ name and exposes the underlying hash for serialization.
	/// </summary>
	public readonly record struct IoChunkId(IoHash Hash)
	{
		public static IoChunkId Empty => new IoChunkId(IoHash.Zero);

		public bool IsEmpty => Hash == IoHash.Zero;

		public static implicit operator IoChunkId(IoHash hash) => new IoChunkId(hash);
		public static implicit operator IoHash(IoChunkId id) => id.Hash;
	}

	/// <summary>
	/// One entry from the cook bulk-data map (mirrors <c>FBulkDataMapEntry</c> on the C++ side).
	/// Carried per file so the chunk-assignment output can describe bulk slicing without
	/// re-reading the oplog at consume time.
	/// </summary>
	public record BulkDataMapEntry(Int64 Offset, Int64 DuplicateOffset, Int64 Size, UInt32 Flags, byte CookedIndex);

	/// <summary>
	/// One inline bulk-data segment (a sub-range inside another IO Store chunk).
	/// </summary>
	public record BulkDataSegments(IoChunkId ChunkId, Int64 Offset, Int64 Size);

	/// <summary>
	/// One file assigned to a <see cref="ContainerChunk"/>.
	/// <para>
	/// <see cref="ChunkId"/> is the identifying IoChunkId from the cook oplog. <see cref="PackageName"/>
	/// and <see cref="FilePath"/> are only carried by writers when the file path cannot be
	/// derived from <see cref="ChunkId"/> alone — i.e. the file doesn't have an IoChunkId
	/// (loose / uncooked / locres / locmeta files), or the actual staged path differs from the
	/// canonical cooked path for its package.
	/// </para>
	/// </summary>
	public class ContainerChunkFile
	{
		public IoChunkId ChunkId { get; set; }

		public string? PackageName { get; set; }

		public string? FilePath { get; set; }

		public string? UfsPath { get; set; }

		public Int64 Size { get; set; }

		public List<BulkDataMapEntry> BulkDataMap { get; init; } = new List<BulkDataMapEntry>();

		public List<BulkDataSegments> BulkDataSegments { get; init; } = new List<BulkDataSegments>();
	};

	/// <summary>
	/// One pak/iostore container chunk produced by the chunk assigner.
	/// </summary>
	public class ContainerChunk
	{
		public int ContainerChunkId { get; set; }

		public string? Name { get; set; }

		public string? EncryptionKeyGuid { get; set; }

		public string? Path { get; set; }

		public bool IsOnDemand { get; set; }

		public bool IsSigned { get; set; }

		public bool EncryptIndex { get; set; }

		public List<ContainerChunkFile> Files { get; init; } = new List<ContainerChunkFile>();
	};

	/// <summary>
	/// Result of <see cref="ChunkAssigner.AssignChunks"/>. Replaces the DataTable-based output
	/// the assigner pipeline used historically.
	/// </summary>
	public class ChunkAssignments
	{
		public List<ContainerChunk> ContainerChunks { get; init; } = new List<ContainerChunk>();
	};
#nullable disable
}
