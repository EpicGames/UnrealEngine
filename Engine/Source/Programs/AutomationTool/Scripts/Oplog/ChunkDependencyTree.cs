// Copyright Epic Games, Inc. All Rights Reserved.

using EpicGames.Core;
using Microsoft.Extensions.Logging;
using System;
using System.Collections.Generic;
using System.Runtime.InteropServices;
using UnrealBuildTool;

namespace AutomationScripts.Oplog
{
#nullable enable
	/// <summary>
	/// Reads the chunk parent hierarchy from
	/// [/Script/CoreUObject.ChunkDependencyInfo] +DependencyArray=(ChunkID=X,ParentChunkID=Y)
	/// and exposes <see cref="FindHighestSharedChunk"/> which returns the deepest common
	/// ancestor chunk for a set of chunk IDs — mirroring
	/// </summary>
	public sealed class ChunkDependencyTree
	{
		// Each entry: chunkId → set of ALL ancestors (including the chunk itself).
		// Mirrors ChildToParentMap in the C++ implementation.
		private Dictionary<int, HashSet<int>> _childToParents;

		// BFS order starting from chunk 0 (root). Used to pick the "highest" (deepest) common ancestor.
		// Mirrors TopologicallySortedChunks in the C++ implementation.
		private List<int> _topologicalOrder;

		// Raw user-specified dependency pairs, preserved across Rebuild() calls.
		private readonly List<(int ChunkId, int ParentChunkId)> _rawDeps;

		private ChunkDependencyTree(
			Dictionary<int, HashSet<int>> childToParents,
			List<int> topologicalOrder,
			List<(int ChunkId, int ParentChunkId)> rawDeps)
		{
			_childToParents = childToParents;
			_topologicalOrder = topologicalOrder;
			_rawDeps = rawDeps;
		}

		/// <summary>
		/// Reads <c>[/Script/CoreUObject.ChunkDependencyInfo] +DependencyArray</c> from the supplied
		/// config and builds the dependency tree.
		/// </summary>
		public static ChunkDependencyTree ReadFromConfig(ConfigHierarchy ini)
		{
			// Read raw DependencyArray entries.
			// Each entry is a struct string like "(ChunkID=10,ParentChunkID=0)".
			ini.GetArray("/Script/CoreUObject.ChunkDependencyInfo", "DependencyArray", out List<string>? rawEntries);

			var deps = new List<(int ChunkId, int ParentChunkId)>();
			if (rawEntries != null)
			{
				foreach (string entry in rawEntries)
				{
					if (TryParseChunkDependency(entry, out int chunkId, out int parentChunkId))
					{
						deps.Add((chunkId, parentChunkId));
					}
					else
					{
						Log.Logger.LogWarning("ChunkDependencyTree: failed to parse DependencyArray entry: {Entry}", entry);
					}
				}
			}

			if (deps.Count == 0)
			{
				Log.Logger.LogWarning("ChunkDependencyTree: no DependencyArray entries found in " +
					"[/Script/CoreUObject.ChunkDependencyInfo]. All chunks will resolve to chunk 0.");
			}

			return Build(deps);
		}

		/// <summary>
		/// Mirrors <c>UChunkDependencyInfo::FindHighestSharedChunk</c>.
		/// Returns the deepest common ancestor chunk for all input chunk IDs.
		/// Returns 0 if no common ancestor is found (chunk 0 is the universal root).
		/// Returns -1 (INDEX_NONE) if any input chunk is not in the tree.
		/// </summary>
		public int FindHighestSharedChunk(IEnumerable<int> chunkIds)
		{
			// Deduplicate inputs
			var unique = new HashSet<int>(chunkIds);

			if (unique.Count == 0)
			{
				return -1;
			}

			// Verify all chunks exist in the map
			foreach (int id in unique)
			{
				if (!_childToParents.ContainsKey(id))
				{
					return -1;
				}
			}

			if (unique.Count == 1)
			{
				foreach (int id in unique)
				{
					return id;
				}
			}

			// Intersect ancestor sets (including self) of all inputs
			HashSet<int>? commonAncestors = null;
			foreach (int id in unique)
			{
				// ancestors of id = _childToParents[id] ∪ {id}
				var ancestorsOfId = new HashSet<int>(_childToParents[id]);
				ancestorsOfId.Add(id);

				if (commonAncestors == null)
				{
					commonAncestors = ancestorsOfId;
				}
				else
				{
					commonAncestors.IntersectWith(ancestorsOfId);
				}
			}

			if (commonAncestors == null || commonAncestors.Count == 0)
			{
				return 0;
			}

			// Return the entry with the highest topological index (deepest in tree)
			int highestIdx = -1;
			foreach (int candidate in commonAncestors)
			{
				int idx = _topologicalOrder.IndexOf(candidate);
				if (idx > highestIdx)
				{
					highestIdx = idx;
				}
			}

			if (highestIdx < 0)
			{
				return 0;
			}

			return _topologicalOrder[highestIdx];
		}

		// ---- Mutation ----
		/// <summary>
		/// returns true if the chunkId was already registered in some pair.
		/// </summary>
		/// <param name="chunkId"></param>
		/// <returns></returns>
		public bool HasChunk(int inChunkId)
		{
			if (inChunkId == 0)
			{
				return true;
			}

			foreach (var (chunkId, parentChunkId) in _rawDeps)
			{
				if (inChunkId == chunkId || inChunkId == parentChunkId)
				{
					return true; 
				}
			}

			return false;
		}

		/// <summary>
		/// Registers a new chunk→parent dependency pair.
		/// Returns <c>false</c> (and does nothing) if:
		/// <list type="bullet">
		///   <item><paramref name="chunkId"/> is 0 — the root cannot have a parent</item>
		///   <item><paramref name="chunkId"/> equals <paramref name="parentChunkId"/> — self-reference</item>
		///   <item><paramref name="parentChunkId"/> is not present in the current tree</item>
		/// </list>
		/// The tree is not updated until <see cref="Rebuild"/> is called.
		/// </summary>
		public bool TryAddDependency(int chunkId, int parentChunkId)
		{
			if (chunkId == 0)
			{
				Log.Logger.LogWarning("ChunkDependencyTree: chunk 0 cannot have a parent (ParentChunkID={ParentChunkId}).", parentChunkId);
				return false;
			}
			if (chunkId == parentChunkId)
			{
				Log.Logger.LogWarning("ChunkDependencyTree: ignoring self-referencing entry ChunkID={ChunkId}.", chunkId);
				return false;
			}
			if (!_childToParents.ContainsKey(parentChunkId))
			{
				Log.Logger.LogWarning("ChunkDependencyTree: parent chunk {ParentChunkId} does not exist in the tree.", parentChunkId);
				return false;
			}
			_rawDeps.Add((chunkId, parentChunkId));
			return true;
		}

		/// <summary>
		/// Attempts to update an existing chunk -> parentChunk pair in the known dependencies.
		/// </summary>
		/// <param name="chunkId"></param>
		/// <param name="parentChunkId"></param>
		/// <param name="logger"></param>
		/// <returns></returns>
		public bool TryUpdateParentChunk(int chunkId, int parentChunkId)
		{
			if (chunkId == 0)
			{
				Log.Logger.LogWarning("ChunkDependencyTree: chunk 0 cannot have a parent (ParentChunkID={ParentChunkId}).", parentChunkId);
				return false;
			}
			if (chunkId == parentChunkId)
			{
				Log.Logger.LogWarning("ChunkDependencyTree: ignoring self-referencing entry ChunkID={ChunkId}.", chunkId);
				return false;
			}
			if (!_childToParents.ContainsKey(parentChunkId))
			{
				Log.Logger.LogWarning("ChunkDependencyTree: parent chunk {ParentChunkId} does not exist in the tree.", parentChunkId);
				return false;
			}
			foreach (ref var dep in CollectionsMarshal.AsSpan(_rawDeps))
			{
				if (dep.ChunkId == chunkId)
				{
					dep.ParentChunkId = parentChunkId;
					return true;
				}
			}
			return false;
		}

		/// <summary>
		/// Rebuilds the internal dependency maps from all registered pairs — both the
		/// original entries and any added via <see cref="TryAddDependency"/>.
		/// Safe to call multiple times.
		/// </summary>
		public void Rebuild()
		{
			ChunkDependencyTree rebuilt = Build(_rawDeps);
			_childToParents = rebuilt._childToParents;
			_topologicalOrder = rebuilt._topologicalOrder;
		}

		// ---- Build ----

		private static ChunkDependencyTree Build(IEnumerable<(int ChunkId, int ParentChunkId)> depsInput)
		{
			// Work on a local copy so the caller's list is never mutated.
			var rawDeps = new List<(int ChunkId, int ParentChunkId)>(depsInput);
			var deps = new List<(int ChunkId, int ParentChunkId)>(rawDeps);

			// Validate: remove self-referencing entries; chunk 0 must not have a parent
			for (int i = deps.Count - 1; i >= 0; i--)
			{
				var (chunkId, parentChunkId) = deps[i];
				if (chunkId == parentChunkId)
				{
					Log.Logger.LogWarning("ChunkDependencyTree: ignoring self-referencing entry ChunkID={ChunkId}.", chunkId);
					deps.RemoveAt(i);
					continue;
				}
				if (chunkId == 0)
				{
					Log.Logger.LogWarning("ChunkDependencyTree: chunk 0 must not have a parent (ParentChunkID={Parent}). Ignoring.", parentChunkId);
					deps.RemoveAt(i);
				}
			}

			// Determine highest chunk ID
			int highestChunk = 0;
			foreach (var (chunkId, parentChunkId) in deps)
			{
				if (chunkId > highestChunk)
				{
					highestChunk = chunkId;
				}
				if (parentChunkId > highestChunk)
				{
					highestChunk = parentChunkId;
				}
			}

			// Fill missing links: any chunk 1..highestChunk not explicitly listed defaults to parent=0
			var explicitChunks = new HashSet<int>();
			foreach (var (chunkId, _) in deps)
			{
				explicitChunks.Add(chunkId);
			}
			for (int i = 1; i <= highestChunk; i++)
			{
				if (!explicitChunks.Contains(i))
				{
					deps.Add((i, 0));
				}
			}

			// Build parent→children adjacency for the tree
			var children = new Dictionary<int, List<int>>();
			children[0] = new List<int>();
			foreach (var (chunkId, parentChunkId) in deps)
			{
				if (!children.ContainsKey(chunkId))
				{
					children[chunkId] = new List<int>();
				}
				if (!children.ContainsKey(parentChunkId))
				{
					children[parentChunkId] = new List<int>();
				}
				children[parentChunkId].Add(chunkId);
			}

			// BFS from root (chunk 0) to build topological order
			var topologicalOrder = new List<int>();
			var visited = new HashSet<int>();
			var queue = new Queue<int>();
			queue.Enqueue(0);

			while (queue.Count > 0)
			{
				int current = queue.Dequeue();
				if (!visited.Add(current))
				{
					continue;
				}
				topologicalOrder.Add(current);
				if (children.TryGetValue(current, out List<int>? childList))
				{
					foreach (int child in childList)
					{
						queue.Enqueue(child);
					}
				}
			}

			// Build ChildToParents: transitive closure — each chunk → all ancestors (excluding self)
			// Mirror of AddChildrenRecursive which accumulates Parents set as it recurses
			var childToParents = new Dictionary<int, HashSet<int>>();
			BuildTransitiveClosure(0, new HashSet<int>(), children, childToParents);

			// Ensure chunk 0 has an entry (it has no parents)
			if (!childToParents.ContainsKey(0))
			{
				childToParents[0] = new HashSet<int>();
			}

			return new ChunkDependencyTree(childToParents, topologicalOrder, rawDeps);
		}

		private static void BuildTransitiveClosure(
			int node,
			HashSet<int> parentsSoFar,
			Dictionary<int, List<int>> children,
			Dictionary<int, HashSet<int>> childToParents)
		{
			if (parentsSoFar.Count > 0)
			{
				if (!childToParents.TryGetValue(node, out HashSet<int>? existing))
				{
					existing = new HashSet<int>();
					childToParents[node] = existing;
				}
				existing.UnionWith(parentsSoFar);
			}
			else if (!childToParents.ContainsKey(node))
			{
				childToParents[node] = new HashSet<int>();
			}

			if (!children.TryGetValue(node, out List<int>? childList))
			{
				return;
			}

			// Add current node to parents set for children
			var newParents = new HashSet<int>(parentsSoFar) { node };

			foreach (int child in childList)
			{
				BuildTransitiveClosure(child, newParents, children, childToParents);
			}
		}

		// ---- Parsing ----

		/// <summary>
		/// Parses "(ChunkID=10,ParentChunkID=0)" → (10, 0).
		/// </summary>
		private static bool TryParseChunkDependency(string entry, out int chunkId, out int parentChunkId)
		{
			chunkId = 0;
			parentChunkId = 0;

			// Strip outer parens if present
			string s = entry.Trim();
			if (s.StartsWith("(", StringComparison.Ordinal))
			{
				s = s.Substring(1);
			}
			if (s.EndsWith(")", StringComparison.Ordinal))
			{
				s = s.Substring(0, s.Length - 1);
			}

			bool gotChunkId = false;
			bool gotParentId = false;

			foreach (string part in s.Split(','))
			{
				int eq = part.IndexOf('=');
				if (eq < 0)
				{
					continue;
				}

				string key = part.Substring(0, eq).Trim();
				string val = part.Substring(eq + 1).Trim();

				if (key.Equals("ChunkID", StringComparison.OrdinalIgnoreCase))
				{
					if (int.TryParse(val, out chunkId))
					{
						gotChunkId = true;
					}
				}
				else if (key.Equals("ParentChunkID", StringComparison.OrdinalIgnoreCase))
				{
					if (int.TryParse(val, out parentChunkId))
					{
						gotParentId = true;
					}
				}
			}

			return gotChunkId && gotParentId;
		}
	}
#nullable disable
}
