// Copyright Epic Games, Inc. All Rights Reserved.

using EpicGames.Core;
using Microsoft.Extensions.Logging;
using System.Collections.Generic;
using UnrealBuildTool;

namespace AutomationScripts.Oplog
{
#nullable enable
	/// <summary>
	/// Engine-default chunk assignment implementation.
	/// Gives a quick way users can write their own method for chunk assignment.
	/// This implementation assigns all packages to chunk 0 or 10.
	/// </summary>
	[ChunkAssigner("DefaultChunkAssigner")]
	public sealed class DefaultChunkAssigner : ChunkAssigner
	{
		public override ChunkAssignments AssignChunks(IReadOnlyList<OplogEntry> OplogEntries, UnrealTargetPlatform TargetPlatform, TargetReceipt Receipt)
		{
			PackageGraph graph = PackageGraphBuilder.Build(OplogEntries);
			Log.Logger.LogInformation("DefaultChunkAssigner: assigning chunks for {Count} packages.", graph.NodeCount);

			int[] assignedChunk = new int[graph.NodeCount];
			for (int i = 0; i < graph.NodeCount; i++)
			{
				assignedChunk[i] = -1;
			}

			// Startup packages → chunk 0
			foreach (PackageNode node in graph.GetStartupPackages())
			{
				assignedChunk[node.Index] = 0;
			}

			var byChunk = new SortedDictionary<int, ContainerChunk>();
			int totalFiles = 0;

			foreach (PackageNode node in graph.AllNodes)
			{
				int chunk = assignedChunk[node.Index];
				if (chunk < 0)
				{
					chunk = 10;
				}

				ContainerChunk container = GetOrAddChunk(byChunk, chunk);
				PackageMetadata meta = graph.GetMetadata(node);

				foreach (PackageFileInfo file in meta.PackageFiles)
				{
					container.Files.Add(new ContainerChunkFile
					{
						ChunkId     = new IoChunkId(file.ChunkId),
						PackageName = meta.PackageName,
						UfsPath     = file.Filename,
						Size        = file.Size,
					});
					totalFiles++;
				}
				foreach (PackageFileInfo file in meta.BulkDataFiles)
				{
					container.Files.Add(new ContainerChunkFile
					{
						ChunkId     = new IoChunkId(file.ChunkId),
						PackageName = meta.PackageName,
						UfsPath     = file.Filename,
						Size        = file.Size,
					});
					totalFiles++;
				}
			}

			Log.Logger.LogInformation("DefaultChunkAssigner: built {Files} file entries across {Nodes} packages in {Chunks} chunks.",
				totalFiles, graph.NodeCount, byChunk.Count);

			var result = new ChunkAssignments();
			result.ContainerChunks.AddRange(byChunk.Values);
			return result;
		}

		private static ContainerChunk GetOrAddChunk(SortedDictionary<int, ContainerChunk> map, int chunkId)
		{
			if (!map.TryGetValue(chunkId, out ContainerChunk? container))
			{
				container = new ContainerChunk
				{
					ContainerChunkId = chunkId,
					Name             = $"pakchunk{chunkId}",
				};
				map[chunkId] = container;
			}
			return container;
		}
	}
#nullable disable
}
