// Copyright Epic Games, Inc. All Rights Reserved.

using EpicGames.Serialization;

namespace AutomationScripts.Oplog
{
#nullable enable
	/// <summary>
	/// Engine-default per-package metadata reader. Reads the "meta.cook.package" attachment
	/// written by FGlobalCookArtifact::AppendPackageMetadata. Populated for every
	/// cooked package that has a known asset class.
	/// Registered via +OplogPackageMetadata=GlobalPackageMetaOp in BaseGame.ini.
	/// </summary>
	[OplogEntry("GlobalPackageMetaOp")]
	public sealed class GlobalPackageMetaOp : OplogEntry
	{
		public override string GetKey() => "meta.cook.package";

		public override bool ParseData(CbField entryField, string BaseOplogURL)
		{
			CbObject cbObject = entryField.AsObject();
			if (cbObject == CbObject.Empty)
			{
				return false;
			}
			CbField classField = cbObject["class"];
			if (classField.HasValue())
			{
				Class = classField.AsString();
			}
			CbField primaryAssetField = cbObject["isprimaryasset"];
			if (primaryAssetField.HasValue())
			{
				IsPrimaryAsset = primaryAssetField.AsBool();
			}
			return true;
		}

		public string Class { get; private set; } = string.Empty;
		public bool IsPrimaryAsset { get; private set; } = false;
	}
#nullable disable
}
