// Copyright Epic Games, Inc. All Rights Reserved.

using EpicGames.Serialization;
using System.Collections.Generic;

namespace AutomationScripts.Oplog
{
#nullable enable
	/// <summary>
	/// Oplog entry emitted by FLocalizationCookArtifact at the end of cook.
	/// Contains set of localization file paths:
	///   LocFiles — relative to the workspace root, for game and plugin source localization files.
	///   Cultures — the cultures that were being cooked for.
	/// </summary>
	[OplogEntry("LocalizationOp")]
	public class LocalizationOp : OplogEntry
	{
		public override string GetKey() { return "Cook.Localization"; }

		public override bool ParseData(CbField entryField, string BaseOplogURL)
		{
			CbField attachmentData = entryField["value"];
			CbObject attachment = DownloadAttachment(BaseOplogURL, attachmentData.AsAttachment());
			if (attachment == CbObject.Empty)
			{
				return false;
			}

			foreach (CbField f in attachment["files"].AsArray())
			{
				string path = f.AsString();
				if (!string.IsNullOrEmpty(path))
				{
					LocFiles.Add(path);
				}
			}
			foreach (CbField f in attachment["cultures"].AsArray())
			{
				string culture = f.AsString();
				if (!string.IsNullOrEmpty(culture))
				{
					Cultures.Add(culture);
				}
			}
			
			return true;
		}
		public HashSet<string> LocFiles = new HashSet<string>();
		public HashSet<string> Cultures = new HashSet<string>();
	}
#nullable disable
}
