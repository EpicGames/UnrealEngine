// Copyright Epic Games, Inc. All Rights Reserved.

using EpicGames.Core;
using EpicGames.Serialization;
using Microsoft.Extensions.Logging;
using System;
using System.Buffers.Binary;
using System.Collections.Generic;
using System.Net.Http;
using System.Threading.Tasks;
using UnrealBuildTool;
using static AutomationScripts.Oplog.OplogChunkAssigner;

namespace AutomationScripts.Oplog
{
#nullable enable
	/// <summary>
	/// Used as the base class for all Oplog Entries and their metadata.
	/// Allows for custom game oplog entries or custom metadata on each package op.
	/// 
	/// 
	/// </summary>
	public abstract class OplogEntry
	{
		// returns the key this OplogEntry or metada on a package entry this subclass should be attempting to parse.
		public abstract string GetKey();

		// the CbField read from the oplog. This allows subclass to process the data for their custom interpretation.
		public abstract bool ParseData(CbField entryField, string BaseOplogURL);

		public static CbObject DownloadAttachment(string BaseOplogURL, IoHash Hash)
		{
			CbObject returnObj = CbObject.Empty;
			if (OplogReader.HttpClient is null)
			{
				Log.Logger.LogError("OplogReader HTTP Client was null. Create an OplogReader first");
				return returnObj;
			}

			using (HttpRequestMessage request = new(HttpMethod.Get, $"{BaseOplogURL}/{Hash.ToUtf8String()}"))
			{
				HttpResponseMessage response;
				try
				{
					response = OplogReader.HttpClient.Send(request);
				}
				catch (HttpRequestException ex)
				{
					throw new AutomationTool.AutomationException($"Failed to send op attachment request to Zen at {BaseOplogURL}: {ex.Message}");
				}

				if (!response.IsSuccessStatusCode)
				{
					throw new AutomationTool.AutomationException(
						$"Failed to read op from Zen at {BaseOplogURL} for attachment {Hash.ToUtf8String()}.\n" +
						$"HTTP {response.StatusCode}.\n" +
						"Ensure that cooking was successful.");
				}

				Task<byte[]> readTask = response.Content.ReadAsByteArrayAsync();
				readTask.Wait();
				returnObj = new CbObject(readTask.Result);
			}
			return returnObj;
		}


		/// <summary>
		/// Creates a <see cref="OplogEntry"/> by reading the ini key
		/// <c>[/Script/UnrealEd.ProjectPackagingSettings] OplogEntry</c>
		/// <c>[/Script/UnrealEd.ProjectPackagingSettings] OplogPackageMetadata</c>
		/// from the supplied game config. 
		/// 
		/// OplogEntry defines the particular ops we'll attempt to read
		/// OplogPackageMetadata defines the additional metadata reader each <see cref="PackageOplogEntry"/> op will read.
		/// 
		/// All classes should be tagged to be a valid reader.
		/// [OplogEntry("MyCustomClass")]
		/// </summary>
		public static OplogEntry? CreateClass(string name)
		{
			if (s_registry == null)
			{
				s_registry = BuildRegistry([typeof(OplogEntry)]);
			}

			if (!s_registry!.TryGetValue(name, out Type? type) || type == null)
			{
				Log.Logger.LogInformation($"Unknown custom oplog package metadata '{name}'. " +
					"Ensure the class is tagged with [OplogEntry(\"{name}\")] " +
					"and is in a loaded script assembly.");
				return null;
			}

			OplogEntry? instance = (OplogEntry?)Activator.CreateInstance(type);
			if (instance == null)
			{
				throw new BuildException($"Could not instantiate custom OplogEntry '{name}'.");
			}

			return instance;
		}

		internal static Dictionary<string, Type>? s_registry;
	}

	/// <summary>
	/// One entry from an oplog op's <c>files</c> array. The same shape appears on
	/// package ops and the EndCook op (each file has an id, a chunkid hash, the
	/// server-side cooked path, and the client-side staged path).
	/// </summary>
	public sealed class OplogFileRecord
	{
		/// <summary>The 24-hex CbObjectId from the <c>id</c> field.</summary>
		public string Id { get; init; } = string.Empty;

		/// <summary>The IoHash chunkid from the <c>data</c> field.</summary>
		public IoHash ChunkId { get; init; }

		/// <summary>The path from the <c>serverpath</c> field (cook-server relative).</summary>
		public string ServerPath { get; init; } = string.Empty;

		/// <summary>The path from the <c>clientpath</c> field (with <c>/{engine}/</c> and <c>/{project}/</c> placeholders).</summary>
		public string ClientPath { get; init; } = string.Empty;

		/// <summary>Read a single record from a <c>files</c> array element.</summary>
		public static OplogFileRecord ReadFrom(CbField fileField)
		{
			CbObject obj = fileField.AsObject();
			byte[] idBytes = obj.Find("id").AsObjectId().ToByteArray();
			return new OplogFileRecord
			{
				Id         = Convert.ToHexString(idBytes).ToLowerInvariant(),
				ChunkId    = obj.Find("data").AsHash(),
				ServerPath = obj.Find("serverpath").AsString(),
				ClientPath = obj.Find("clientpath").AsString(),
			};
		}
	}

	public sealed class NullOplogEntry : OplogEntry
	{
		public List<CbObject> Files { get; private set; } = new List<CbObject>();

		public override string GetKey() { return string.Empty; }

		public override bool ParseData(CbField MetaField, string BaseOplogURL)
		{
			Log.Logger.LogError("Attempting to parse null oplog entry.");
			return false;
		}
	}

	/// <summary>
	/// Attribute used to register a <see cref="OplogEntryAttribute"/> subclass by name.
	/// </summary>
	[AttributeUsage(AttributeTargets.Class)]
	public sealed class OplogEntryAttribute : OplogAttribute
	{
		public OplogEntryAttribute(string name) : base(name)
		{
		}
	}

	/// <summary>
	/// Internal DTO representing a single parsed oplog entry before graph construction.
	/// One instance per package entry read from the Zen server oplog.
	/// </summary>
	[OplogEntry("PackageOplogEntry")]
	public sealed class PackageOplogEntry : OplogEntry
	{
		public string PackageName { get; private set; } = string.Empty;

		/// <summary> FPackageId value — 64-bit hash of the package name.</summary>
		public ulong PackageId { get; private set; }

		/// <summary> EPackageStoreEntryFlags bits from the oplog.</summary>
		public uint Flags { get; private set; }

		/// <summary> Cooked filename from the oplog packagedata (e.g. "MyGame/Saved/Cooked/.../Foo.uasset"). Empty if absent.</summary>
		public string Filename { get; private set; } = string.Empty;

		/// <summary> Hard dependencies: FPackageId values from importedpackageids.</summary>
		public List<ulong> ImportedPackageIds { get; private set; } = new();

		/// <summary> Soft dependencies: FPackageId values from softpackagereferences.</summary>
		public List<ulong> SoftPackageReferences { get; private set; } = new();

		/// <summary> Optional segment imported package IDs. Additional hard dependencies </summary>
		public List<ulong> OptionalSegmentImportedPackageIds { get; private set; } = new();

		/// <summary> Runtime package dependencies taken from the cook.artifacts </summary>
		public List<string> RuntimeDependencies { get; private set; } = new();

		/// <summary>All files from the oplog packagedata array for this package (e.g. .uasset).</summary>
		public IReadOnlyList<PackageFileInfo> PackageFiles { get; private set; } = [];

		/// <summary>All files from the oplog bulkdata array for this package (e.g. .ubulk, .uptnl).</summary>
		public IReadOnlyList<PackageFileInfo> BulkDataFiles { get; private set; } = [];

		/// <summary>
		/// All files from the oplog top-level <c>files</c> array on this package entry.
		/// These are auxiliary files (fonts, shader caches, etc.) referenced by the package
		/// with id/serverpath/clientpath plus an IoHash chunkid in <c>data</c>.
		/// </summary>
		public IReadOnlyList<OplogFileRecord> Files { get; private set; } = [];

		/// <summary> Additional metadata which knows how to read the custom additional metadata keys that might be in each op. </summary>
		public Dictionary<System.Type, OplogEntry> AdditionalMetadata = new();

		public override string GetKey() { return PackageName; }

		public override bool ParseData(CbField entryField, string BaseOplogURL)
		{
			CbField storeEntry = entryField["packagestoreentry"];
			CbField packageData = entryField["packagedata"];

			CbObject pse = storeEntry.AsObject();
			PackageName = pse["packagename"].AsString();
			Flags = pse["flags"].AsUInt32();

			// Hard dependencies
			ImportedPackageIds.Capacity = pse["importedpackageids"].AsArray().Count;
			foreach (CbField idField in pse["importedpackageids"].AsArray())
			{
				ImportedPackageIds.Add(idField.AsUInt64());
			}

			// Soft references
			SoftPackageReferences.Capacity = pse["softpackagereferences"].AsArray().Count;
			foreach (CbField idField in pse["softpackagereferences"].AsArray())
			{
				SoftPackageReferences.Add(idField.AsUInt64());
			}

			// Optional segment deps
			OptionalSegmentImportedPackageIds.Capacity = pse["optionalsegmentimportedpackageids"].AsArray().Count;
			foreach (CbField idField in pse["optionalsegmentimportedpackageids"].AsArray())
			{
				OptionalSegmentImportedPackageIds.Add(idField.AsUInt64());
			}

			// packagedata — capture all files; set PackageId/Filename from the first entry for backward compat
			ulong id = 0;
			var packageFiles = new List<PackageFileInfo>();
			foreach (var item in packageData.AsArray())
			{
				CbObject itemObj = item.AsObject();
				CbField pkgIdField = itemObj.Find("id");
				byte[] idBytes = pkgIdField.AsObjectId().ToByteArray();
				string fileId = Convert.ToHexString(idBytes).ToLowerInvariant();
				long size = (long)itemObj.Find("size").AsUInt64();
				CbField filenameField = itemObj.Find("filename");
				string filename = filenameField.HasValue() ? filenameField.AsString() : string.Empty;
				IoHash chunkId = itemObj.Find("data").AsHash();
				packageFiles.Add(new PackageFileInfo(fileId, size, filename, EPackageFileType.PackageData, chunkId));
				if (packageFiles.Count == 1)
				{
					id = BinaryPrimitives.ReadUInt64LittleEndian(idBytes);
					Filename = filename;
				}
			}
			PackageFiles = packageFiles;
			PackageId = id;

			// Runtime Deps
			CbField cookArtifactsField = entryField["meta.cook.artifacts"];
			if (cookArtifactsField.HasValue())
			{
				CbObject cookArtifacts = DownloadAttachment(BaseOplogURL, cookArtifactsField.AsAttachment());
				if (cookArtifacts != CbObject.Empty)
				{
					RuntimeDependencies.Capacity = cookArtifacts["RuntimeDependencies"].AsArray().Count;
					foreach (CbField pkg in cookArtifacts["RuntimeDependencies"].AsArray())
					{
						string pkgName = pkg.AsString();
						if (!string.IsNullOrEmpty(pkgName))
						{
							RuntimeDependencies.Add(pkgName);
						}
					}
					// IOStoreChunk ID can be resolved from the Key or obtained from the meta.cook.artifacts.
					if (PackageId == 0 && cookArtifacts["IoStoreChunkId"].HasValue())
					{
						byte[] idBytes = cookArtifacts["IoStoreChunkId"].AsObjectId().ToByteArray();
						PackageId = BinaryPrimitives.ReadUInt64LittleEndian(idBytes);
					}
				}
			}

			// read all potential metdata blobs
			foreach (KeyValuePair<string, OplogEntry> kvp in _keyToPackageOplogMeta)
			{
				CbField metaData = entryField[kvp.Key];
				if (metaData.HasValue())
				{
					OplogEntry? newOplogPackageMetadata = OplogEntry.CreateClass(kvp.Value.GetType().Name);
					if (newOplogPackageMetadata == null)
					{
						Log.Logger.LogWarning($"Failed to parse metadata for {kvp.Key}");
						continue;
					}
					if (newOplogPackageMetadata.ParseData(metaData, BaseOplogURL))
					{
						AdditionalMetadata.Add(kvp.Value.GetType(), newOplogPackageMetadata);
					}
				}
			}

			// bulkdata — capture all bulk files with their type
			var bulkFiles = new List<PackageFileInfo>();
			foreach (var item in entryField["bulkdata"].AsArray())
			{
				CbObject itemObj = item.AsObject();
				string fileId = Convert.ToHexString(itemObj.Find("id").AsObjectId().ToByteArray()).ToLowerInvariant();
				long size = (long)itemObj.Find("size").AsUInt64();
				CbField filenameField = itemObj.Find("filename");
				string filename = filenameField.HasValue() ? filenameField.AsString() : string.Empty;
				CbField typeField = itemObj.Find("type");
				EPackageFileType fileType = typeField.HasValue() && typeField.AsString() == "Optional"
					? EPackageFileType.BulkOptional
					: EPackageFileType.Bulk;
				IoHash chunkId = itemObj.Find("data").AsHash();
				bulkFiles.Add(new PackageFileInfo(fileId, size, filename, fileType, chunkId));
			}
			BulkDataFiles = bulkFiles;

			// files — auxiliary per-package files (id/data/serverpath/clientpath)
			CbField filesField = entryField["files"];
			if (filesField.HasValue())
			{
				var files = new List<OplogFileRecord>(filesField.AsArray().Count);
				foreach (CbField item in filesField.AsArray())
				{
					files.Add(OplogFileRecord.ReadFrom(item));
				}
				Files = files;
			}

			return true;
		}

		static internal readonly Dictionary<string, OplogEntry> _keyToPackageOplogMeta = new Dictionary<string, OplogEntry>();
	}

	[OplogEntry("ReferencedSetOplogEntry")]
	public sealed class ReferencedSetOplogEntry : OplogEntry
	{
		public List<CbObject> Files { get; private set; } = new List<CbObject>();

		public override string GetKey() { return "ReferencedSet"; }

		public override bool ParseData(CbField entryField, string BaseOplogURL)
		{
			Files.Capacity = entryField["files"].AsArray().Count;
			foreach (CbField fileObj in entryField["files"].AsArray())
			{
				Files.Add(fileObj.AsObject());
			}
			return true;
		}
	}

	[OplogEntry("EndCookOplogEntry")]
	public sealed class EndCookOplogEntry : OplogEntry
	{
		public List<OplogFileRecord> Files { get; private set; } = new List<OplogFileRecord>();

		public override string GetKey() { return "EndCook"; }

		public override bool ParseData(CbField entryField, string BaseOplogURL)
		{
			CbField filesField = entryField["files"];
			Files.Capacity = filesField.AsArray().Count;
			foreach (CbField fileObj in filesField.AsArray())
			{
				Files.Add(OplogFileRecord.ReadFrom(fileObj));
			}
			return true;
		}
	}

	[OplogEntry("CookStartupPackagesOp")]
	public class CookStartupPackagesOp : OplogEntry
	{
		public override string GetKey() { return "Cook.StartupPackages"; }

		public override bool ParseData(CbField entryField, string BaseOplogURL)
		{
			CbField attachmentData = entryField["value"];
			CbObject attachment = DownloadAttachment(BaseOplogURL, attachmentData.AsAttachment());
			if (attachment != CbObject.Empty)
			{
				StartupPackages = new HashSet<string>(capacity: attachment["StartupPackages"].AsArray().Count, StringComparer.OrdinalIgnoreCase);
				foreach (CbField pkg in attachment["StartupPackages"].AsArray())
				{
					string pkgName = pkg.AsString();
					if (!string.IsNullOrEmpty(pkgName))
					{
						StartupPackages.Add(pkgName);
					}
				}
				return true;
			}
			return false;
		}

		public HashSet<string> StartupPackages = new HashSet<string>();
	}
#nullable disable
}
