// Copyright Epic Games, Inc. All Rights Reserved.

using AutomationUtils;
using Dapper;
using EpicGames.Core;
using EpicGames.Serialization;
using Microsoft.Extensions.Logging;
using System;
using System.Collections.Concurrent;
using System.Collections.Generic;
using System.Net.Http;
using System.Threading.Tasks;
using UnrealBuildTool;
using static AutomationScripts.Oplog.OplogChunkAssigner;

namespace AutomationScripts.Oplog
{
#nullable enable
	/// <summary>
	/// Reads cooked package entries from the Zen store oplog via HTTP.
	/// The oplog is served as CompactBinary (application/x-ue-cb).
	/// </summary>
	public sealed class OplogReader
	{
		private readonly string _socketHostNameAndPort;
		private readonly string _httpHostNameAndPort;
		private readonly string _projectId;
		private readonly string _oplogId;
		private readonly Dictionary<string, OplogEntry> _keyToOplogEntry = new Dictionary<string, OplogEntry>();

		public static HttpClient? HttpClient { get; private set; }

		/// <param name="host">Zen server hostname (e.g. "localhost").</param>
		/// <param name="port">Zen server port (e.g. 8558).</param>
		/// <param name="projectId">Zen project ID.</param>
		/// <param name="oplogId">Zen oplog ID.</param>
		public OplogReader(string SocketHostNameAndPort, string HttpHostNameAndPort, string projectId, string oplogId, ConfigHierarchy? gameIni)
		{
			_socketHostNameAndPort = SocketHostNameAndPort;
			_httpHostNameAndPort = HttpHostNameAndPort;
			_projectId = projectId;
			_oplogId   = oplogId;

			if (gameIni != null)
			{
				if (gameIni.GetArray("/Script/UnrealEd.ProjectPackagingSettings", "OplogEntry", out List<string>? OplogEndCookMetadatas))
				{
					foreach (string className in OplogEndCookMetadatas)
					{
						OplogEntry? newOplogPackageMetadata = OplogEntry.CreateClass(className);
						if (newOplogPackageMetadata != null)
						{
							string newKey = newOplogPackageMetadata.GetKey();
							if (_keyToOplogEntry.ContainsKey(newKey))
							{
								OplogEntry existingOplogPackageMeta = _keyToOplogEntry[newKey];
								throw new BuildException($"Existing OplogEndCookMetadata looking for the same key '{existingOplogPackageMeta.GetType().Name}'");
							}
							_keyToOplogEntry.Add(newKey, newOplogPackageMetadata);
						}
					}
				}
				PackageOplogEntry._keyToPackageOplogMeta.Clear();
				if (gameIni.GetArray("/Script/UnrealEd.ProjectPackagingSettings", "OplogPackageMetadata", out List<string>? OplogPackageMetadatas))
				{
					foreach (string className in OplogPackageMetadatas)
					{
						OplogEntry? newOplogPackageMetadata = OplogEntry.CreateClass(className);
						if (newOplogPackageMetadata != null)
						{
							string newKey = newOplogPackageMetadata.GetKey();
							if (PackageOplogEntry._keyToPackageOplogMeta.ContainsKey(newKey))
							{
								OplogEntry existingOplogPackageMeta = PackageOplogEntry._keyToPackageOplogMeta[newKey];
								throw new BuildException($"Existing OplogPackageMetadata looking for the same key '{existingOplogPackageMeta.GetType().Name}'");
							}
							PackageOplogEntry._keyToPackageOplogMeta.Add(newKey, newOplogPackageMetadata);
						}
					}
				}
			}

			HttpClient = ZenUtils.CreateHttpClient(_socketHostNameAndPort);
			HttpClient.Timeout = TimeSpan.FromMinutes(20);
		}

		/// <summary>
		/// Fetches all package entries from the oplog and returns them as raw DTOs.
		/// Entries that do not have a valid <c>packagestoreentry</c> field (e.g. the
		/// "EndCook" sentinel) are silently skipped.
		/// </summary>
		/// <exception cref="AutomationTool.AutomationException">Thrown if the Zen server is unreachable or returns an error.</exception>
		public IReadOnlyList<OplogEntry> ReadEntries()
		{
			if (!ZenUtils.IsZenServerRunning(_socketHostNameAndPort))
			{
				throw new AutomationTool.AutomationException(
					$"Zen server is not running at {_socketHostNameAndPort}. " +
					"Ensure a cook has been completed and the Zen server is started.");
			}

			byte[] oplogData = FetchRawOplog();
			return ParseEntries(oplogData);
		}

		// ---- Private helpers ----

		public string GetBaseOplogURL()
		{
			return $"http://{_httpHostNameAndPort}/prj/{_projectId}/oplog/{_oplogId}";
		}
		private byte[] FetchRawOplog()
		{
			string uri = $"{GetBaseOplogURL()}/entries?trim_by_referencedset=true";

			Log.Logger.LogInformation("Reading oplog from {Uri}", uri);

			using var request = new HttpRequestMessage(HttpMethod.Get, uri);
			request.Headers.Add("Accept", "application/x-ue-cb");

			if (HttpClient is null)
			{
				throw new AutomationTool.AutomationException($"Failed to create HTTP Client during OplogReader constructor");
			}

			HttpResponseMessage response;
			try
			{
				response = HttpClient.Send(request);
			}
			catch (Exception ex)
			{
				throw new AutomationTool.AutomationException(
					$"Failed to send oplog request to Zen at {_socketHostNameAndPort}: {ex.Message}");
			}

			if (!response.IsSuccessStatusCode)
			{
				throw new AutomationTool.AutomationException(
					$"Failed to read oplog from Zen at {_socketHostNameAndPort} for " +
					$"{_projectId}.{_oplogId}: HTTP {response.StatusCode}. " +
					"Ensure that cooking was successful.");
			}

			Task<byte[]> readTask = response.Content.ReadAsByteArrayAsync();
			readTask.Wait();
			return readTask.Result;
		}

		private IReadOnlyList<OplogEntry> ParseEntries(byte[] oplogData)
		{
			var results = new ConcurrentBag<OplogEntry>();

			CbObject oplogObject = new CbField(oplogData).AsObject();
			Parallel.ForEach(oplogObject["entries"].AsArray(), entryField =>
			{
				string entryKey = entryField["key"].AsString();

				CbField storeEntry = entryField["packagestoreentry"];
				if (string.IsNullOrEmpty(entryKey))
				{
					Log.Logger.LogDebug("Encountered op with null or empty key. Skipping parsing of this op.");
					return;
				}
				if (storeEntry.HasValue())
				{
					var packageEntry = new PackageOplogEntry();
					if (packageEntry.ParseData(entryField, GetBaseOplogURL()))
					{
						results.Add(packageEntry);
					}
					else
					{
						Log.Logger.LogWarning($"Failed to parse PackageOplogEntry {entryKey}");
					}
				}
				if (_keyToOplogEntry.ContainsKey(entryKey))
				{
					OplogEntry? newEntry = OplogEntry.CreateClass(_keyToOplogEntry[entryKey].GetType().Name);
					if (newEntry == null)
					{
						Log.Logger.LogWarning($"Failed to create op for ({entryKey})");
						return;
					}
					if (newEntry.ParseData(entryField, GetBaseOplogURL()))
					{
						results.Add(newEntry);
					}
					else
					{
						Log.Logger.LogWarning($"Failed to parse op {entryKey} of type {_keyToOplogEntry[entryKey].GetType().Name}");
					}
				}
				else
				{
					Log.Logger.LogDebug("Encountered an op not intended to be parsed based upon /Script/UnrealEd.ProjectPackagingSettings. Skipping parsing of this op.");
					return;
				}
			});

			Log.Logger.LogInformation("Parsed {Count} package entries from oplog.", results.Count);
			List<OplogEntry> returnResults = results.AsList();
			returnResults.SortBy( entry => entry.GetKey() );

			// The ChunkAssignment op stores only packagedata/bulkdata ids — resolve them
			// to filenames against the package entries we just parsed before handing the
			// entry list back to consumers.
			foreach (OplogEntry entry in returnResults)
			{
				if (entry is ChunkAssignmentOp chunkAssignment)
				{
					chunkAssignment.ResolveFilenames(returnResults);
				}
			}
			return returnResults;
		}
	}
#nullable disable
}
