// Copyright Epic Games, Inc. All Rights Reserved.

using AutomationUtils;
using EpicGames.Core;
using EpicGames.Compression;
using EpicGames.Serialization;
using Microsoft.Extensions.Logging;
using System;
using System.IO;
using System.Net.Http;
using System.Net.Http.Headers;
using System.Threading.Tasks;

namespace AutomationScripts.Oplog
{
#nullable enable
	/// <summary>
	/// Writes a single op into a Zen store oplog by POSTing a CbPackage to
	/// <c>/prj/{projectId}/oplog/{oplogId}/new</c>.
	///
	/// Mirrors the C++ <c>FZenStoreWriter::AppendOp(FName, FCbObject)</c> path
	/// in <c>Engine/Source/Developer/IoStoreUtilities/Private/ZenStoreWriter.cpp</c>:
	/// the payload is wrapped in a <c>{ key, value: BinaryAttachment }</c> root
	/// CbObject with a single Oodle-compressed-binary attachment.
	/// </summary>
	public sealed class OplogWriter
	{
		private readonly string _socketHostNameAndPort;
		private readonly string _httpHostNameAndPort;
		private readonly string _projectId;
		private readonly string _oplogId;
		private readonly HttpClient _httpClient;

		private static readonly MediaTypeHeaderValue CbPackageMediaType = new("application/x-ue-cbpkg");

		public OplogWriter(string SocketHostNameAndPort, string HttpHostNameAndPort, string projectId, string oplogId)
		{
			_socketHostNameAndPort = SocketHostNameAndPort;
			_httpHostNameAndPort   = HttpHostNameAndPort;
			_projectId             = projectId;
			_oplogId               = oplogId;

			// Reuse the static client created by OplogReader when present — it has already
			// been configured with the same 20-minute timeout. Setting Timeout on an
			// HttpClient that has already sent a request throws InvalidOperationException
			// ("This instance has already started one or more requests"), so only set it
			// when we created a fresh client.
			if (OplogReader.HttpClient != null)
			{
				_httpClient = OplogReader.HttpClient;
			}
			else
			{
				_httpClient = ZenUtils.CreateHttpClient(_socketHostNameAndPort);
				_httpClient.Timeout = TimeSpan.FromMinutes(20);
			}
		}

		/// <summary>
		/// Append a new op to the oplog. The payload is wrapped in a Zen CbPackage
		/// with one Oodle-compressed binary attachment containing the serialized
		/// CbObject. Throws <see cref="AutomationTool.AutomationException"/> on
		/// network or server failure.
		/// </summary>
		public void AppendOp(string keyName, CbObject payload)
		{
			if (!ZenUtils.IsZenServerRunning(_socketHostNameAndPort))
			{
				throw new AutomationTool.AutomationException(
					$"Zen server is not running at {_socketHostNameAndPort}. " +
					$"Cannot append op '{keyName}' to {_projectId}.{_oplogId}.");
			}

			byte[] body = BuildCbPackage(keyName, payload);

			string uri = $"{GetBaseOplogURL()}/new";
			using var request = new HttpRequestMessage(HttpMethod.Post, uri);
			request.Content = new ByteArrayContent(body);
			request.Content.Headers.ContentType = CbPackageMediaType;

			HttpResponseMessage response;
			try
			{
				response = _httpClient.Send(request);
			}
			catch (Exception ex)
			{
				throw new AutomationTool.AutomationException(
					$"Failed to POST oplog op '{keyName}' to Zen at {_socketHostNameAndPort}: {ex.Message}");
			}

			if (response.StatusCode != System.Net.HttpStatusCode.OK &&
			    response.StatusCode != System.Net.HttpStatusCode.Created)
			{
				Task<string> readTask = response.Content.ReadAsStringAsync();
				readTask.Wait();
				throw new AutomationTool.AutomationException(
					$"Failed to append oplog op '{keyName}' to {_projectId}.{_oplogId}: " +
					$"HTTP {(int)response.StatusCode}. Body: {readTask.Result}");
			}

			Log.Logger.LogInformation("OplogWriter: appended op '{Key}' ({Bytes} bytes) to {Proj}.{Oplog}",
				keyName, body.Length, _projectId, _oplogId);
		}

		public string GetBaseOplogURL()
		{
			return $"http://{_httpHostNameAndPort}/prj/{_projectId}/oplog/{_oplogId}";
		}

		// ---- Private helpers ----

		/// <summary>
		/// Builds the Zen CbPackage wire format produced by C++ <c>SaveCbPackage</c>:
		/// a sequence of top-level CB fields with no framing — root object,
		/// object-attachment hash, then per-attachment (binary blob + binary-attachment
		/// hash), then a terminating null. NOT the same format as
		/// <c>EpicGames.Serialization.CbPackageBuilder</c>, which adds a Jupiter header.
		/// </summary>
		private static byte[] BuildCbPackage(string keyName, CbObject payload)
		{
			// 1. Compress the payload bytes. CompressContent returns the IoHash of the
			//    UNCOMPRESSED content — that is what the BinaryAttachment field references.
			byte[] payloadBytes = payload.GetView().ToArray();

			byte[] compressed;
			IoHash rawHash;
			using (var ms = new MemoryStream())
			{
				rawHash = CompressedBuffer.CompressContent(
					ms,
					OodleCompressorType.Mermaid,
					OodleCompressionLevel.Normal,
					payloadBytes);
				compressed = ms.ToArray();
			}

			// 2. Build the wrapper root object: { key: <keyName>, value: <BinaryAttachment rawHash> }.
			var rootWriter = new CbWriter();
			rootWriter.BeginObject();
			rootWriter.WriteString("key", keyName);
			rootWriter.WriteBinaryAttachment("value", rawHash);
			rootWriter.EndObject();
			byte[] rootBytes = rootWriter.ToByteArray();
			IoHash rootHash = IoHash.Compute(rootBytes);
			CbObject rootObj = new CbObject(rootBytes);

			// 3. Build the Zen CbPackage stream as top-level (unnamed) fields.
			var pkgWriter = new CbWriter();
			pkgWriter.WriteObject(rootObj);
			pkgWriter.WriteObjectAttachmentValue(rootHash);
			pkgWriter.WriteBinarySpanValue(compressed);
			pkgWriter.WriteBinaryAttachmentValue(rawHash);
			pkgWriter.WriteNullValue();
			return pkgWriter.ToByteArray();
		}
	}
#nullable disable
}
