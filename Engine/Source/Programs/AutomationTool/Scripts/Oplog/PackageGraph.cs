// Copyright Epic Games, Inc. All Rights Reserved.

using EpicGames.Core;
using System;
using System.Collections;
using System.Collections.Concurrent;
using System.Collections.Generic;
using System.Threading.Tasks;

namespace AutomationScripts.Oplog
{
#nullable enable
	/// <summary>
	/// Opaque handle to a node in a <see cref="PackageGraph"/>.
	/// The internal representation (index) is an implementation detail that can change.
	/// Instances are only obtained via <see cref="PackageGraph"/> lookup methods.
	/// </summary>
	public readonly struct PackageNode : IEquatable<PackageNode>
	{
		/// <summary>Internal index into the graph's data arrays. Not part of the public contract.</summary>
		public readonly int Index;

		/// <summary>The graph this node belongs to. Prevents cross-graph comparisons.</summary>
		internal readonly PackageGraph Graph;

		internal PackageNode(PackageGraph graph, int index)
		{
			Graph = graph;
			Index = index;
		}

		public bool Equals(PackageNode other) => Graph == other.Graph && Index == other.Index;
		public override bool Equals(object? obj) => obj is PackageNode other && Equals(other);
		public override int GetHashCode() => HashCode.Combine(Graph, Index);

		public static bool operator ==(PackageNode left, PackageNode right) => left.Equals(right);
		public static bool operator !=(PackageNode left, PackageNode right) => !left.Equals(right);
	}

	/// <summary>Identifies which oplog array a <see cref="PackageFileInfo"/> was read from.</summary>
	public enum EPackageFileType
	{
		PackageData,   // from the packagedata array (e.g. .uasset)
		Bulk,          // from bulkdata array, type = "Standard" (e.g. .ubulk)
		BulkOptional,  // from bulkdata array, type = "Optional" (e.g. .uptnl)
	}

	/// <summary>Metadata for a single physical file associated with a cooked package.</summary>
	public readonly record struct PackageFileInfo(
		string Id,
		long Size,
		string Filename,
		EPackageFileType FileType,
		IoHash ChunkId
	);

	/// <summary>
	/// Metadata for a package node in the package graph.
	/// All fields reflect what was stored in the oplog at cook time.
	/// Fields marked "cook TODO" default to safe values until the cook step is updated to write them.
	/// </summary>
	public sealed class PackageMetadata
	{
		/// <summary>Lowercase package name (e.g. "/game/foo/bar").</summary>
		public string PackageName { get; }

		/// <summary>FPackageId value — 64-bit hash of the package name.</summary>
		public ulong PackageId { get; }

		/// <summary>EPackageStoreEntryFlags bits (HasPackageData, AutoOptional, etc.).</summary>
		public uint Flags { get; }

		/// <summary>
		/// Whether this package is a startup package that must live in chunk 0.
		/// Defaults to false until the cook step writes this field.
		/// </summary>
		public bool IsStartupPackage { get; }

		/// <summary>Full class path (e.g. "/Script/Engine.World"). Empty if unknown.</summary>
		public string Class { get; }

		/// <summary>True if the asset's class derives from UPrimaryDataAsset.</summary>
		public bool IsPrimaryAsset { get; }

		/// <summary>Files from the oplog packagedata array for this package (e.g. .uasset).</summary>
		public IReadOnlyList<PackageFileInfo> PackageFiles { get; }

		/// <summary>Files from the oplog bulkdata array for this package (e.g. .ubulk, .uptnl).</summary>
		public IReadOnlyList<PackageFileInfo> BulkDataFiles { get; }

		internal PackageMetadata(
			string packageName,
			ulong packageId,
			uint flags,
			bool isStartupPackage,
			string className = "",
			bool isPrimaryAsset = false,
			IReadOnlyList<PackageFileInfo>? packageFiles = null,
			IReadOnlyList<PackageFileInfo>? bulkDataFiles = null)
		{
			PackageName = packageName;
			PackageId = packageId;
			Flags = flags;
			IsStartupPackage = isStartupPackage;
			Class = className;
			IsPrimaryAsset = isPrimaryAsset;
			PackageFiles = packageFiles ?? [];
			BulkDataFiles = bulkDataFiles ?? [];
		}
	}

	/// <summary>
	/// Compact internal storage for the package graph.
	/// All arrays are indexed by the node's integer index.
	/// This class is an implementation detail — callers use <see cref="PackageGraph"/>.
	/// </summary>
	internal sealed class PackageGraphData
	{
		internal int NodeCount;

		// Per-node metadata — single array; adding fields to PackageMetadata does not require
		// changing this class.
		internal PackageMetadata[] Metadata;

		// Edge arrays: each element is an array of node indices
		internal int[][] HardDeps;     // importedpackageids (+ optional segment) resolved to indices
		internal int[][] SoftDeps;     // softpackagereferences resolved to indices
		internal int[][] RuntimeDeps;  // cook.artifacts RuntimeDependencies resolved to indices
		internal int[][] HardRefs;     // reverse of HardDeps (who hard-imports this package)
		internal int[][] SoftRefs;     // reverse of SoftDeps
		internal int[][] RuntimeRefs;  // reverse of RuntimeDeps

		// Lookup tables
		internal Dictionary<ulong, int>  IndexById;
		internal Dictionary<string, int> IndexByName;  // key is lowercase package name

		internal PackageGraphData(int capacity)
		{
			NodeCount   = 0;
			Metadata    = new PackageMetadata[capacity];
			HardDeps    = new int[capacity][];
			SoftDeps    = new int[capacity][];
			RuntimeDeps = new int[capacity][];
			HardRefs    = new int[capacity][];
			SoftRefs    = new int[capacity][];
			RuntimeRefs = new int[capacity][];
			IndexById   = new Dictionary<ulong, int>(capacity);
			IndexByName = new Dictionary<string, int>(capacity, StringComparer.OrdinalIgnoreCase);
		}
	}

	/// <summary>
	/// Immutable directed graph of cooked packages, built from the Zen store oplog.
	/// <para>
	/// Nodes are <see cref="PackageNode"/> handles (opaque). Use <see cref="TryGetNode"/> to obtain
	/// a handle by package name or ID, then traverse edges with <see cref="GetDependencies"/> /
	/// <see cref="GetReferencers"/> and read metadata with <see cref="GetMetadata"/>.
	/// </para>
	/// <para>
	/// The internal storage format is an implementation detail and may change.
	/// </para>
	/// </summary>
	public sealed class PackageGraph
	{
		private readonly PackageGraphData _data;

		internal PackageGraph(PackageGraphData data)
		{
			_data = data;
		}

		/// <summary>Total number of package nodes in the graph.</summary>
		public int NodeCount => _data.NodeCount;

		/// <summary>Enumerate all nodes in the graph.</summary>
		public IEnumerable<PackageNode> AllNodes
		{
			get
			{
				for (int i = 0; i < _data.NodeCount; i++)
				{
					yield return new PackageNode(this, i);
				}
			}
		}

		// ---- Lookup ----

		/// <summary>Look up a node by lowercase package name (e.g. "/game/foo/bar").</summary>
		public bool TryGetNode(string packageName, out PackageNode node)
		{
			if (_data.IndexByName.TryGetValue(packageName, out int idx))
			{
				node = new PackageNode(this, idx);
				return true;
			}
			node = default;
			return false;
		}

		/// <summary>Look up a node by FPackageId (64-bit value).</summary>
		public bool TryGetNode(ulong packageId, out PackageNode node)
		{
			if (_data.IndexById.TryGetValue(packageId, out int idx))
			{
				node = new PackageNode(this, idx);
				return true;
			}
			node = default;
			return false;
		}

		// ---- Traversal ----

		/// <summary>
		/// Returns the hard dependencies (imported packages) of the given node.
		/// <para>
		/// By default the result also includes runtime dependencies (entries from the cook
		/// artifacts <c>RuntimeDependencies</c> set), since callers historically treated runtime
		/// deps as hard deps. Pass <paramref name="includeRuntimeDependencies"/> = <c>false</c>
		/// to get only the package-store hard imports.
		/// </para>
		/// </summary>
		public IReadOnlyList<PackageNode> GetDependencies(PackageNode node, bool includeRuntimeDependencies = true)
		{
			ValidateNode(node);
			int[] hard = _data.HardDeps[node.Index];
			if (!includeRuntimeDependencies)
			{
				return ToNodeList(hard);
			}

			int[] runtime = _data.RuntimeDeps[node.Index];
			if (runtime == null || runtime.Length == 0)
			{
				return ToNodeList(hard);
			}
			if (hard == null || hard.Length == 0)
			{
				return ToNodeList(runtime);
			}

			var combined = new int[hard.Length + runtime.Length];
			Array.Copy(hard, 0, combined, 0, hard.Length);
			Array.Copy(runtime, 0, combined, hard.Length, runtime.Length);
			return ToNodeList(combined);
		}

		/// <summary>Returns only the runtime dependencies (from cook artifacts) of the given node.</summary>
		public IReadOnlyList<PackageNode> GetRuntimeDependencies(PackageNode node)
		{
			ValidateNode(node);
			return ToNodeList(_data.RuntimeDeps[node.Index]);
		}

		/// <summary>Returns the soft dependencies (soft package references) of the given node.</summary>
		public IReadOnlyList<PackageNode> GetSoftDependencies(PackageNode node)
		{
			ValidateNode(node);
			return ToNodeList(_data.SoftDeps[node.Index]);
		}

		/// <summary>
		/// Returns all packages that hard-import the given node.
		/// <para>
		/// By default the result also includes packages that runtime-depend on this node
		/// (i.e. reverse edges of <see cref="GetRuntimeDependencies"/>). Pass
		/// <paramref name="includeRuntimeReferencers"/> = <c>false</c> to get only the
		/// package-store hard referencers.
		/// </para>
		/// </summary>
		public IReadOnlyList<PackageNode> GetReferencers(PackageNode node, bool includeRuntimeReferencers = true)
		{
			ValidateNode(node);
			int[] hard = _data.HardRefs[node.Index];
			if (!includeRuntimeReferencers)
			{
				return ToNodeList(hard);
			}

			int[] runtime = _data.RuntimeRefs[node.Index];
			if (runtime == null || runtime.Length == 0)
			{
				return ToNodeList(hard);
			}
			if (hard == null || hard.Length == 0)
			{
				return ToNodeList(runtime);
			}

			var combined = new int[hard.Length + runtime.Length];
			Array.Copy(hard, 0, combined, 0, hard.Length);
			Array.Copy(runtime, 0, combined, hard.Length, runtime.Length);
			return ToNodeList(combined);
		}

		/// <summary>Returns only the packages that runtime-depend on the given node.</summary>
		public IReadOnlyList<PackageNode> GetRuntimeReferencers(PackageNode node)
		{
			ValidateNode(node);
			return ToNodeList(_data.RuntimeRefs[node.Index]);
		}

		/// <summary>Returns all packages that soft-reference the given node.</summary>
		public IReadOnlyList<PackageNode> GetSoftReferencers(PackageNode node)
		{
			ValidateNode(node);
			return ToNodeList(_data.SoftRefs[node.Index]);
		}

		/// <summary>
		/// Returns the set of nodes that are <i>exclusively</i> dependencies of the given seed set —
		/// i.e. every node reachable from <paramref name="seeds"/> that cannot also be reached from any
		/// node outside <paramref name="seeds"/>. The seeds themselves are always included in the result.
		/// 
		/// Tests under - \Engine\Source\Programs\AutomationTool.Tests\Oplog\PackageGraphTests.cs
		/// dotnet test "<FullPath>/Engine/Source/Programs/AutomationTool.Tests/AutomationTool.Tests.csproj"
		/// <para>
		/// Formally: let R be the forward-reachable closure of <paramref name="seeds"/> and let T ⊂ R\seeds
		/// be the set of nodes in R that have at least one path from a node in V\R, or transitively from
		/// another tainted R-node. The returned set is R \ T (which always contains the seeds).
		/// </para>
		/// <para>
		/// Edges traversed follow the same hard/runtime semantics as <see cref="GetDependencies"/>:
		/// soft dependencies are <b>not</b> followed. <paramref name="includeRuntimeDependencies"/>
		/// toggles whether runtime deps (from cook artifacts) count as edges, on both the forward
		/// and reverse traversals.
		/// </para>
		/// </summary>
		/// <param name="seeds">The input super-set. Duplicates are tolerated; all must belong to this graph.</param>
		/// <param name="includeRuntimeDependencies">Match <see cref="GetDependencies"/>'s default (true).</param>
		/// <returns>A new <see cref="HashSet{PackageNode}"/> containing the seeds plus all exclusive dependencies.</returns>
		public HashSet<PackageNode> GetExclusiveDependencyClosure(
			IEnumerable<PackageNode> seeds,
			bool includeRuntimeDependencies = true)
		{
			if (seeds == null)
			{
				throw new ArgumentNullException(nameof(seeds));
			}

			int N = _data.NodeCount;
			var result = new HashSet<PackageNode>();
			if (N == 0)
			{
				return result;
			}

			// Step 0: materialize seed indices.
			var seedMask = new BitArray(N);
			var queue = new Queue<int>();
			foreach (PackageNode seed in seeds)
			{
				ValidateNode(seed);
				if (!seedMask[seed.Index])
				{
					seedMask[seed.Index] = true;
					queue.Enqueue(seed.Index);
				}
			}
			if (queue.Count == 0)
			{
				return result;
			}

			// Step 1: R = forward-reachable closure of seeds (hard + optional runtime edges).
			var reachMask = new BitArray(N);
			foreach (int idx in queue)
			{
				reachMask[idx] = true;
			}

			while (queue.Count > 0)
			{
				int n = queue.Dequeue();
				ExpandForward(n, reachMask, queue, includeRuntimeDependencies);
			}

			// Step 2: seed taint set T with non-seed R-nodes that have a predecessor outside R.
			// Direct external entry points are the only way an outside node can first enter R;
			// Step 3 then handles all transitive cases by forward-propagating within R.
			var taintMask = new BitArray(N);
			var taintQueue = new Queue<int>();
			for (int n = 0; n < N; n++)
			{
				if (!reachMask[n] || seedMask[n])
				{
					continue;
				}

				if (HasExternalReferencer(n, reachMask, includeRuntimeDependencies))
				{
					taintMask[n] = true;
					taintQueue.Enqueue(n);
				}
			}

			// Step 3: propagate taint forward within R. Seeds are never tainted.
			while (taintQueue.Count > 0)
			{
				int n = taintQueue.Dequeue();
				PropagateTaint(n, reachMask, seedMask, taintMask, taintQueue, includeRuntimeDependencies);
			}

			// Step 4: result = R \ T. Seeds survive because they are excluded from taint.
			for (int i = 0; i < N; i++)
			{
				if (reachMask[i] && !taintMask[i])
				{
					result.Add(new PackageNode(this, i));
				}
			}
			return result;
		}

		// ---- Metadata ----

		/// <summary>Returns the metadata for the given node.</summary>
		public PackageMetadata GetMetadata(PackageNode node)
		{
			ValidateNode(node);
			return _data.Metadata[node.Index];
		}

		// ---- Metadata queries ----

		/// <summary>
		/// Filters all nodes by the given metadata predicate.
		/// <example>
		/// <code>
		/// // All primary assets of class "Texture2d"
		/// var nodes = graph.QueryNodes(m => m.IsPrimaryAsset &amp;&amp; m.ClassType == "Texture2d");
		/// </code>
		/// </example>
		/// </summary>
		public IEnumerable<PackageNode> QueryNodes(Predicate<PackageMetadata> predicate)
		{
			var Results = new ConcurrentBag<PackageNode>();
			Parallel.For(0, _data.NodeCount, Idx =>
			{
				if (predicate(_data.Metadata[Idx]))
				{
					Results.Add(new PackageNode(this, Idx));
				}
			});
			return Results;
		}

		/// <summary>Returns all startup packages (IsStartupPackage == true).</summary>
		public IEnumerable<PackageNode> GetStartupPackages()
		{
			var Results = new ConcurrentBag<PackageNode>();
			Parallel.For(0, _data.NodeCount, Idx =>
			{
				if (_data.Metadata[Idx].IsStartupPackage)
				{
					Results.Add(new PackageNode(this, Idx));
				}
			});
			return Results;
		}

		// ---- Internal helpers ----

		private void ValidateNode(PackageNode node)
		{
			if (node.Graph != this)
			{
				throw new ArgumentException("PackageNode does not belong to this graph.", nameof(node));
			}
		}

		private PackageNode[] ToNodeList(int[] indices)
		{
			if (indices == null || indices.Length == 0)
			{
				return Array.Empty<PackageNode>();
			}

			var result = new PackageNode[indices.Length];
			for (int i = 0; i < indices.Length; i++)
			{
				result[i] = new PackageNode(this, indices[i]);
			}
			return result;
		}

		private void ExpandForward(int n, BitArray reachMask, Queue<int> queue, bool includeRuntime)
		{
			int[] hard = _data.HardDeps[n];
			if (hard != null)
			{
				for (int i = 0; i < hard.Length; i++)
				{
					int outIdx = hard[i];
					if (!reachMask[outIdx])
					{
						reachMask[outIdx] = true;
						queue.Enqueue(outIdx);
					}
				}
			}

			if (!includeRuntime)
			{
				return;
			}

			int[] runtime = _data.RuntimeDeps[n];
			if (runtime != null)
			{
				for (int i = 0; i < runtime.Length; i++)
				{
					int outIdx = runtime[i];
					if (!reachMask[outIdx])
					{
						reachMask[outIdx] = true;
						queue.Enqueue(outIdx);
					}
				}
			}
		}

		private bool HasExternalReferencer(int n, BitArray reachMask, bool includeRuntime)
		{
			int[] hard = _data.HardRefs[n];
			if (hard != null)
			{
				for (int i = 0; i < hard.Length; i++)
				{
					if (!reachMask[hard[i]])
					{
						return true;
					}
				}
			}

			if (!includeRuntime)
			{
				return false;
			}

			int[] runtime = _data.RuntimeRefs[n];
			if (runtime != null)
			{
				for (int i = 0; i < runtime.Length; i++)
				{
					if (!reachMask[runtime[i]])
					{
						return true;
					}
				}
			}
			return false;
		}

		private void PropagateTaint(int n, BitArray reachMask, BitArray seedMask, BitArray taintMask, Queue<int> taintQueue, bool includeRuntime)
		{
			int[] hard = _data.HardDeps[n];
			if (hard != null)
			{
				for (int i = 0; i < hard.Length; i++)
				{
					int outIdx = hard[i];
					if (reachMask[outIdx] && !seedMask[outIdx] && !taintMask[outIdx])
					{
						taintMask[outIdx] = true;
						taintQueue.Enqueue(outIdx);
					}
				}
			}

			if (!includeRuntime)
			{
				return;
			}

			int[] runtime = _data.RuntimeDeps[n];
			if (runtime != null)
			{
				for (int i = 0; i < runtime.Length; i++)
				{
					int outIdx = runtime[i];
					if (reachMask[outIdx] && !seedMask[outIdx] && !taintMask[outIdx])
					{
						taintMask[outIdx] = true;
						taintQueue.Enqueue(outIdx);
					}
				}
			}
		}
	}
#nullable enable
}
