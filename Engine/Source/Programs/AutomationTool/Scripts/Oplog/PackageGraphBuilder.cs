// Copyright Epic Games, Inc. All Rights Reserved.

using System.Collections.Generic;
using System.Linq;

namespace AutomationScripts.Oplog
{
#nullable enable
	/// <summary>
	/// Builds a <see cref="PackageGraph"/> from a list of <see cref="OplogEntry"/> objects.
	/// Uses a two-pass approach:
	/// <list type="number">
	///   <item>Pass 1 — allocate node arrays and assign indices; populate all metadata.</item>
	///   <item>Pass 2 — resolve PackageId references to node indices; fill edge arrays and build reverse edges.</item>
	/// </list>
	/// </summary>
	public static class PackageGraphBuilder
	{
		public static PackageGraph Build(IReadOnlyList<OplogEntry> entries)
		{
			List<PackageOplogEntry> PackageEntries = new List<PackageOplogEntry>(entries.Count);
			CookStartupPackagesOp? CookStartupPackagesEntry = null;
			foreach (OplogEntry OplogEntry in entries)
			{
				var packageEntry = OplogEntry as PackageOplogEntry;
				var startupPackagesEntry = OplogEntry as CookStartupPackagesOp;
				if (packageEntry != null)
				{
					PackageEntries.Add(packageEntry);
				}
				else if (startupPackagesEntry != null)
				{
					CookStartupPackagesEntry = startupPackagesEntry;
				}
			}

			HashSet<string> StartupPackages = new HashSet<string>();
			if (CookStartupPackagesEntry != null)
			{
				StartupPackages = CookStartupPackagesEntry.StartupPackages;
			}

			int count = PackageEntries.Count;
			var data = new PackageGraphData(count);

			// --- Pass 1: allocate nodes and populate metadata ---
			for (int i = 0; i < count; i++)
			{
				var entry = PackageEntries[i];

				string className = string.Empty;
				bool isPrimaryAsset = false;
				if (entry.AdditionalMetadata.TryGetValue(typeof(GlobalPackageMetaOp), out OplogEntry? globalMeta)
					&& globalMeta is GlobalPackageMetaOp gm)
				{
					className = gm.Class;
					isPrimaryAsset = gm.IsPrimaryAsset;
				}

				data.Metadata[i] = new PackageMetadata(
					entry.PackageName,
					entry.PackageId,
					entry.Flags,
					StartupPackages.Contains(entry.PackageName),
					className,
					isPrimaryAsset,
					entry.PackageFiles,
					entry.BulkDataFiles);

				data.IndexById[entry.PackageId]    = i;
				data.IndexByName[entry.PackageName] = i;
			}

			data.NodeCount = count;

			// --- Pass 2: resolve PackageId references to indices, build edge arrays ---
			// Accumulate reverse edges during the forward pass.
			var hardRefsAccumulator    = new List<int>[count];
			var softRefsAccumulator    = new List<int>[count];
			var runtimeRefsAccumulator = new List<int>[count];
			for (int i = 0; i < count; i++)
			{
				hardRefsAccumulator[i]    = new List<int>();
				softRefsAccumulator[i]    = new List<int>();
				runtimeRefsAccumulator[i] = new List<int>();
			}

			for (int i = 0; i < count; i++)
			{
				var entry = PackageEntries[i];

				PackageMetadata Metadata = data.Metadata[i];
				// Hard dependencies — package-store imports plus optional-segment imports.
				// Runtime dependencies are tracked separately so callers can opt out of them.
				var hardDeps = ResolveIds(entry.ImportedPackageIds, data.IndexById);
				var optionalHardDeps = ResolveIds(entry.OptionalSegmentImportedPackageIds, data.IndexById);
				foreach (int depIdx in hardDeps)
				{
					hardRefsAccumulator[depIdx].Add(i);
				}
				foreach (int depIdx in optionalHardDeps)
				{
					hardRefsAccumulator[depIdx].Add(i);
				}
				data.HardDeps[i] = new List<int>(hardDeps).Concat(optionalHardDeps).ToArray();

				// Runtime dependencies (from cook.artifacts) — kept in their own category.
				var runtimeDeps = ResolveIds(entry.RuntimeDependencies, data.IndexByName);
				foreach (int depIdx in runtimeDeps)
				{
					runtimeRefsAccumulator[depIdx].Add(i);
				}
				data.RuntimeDeps[i] = runtimeDeps;

				// Soft dependencies
				var softDeps = ResolveIds(entry.SoftPackageReferences, data.IndexById);
				data.SoftDeps[i] = softDeps;
				foreach (int depIdx in softDeps)
				{
					softRefsAccumulator[depIdx].Add(i);
				}
			}

			// Materialize reverse-edge lists into arrays
			for (int i = 0; i < count; i++)
			{
				data.HardRefs[i]    = hardRefsAccumulator[i].ToArray();
				data.SoftRefs[i]    = softRefsAccumulator[i].ToArray();
				data.RuntimeRefs[i] = runtimeRefsAccumulator[i].ToArray();
			}

			return new PackageGraph(data);
		}

		/// <summary>
		/// Resolves a list of PackageId values to node indices.
		/// PackageIds that are not found in the graph (e.g. engine/script packages) are silently skipped.
		/// </summary>
		private static int[] ResolveIds(List<ulong> ids, Dictionary<ulong, int> indexById)
		{
			if (ids.Count == 0)
			{
				return System.Array.Empty<int>();
			}

			var result = new List<int>(ids.Count);
			foreach (ulong id in ids)
			{
				if (indexById.TryGetValue(id, out int idx))
				{
					result.Add(idx);
				}
				// Else: skip — script/engine packages are not in the oplog
			}
			return result.ToArray();
		}

		/// <summary>
		/// Resolves a list of PackageId values to node indices.
		/// PackageIds that are not found in the graph (e.g. engine/script packages) are silently skipped.
		/// </summary>
		private static int[] ResolveIds(List<string> packageNames, Dictionary<string, int> indexByPackage)
		{
			if (packageNames.Count == 0)
			{
				return System.Array.Empty<int>();
			}

			var result = new List<int>(packageNames.Count);
			foreach (string id in packageNames)
			{
				if (indexByPackage.TryGetValue(id, out int idx))
				{
					result.Add(idx);
				}
				// Else: skip — script/engine packages are not in the oplog
			}
			return result.ToArray();
		}
	}
#nullable disable
}
