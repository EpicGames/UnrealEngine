// Copyright Epic Games, Inc. All Rights Reserved.

using AutomationTool;
using AutomationUtils;
using EpicGames.Core;
using Microsoft.Extensions.Logging;
using System;
using System.Collections.Concurrent;
using System.Collections.Generic;
using System.Data;
using System.IO;
using System.Linq;
using System.Reflection;
using System.Text;
using System.Threading.Tasks;
using UnrealBuildTool;

namespace AutomationScripts.Oplog
{
	/// <summary>
	/// Reads a cooked package oplog from the Zen store, builds a dependency graph, assigns
	/// packages to chunks using a configurable <see cref="ChunkAssigner"/>, and writes
	/// pakchunkN.txt / pakchunklist.txt manifests compatible with CreatePaksUsingChunkManifests.
	/// </summary>
	/// <remarks>
	/// Parameters:
	/// <code>
	///   -Project=&lt;path&gt;      Path to the .uproject file (required — derives the project name)
	///   -ZenHost=&lt;host&gt;      Zen server hostname  (default: localhost)
	///   -ZenPort=&lt;port&gt;      Zen server port       (default: 8558)
	///   -ProjectId=&lt;id&gt;     Zen project ID
	///   -OplogId=&lt;id&gt;       Zen oplog ID
	///   -Platform=&lt;name&gt;    Target platform, e.g. WindowsClient
	///   -CookDir=&lt;path&gt;     Root of the cooked output directory
	///   -OutputDir=&lt;path&gt;   Where to write the chunk manifest files
	///   -GameIni=&lt;path&gt;     Optional: path to a Game.ini used to select a ChunkAssigner
	/// </code>
	/// The chunk assigner can be overridden per-project via DefaultGame.ini:
	/// <code>
	///   [/Script/UnrealEd.ProjectPackagingSettings]
	///   ChunkAssigner=MyCustomAssignerName
	/// </code>
	/// </remarks>
	[Help("Generates pak chunk manifests from an oplog fetched from the Zen cook store.")]
	[Help("Project=<path>", "Path to the .uproject for this operation.")]
	[Help("ZenHost=<host>", "Zen server hostname (default: localhost)")]
	[Help("ZenPort=<port>", "Zen server port (default: 8558)")]
	[Help("ProjectId=<id>", "Zen project ID")]
	[Help("OplogId=<id>", "Zen oplog ID")]
	[Help("Target=<name>", "Target name to read receipt for (e.g. MyGameClient)")]
	[Help("Configuration=<name>", "Target configuration (default: Development)")]
	[Help("Platform=<name>", "Target platform (e.g. Win64)")]
	[Help("CookPlatform=<name>", "Target platform (e.g. WindowsClient)")]
	[Help("CookDir=<path>", "Root of the cooked output directory")]
	[Help("OutputDir=<path>", "Directory to write pakchunkN.txt / pakchunklist.txt")]
	[Help("GameIni=<path>", "Optional path to a Game.ini for custom assigner selection")]
	[Help("SkipLocalization", "If localization data should not be generated")]
	public class OplogChunkAssigner : BuildCommand
	{
#nullable enable
		public override void ExecuteBuild()
		{
			// ---- Parse parameters ----
			string ProjectFilePath = ParseParamValue("Project") ?? throw new AutomationException("-Project is required");
			FileReference ProjectFile = new FileReference(ProjectFilePath);

			string zenHost = ParseParamValue("ZenHost", "localhost");
			int zenPort = int.Parse(ParseParamValue("ZenPort", "8558") ?? "8558");
			string projectId = ParseParamValue("ProjectId", ProjectUtils.GetProjectPathId(ProjectFile));
			string oplogId = ParseParamValue("OplogId") ?? throw new AutomationException("-OplogId is required");
			string targetName = ParseParamValue("Target") ?? throw new AutomationException("-Target is required");
			string configName = ParseParamValue("Configuration", "Development") ?? "Development";
			string platform = ParseParamValue("Platform") ?? throw new AutomationException("-Platform is required");
			string cookPlatform = ParseParamValue("CookPlatform") ?? throw new AutomationException("-CookPlatform is required");
			string cookDir = ParseParamValue("CookDir") ?? throw new AutomationException("-CookDir is required");
			string outputDir = ParseParamValue("OutputDir") ?? throw new AutomationException("-OutputDir is required");
			bool skipLocalization = ParseParam("SkipLocalization");

			ZenRunContext.GetRobustHostNameAndPortStrings(zenHost, zenPort,
				out string SocketHostNameAndPort, out string HttpHostNameAndPort);
			Run(ProjectFile, SocketHostNameAndPort, HttpHostNameAndPort, projectId, oplogId, targetName, configName, platform, cookPlatform, cookDir, outputDir, skipLocalization, this);
		}

		/// <summary>
		/// Run OplogChunkAssigner from a staging context, deriving parameters from ProjectParams and DeploymentContext.
		/// </summary>
		public static void RunFromStagingContext(ProjectParams Params, DeploymentContext SC)
		{
			if (string.IsNullOrEmpty(SC.PackageStoreData.ZenServerStore.ProjectId) || string.IsNullOrEmpty(SC.PackageStoreData.ZenServerStore.OplogId))
			{
				throw new AutomationException("SC.PackageStoreData.ZenServerStore must be loaded before performing chunk assignment. Ensure LoadPackageStoreData was called.");
			}

			FileReference ProjectFile = Params.RawProjectPath;
			string projectId = SC.PackageStoreData.ZenServerStore.ProjectId;
			string oplogId = SC.PackageStoreData.ZenServerStore.OplogId;
			string targetName = SC.StageTargets.FirstOrDefault().Receipt.TargetName;
			string configName = SC.StageTargetConfigurations.FirstOrDefault().ToString();
			string platform = SC.StageTargetPlatform.PlatformType.ToString();
			string cookPlatform = SC.CookPlatform;
			string cookDir = SC.PlatformCookDir.FullName;
			string outputDir = Path.Combine(SC.MetadataDir.FullName, "ChunkManifest");
			bool skipLocalization = false;

			ZenRunContext.GetRobustHostNameAndPortStrings(SC.PackageStoreData.ZenServerStore.HostName, SC.PackageStoreData.ZenServerStore.HostPort,
				out string SocketHostNameAndPort, out string HttpHostNameAndPort);
			Run(ProjectFile, SocketHostNameAndPort, HttpHostNameAndPort, projectId, oplogId, targetName, configName, platform, cookPlatform, cookDir, outputDir, skipLocalization, context: null);
		}

		private static void Run(
			FileReference ProjectFile,
			string SocketHostNameAndPort,
			string HttpHostNameAndPort,
			string projectId,
			string oplogId,
			string targetName,
			string configName,
			string platform,
			string cookPlatform,
			string cookDir,
			string outputDir,
			bool skipLocalization,
			BuildCommand? context)
		{
			Logger.LogInformation("OplogChunkAssigner starting.");
			Logger.LogInformation("  Zen:      {HttpHostNameAndPort}, project={ProjectId}, oplog={OplogId}", HttpHostNameAndPort, projectId, oplogId);
			Logger.LogInformation("  Target:   {Target} {Configuration}", targetName, configName);
			Logger.LogInformation("  Platform: {Platform}", platform);
			Logger.LogInformation("  CookPlatform: {CookPlatform}", cookPlatform);
			Logger.LogInformation("  CookDir:  {CookDir}", cookDir);
			Logger.LogInformation("  OutputDir:{OutputDir}", outputDir);

			Logger.LogInformation($"Clearing output dir: {outputDir}");
			var OutputFilesDir = new DirectoryInfo(outputDir);
			if (OutputFilesDir.Exists)
			{
				OutputFilesDir.GetFiles("*.txt", SearchOption.AllDirectories).ToList().ForEach(File => File.Delete());
				OutputFilesDir.GetFiles("*.json", SearchOption.AllDirectories).ToList().ForEach(File => File.Delete());
			}

			// ---- Read target receipt ----
			UnrealTargetPlatform TargetPlatform = UnrealTargetPlatform.Parse(platform);
			UnrealTargetConfiguration TargetConfig = (UnrealTargetConfiguration)Enum.Parse(typeof(UnrealTargetConfiguration), configName, ignoreCase: true);
			DirectoryReference ReceiptBaseDir = ProjectFile.Directory;
			FileReference ReceiptFileName = TargetReceipt.GetDefaultPath(ReceiptBaseDir, targetName, TargetPlatform, TargetConfig, null);
			if (!FileReference.Exists(ReceiptFileName))
			{
				throw new AutomationException($"Missing receipt '{ReceiptFileName}'. Check that this target has been built. UBT can be ran with SkipBuild=\"true\" to only create this file.");
			}
			TargetReceipt? Receipt;
			if (!TargetReceipt.TryRead(ReceiptFileName, out Receipt))
			{
				throw new AutomationException("Missing or invalid target receipt ({0})", ReceiptFileName);
			}
			Logger.LogInformation("  Receipt:  {ReceiptFileName}", ReceiptFileName);

			// ---- Initialize class registries ----
			{
				Dictionary<string, Type> classRegistry = BuildRegistry([typeof(OplogEntry), typeof(ChunkAssigner)]);
				OplogEntry.s_registry = classRegistry;
				ChunkAssigner.s_registry = classRegistry;
			}

			// ---- Select chunk assigner ----
			ConfigHierarchy engineIni = ConfigCache.ReadHierarchy(
				ConfigHierarchyType.Engine,
				ProjectFile.Directory,
				TargetPlatform);
			ConfigHierarchy gameIni = ConfigCache.ReadHierarchy(
				ConfigHierarchyType.Game,
				ProjectFile.Directory,
				TargetPlatform);
			ChunkAssigner Assigner = ChunkAssigner.CreateFromConfig(ProjectFile, engineIni, gameIni);

			// ---- Read oplog entries from Zen server ----
			var reader = new OplogReader(SocketHostNameAndPort, HttpHostNameAndPort, projectId, oplogId, gameIni);
			IReadOnlyList<OplogEntry> entries = reader.ReadEntries();

			// ---- Assign packages to chunks ----
			ChunkAssignments assignments = Assigner.AssignChunks(entries, TargetPlatform, Receipt);

			string normalizedCookDir = cookDir.Replace('/', '\\').TrimEnd('\\');

			// ---- Write chunk_assignments.json (debug-only; oplog op is authoritative) ----
			var jsonWriter = new ChunkAssignmentJsonWriter();
			jsonWriter.Write(assignments, outputDir);

			// ---- Append ChunkAssignment op to the Zen oplog ----
			try
			{
				var oplogWriter = new OplogWriter(SocketHostNameAndPort, HttpHostNameAndPort, projectId, oplogId);
				jsonWriter.WriteOp(assignments, oplogWriter);
			}
			catch (System.Exception ex)
			{
				Logger.LogWarning("Failed to append ChunkAssignment op to oplog ({Proj}.{Oplog}): {Msg}. " +
					"chunk_assignments.json was still written.",
					projectId, oplogId, ex.Message);
			}

			// ---- Run Commandlet ----
			// Commandlet is used to fixup the AssetRegistry.bin as well as write out the localization files.
			StringBuilder CommandletArgs = new();
			if (!skipLocalization)
			{
				CommandletArgs.Append(" -GenerateLocalization");
			}
			CommandletArgs.Append($" -CookDir=\"{normalizedCookDir}\"");
			CommandletArgs.Append($" -Platform=\"{cookPlatform}\"");
			// ChunkAssignerCommandlet reads chunk assignments from the per-platform Zen
			// oplog (ChunkAssignment op written above)
			CommandletArgs.Append($" -ProjectId=\"{projectId}\"");
			CommandletArgs.Append($" -OplogId=\"{oplogId}\"");

			FileReference AssetRegistryFile = new FileReference($"{normalizedCookDir}/{ProjectFile.GetFileNameWithoutAnyExtensions()}/AssetRegistry.bin");
			if (AssetRegistryFile.ToFileInfo().Exists)
			{
				CommandletArgs.Append($" -AssetRegistryFile=\"{AssetRegistryFile.FullName}\"");
			}
			else
			{
				Logger.LogWarning($"AssetRegistry.bin does not exist at {AssetRegistryFile.FullName}");
			}

			FileReference DevelopmentAssetRegistryFile = new FileReference($"{normalizedCookDir}/{ProjectFile.GetFileNameWithoutAnyExtensions()}/Metadata/DevelopmentAssetRegistry.bin");
			if (DevelopmentAssetRegistryFile.ToFileInfo().Exists)
			{
				CommandletArgs.Append($" -DevelopmentAssetRegistryFile=\"{DevelopmentAssetRegistryFile.FullName}\"");
			}
			else
			{
				Logger.LogWarning($"DevelopmentAssetRegistry.bin does not exist at {DevelopmentAssetRegistryFile.FullName}");
			}

			CommandUtils.RunCommandlet(ProjectFile, UnrealExe: null, "ChunkAssignerCommandlet", CommandletArgs.ToString());
			Logger.LogInformation("OplogChunkAssigner complete.");
		}


		public class OplogAttribute : Attribute
		{
			public string Name { get; }
			public OplogAttribute(string name) => Name = name;
		}
		internal static Dictionary<string, Type> BuildRegistry(Type[] Subclasses)
		{
			var registry = new ConcurrentDictionary<string, Type>(StringComparer.OrdinalIgnoreCase);
			Parallel.ForEach(ScriptManager.AllScriptAssemblies, asm =>
			{
				foreach (Type t in asm.GetTypes())
				{
					if (t.IsAbstract)
					{
						continue;
					}
					bool bIsSubclass = false;
					foreach (Type subclass in Subclasses)
					{
						if (t.IsSubclassOf(subclass))
						{
							bIsSubclass = true;
							break;
						}
					}
					if (!bIsSubclass)
					{
						continue;
					}

					OplogAttribute? attr = t.GetCustomAttribute<OplogAttribute>();
					if (attr != null)
					{
						registry[attr.Name] = t;
					}
				}
			});

			return new Dictionary<string, Type>(registry, StringComparer.OrdinalIgnoreCase);
		}
	}
}
