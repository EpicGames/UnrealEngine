// Copyright Epic Games, Inc. All Rights Reserved.
using System;
using System.Collections.Generic;
using System.IO;
using System.Threading;
using System.Reflection;
using AutomationTool;
using UnrealBuildTool;
using Microsoft.Extensions.Logging;

namespace AutomationScripts
{
	public partial class Project : CommandUtils
	{
		public static void Package(ProjectParams Params, int WorkingCL = -1)
		{
			if ((!Params.SkipStage || Params.Package) && !Params.SkipPackage)
			{
				Params.ValidateAndLog();
				List<DeploymentContext> DeployContextList = new List<DeploymentContext>();
				if (!Params.NoClient)
				{
					DeployContextList.AddRange(CreateDeploymentContext(Params, false, false));
				}
				if (Params.DedicatedServer)
				{
					DeployContextList.AddRange(CreateDeploymentContext(Params, true, false));
				}

				bool bShouldPackage = false; 
				foreach (var SC in DeployContextList)
				{
					if (Params.Package || (SC.StageTargetPlatform.RequiresPackageToDeploy(Params) && Params.Deploy))
					{
						bShouldPackage = true;
						break;
					}
				}

				if (bShouldPackage)
				{
					Logger.LogInformation("********** PACKAGE COMMAND STARTED **********");
					var StartTime = DateTime.UtcNow;

					foreach (var SC in DeployContextList)
					{
						if (Params.Package || (SC.StageTargetPlatform.RequiresPackageToDeploy(Params) && Params.Deploy))
						{
							if (SC.CustomDeployment == null || !SC.CustomDeployment.PrePackage(Params, SC, WorkingCL))
							{
								SC.StageTargetPlatform.Package(Params, SC, WorkingCL);
							}
							SC.CustomDeployment?.PostPackage(Params, SC, WorkingCL);
							SC.CustomStagingHandlers.ForEach(H => H.Package(Params, SC, WorkingCL));
						}
					}

					Logger.LogInformation("Package command time: {0:0.00} s", (DateTime.UtcNow - StartTime).TotalMilliseconds / 1000);
					Logger.LogInformation("********** PACKAGE COMMAND COMPLETED **********");
				}
			}
		}
	}
}
