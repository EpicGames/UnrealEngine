// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.IO;
using System.Linq;
using System.Text;
using System.Text.RegularExpressions;
using System.Threading.Tasks;
using EpicGames.Core;

namespace AutomationTool
{
	[Help("Parses Visual C++ timing information (as generated by UBT with the -Timing flag), and converts it into JSON format which can be visualized using the chrome://tracing tab")]
	[Help("-File=<Path>", "Path to the input file")]
	class ParseMsvcTimingInfo : BuildCommand
	{
		public override void ExecuteBuild()
		{
			FileReference InputFile = ParseRequiredFileReferenceParam("File");

			string[] Lines = FileReference.ReadAllLines(InputFile);
			for(int LineIdx = 0; LineIdx < Lines.Length; )
			{
				string Line = Lines[LineIdx];
				if(Line.StartsWith("Include Headers:", StringComparison.Ordinal))
				{
					LineIdx = ParseIncludeHeaders(Lines, LineIdx + 1, InputFile.ChangeExtension(".json"));
				}
				else if(Line.StartsWith("Class Definitions:", StringComparison.Ordinal))
				{
					LineIdx = ParseDefinitions(Lines, LineIdx + 1, InputFile.ChangeExtension(".classes.txt"));
				}
				else if(Line.StartsWith("Function Definitions:", StringComparison.Ordinal))
				{
					LineIdx = ParseDefinitions(Lines, LineIdx + 1, InputFile.ChangeExtension(".functions.txt"));
				}
				else
				{
					LineIdx++;
				}
			}
		}

		int ParseIncludeHeaders(string[] Lines, int LineIdx, FileReference OutputFile)
		{
			if(LineIdx < Lines.Length && Lines[LineIdx].StartsWith("\tCount:", StringComparison.Ordinal))
			{
				LineIdx++;
			}

			using(JsonWriter Writer = new JsonWriter(OutputFile))
			{
				Writer.WriteObjectStart();
				Writer.WriteArrayStart("traceEvents");

				Stack<float> FinishTimesForIndent = new Stack<float>();
				FinishTimesForIndent.Push(0.0f);

				float StartTime = 0.0f;
				for(; LineIdx < Lines.Length; LineIdx++)
				{
					Match Match = Regex.Match(Lines[LineIdx], "^\t\t(\t*)([^\t]+):\\s*([0-9\\.]+)s$");
					if(!Match.Success)
					{
						break;
					}

					int Indent = Match.Groups[1].Length;
					string FileName = Match.Groups[2].Value;
					float Duration = float.Parse(Match.Groups[3].Value);

					while(Indent <= FinishTimesForIndent.Count - 1)
					{
						StartTime = FinishTimesForIndent.Pop();
					}

					Writer.WriteObjectStart();
					Writer.WriteValue("pid", 1);
					Writer.WriteValue("tid", 1);
					Writer.WriteValue("ts", (long)(StartTime * 1000.0f * 1000.0f));
					Writer.WriteValue("dur", (long)(Duration * 1000.0f * 1000.0f));
					Writer.WriteValue("ph", "X");
					Writer.WriteValue("name", Path.GetFileName(FileName));
					Writer.WriteObjectStart("args");
					Writer.WriteValue("path", FileName);
					Writer.WriteObjectEnd();
					Writer.WriteObjectEnd();

					while(Indent >= FinishTimesForIndent.Count)
					{
						FinishTimesForIndent.Push(StartTime + Duration);
					}
				}

				Writer.WriteArrayEnd();
				Writer.WriteObjectEnd();
			}
			return LineIdx;
		}

		int ParseDefinitions(string[] Lines, int LineIdx, FileReference OutputFile)
		{
			if(LineIdx < Lines.Length && Lines[LineIdx].StartsWith("\tCount:", StringComparison.Ordinal))
			{
				LineIdx++;
			}

			Dictionary<string, float> ClassNameToTime = new Dictionary<string, float>();
			for(; LineIdx < Lines.Length; LineIdx++)
			{
				Match Match = Regex.Match(Lines[LineIdx], "^\t\t\t*([^\t]+):\\s*([0-9\\.]+)s$");
				if(!Match.Success)
				{
					break;
				}

				string ClassName = Match.Groups[1].Value;

				int TemplateIdx = ClassName.IndexOf('<');
				if(TemplateIdx != -1)
				{
					ClassName = ClassName.Substring(0, TemplateIdx) + "<>";
				}

				float Time;
				ClassNameToTime.TryGetValue(ClassName, out Time);

				Time += float.Parse(Match.Groups[2].Value);
				ClassNameToTime[ClassName] = Time;
			}

			using(StreamWriter Writer = new StreamWriter(OutputFile.FullName))
			{
				foreach(KeyValuePair<string, float> Pair in ClassNameToTime.OrderByDescending(x => x.Value))
				{
					Writer.WriteLine("{0,7:0.000}: {1}", Pair.Value, Pair.Key);
				}
			}

			return LineIdx;
		}
	}
}
