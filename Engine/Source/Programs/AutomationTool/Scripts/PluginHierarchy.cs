// Copyright Epic Games, Inc. All Rights Reserved.

using EpicGames.Core;
using Microsoft.Extensions.Logging;
using System;
using System.Collections.Concurrent;
using System.Collections.Generic;
using System.IO;
using System.Text.Json;
using System.Threading.Tasks;
using UnrealBuildBase;
using UnrealBuildTool;

namespace AutomationScripts
{
#nullable enable
	/// <summary>
	/// Engine-level utility that scans all <c>.uplugin</c> files under the engine and project
	/// plugin directories and provides access to their <see cref="PluginDescriptor"/> objects
	/// and transitive dependency sets.
	/// <para>
	/// This is intentionally engine-generic — no game-specific logic lives here.
	/// </para>
	/// </summary>
	public sealed class PluginHierarchy
	{
		// Populated by the parallel scan in ReadFromProject
		private readonly IReadOnlyDictionary<string, PluginDescriptor> _descriptors;
		private readonly IReadOnlyDictionary<string, FileReference>    _pluginFiles;

		// Lazy transitive-closure cache
		private readonly Dictionary<string, IReadOnlySet<string>> _transitiveCache =
			new Dictionary<string, IReadOnlySet<string>>(StringComparer.OrdinalIgnoreCase);

		private PluginHierarchy(
			IReadOnlyDictionary<string, PluginDescriptor> descriptors,
			IReadOnlyDictionary<string, FileReference>    pluginFiles)
		{
			_descriptors = descriptors;
			_pluginFiles  = pluginFiles;
		}

		// ---- Public API ----

		/// <summary>All known plugin names (from the scan).</summary>
		public IEnumerable<string> AllPluginNames => _descriptors.Keys;

		/// <summary>
		/// Returns the plugin-name → <see cref="PluginDescriptor"/> dictionary built during the scan.
		/// The dictionary is populated once (during <see cref="ReadFromProject"/>) and cached.
		/// </summary>
		public IReadOnlyDictionary<string, PluginDescriptor> GetPluginDescriptors() => _descriptors;

		/// <summary>
		/// Tries to return the <c>.uplugin</c> <see cref="FileReference"/> for the given plugin.
		/// Callers that need to read additional non-standard JSON fields (e.g. game-specific metadata)
		/// or inspect the on-disk location of a plugin can use this to open the file directly.
		/// </summary>
		public bool TryGetPluginFile(string pluginName, out FileReference? file) =>
			_pluginFiles.TryGetValue(pluginName, out file);

		/// <summary>
		/// Returns the transitive set of plugin dependencies for <paramref name="pluginName"/>,
		/// including the plugin itself. Computed lazily and cached on first call.
		/// Returns an empty set if the plugin is not known.
		/// </summary>
		internal IReadOnlySet<string> GetTransitiveDependencies(string pluginName)
		{
			lock (_transitiveCache)
			{
				if (_transitiveCache.TryGetValue(pluginName, out IReadOnlySet<string>? cached))
				{
					return cached;
				}
			}

			var visited = new HashSet<string>(StringComparer.OrdinalIgnoreCase);
			ComputeTransitiveDeps(pluginName, visited);
			var result = new HashSet<string>(visited, StringComparer.OrdinalIgnoreCase);

			lock (_transitiveCache)
			{
				_transitiveCache[pluginName] = result;
			}
			return result;
		}

		// ---- Factory ----

		/// <summary>
		/// Scans <c>.uplugin</c> files under <c>{root}/Engine/Plugins/</c> and
		/// <c>{root}/{projectName}/Plugins/</c> in parallel and builds the hierarchy.
		/// </summary>
		public static PluginHierarchy ReadFromProject(FileReference projectFile)
		{
			DirectoryReference enginePluginsDir  = new DirectoryReference(Path.Combine(Unreal.RootDirectory.FullName, "Engine"));
			DirectoryReference projectPluginsDir = new DirectoryReference(Path.Combine(Unreal.RootDirectory.FullName, projectFile.GetFileNameWithoutAnyExtensions(), "Plugins"));

			var descriptors  = new ConcurrentDictionary<string, PluginDescriptor>(StringComparer.OrdinalIgnoreCase);
			var pluginFiles  = new ConcurrentDictionary<string, FileReference>(StringComparer.OrdinalIgnoreCase);

			ScanPluginDirectory(enginePluginsDir,  descriptors, pluginFiles);
			ScanPluginDirectory(projectPluginsDir, descriptors, pluginFiles);

			Log.Logger.LogInformation("PluginHierarchy: scanned {Count} plugins.", descriptors.Count);

			return new PluginHierarchy(descriptors, pluginFiles);
		}

		// ---- Scanning ----

		private static void ScanPluginDirectory(
			DirectoryReference dir,
			ConcurrentDictionary<string, PluginDescriptor> descriptors,
			ConcurrentDictionary<string, FileReference>    pluginFiles)
		{
			if (!DirectoryReference.Exists(dir))
			{
				return;
			}

			List<FileReference> PluginsList = PluginsBase.EnumeratePlugins(dir);
			Parallel.ForEach(PluginsList, upluginFile =>
			{
				string pluginName = upluginFile.GetFileNameWithoutExtension();
				try
				{
					PluginDescriptor descriptor = PluginDescriptor.FromFile(upluginFile);

					// Use TryAdd so that if engine and project have the same plugin name the first
					// entry (engine) wins and the project override is silently skipped.
					if (descriptors.TryAdd(pluginName, descriptor))
					{
						pluginFiles[pluginName] = upluginFile;
					}
				}
				catch (JsonException ex)
				{
					Log.Logger.LogWarning("PluginHierarchy: failed to read {File}: {Message}", upluginFile, ex.Message);
				}
			});
		}

		// ---- Transitive closure ----

		private void ComputeTransitiveDeps(string pluginName, HashSet<string> visited)
		{
			if (!visited.Add(pluginName))
			{
				return;
			}

			if (!_descriptors.TryGetValue(pluginName, out PluginDescriptor? descriptor) || descriptor.Plugins == null)
			{
				return;
			}

			foreach (PluginReferenceDescriptor pluginRef in descriptor.Plugins)
			{
				if (pluginRef.bEnabled && !string.IsNullOrEmpty(pluginRef.Name))
				{
					ComputeTransitiveDeps(pluginRef.Name, visited);
				}
			}
		}
	}
#nullable disable
}
