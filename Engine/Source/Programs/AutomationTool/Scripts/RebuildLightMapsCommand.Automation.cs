// Copyright Epic Games, Inc. All Rights Reserved.
using System;
using System.Collections.Generic;
using System.IO;
using System.Threading;
using System.Reflection;
using System.Linq;
using System.Net.Mail;
using AutomationTool;
using UnrealBuildTool;
using EpicGames.Core;
using UnrealBuildBase;
using Microsoft.Extensions.Logging;

namespace AutomationScripts.Automation
{
	/// <summary>
	/// Helper command used for rebuilding a projects light maps.
	/// </summary>
	[RequireP4]
	[Help("Helper command used for rebuilding a projects light maps")]
	[Help("Project", "Absolute path to a .uproject file")]
	[Help("MapsToRebuildLightMaps", "A list of '+' delimited maps we wish to build lightmaps for.")]
	[Help("CommandletTargetName", "The Target used in running the commandlet")]
	[Help("StakeholdersEmailAddresses", "Users to notify of completion")]
	public class RebuildLightMaps : BuildCommand
	{
		public override void ExecuteBuild()
		{
			Logger.LogInformation("********** REBUILD LIGHT MAPS COMMAND STARTED **********");
			int SubmittedCL = 0;
			try
			{
				var Params = new ProjectParams
				(
					Command: this,
					// Shared
					RawProjectPath: ProjectPath
				);

				// Sync and build our targets required for the commandlet to run correctly.
				// P4.Sync(String.Format("-f {0}/...#head", P4Env.BuildRootP4));

                bool NoBuild = ParseParam("nobuild");

                if (!NoBuild)
                {
                    BuildNecessaryTargets();
                }
				CreateChangelist(Params);
				RunRebuildLightmapsCommandlet(Params);
				SubmitRebuiltMaps(ref SubmittedCL);
			}
			catch (Exception ProcessEx)
			{
				Logger.LogInformation("********** REBUILD LIGHT MAPS COMMAND FAILED **********");
                Logger.LogInformation("Error message: {Arg0}", ProcessEx.Message);
				HandleFailure(ProcessEx.Message);
				throw;
			}

			// The processes steps have completed successfully.
			HandleSuccess(SubmittedCL);

			Logger.LogInformation("********** REBUILD LIGHT MAPS COMMAND COMPLETED **********");
		}

		private void BuildNecessaryTargets()
		{
			Logger.LogInformation("Running Step:- RebuildLightMaps::BuildNecessaryTargets");
			UnrealBuild.BuildAgenda Agenda = new UnrealBuild.BuildAgenda();
			Agenda.AddTarget("ShaderCompileWorker", UnrealBuildTool.UnrealTargetPlatform.Win64, UnrealBuildTool.UnrealTargetConfiguration.Development);
			Agenda.AddTarget("UnrealLightmass", UnrealBuildTool.UnrealTargetPlatform.Win64, UnrealBuildTool.UnrealTargetConfiguration.Development);
			Agenda.AddTarget(CommandletTargetName, UnrealBuildTool.UnrealTargetPlatform.Win64, UnrealBuildTool.UnrealTargetConfiguration.Development);

			try
			{
				UnrealBuild Builder = new UnrealBuild(this);
				Builder.Build(Agenda, InDeleteBuildProducts: true, InUpdateVersionFiles: true, InForceNoXGE: false, InChangelistNumberOverride: GetLatestCodeChange());
				UnrealBuild.CheckBuildProducts(Builder.BuildProductFiles);
			}
			catch (AutomationException)
			{
				Logger.LogError("Rebuild Light Maps has failed.");
				throw;
			}
		}

		private int GetLatestCodeChange()
		{
			List<P4Connection.ChangeRecord> ChangeRecords;
			if(!P4.Changes(out ChangeRecords, String.Format("-m 1 //{0}/....cpp@<{1} //{0}/....h@<{1} //{0}/....cs@<{1} //{0}/....usf@<{1} //{0}/....ush@<{1}", P4Env.Client, P4Env.Changelist), WithClient: true))
			{
				throw new AutomationException("Couldn't enumerate latest change from branch");
			}
			return ChangeRecords.Max(x => x.CL);
		}

		private void CreateChangelist(ProjectParams Params)
		{
			Logger.LogInformation("Running Step:- RebuildLightMaps::CheckOutMaps");
			// Setup a P4 Cl we will use to submit the new lightmaps
			WorkingCL = P4.CreateChange(P4Env.Client, String.Format("{0} rebuilding lightmaps from changelist {1}\n#rb None\n#tests None", Params.ShortProjectName, P4Env.Changelist));
			Logger.LogInformation("Working in {WorkingCL}", WorkingCL);

		}

		private void RunRebuildLightmapsCommandlet(ProjectParams Params)
		{
			Logger.LogInformation("Running Step:- RebuildLightMaps::RunRebuildLightmapsCommandlet");

			// Find the commandlet binary
			string UEEditorExe = HostPlatform.Current.GetUnrealExePath(Params.UnrealExe);
			if (!FileExists(UEEditorExe))
			{
				Logger.LogError("Missing " + UEEditorExe + " executable. Needs to be built first.");
				throw new AutomationException("Missing " + UEEditorExe + " executable. Needs to be built first.");
			}

			// Now let's rebuild lightmaps for the project
			try
			{
				var CommandletParams = IsBuildMachine ? "-unattended -buildmachine -fileopenlog" : "-fileopenlog";
                CommandletParams += " -AutoCheckOutPackages";
                if (P4Enabled)
                {
                    CommandletParams += String.Format(" -SCCProvider={0} -P4Port={1} -P4User={2} -P4Client={3} -P4Changelist={4} -P4Passwd={5}", "Perforce", P4Env.ServerAndPort, P4Env.User, P4Env.Client, WorkingCL.ToString(), P4.GetAuthenticationToken());
                }
				RebuildLightMapsCommandlet(Params.RawProjectPath, Params.UnrealExe, Params.MapsToRebuildLightMaps.ToArray(), CommandletParams);
			}
			catch (Exception Ex)
			{
                string FinalLogLines = "No log file found";
                CommandletException AEx = Ex as CommandletException;
                if ( AEx != null )
                {
                    string LogFile = AEx.LogFileName;
                    Logger.LogWarning("Attempting to load file {File}", LogFile);
                    if ( LogFile != "")
                    {
                        
                        Logger.LogWarning("Attempting to read file {File}", LogFile);
                        try
                        {
                            string[] AllLogFile = ReadAllLines(LogFile);

                            FinalLogLines = "Important log entries\n";
                            foreach (string LogLine in AllLogFile)
                            {
                                if (LogLine.Contains("[REPORT]") || LogLine.Contains("Error:"))
                                {
                                    FinalLogLines += LogLine + "\n";
                                }
                            }
                        }
                        catch (Exception)
                        {
                            // we don't care about this because if this is hit then there is no log file the exception probably has more info
                            Logger.LogError("{Text}", "Could not find log file " + LogFile);
                        }
                    }
                }

				// Something went wrong with the commandlet. Abandon this run, don't check in any updated files, etc.
				Logger.LogError("{Text}", "Rebuild Light Maps has failed. because "+ Ex.ToString());
				throw new AutomationException(ExitCode.Error_Unknown, Ex, "RebuildLightMaps failed. {0}", FinalLogLines);
			}
		}

		private void SubmitRebuiltMaps(ref int SubmittedCL)
		{
			Logger.LogInformation("Running Step:- RebuildLightMaps::SubmitRebuiltMaps");

			// Check everything in!
			if (WorkingCL != -1)
			{
                Logger.LogInformation("{Text}", "Running Step:- Submitting CL " + WorkingCL);
				P4.Submit(WorkingCL, out SubmittedCL, true, true);
				Logger.LogInformation("{Text}", "INFO: Lightmaps successfully submitted in cl "+ SubmittedCL.ToString());
			}
		}

		/**
		 * Parse the P4 output for any errors that we really care about.
		 * e.g. umaps and assets are exclusive checkout files, if we cant check out a map for this reason
		 *		then we need to stop.
		 */
		private bool FoundCheckOutErrorInP4Output(string Output)
		{
			bool bHadAnError = false;

			var Lines = Output.Split(new string[] { Environment.NewLine }, StringSplitOptions.RemoveEmptyEntries);
			foreach (string Line in Lines)
			{
				// Check for log spew that matches exclusive checkout failure
				// http://answers.perforce.com/articles/KB/3114
				if (Line.Contains("can't edit exclusive file already opened"))
				{
					bHadAnError = true;
					break;
				}
			}

			return bHadAnError;
		}

		/**
		 * Cleanup anything this build may leave behind and inform the user
		 */
		private void HandleFailure(String FailureMessage)
		{
			try
			{
				if (WorkingCL != -1)
				{
					P4.RevertAll(WorkingCL);
					P4.DeleteChange(WorkingCL);
				}
				SendCompletionMessage(false, FailureMessage);
			}
			catch (P4Exception P4Ex)
			{
				Logger.LogError("{Text}", "Failed to clean up P4 changelist: " + P4Ex.Message);
			}
			catch (Exception SendMailEx)
			{
				Logger.LogError("{Text}", "Failed to notify that build succeeded: " + SendMailEx.Message);
			}
		}

		/**
		 * Perform any post completion steps needed. I.e. Notify stakeholders etc.
		 */
		private void HandleSuccess(int SubmittedCL)
		{
			try
			{
				SendCompletionMessage(true, String.Format( "Successfully rebuilt lightmaps to cl {0}.", SubmittedCL ));
			}
			catch (Exception SendMailEx)
			{
				Logger.LogError("{Text}", "Failed to notify that build succeeded: " + SendMailEx.Message);
			}
		}

		/**
		 * Notify stakeholders of the commandlet results
		 */
		void SendCompletionMessage(bool bWasSuccessful, String MessageBody)
		{
			if (StakeholdersEmailAddresses != null)
			{
				MailMessage Message = new System.Net.Mail.MailMessage();
				Message.Priority = MailPriority.High;
				Message.From = new MailAddress("unrealbot@epicgames.com");

				string Branch = "Unknown";
				if (P4Enabled)
				{
					Branch = P4Env.Branch;
				}

				foreach (String NextStakeHolder in StakeholdersEmailAddresses)
				{
					Message.To.Add(new MailAddress(NextStakeHolder));
				}

				ConfigHierarchy Ini = ConfigCache.ReadHierarchy(ConfigHierarchyType.EditorPerProjectUserSettings, DirectoryReference.FromFile(ProjectFullPath), UnrealTargetPlatform.Win64);
				List<string> Emails;
				Ini.GetArray("RebuildlightingSettings", "EmailNotification", out Emails);
				foreach (var Email in Emails)
				{
					Message.CC.Add(new MailAddress(Email));
				}

				Message.Subject = String.Format("Nightly lightmap rebuild {0} for {1}", bWasSuccessful ? "[SUCCESS]" : "[FAILED]", Branch);
				Message.Body = MessageBody;
				/*Attachment Attach = new Attachment();
				Message.Attachments.Add()*/
				try
				{
					SmtpClient MailClient = new SmtpClient("smtp.epicgames.net");
					MailClient.Send(Message);
				}
				catch (Exception Ex)
				{
					Logger.LogError("Failed to send notify email to {Arg0} ({Arg1})", String.Join(", ", StakeholdersEmailAddresses.ToArray()), Ex.Message);
				}
			}
		}

		// Users to notify if the process fails or succeeds.
		List<String> StakeholdersEmailAddresses
		{
			get
			{
				String UnprocessedEmailList = ParseParamValue("StakeholdersEmailAddresses");
				if (String.IsNullOrEmpty(UnprocessedEmailList) == false)
				{
					return UnprocessedEmailList.Split('+').ToList();
				}
				else
				{
					return null;
				}
			}
		}

		// The Changelist used when doing the work.
		private int WorkingCL = -1;

		// The target name of the commandlet binary we wish to build and run.
		private String CommandletTargetName
		{
			get
			{
				return ParseParamValue("CommandletTargetName", "");
			}
		}

		// Process command-line and find a project file. This is necessary for the commandlet to run successfully
		private FileReference ProjectFullPath;
		public virtual FileReference ProjectPath
		{
			get
			{
				if (ProjectFullPath == null)
				{
					var OriginalProjectName = ParseParamValue("project", "");
					var ProjectName = OriginalProjectName;
					ProjectName = ProjectName.Trim(new char[] { '\"' });
					if (ProjectName.IndexOfAny(new char[] { '\\', '/' }) < 0)
					{
						ProjectName = CombinePaths(CmdEnv.LocalRoot, ProjectName, ProjectName + ".uproject");
					}
					else if (!FileExists_NoExceptions(ProjectName))
					{
						ProjectName = CombinePaths(CmdEnv.LocalRoot, ProjectName);
					}
					if (FileExists_NoExceptions(ProjectName))
					{
						ProjectFullPath = new FileReference(ProjectName);
					}
					else
					{
						var Branch = new BranchInfo();
						var GameProj = Branch.FindGame(OriginalProjectName);
						if (GameProj != null)
						{
							ProjectFullPath = GameProj.FilePath;
						}
						if (!FileExists_NoExceptions(ProjectFullPath.FullName))
						{
							throw new AutomationException("Could not find a project file {0}.", ProjectName);
						}
					}
				}
				return ProjectFullPath;
			}
		}
	}
}
