// Copyright Epic Games, Inc. All Rights Reserved.
using System;
using System.Collections.Generic;
using System.Text;
using System.IO;
using AutomationTool;
using UnrealBuildTool;
using EpicGames.Core;
using System.Linq;
using Microsoft.Extensions.Logging;

[Help("UAT command to run performance test demo using different RHIs and compare results")]
class RecordPerformance : BuildCommand
{
	private string[] ProjectsToRun =
	{
		"SubwaySequencer",		// 0
		"InfiltratorDemo",		// 1
		"ElementalDemo",		// 2
		"ShowdownDemo"			// 3
	};

	private string ProjectToRun;
	private DirectoryReference RootDir;
	private const string PerformanceMonitorConfig = "Default";
	private DirectoryReference OutputDir;

	public override void ExecuteBuild()
	{
		SetupPaths();

		// Project settings should be appropriate (PerformanceMonitor plugin included and configured, per-platform settings set).
		int Index = int.Parse(ParseParamValue("DemoIndex", "1"));
		Index = Math.Max(0, Index);
		Index = Math.Min(ProjectsToRun.Length - 1, Index);
		ProjectToRun = ProjectsToRun[Index];

		// Which platforms to test?
		var PlatformsToTest = ParseMultipleParams("Platforms");
		if (PlatformsToTest.Count() == 0)
		{
			throw new AutomationException("Missing -Platforms=Win64+Linux+... parameter");
		}

		// How many runs of each test to accumulate data?
		var NumOfRuns = int.Parse(ParseParamValue("NumOfRuns", "3"));

		// Which stats to present?
		var StatsToPresent = ParseMultipleParams("Stats");
		if (StatsToPresent.Count() == 0)
		{
			StatsToPresent = new string[] { "FrameTime", "GPUFrameTime", "RenderThreadTime", "GameThreadTime" };
		}

		if (!ParseParam("SkipBuild"))
		{
			// First cook and deploy all the platforms
			foreach (var PlatformName in PlatformsToTest)
			{
				var Platform = GetPlatformByName(PlatformName);
				if (Platform == null)
				{
					Logger.LogInformation("Cannot find platform '{PlatformName}'. Skipping.", PlatformName);
					continue;
				}

				BuildAndCookPlatform(Platform.PlatformType);
			}
		}

		// Now run tests
		foreach (var PlatformName in PlatformsToTest)
		{
			var Platform = GetPlatformByName(PlatformName);
			if (Platform == null)
			{
				continue;
			}

			var RhisToTest = GetRhisForPlatform(PlatformName);

			foreach (var Rhi in RhisToTest)
			{
				// First run to make sure all caches are populated
				RunGame(Platform, Rhi, 0, false);

				// Actual runs to gather perf data
				for (var i = 1; i <= NumOfRuns; ++i)
				{
					RunGame(Platform, Rhi, i, true);
				}

				CombineStats(Platform, Rhi);
			}
		}

		foreach (var Stat in StatsToPresent)
		{
			CreateChart(Stat);
		}

		Logger.LogInformation("Performance charts stored in {OutputDir}", OutputDir);
	}

	private IEnumerable<string> ParseMultipleParams(string ParamName)
	{
		List<string> Params = new List<string>();

		var ParamsString = ParseParamValue(ParamName);
		if (!string.IsNullOrEmpty(ParamsString))
		{
			var ParamNames = new ParamList<string>(ParamsString.Split('+'));
			foreach (var Param in ParamNames)
			{
				Params.Add(Param);
			}
		}

		return Params;
	}

	private IEnumerable<string> GetRhisForPlatform(string PlatformName)
	{
		if (PlatformName == "Win64")
		{
			return new string[] { "d3d11", "d3d12", "vulkan" };
		}
		else if (PlatformName == "Linux")
		{
			return new string[] { "vulkan" };
		}
		else
		{
			return new string[] { "default" };
		}
	}

	private Platform GetPlatformByName(string PlatformName)
	{
		UnrealTargetPlatform TargetPlatform;
		if (UnrealTargetPlatform.TryParse(PlatformName, out TargetPlatform))
		{
			return null;
		}

		return Platform.GetPlatform(TargetPlatform);
	}

	private void RunCsvTool(string ToolName, string Arguments)
	{
		var ToolPath = FileReference.Combine(DirectoryReference.Combine(RootDir, "Engine", "Source", "Programs", "CSVTools", "Binaries"), ToolName + ".exe");
		Run(ToolPath.FullName, Arguments);
	}

	private void CreateChart(string StatName)
	{
		var StatsDir = DirectoryReference.Combine(OutputDir, "Combined");
		var ChartFileName = FileReference.Combine(OutputDir, StatName + ".svg");
		RunCsvTool("CSVToSVG", string.Format("-csvDir {0} -o {1} -stats {2} -maxY 100", StatsDir.FullName, ChartFileName.FullName, StatName));
	}

	private void CombineStats(Platform Platform, string Rhi)
	{
		var StatsDir = FileReference.Combine(OutputDir, string.Format("{0}-{1}", Platform.PlatformType, Rhi));
		var OutputFile = FileReference.Combine(OutputDir, "Combined", string.Format("{0}-{1}.csv", Platform.PlatformType, Rhi));

		CreateDirectory(OutputFile.Directory.FullName);

		RunCsvTool("CSVCollate", string.Format("-csvDir {0} -o {1} -avg", StatsDir.FullName, OutputFile.FullName));
	}

	private void SetupPaths()
	{
		var TestName = DateTime.Now.ToString("yyyyMMddHHmm");

		RootDir = new DirectoryReference(CmdEnv.LocalRoot);
		OutputDir = DirectoryReference.Combine(RootDir, "Engine", "Saved", "RecordPerformance", TestName);
		CreateDirectory(OutputDir.FullName);
	}

	private void RunGame(Platform Platform, string Rhi, int RunNumber, bool UseStats)
	{
		var AdditionalCommandLineParameters = "-savevulkanpsocacheonexit";
		var StatFileName = Path.Combine(ProjectToRun, "Saved", "FXPerformance", PerformanceMonitorConfig + ".csv");

		Logger.LogInformation("Running {Rhi} on {Arg1}...", Rhi, Platform.PlatformType);

		var RhiParam = (Rhi != "default") ? "-" + Rhi : "";

		try
		{
			RunBCR(string.Format("-project={0} -platform={1} -skipbuild -skipcook -skipstage -skipdeploy -run -getfile=\"{5}\" -addcmdline=\"{2} {4} -benchmark -execcmds=\\\"PerformanceMonitor addtimer Time,PerformanceMonitor cvstoolsmode true,PerformanceMonitor start {3}, ce start\\\"\"",
				ProjectToRun, Platform.PlatformType, RhiParam, PerformanceMonitorConfig, AdditionalCommandLineParameters, StatFileName));
		}
		catch
		{
			Logger.LogInformation("Running game failed. Stats might be wrong / incomplete.");
		}

		if (UseStats)
		{
			// Create output directory
			var DestinationDirectory = DirectoryReference.Combine(OutputDir, string.Format("{0}-{1}", Platform.PlatformType, Rhi));
			CreateDirectory(DestinationDirectory.FullName);

			// Copy log
			var DestLogFile = FileReference.Combine(DestinationDirectory, string.Format("{0}.log", RunNumber));
			var LogFileName = Path.Combine(CmdEnv.LogFolder, "BCR", "Client.log");
			CopyFile(LogFileName, DestLogFile.FullName);

			// Copy stat file
			var DestStatsFile = FileReference.Combine(DestinationDirectory, string.Format("{0}.csv", RunNumber));
			CopyFile(CombinePaths(CmdEnv.EngineSavedFolder, Path.GetFileName(StatFileName)), DestStatsFile.FullName);
		}
	}

	private void BuildAndCookPlatform(UnrealTargetPlatform Platform)
	{
		Logger.LogInformation("Building {Arg0}...", Platform.ToString());

		RunBCR(string.Format("-project={0} -platform={1} -build -cook -stage -pak -deploy", ProjectToRun, Platform));
	}

	private void RunBCR(string Arguments)
	{
		RunUAT(CmdEnv, "BuildCookRun " + Arguments, "BCR");
	}
}
