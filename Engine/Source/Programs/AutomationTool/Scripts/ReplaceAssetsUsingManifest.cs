// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Linq;
using System.Text;
using System.Threading.Tasks;
using System.IO;
using AutomationTool;
using UnrealBuildTool;
using EpicGames.Core;

public class ReplaceAssetsUsingManifest : BuildCommand
{
	public override void ExecuteBuild()
	{
		// Command parameters - not all required if using existing manifest
		string ProjectPath = ParseParamValue("ProjectPath");
		string ManifestFile = ParseParamValue("ManifestFile");
		// by defaulting to null here, we will perform a lookup of what editor the project wants to cook, etc with, based on target receipt
		string UnrealExe = ParseParamValue("UnrealExe", null, ObsoleteParam: "UE4Exe");
		string ReplacedPaths = ParseParamValue("ReplacedPaths", "");
		string ReplacedClasses = ParseParamValue("ReplacedClasses", "");
		string ExcludedPaths = ParseParamValue("ExcludedPaths", "");
		string ExcludedClasses = ParseParamValue("ExcludedClasses", "");
		DirectoryReference BaseDir = new DirectoryReference(ParseParamValue("BaseDir"));
		DirectoryReference AssetSourcePath = new DirectoryReference(ParseParamValue("AssetSourcePath"));
		bool UseExistingManifest = ParseParam("UseExistingManifest");

		if (!UseExistingManifest || !FileExists_NoExceptions(ManifestFile))
		{
			// Run commandlet to generate list of assets to replace
			FileReference Project = new FileReference(ProjectPath);
			var Dir = Path.GetDirectoryName(ManifestFile);
			var Filename = Path.GetFileName(ManifestFile);
			if (String.IsNullOrEmpty(Dir) || String.IsNullOrEmpty(Filename))
			{
				throw new AutomationException("GenerateDistillFileSets should have a full path and file for {0}.", ManifestFile);
			}
			CreateDirectory(Dir);
			if (FileExists_NoExceptions(ManifestFile))
			{
				DeleteFile(ManifestFile);
			}

			RunCommandlet(Project, UnrealExe, "GenerateAssetManifest", String.Format("-ManifestFile={0} -IncludedPaths={1} -IncludedClasses={2} -ExcludedPaths={3} -ExcludedClasses={4}", CommandUtils.MakePathSafeToUseWithCommandLine(ManifestFile), ReplacedPaths, ReplacedClasses, ExcludedPaths, ExcludedClasses));

			if (!FileExists_NoExceptions(ManifestFile))
			{
				throw new AutomationException("GenerateAssetManifest did not produce a manifest for {0}.", Project);
			}
		}

		// Read Source Files from Manifest
		List<string> Lines = new List<string>(ReadAllLines(ManifestFile));
		if (Lines.Count < 1)
		{
			throw new AutomationException("Manifest file {0} does not list any files.", ManifestFile);
		}
		List<string> SourceFiles = new List<string>();
		foreach (var ThisFile in Lines)
		{
			var TestFile = CombinePaths(ThisFile);
			if (!FileExists_NoExceptions(TestFile))
			{
				throw new AutomationException("GenerateAssetManifest produced {0}, but {1} doesn't exist.", ThisFile, TestFile);
			}
			// we correct the case here
			var TestFileInfo = new FileInfo(TestFile);
			var FinalFile = CombinePaths(TestFileInfo.FullName);
			if (!FileExists_NoExceptions(FinalFile))
			{
				throw new AutomationException("GenerateDistillFileSets produced {0}, but {1} doesn't exist.", ThisFile, FinalFile);
			}
			SourceFiles.Add(FinalFile);
		}

		// Delete all original files - might not always be an asset to replace them with
		foreach(string SourceFile in SourceFiles)
		{
			CommandUtils.DeleteFile(SourceFile);
		}

		// Convert Source file paths to Asset Source Paths
		foreach (string ReplacedFile in SourceFiles)
		{
			FileReference ReplacementFile = FileReference.Combine(AssetSourcePath, new FileReference(ReplacedFile).MakeRelativeTo(BaseDir));
			if (FileReference.Exists(ReplacementFile))
			{
				CommandUtils.CopyFile(ReplacementFile.FullName, ReplacedFile);
			}
		}
	}
};
