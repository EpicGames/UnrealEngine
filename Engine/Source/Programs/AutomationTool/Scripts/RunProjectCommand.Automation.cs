// Copyright Epic Games, Inc. All Rights Reserved.
using System;
using System.Collections.Generic;
using System.Diagnostics;
using System.IO;
using System.Text;
using System.Text.RegularExpressions;
using System.Threading;
using System.Reflection;
using System.Linq;
using System.Net.NetworkInformation;
using System.Collections;
using AutomationTool;
using UnrealBuildTool;
using EpicGames.Core;
using UnrealBuildBase;
using Microsoft.Extensions.Logging;
using EpicGames.Serialization;
using AutomationUtils;

namespace AutomationScripts
{
	/// <summary>
	/// Helper command to run a game.
	/// </summary>
	/// <remarks>
	/// Uses the following command line params:
	/// -cooked
	/// -cookonthefly
	/// -dedicatedserver
	/// -win32
	/// -noclient
	/// -logwindow
	/// </remarks>
	public partial class Project : CommandUtils
	{
		/// <summary>
		/// Thread used to read client log file.
		/// </summary>
		private static Thread ClientLogReaderThread = null;

		/// <summary>
		/// Process for the cook server, can be set by the cook command when a cook on the fly server is used
		/// </summary>
		public static IProcessResult CookServerProcess;

		/// <summary>
		/// Process for the dedicated server
		/// </summary>
		private static IProcessResult DedicatedServerProcess;

		// debug commands for the engine to crash
		public static string[] CrashCommands =
		{
		"crash",
		"CHECK",
		"GPF",
		"ASSERT",
		"ENSURE",
		"RENDERCRASH",
		"RENDERCHECK",
		"RENDERGPF",
		"THREADCRASH",
		"THREADCHECK",
		"THREADGPF",
	};

		/// <summary>
		/// For not-installed runs, returns a temp log folder to make sure it doesn't fall into sandbox paths
		/// </summary>
		/// <returns></returns>
		private static string GetLogFolderOutsideOfSandbox()
		{
			return Unreal.IsEngineInstalled() ?
				CmdEnv.LogFolder :
				CombinePaths(Path.GetTempPath(), CommandUtils.EscapePath(CmdEnv.LocalRoot), "Logs");
		}

		/// <summary>
		/// For not-installed runs, copies all logs from the temp log folder back to the UAT log folder.
		/// </summary>
		private static void CopyLogsBackToLogFolder()
		{
			if (!Unreal.IsEngineInstalled())
			{
				var LogFolderOutsideOfSandbox = GetLogFolderOutsideOfSandbox();
				var TempLogFiles = FindFiles_NoExceptions("*", false, LogFolderOutsideOfSandbox);
				Logger.LogInformation("Found {Arg0} temp logs to copy from {LogFolderOutsideOfSandbox} to {Arg2}", TempLogFiles.Length, LogFolderOutsideOfSandbox, CmdEnv.LogFolder);
				foreach (var LogFilename in TempLogFiles)
				{
					var DestFilename = CombinePaths(CmdEnv.LogFolder, Path.GetFileName(LogFilename));
					CopyFile_NoExceptions(LogFilename, DestFilename);
				}
			}
		}

		private static bool WaitForProcessReady(IProcessResult Process, string Name, string LogFile, string[] ReadyTexts, int MaxLogWaitTimeInSeconds = -1)
		{
			var StartTime = DateTime.UtcNow;
			var bFirst = true;
			while (!FileExists(LogFile) && !Process.HasExited)
			{
				if (bFirst)
				{
					Logger.LogInformation("Waiting for {Name} to start logging at: {LogFile}", Name, LogFile);
					bFirst = false;
				}
				else if (MaxLogWaitTimeInSeconds > 0)
				{
					var Duration = (DateTime.UtcNow - StartTime).Seconds;
					var TimeLeft = MaxLogWaitTimeInSeconds - Duration;
					if (TimeLeft <= 0)
					{
						Logger.LogWarning("Giving up waiting for {Name} to start logging, it may run with logging disabled.", Name);
						return false;
					}
					else
					{
						Logger.LogInformation("Waiting for {TimeLeft} seconds for {Name} to start logging...", TimeLeft, Name);
					}
				}
				else
				{
					Logger.LogInformation("Waiting for {Name} to start logging...", Name);
				}
				Thread.Sleep(2000);
			}

			if (!FileExists(LogFile))
			{
				throw new AutomationException("{0} exited without creating a log file at: {1}", Name, LogFile);
			}

			Logger.LogInformation("Logging started for {Name} at: {LogFile}", Name, LogFile);
			Thread.Sleep(1000);

			if (ReadyTexts == null)
			{
				Logger.LogInformation("{Name} is ready!", Name);
				return true;
			}

			using (FileStream ProcessLog = File.Open(LogFile, FileMode.Open, FileAccess.Read, FileShare.ReadWrite))
			{
				StreamReader LogReader = new StreamReader(ProcessLog);

				// Read until the process has exited or text has been found
				while (!Process.HasExited)
				{
					while (!LogReader.EndOfStream)
					{
						string Output = LogReader.ReadToEnd();
						if (!String.IsNullOrEmpty(Output))
						{
							foreach (string ReadyText in ReadyTexts)
							{
								if (Output.Contains(ReadyText))
								{
									Logger.LogInformation("{Name} is ready! \"{ReadyText}\" was found in log.", ReadyText, Name);
									return true;
								}
							}
						}
					}
					Logger.LogInformation("Waiting for {Name} to get ready...", Name);
					Thread.Sleep(2000);
				}
			}
			throw new AutomationException("{0} exited before we asked it to (see {1} for more info)", Name, LogFile);
		}

		public static void Run(ProjectParams Params)
		{
			Params.ValidateAndLog();
			if (!Params.Run)
			{
				return;
			}

			Logger.LogInformation("********** RUN COMMAND STARTED **********");
			var StartTime = DateTime.UtcNow;

			var LogFolderOutsideOfSandbox = GetLogFolderOutsideOfSandbox();
			if (!Unreal.IsEngineInstalled() && CookServerProcess == null)
			{
				// In the installed runs, this is the same folder as CmdEnv.LogFolder so delete only in not-installed
				DeleteDirectory(LogFolderOutsideOfSandbox);
				CreateDirectory(LogFolderOutsideOfSandbox);
			}
			var CookServerLogFile = CombinePaths(LogFolderOutsideOfSandbox, "CookServer.log");
			var DedicatedServerLogFile = CombinePaths(LogFolderOutsideOfSandbox, "DedicatedServer.log");
			var ClientLogFile = CombinePaths(LogFolderOutsideOfSandbox, Params.EditorTest ? "Editor.log" : "Client.log");

			try
			{
				RunInternal(Params, CookServerLogFile, DedicatedServerLogFile, ClientLogFile);
			}
			catch
			{
				throw;
			}
			finally
			{
				if (!GlobalCommandLine.NoKill)
				{
					if (CookServerProcess != null)
					{
						if (!CookServerProcess.HasExited)
						{
							Logger.LogInformation("Stopping cook server...");
							CookServerProcess.StopProcess();
						}
						Logger.LogInformation("Cook server exited with error code: {ExitCode} (see {File} for more info)", CookServerProcess.ExitCode, CookServerLogFile);
					}
					if (DedicatedServerProcess != null)
					{
						if (!DedicatedServerProcess.HasExited)
						{
							Logger.LogInformation("Stopping dedicated server...");
							DedicatedServerProcess.StopProcess();
						}
						Logger.LogInformation("Dedicated server exited with error code: {ExitCode} (see {File} for more info)", DedicatedServerProcess.ExitCode, DedicatedServerLogFile);
					}
				}
				CopyLogsBackToLogFolder();
			}

			Logger.LogInformation("Run command time: {0:0.00} s", (DateTime.UtcNow - StartTime).TotalMilliseconds / 1000);
			Logger.LogInformation("********** RUN COMMAND COMPLETED **********");
		}

		private static void RunInternal(
				ProjectParams Params,
				string CookServerLogFile,
				string DedicatedServerLogFile,
				string ClientLogFile)
		{
			StartUnrealTrace();

			if (CookServerProcess != null)
			{
				WaitForProcessReady(CookServerProcess, "Cook server", CookServerLogFile,
					new string[] {"Unreal Network File Server is ready"});
			}

			if (Params.DedicatedServer && !Params.SkipServer)
			{
				// With dedicated server, the client connects to local host to load a map,
				// unless client parameters are already specified
				if (String.IsNullOrEmpty(Params.ClientCommandline))
				{
					if (!String.IsNullOrEmpty(Params.ServerDeviceAddress))
					{
						Params.ClientCommandline = Params.ServerDeviceAddress;
					}
					else
					{
						Params.ClientCommandline = "127.0.0.1";
					}
				}

				DedicatedServerProcess = RunDedicatedServer(Params, DedicatedServerLogFile, Params.RunCommandline);
				int MaxLogCreationWaitTimeInSeconds = 10;
				WaitForProcessReady(DedicatedServerProcess, "Dedicated server", DedicatedServerLogFile,
					new string[] {"Game Engine Initialized"}, MaxLogCreationWaitTimeInSeconds);
			}

			if (!Params.NoClient)
			{
				var SC = CreateDeploymentContext(Params, false);
				ERunOptions ClientRunFlags;
				string ClientApp;
				string ClientCmdLine;

				if (!string.IsNullOrEmpty(Params.ZenWorkspaceSharePath))
				{
					ZenRunContext ZenServerContext = ZenRunContext.ReadFromContextFile(new FileReference(ZenUtils.GetZenRunContextFile()));
					if (ZenServerContext != null && ZenServerContext.IsServiceRunning())
					{
						DirectoryReference WorkspaceShareDir = new(Params.ZenWorkspaceSharePath);
						if (SC.Count > 0 && DirectoryReference.Exists(WorkspaceShareDir))
						{
							if (ZenServerContext.CreateWorkspaceAndShare(WorkspaceShareDir) == null)
							{
								throw new AutomationException("Couldn't createa a Zen workspace at {0}", WorkspaceShareDir.FullName);
							}
						}
					}
					else
					{
						throw new AutomationException("Zen service is not running or wasn't detected. Can't run with ZenWorkspaceSharePath");
					}
				}

				SetupClientParams(SC, Params, ClientLogFile, out ClientRunFlags, out ClientApp, out ClientCmdLine);
				RunClient(SC, ClientLogFile, ClientRunFlags, ClientApp, ClientCmdLine, Params);
			}
		}

		private static IProcessResult RunClientInternal( ERunOptions ClientRunFlags, string ClientApp, string ClientCmdLine, ProjectParams Params, DeploymentContext SC )
		{
			IProcessResult CustomResult = null;
			foreach (CustomStagingHandler PackagingHandler in SC.CustomStagingHandlers)
			{
				CustomResult = PackagingHandler.TryRunClient(ClientRunFlags, ClientApp, ClientCmdLine, Params, SC);
				if (CustomResult != null)
				{
					break;
				}
			}

			if (CustomResult == null)
			{
				CustomResult = SC.CustomDeployment?.RunClient(ClientRunFlags, ClientApp, ClientCmdLine, Params, SC);
			}

			return CustomResult ?? SC.StageTargetPlatform.RunClient(ClientRunFlags, ClientApp, ClientCmdLine, Params, SC);
		}

		private static void RunClient(
				List<DeploymentContext> DeployContextList,
				string ClientLogFile,
				ERunOptions ClientRunFlags,
				string ClientApp,
				string ClientCmdLine,
				ProjectParams Params)
		{
			var ExtraClients = new List<IProcessResult>();
			int NumClients = Params.NumClients;
			int RunTimeoutSeconds = Params.RunTimeoutSeconds;
			IProcessResult ClientProcess = null;
			var SC = DeployContextList[0];
			bool bTestExitTextFound = false;
			ERunOptions ExtraClientRunFlags = ClientRunFlags | ERunOptions.NoStdOutRedirect;

			DateTime ClientStartTime = DateTime.UtcNow;

			if (Params.Unattended)
			{
				string LookFor = "Bringing up level for play took";
				bool bCommandlet = false;
				bool bAutomation = false;

				if (Params.RunAutomationTest != "" || Params.RunAutomationTests)
				{
					LookFor = "Automation Test Queue Empty";
					bAutomation = true;
				}
				else if (Params.EditorTest)
				{
					LookFor = "Asset discovery search completed in";
				}
				// If running a commandlet, just detect a normal exit
				else if (ClientCmdLine.IndexOf("-run=", StringComparison.InvariantCultureIgnoreCase) >= 0)
				{
					LookFor = "Game engine shut down";
					bCommandlet = true;
				}
				else if (Params.DedicatedServer)
				{
					LookFor = "Welcomed by server";
				}
				ClientCmdLine += "-testexit=\"" + LookFor + "\"";

				string AllClientOutput = "";
				int AllClientOutputLength = 0;
				int LastAutoFailIndex = -1;

				Logger.LogInformation("Starting Client for unattended test....");
				ClientProcess = RunClientInternal(ClientRunFlags, ClientApp, ClientCmdLine, Params, SC);

				if (DedicatedServerProcess != null)
				{
					if (NumClients > 1 && NumClients < 9)
					{
						for (int i = 1; i < NumClients; i++)
						{
							Logger.LogInformation("Starting Extra Client {i} for unattended test....", i);
							ExtraClients.Add(RunClientInternal(ExtraClientRunFlags, ClientApp, ClientCmdLine, Params, SC));
						}
					}
				}

				bool bKeepReading = ClientProcess != null;
				while (bKeepReading)
				{
					Thread.Sleep(100);

					if (RunTimeoutSeconds > 0)
					{
						if ((DateTime.UtcNow - ClientStartTime).TotalSeconds > RunTimeoutSeconds)
						{
							Logger.LogInformation("The run timed out after {RunTimeoutSeconds} seconds. Stopping client...", RunTimeoutSeconds);
							ClientProcess.StopProcess();
						}
					}
					if (ClientProcess.HasExited)
					{
						Logger.LogInformation("Client exited, waiting for stdout...");
						ClientProcess.WaitForExit();
						Logger.LogInformation("Client exited, logging done! (see {ClientLogFile} for more info)", ClientLogFile);
						bKeepReading = false;
					}

					AllClientOutput = ClientProcess.Output;
					if (AllClientOutput.Length > AllClientOutputLength)
					{
						int StartIndex = AllClientOutputLength;
						AllClientOutputLength = AllClientOutput.Length;

						// look for the test exit phrase, but ignore any output of the actual commandline string
						// which can look like either -testexit=Bringing, -testexit="Bringing, -testexit=\"Bringing
						if (AllClientOutput.IndexOf(LookFor, StartIndex) > 0 &&
							AllClientOutput.IndexOf("-testexit=", Math.Max(0, StartIndex - 12)) < 0)
						{
							if (DedicatedServerProcess != null)
							{
								Logger.LogInformation("Welcomed by server or client loaded");
							}
							Logger.LogInformation("Test complete");
							Logger.LogInformation("**** UNATTENDED TEST COMPLETE: {Time} seconds ****", $"{(DateTime.UtcNow - ClientStartTime).TotalMilliseconds / 1000:0.00}");
							bTestExitTextFound = true;
							bKeepReading = false;
						}
						if (bAutomation)
						{
							int FailIndex = AllClientOutput.LastIndexOf("Automation Test Failed");
							int ParenIndex = AllClientOutput.LastIndexOf(")");
							if (FailIndex >= 0 && ParenIndex > FailIndex && FailIndex > LastAutoFailIndex)
							{
								string Tail = AllClientOutput.Substring(FailIndex);
								int CloseParenIndex = Tail.IndexOf(")");
								int OpenParenIndex = Tail.IndexOf("(");
								string Test = "";
								if (OpenParenIndex >= 0 && CloseParenIndex > OpenParenIndex)
								{
									Test = Tail.Substring(OpenParenIndex + 1, CloseParenIndex - OpenParenIndex - 1);
									Logger.LogError("Automated test failed ({Test}).", Test);
									LastAutoFailIndex = FailIndex;
								}
							}
						}
						// Detect commandlet failure
						else if (bCommandlet)
						{
							const string ResultLog = "Commandlet->Main return this error code: ";

							int ResultStart = AllClientOutput.LastIndexOf(ResultLog);
							int ResultValIdx = ResultStart + ResultLog.Length;

							if (ResultStart >= 0 && ResultValIdx < AllClientOutput.Length &&
									AllClientOutput.Substring(ResultValIdx, 1) == "1")
							{
								// Parse the full commandlet warning/error summary
								string FullSummary = "";
								int SummaryStart = AllClientOutput.LastIndexOf("Warning/Error Summary");

								if (SummaryStart >= 0 && SummaryStart < ResultStart)
								{
									FullSummary = AllClientOutput.Substring(SummaryStart, ResultStart - SummaryStart);
								}

								if (FullSummary.Length > 0)
								{
									Logger.LogError("{Text}", "Commandlet failed, summary:" + Environment.NewLine + FullSummary);
								}
								else
								{
									Logger.LogError("Commandlet failed.");
								}
							}
						}
					}
					if (DedicatedServerProcess != null && DedicatedServerProcess.HasExited)
					{
						Logger.LogInformation("Dedicated server exited, stopping client...");
						ClientProcess.StopProcess();
					}
					else if (CookServerProcess != null && CookServerProcess.HasExited)
					{
						Logger.LogInformation("Cook server exited, stopping client...");
						ClientProcess.StopProcess();
					}
				}

				if (ClientProcess != null && !ClientProcess.HasExited)
				{
					Logger.LogInformation("Client is supposed to exit, lets wait 20 seconds for it to exit naturally...");
					for (int i = 0; i < 20 && !ClientProcess.HasExited; i++)
					{
						Thread.Sleep(1000);
					}
					if (!ClientProcess.HasExited)
					{
						Logger.LogInformation("Stopping client...");
						ClientProcess.StopProcess();
					}
				}
			}
			else
			{
				Logger.LogInformation("Starting Client....");
				ClientProcess = RunClientInternal(ClientRunFlags, ClientApp, ClientCmdLine, Params, SC);

				if (DedicatedServerProcess != null)
				{
					if (NumClients > 1 && NumClients < 9)
					{
						for (int i = 1; i < NumClients; i++)
						{
							Logger.LogInformation("Starting Extra Client {i}....", i);
							ExtraClients.Add(RunClientInternal(ExtraClientRunFlags, ClientApp, ClientCmdLine, Params, SC));
						}
					}
				}

				if (ClientProcess != null)
				{
					// If the client runs with LogWindow (without StdOut redirect),
					// then fetch output from log file on a separate thread.
					if (SC.StageTargetPlatform.UseAbsLog)
					{
						if ((ClientRunFlags & ERunOptions.NoStdOutRedirect) == ERunOptions.NoStdOutRedirect)
						{
							ClientLogReaderThread = new System.Threading.Thread(ClientLogReaderProc);
							ClientLogReaderThread.Start(new object[] { ClientLogFile, ClientProcess });
						}
					}

					do
					{
						Thread.Sleep(100);
						if (RunTimeoutSeconds > 0)
						{
							if ((DateTime.UtcNow - ClientStartTime).TotalSeconds > RunTimeoutSeconds)
							{
								Logger.LogInformation("The run timed out after {RunTimeoutSeconds} seconds. Stopping client...", RunTimeoutSeconds);
								ClientProcess.StopProcess();
							}
						}
						if (DedicatedServerProcess != null && DedicatedServerProcess.HasExited)
						{
							Logger.LogInformation("Dedicated server exited, stopping client...");
							ClientProcess.StopProcess();
						}
						else if (CookServerProcess != null && CookServerProcess.HasExited)
						{
							Logger.LogInformation("Cook server exited, stopping client...");
							ClientProcess.StopProcess();
						}
					}
					while (ClientProcess.HasExited == false);
				}
			}

			if (ExtraClients.Count > 0)
			{
				Logger.LogInformation("Client exited, stopping extra clients...");
				foreach (var OtherClient in ExtraClients)
				{
					if (OtherClient != null && !OtherClient.HasExited)
					{
						OtherClient.StopProcess();
					}
				}
			}

			SC.StageTargetPlatform.PostRunClient(ClientProcess, Params);

			if (Params.Unattended)
			{
				// In unattended/-testexit mode we only throw if testexit text was not found
				if (!bTestExitTextFound)
				{
					throw new AutomationException("Client exited before we asked it to (see {0} for more info)", ClientLogFile);
				}
			}
			else
			{
				// Any non-zero exit code should propagate an exception. The PostRunClient function above may have
				// already thrown a more specific exception or given a more specific ErrorCode, but this catches the rest.
				if (ClientProcess != null && !ClientProcess.bExitCodeSuccess)
				{
					throw new AutomationException("Client exited with error code: {0} (see {1} for more info)", ClientProcess.ExitCode, ClientLogFile);
				}
			}
			if (ClientProcess != null)
			{
				Logger.LogInformation("Client exited with error code: {Arg0} (see {ClientLogFile} for more info)", ClientProcess.ExitCode, ClientLogFile);
			}
		}

		private static void SetupClientParams(List<DeploymentContext> DeployContextList, ProjectParams Params, string ClientLogFile, out ERunOptions ClientRunFlags, out string ClientApp, out string ClientCmdLine)
		{
			if (Params.ClientTargetPlatforms.Count == 0)
			{
				throw new AutomationException("No ClientTargetPlatform set for SetupClientParams.");
			}

			if (DeployContextList.Count == 0)
			{
				throw new AutomationException("No DeployContextList for SetupClientParams.");
			}

			// set default output variables
			ClientRunFlags = ERunOptions.Default | ERunOptions.NoWaitForExit;
			ClientApp = "";
			ClientCmdLine = "";

			var TargetPlatform = Params.ClientTargetPlatforms[0];
			var SC = DeployContextList[0];

			// Get client app name and command line.
			string TempCmdLine = SC.ProjectArgForCommandLines + " ";
			var PlatformName = TargetPlatform.ToString();

			String FileHostCommandline = GetFileHostCommandline(Params, SC);
			if (!string.IsNullOrEmpty(FileHostCommandline))
			{
				TempCmdLine += FileHostCommandline + " ";
			}

			if (Params.Cook || Params.CookOnTheFly)
			{
				List<FileReference> Exes = SC.StageTargetPlatform.GetExecutableNames(SC);
				ClientApp = Exes[0].FullName;

				if (!String.IsNullOrEmpty(Params.ClientCommandline))
				{
					TempCmdLine += Params.ClientCommandline + " ";
				}
				else
				{
					TempCmdLine += Params.MapToRun + " ";
				}

				if (Params.CookOnTheFly || Params.FileServer)
				{
					if (Params.CookOnTheFlyStreaming)
					{
						TempCmdLine += "-streaming ";
					}
				}
				else if (Params.UsePak(SC.StageTargetPlatform))
				{
					if (Params.SignedPak)
					{
						TempCmdLine += "-signedpak ";
					}
				}
				else if (!Params.Stage)
				{
					var SandboxPath = CombinePaths(SC.RuntimeProjectRootDir.FullName, "Saved", "Cooked", SC.CookPlatform);
					if (!SC.StageTargetPlatform.LaunchViaUFE)
					{
						TempCmdLine += "-sandbox=" + CommandUtils.MakePathSafeToUseWithCommandLine(SandboxPath) + " ";
					}
					else
					{
						TempCmdLine += "-sandbox=\'" + SandboxPath + "\' ";
					}
				}
			}
			else
			{
				ClientApp = CombinePaths(CmdEnv.LocalRoot, "Engine/Binaries", PlatformName, "UnrealEditor.exe");
				if (!Params.EditorTest)
				{
					TempCmdLine += "-game " + Params.MapToRun + " ";
				}
				else
				{
					TempCmdLine += Params.MapToRun + " ";
				}

				if (Params.HasDDCGraph)
				{
					TempCmdLine += "-ddc=" + Params.DDCGraph + " ";
				}
			}

			if (Params.Unattended)
			{
				TempCmdLine += "-unattended ";
			}
			if (IsBuildMachine)
			{
				TempCmdLine += "-buildmachine ";
			}

			if (Params.CrashIndex > 0)
			{
				int RealIndex = Params.CrashIndex - 1;
				if (RealIndex >= CrashCommands.Count())
				{
					throw new AutomationException("CrashIndex {0} is out of range...max={1}", Params.CrashIndex, CrashCommands.Count());
				}
				TempCmdLine += String.Format("-execcmds=\"debug {0}\" ", CrashCommands[RealIndex]);
			}
			else if (Params.RunAutomationTest != "")
			{
				TempCmdLine += "-execcmds=\"automation runtests " + Params.RunAutomationTest + ";softquit\" ";
			}
			else if (Params.RunAutomationTests)
			{
				TempCmdLine += "-execcmds=\"automation runall;softquit;\" ";
			}
			if (SC.StageTargetPlatform.UseAbsLog)
			{
				TempCmdLine += "-abslog=" + CommandUtils.MakePathSafeToUseWithCommandLine(ClientLogFile) + " ";
			}
			if (SC.StageTargetPlatform.PlatformType == BuildHostPlatform.Current.Platform)
			{
				if (Params.LogWindow && !Params.Unattended)
				{
					// Without NoStdOutRedirect '-log' doesn't log anything to the window
					ClientRunFlags |= ERunOptions.NoStdOutRedirect;
					TempCmdLine += "-log ";
				}
				else
				{
					// unattended run logic depends on parsing stdout
					TempCmdLine += "-stdout -AllowStdOutLogVerbosity ";
				}
			}
			if (SC.StageTargetPlatform.PlatformType == UnrealTargetPlatform.Win64)
			{
				TempCmdLine += "-Windowed ";
			}

			TempCmdLine += "-Messaging ";

			if (Params.NullRHI && SC.StageTargetPlatform.PlatformType != UnrealTargetPlatform.Mac) // all macs have GPUs, and currently the mac dies with nullrhi
			{
				TempCmdLine += "-nullrhi ";
			}
			if (!String.IsNullOrEmpty(Params.Trace))
			{
				TempCmdLine += Params.Trace + " ";
			}
			if (!String.IsNullOrEmpty(Params.TraceHost))
			{
				TempCmdLine += Params.TraceHost + " ";
			}
			if (!String.IsNullOrEmpty(Params.TraceFile))
			{
				TempCmdLine += Params.TraceFile + " ";
			}
			if (!String.IsNullOrEmpty(Params.SessionLabel))
			{
				TempCmdLine += Params.SessionLabel + " ";
			}

			if (!String.IsNullOrEmpty(Params.ZenWorkspaceSharePath))
			{
				TempCmdLine += "-ZenPak -ZenWorkspaceSharePath=" + Params.ZenWorkspaceSharePath + " ";
			}

			TempCmdLine += SC.StageTargetPlatform.GetLaunchExtraCommandLine(Params);

			TempCmdLine += "-CrashForUAT ";
			TempCmdLine += Params.RunCommandline;

			// todo: move this into the platform
			if (SC.StageTargetPlatform.LaunchViaUFE)
			{
				ClientCmdLine = "-run=Launch ";
				ClientCmdLine += "-Device=" + Params.Devices[0];
				for (int DeviceIndex = 1; DeviceIndex < Params.Devices.Count; DeviceIndex++)
				{
					ClientCmdLine += "+" + Params.Devices[DeviceIndex];
				}
				ClientCmdLine += " ";
				ClientCmdLine += "-Exe=\"" + ClientApp + "\" ";
				ClientCmdLine += "-Targetplatform=" + PlatformName + " ";
				ClientCmdLine += "-Params=\"" + TempCmdLine + "\"";
				ClientApp = CombinePaths(CmdEnv.LocalRoot, "Engine/Binaries/Win64/UnrealFrontend.exe");

				Logger.LogInformation("Launching via UFE:");
				Logger.LogInformation("\tClientCmdLine: " + ClientCmdLine + "");
			}
			else
			{
				ClientCmdLine = TempCmdLine;
			}
		}

		private static List<string> GetFileHostAddresses(DeploymentContext SC)
		{
			List<string> HostAddresses = new List<string>();

			// Add localhost first for host platforms and skip it completely for other platforms.
			// Any Platform can implement ModifyFileHostAddresses to tweak this default behavior.
			string LocalHost = "127.0.0.1";
			if (UnrealBuildTool.BuildHostPlatform.Current.Platform == SC.StageTargetPlatform.PlatformType)
			{
				HostAddresses.Add(LocalHost);
			}

			bool bIsMac = UnrealBuildTool.BuildHostPlatform.Current.Platform == UnrealTargetPlatform.Mac;
			NetworkInterface[] Interfaces = NetworkInterface.GetAllNetworkInterfaces();
			foreach (NetworkInterface Adapter in Interfaces)
			{
				if (bIsMac)
				{
					if (Adapter.NetworkInterfaceType == NetworkInterfaceType.Loopback)
					{
						continue;
					}
				}
				else
				{
					if (Adapter.OperationalStatus != OperationalStatus.Up)
					{
						continue;
					}
				}

				IPInterfaceProperties IP = Adapter.GetIPProperties();
				foreach (UnicastIPAddressInformation UnicastAddress in IP.UnicastAddresses)
				{
					if (!InternalUtils.IsDnsEligible(UnicastAddress))
					{
						continue;
					}

					if (UnicastAddress.Address.AddressFamily != System.Net.Sockets.AddressFamily.InterNetwork)
					{
						continue;
					}

					string HostAddress = UnicastAddress.Address.ToString();
					if (HostAddress == LocalHost)
					{
						continue;
					}
					HostAddresses.Add(HostAddress);
				}
			}
			return HostAddresses.ToList();
		}

		private static string GetFileHostCommandline(ProjectParams Params, DeploymentContext SC)
		{
			string FileHostParams = "";
			if (!Params.CookOnTheFly && !Params.FileServer)
			{
				return FileHostParams;
			}

			List<string> HostAddresses = GetFileHostAddresses(SC);
			SC.StageTargetPlatform.ModifyFileHostAddresses(HostAddresses);

			if (!IsNullOrEmpty(Params.Port))
			{
				foreach (var Port in Params.Port)
				{
					string[] PortProtocol = Port.Split(new char[] { ':' });
					for (int I = 0; I < HostAddresses.Count; I++)
					{
						if (PortProtocol.Length > 1)
						{
							HostAddresses[I] = String.Format("{0}://{1}:{2}", PortProtocol[0], HostAddresses[I], PortProtocol[1]);
						}
						else
						{
							HostAddresses[I] = String.Format("{0}:{1}", HostAddresses[I], Port);
						}
					}
				}
			}

			FileHostParams += "-filehostip=";
			FileHostParams += String.Join("+", HostAddresses);
			return FileHostParams;
		}

		private static void ClientLogReaderProc(object ArgsContainer)
		{
			var Args = ArgsContainer as object[];
			var ClientLogFile = (string)Args[0];
			var ClientProcess = (IProcessResult)Args[1];
			LogFileReaderProcess(ClientLogFile, ClientProcess, (string Output) =>
			{
				if (String.IsNullOrEmpty(Output) == false)
				{
					Logger.LogInformation("{Text}", Output);
				}
				return true;
			});
		}

		private static IProcessResult RunDedicatedServer(ProjectParams Params, string ServerLogFile, string AdditionalCommandLine)
		{
			ProjectParams ServerParams = new ProjectParams(Params);
			ServerParams.Devices = new ParamList<string>(Params.ServerDevice);

			if (ServerParams.ServerTargetPlatforms.Count == 0)
			{
				throw new AutomationException("No ServerTargetPlatform set for RunDedicatedServer.");
			}

			var DeployContextList = CreateDeploymentContext(ServerParams, true);

			if (DeployContextList.Count == 0)
			{
				throw new AutomationException("No DeployContextList for RunDedicatedServer.");
			}

			var SC = DeployContextList[0];

			TargetPlatformDescriptor ServerPlatformDesc = ServerParams.ServerTargetPlatforms[0];
			var ServerApp = CombinePaths(CmdEnv.LocalRoot, "Engine/Binaries/Win64/UnrealEditor.exe");
			bool bCooked = ServerParams.Cook || ServerParams.CookOnTheFly;
			if (bCooked)
			{
				List<FileReference> Exes = SC.StageTargetPlatform.GetExecutableNames(SC);
				ServerApp = Exes[0].FullName;
				
				// account for app bundles placed outside Binaries/Mac on macOS
				if(ServerPlatformDesc.Type == UnrealTargetPlatform.Mac)
				{
					DirectoryReference BaseStageDir = new DirectoryReference(Params.BaseStageDirectory);
					DirectoryReference StageSubDir = new FileReference(ServerApp).Directory;

					while (StageSubDir.IsUnderDirectory(BaseStageDir) && StageSubDir.ParentDirectory != BaseStageDir)
					{
					    StageSubDir = StageSubDir.ParentDirectory;
					}
					ServerApp = FileReference.Combine(StageSubDir, 
													  $"{Exes[0].GetFileNameWithoutExtension()}.app/Contents/MacOS/{Exes[0].GetFileNameWithoutExtension()}").FullName;
				}
			}
			var Args = bCooked ? "" : (SC.ProjectArgForCommandLines + " ");
			Console.WriteLine("Running dedicated server on device with address: " + Params.ServerDeviceAddress);
			if (ServerParams.Cook && ServerPlatformDesc.Type == UnrealTargetPlatform.Linux && !String.IsNullOrEmpty(ServerParams.ServerDeviceAddress))
			{
				ServerApp = @"C:\Windows\system32\cmd.exe";

				string plinkPath = MakePathSafeToUseWithCommandLine(CombinePaths(Unreal.RootDirectory.FullName, "Engine", "Extras", "ThirdPartyNotUE", "putty", "PLINK.exe"));
				string exePath = MakePathSafeToUseWithCommandLine(CombinePaths(SC.ShortProjectName, "Binaries", ServerPlatformDesc.Type.ToString(), SC.ShortProjectName + "Server"));
				if (ServerParams.ServerConfigsToBuild[0] != UnrealTargetConfiguration.Development)
				{
					exePath += "-" + ServerPlatformDesc.Type.ToString() + "-" + ServerParams.ServerConfigsToBuild[0].ToString();
				}
				exePath = CombinePaths("LinuxServer", exePath.ToLower()).Replace("\\", "/");
				Args = String.Format("/k {0} -batch -ssh -t -i {1} {2}@{3} {4} {5} {6} -server -Messaging", plinkPath, ServerParams.DevicePassword, ServerParams.DeviceUsername, ServerParams.ServerDeviceAddress, exePath, Args, ServerParams.MapToRun);
			}
			else
			{
				var Map = ServerParams.MapToRun;
				if (!String.IsNullOrEmpty(ServerParams.AdditionalServerMapParams))
				{
					Map += ServerParams.AdditionalServerMapParams;
				}
				if (Params.FakeClient)
				{
					Map += "?fake";
				}
				Args += String.Format("{0} -server -abslog={1} -log -Messaging", Map, CommandUtils.MakePathSafeToUseWithCommandLine(ServerLogFile));
				if (Params.Unattended)
				{
					Args += " -unattended";
				}

				if (Params.ServerCommandline.Length > 0)
				{
					Args += " " + Params.ServerCommandline;
				} 

				String FileHostCommandline = GetFileHostCommandline(Params, SC);
				if (!string.IsNullOrEmpty(FileHostCommandline))
				{
					Args += " " + FileHostCommandline;
				}
			}

			if (ServerParams.UsePak(SC.StageTargetPlatform))
			{
				if (ServerParams.SignedPak)
				{
					Args += " -signedpak";
				}
			}

			if (Params.HasDDCGraph && !bCooked)
			{
				Args += " -ddc=" + Params.DDCGraph;
			}

			if (IsBuildMachine)
			{
				Args += " -buildmachine";
			}

			if (!String.IsNullOrEmpty(Params.Trace))
			{
				Args += " " + Params.Trace;
			}

			if (!String.IsNullOrEmpty(Params.TraceHost))
			{
				Args += " " + Params.TraceHost;
			}

			if (!String.IsNullOrEmpty(Params.TraceFile))
			{
				Args += " " + Params.TraceFile;
			}

			Args += " -CrashForUAT";
			Args += " " + AdditionalCommandLine;

			// Avoid issues with dotnet not allowing any files with an exclusive advisory lock to be opened for read-only or copied
			// https://github.com/dotnet/runtime/issues/34126
			if (ServerPlatformDesc.Type == UnrealTargetPlatform.Linux || ServerPlatformDesc.Type == UnrealTargetPlatform.Mac)
			{
				Args += " -noexclusivelockonwrite";
			}

			if (ServerParams.Cook && ServerPlatformDesc.Type == UnrealTargetPlatform.Linux && !String.IsNullOrEmpty(ServerParams.ServerDeviceAddress))
			{
				Args += String.Format(" 2>&1 > {0}", ServerLogFile);
			}

			PushDir(Path.GetDirectoryName(ServerApp));
			var Result = Run(ServerApp, Args, null, ERunOptions.Default | ERunOptions.NoWaitForExit | ERunOptions.NoStdOutRedirect);
			PopDir();

			Logger.LogInformation("Running DedicatedServer@Process:{ServerApp}@{Arg1}", ServerApp, Result.ProcessObject.Id);
			return Result;
		}

		private static IProcessResult RunCookOnTheFlyServer(ProjectParams Params, string ServerLogFile, string AdditionalCommandLine)
		{
			var ServerApp = HostPlatform.Current.GetUnrealExePath(Params.UnrealExe);
			var Args = String.Format("{0} -run=cook -cookonthefly -unattended -CrashForUAT -AllowStdOutLogVerbosity",
				CommandUtils.MakePathSafeToUseWithCommandLine(Params.RawProjectPath.FullName));
			if (!String.IsNullOrEmpty(ServerLogFile))
			{
				// Issue with dotnet not allowing any files with an exclusive advisory lock to be opened for read-only or copied
				// https://github.com/dotnet/runtime/issues/34126
				if (HostPlatform.Current.HostEditorPlatform == UnrealBuildTool.UnrealTargetPlatform.Linux)
				{
					Args += " -noexclusivelockonwrite";
				}

				Args += " -abslog=" + CommandUtils.MakePathSafeToUseWithCommandLine(ServerLogFile);
			}

			if (!String.IsNullOrEmpty(AdditionalCommandLine))
			{
				Args += " " + AdditionalCommandLine;
			}

			// Run the server with shell execute to launch it in a separate shell with stdout
			PushDir(Path.GetDirectoryName(ServerApp));
			var Result = Run(ServerApp, Args, null, ERunOptions.Default | ERunOptions.NoWaitForExit | ERunOptions.UseShellExecute);
			PopDir();

			Logger.LogInformation("Running CookServer@Process:{ServerApp}@{Arg1}", ServerApp, Result.ProcessObject.Id);
			return Result;
		}

		private static bool StartUnrealTrace()
		{
			// [TEMPORARY] - UnrealTrace server is currently only available on Windows
			if (!OperatingSystem.IsWindows())
			{
				return true;
			}
			// [/TEMPORARY]

			Logger.LogInformation("UnrealTrace: Starting server");

			// Locate the UnrealTrace binary
			var UnrealTracePath = HostPlatform.Current.GetUnrealExePath("UnrealTraceServer.exe");
			if (!File.Exists(UnrealTracePath))
			{
				Logger.LogWarning("{Text}", "UnrealTrace: Unable to locate binary at " + UnrealTracePath);
				return false;
			}

			// Launch UnrealTrace and wait for it to fork and return
			Process Proc = Process.Start(UnrealTracePath, " fork");
			Proc.WaitForExit();
			if (Proc.ExitCode != 0)
			{
				Logger.LogWarning("{Text}", "UnrealTrace: Failed to start server; ExitCode=" + Proc.ExitCode);
				return false;
			}

			return true;
		}
	}
}
