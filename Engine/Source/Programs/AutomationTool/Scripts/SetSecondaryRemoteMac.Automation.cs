// Copyright Epic Games, Inc. All Rights Reserved.

using Microsoft.Extensions.Logging;
using System.Collections.Generic;
using AutomationTool;
using UnrealBuildTool;
using System.IO;
using EpicGames.Core;

namespace AutomationScripts
{
	public class SetSecondaryRemoteMac : BuildCommand
	{
		public override void ExecuteBuild()
		{

			if (ParseParam("usage"))
			{
				Logger.LogInformation("Arguments : \n -platform=<platform> \n -ProjectFilePath=<path/ProjectName.uproject> Mandatory. Path to the project you want to Debug. \n -SourcePackage=<path/PackageName.ipa> Optional. Specify the .ipa to wrangle data for debug from. When not specified, [ProjectPath]/Build/[IOS|TVOS]/[ProjectName].ipa will be used.");
			}

			FileReference ProjectFile = new FileReference(ParseParamValue("ProjectFilePath=", null));
			var Params = new ProjectParams
			(
				Command: this,
				RawProjectPath: ProjectFile
			);

			Logger.LogInformation("======= SetSecondaryRemoteMac - Start =======");

			var SC = Project.CreateDeploymentContext(Params, false);

			SC[0].StageTargetPlatform.SetSecondaryRemoteMac(ParseParamValue("ProjectFilePath=", null), ParseParamValue("platform=", null));

			Logger.LogInformation("======= SetSecondaryRemoteMac - Done ========");
		}
	}
}
