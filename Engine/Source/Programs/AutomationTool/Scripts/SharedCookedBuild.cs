// Copyright Epic Games, Inc. All Rights Reserved.
using System;
using System.Collections.Generic;
using System.IO;
using System.Threading;
using System.Reflection;
using System.Linq;
using System.Threading.Tasks;
using AutomationTool;
using UnrealBuildTool;
using EpicGames.Core;
using System.Text.RegularExpressions;
using UnrealBuildBase;
using Microsoft.Extensions.Logging;

using static AutomationTool.CommandUtils;

public class SharedCookedBuild
{
	private const string SyncedBuildFileName = "SyncedBuild.txt";

	/// <summary>
	/// Types of shared cook base builds
	/// </summary>
	public enum SharedCookType
	{
		/// <summary>
		/// Only allow shared cook build of version identical to local sync
		/// </summary>
		Exact,

		/// <summary>
		/// Allow any previous version that is only a content change from local sync
		/// </summary>
		Content,

		/// <summary>
		/// Closest previous version, regardless of code/content changes
		/// </summary>
		Any,
	}

	public enum SharedCookSource
	{
		Manifest,
		LooseFiles,
	}

	private FileReference ProjectFile { get; set; }
	private DirectoryReference InstallPath { get; set; }
	private HashSet<string> TargetPlatforms { get; set; }
	private SharedCookType BuildType { get; set; }
	private List<ISharedCookedBuild> CandidateBuilds { get; set; }
	private BuildVersion LocalSync { get; set; }

	public SharedCookedBuild(ProjectParams Params)
	{
		List<string> Platforms = new List<string>();
		foreach (TargetPlatformDescriptor ClientPlatform in Params.ClientTargetPlatforms)
		{
			TargetPlatformDescriptor DataPlatformDesc = Params.GetCookedDataPlatformForClientTarget(ClientPlatform);
			Platforms.Add(Platform.Platforms[DataPlatformDesc].GetCookPlatform(false, Params.Client));
		}
		SharedCookType BuildType = (SharedCookType)Enum.Parse(typeof(SharedCookType), Params.IterateSharedCookedBuild, true);
		SharedCookedBuildConstructor(Params.RawProjectPath, Platforms, BuildType);
	}

	public SharedCookedBuild(FileReference ProjectFile, IEnumerable<string> TargetPlatforms, SharedCookType BuildType)
	{
		SharedCookedBuildConstructor(ProjectFile, TargetPlatforms, BuildType);
	}

	private void SharedCookedBuildConstructor(FileReference ProjectFile, IEnumerable<string> Platforms, SharedCookType BuildType)
	{
		TargetPlatforms = new HashSet<string>();
		CandidateBuilds = new List<ISharedCookedBuild>();
		this.ProjectFile = ProjectFile;
		this.BuildType = BuildType;
		InstallPath = DirectoryReference.Combine(ProjectFile.Directory, "Saved", "SharedIterativeBuild");
		LocalSync = GetLocalSync();
		foreach (string Platform in Platforms)
		{
			TargetPlatforms.Add(Platform);
		}
	}

	public void CopySharedCookedBuilds()
	{
		CandidateBuilds = FindBestBuilds();
		if (CandidateBuilds.Count == 0)
		{
			throw new AutomationException("No valid shared cooked builds available");
		}

		foreach (string TargetPlatform in TargetPlatforms)
		{
			// Prefer existing sync if listed
			IEnumerable<ISharedCookedBuild> LocalSync = CandidateBuilds.Where(x => x.GetType() == typeof(ExistingSharedCookedBuild) && x.Platform.Equals(TargetPlatform, StringComparison.InvariantCultureIgnoreCase));
			if (LocalSync.Count() > 0)
			{
				LocalSync.First().CopyBuild(InstallPath);
				continue;
			}

			IEnumerable<ISharedCookedBuild> PlatformBuilds = CandidateBuilds.Where(x => x.Platform.Equals(TargetPlatform, StringComparison.InvariantCultureIgnoreCase));
			if (PlatformBuilds.Count() > 0)
			{
				ISharedCookedBuild Build = PlatformBuilds.First();
				if (!Build.CopyBuild(InstallPath))
				{
					throw new AutomationException("Failed to copy shared build for {0} {1}", Build.Platform, Build.CL);
				}
			}
		}
	}

	public static BuildVersion GetLocalSync()
	{
		BuildVersion P4Version = new BuildVersion();
		if (CommandUtils.P4Enabled)
		{
			P4Version.BranchName = CommandUtils.P4Env.Branch.Replace("/", "+");
			P4Version.Changelist = CommandUtils.P4Env.Changelist;
			P4Version.CompatibleChangelist = CommandUtils.P4Env.CodeChangelist;
		}

		BuildVersion UGSVersion;
		if (BuildVersion.TryRead(BuildVersion.GetDefaultFileName(), out UGSVersion))
		{
			return UGSVersion;
		}
		
		if (!CommandUtils.P4Enabled)
		{
			throw new AutomationException("Cannot determine local sync");
		}

		return P4Version;
	}

	public List<ISharedCookedBuild> FindBestBuilds()
	{
		// Attempt manifest searching first
		ConfigHierarchy Hierarchy = ConfigCache.ReadHierarchy(ConfigHierarchyType.Engine, DirectoryReference.FromFile(ProjectFile), UnrealTargetPlatform.Win64);

		IReadOnlyList<string> RawSharedCookedSources = null;
		Hierarchy.TryGetValues("SharedCookedBuildSettings", "SharedCookedSources", out RawSharedCookedSources);
		if (RawSharedCookedSources == null)
		{
			throw new AutomationException("Unable to locate shared cooked builds. SharedCookedSources not set in Engine.ini [SharedCookedBuildSettings]");
		}

		List<Dictionary<string, string>> ParsedSharedCookSources = new List<Dictionary<string, string>>();
		foreach (string RawConfig in RawSharedCookedSources)
		{
			Dictionary<string, string> ParsedSource = null;
			if (ConfigHierarchy.TryParse(RawConfig, out ParsedSource))
			{
				ParsedSharedCookSources.Add(ParsedSource);
			}
		}

		List<ISharedCookedBuild> CandidateBuilds = new List<ISharedCookedBuild>();

		// If existing sync is present, stick to it. Read version out of sync file
		foreach (string Platform in TargetPlatforms)
		{
			FileReference SyncedBuildFile = new FileReference(CommandUtils.CombinePaths(InstallPath.FullName, Platform, SyncedBuildFileName));
			if (FileReference.Exists(SyncedBuildFile))
			{
				string[] SyncedBuildInfo = FileReference.ReadAllLines(SyncedBuildFile);
				int SyncedCL = int.Parse(SyncedBuildInfo[0]);
				if (IsValidCL(SyncedCL, BuildType, LocalSync))
				{
					CandidateBuilds.Add(new ExistingSharedCookedBuild() { CL = SyncedCL, Platform = Platform });
				}
			}
		}

		foreach (Dictionary<string, string> Source in ParsedSharedCookSources)
		{
			SharedCookSource SourceType = (SharedCookSource)Enum.Parse(typeof(SharedCookSource), Source["Type"], true);
			foreach (string Platform in TargetPlatforms)
			{
				if (SourceType == SharedCookSource.Manifest)
				{
					CandidateBuilds.AddRange(FindValidManifestBuilds(Source["Path"], Platform));
				}
				else if (SourceType == SharedCookSource.LooseFiles)
				{
					CandidateBuilds.AddRange(FindValidLooseBuilds(Source["Path"], Platform));
				}
			}
		}

		// Strip all failed searches
		CandidateBuilds.RemoveAll(x => x == null);

		// Make sure we have a matching CL for all target platforms, regardless of source
		List<int> OrderedDistinctCLs = CandidateBuilds.Select(x => x.CL).Distinct().OrderByDescending(i => i).ToList();
		int BestCL = -1;
		foreach (int CL in OrderedDistinctCLs)
		{
			// Ensure we have a platform for each
			HashSet<string> CLPlatforms = new HashSet<string>(CandidateBuilds.Where(x => x.CL == CL).Select(x => x.Platform).ToList());
			if (CLPlatforms.SetEquals(TargetPlatforms))
			{
				BestCL = CL;
				break;
			}
		}

		if (BestCL < 0)
		{
			Logger.LogError("Could not locate valid shared cooked build for all target platforms");
			Logger.LogError("Current CL: {Arg0}, Current Code CL: {Arg1}", LocalSync.Changelist, LocalSync.CompatibleChangelist);
		}

		return CandidateBuilds.Where(x => x.CL == BestCL).ToList();
	}

	public List<ISharedCookedBuild> FindValidManifestBuilds(string Path, string TargetPlatform)
	{
		List<ISharedCookedBuild> ValidBuilds = new List<ISharedCookedBuild>();
		Tuple<string, string> SplitPath = SplitOnFixedPrefix(Path);
		Regex Pattern = RegexFromWildcards(SplitPath.Item2, LocalSync, TargetPlatform);

		DirectoryReference SearchDir = new DirectoryReference(SplitPath.Item1);
		if (DirectoryReference.Exists(SearchDir))
		{
			foreach (FileReference File in DirectoryReference.EnumerateFiles(SearchDir))
			{
				Match Match = Pattern.Match(File.FullName);
				if (Match.Success)
				{
					int MatchCL = int.Parse(Match.Result("${CL}"));
					if (IsValidCL(MatchCL, BuildType, LocalSync))
					{
						ValidBuilds.Add(new ManifestSharedCookedBuild { CL = MatchCL, Manifest = File, Platform = TargetPlatform });
					}
				}
			}
		}

		return ValidBuilds;
	}

	public List<ISharedCookedBuild> FindValidLooseBuilds(string Path, string TargetPlatform)
	{
		List<ISharedCookedBuild> ValidBuilds = new List<ISharedCookedBuild>();
		Tuple<string, string> SplitPath = SplitOnFixedPrefix(Path);
		Regex Pattern = RegexFromWildcards(SplitPath.Item2, LocalSync, TargetPlatform);

		// Search for all available builds
		const string MetaDataFilename = "\\Metadata\\DevelopmentAssetRegistry.bin";
		string BuildRule = SplitPath.Item2 + MetaDataFilename;
		BuildRule = BuildRule.Replace("[BRANCHNAME]", LocalSync.BranchName);
		BuildRule = BuildRule.Replace("[PLATFORM]", TargetPlatform);
		string IncludeRule = BuildRule.Replace("[CL]", "*");
		string ExcludeRule = BuildRule.Replace("[CL]", "*-PF-*"); // Exclude preflights
		FileFilter BuildSearch = new FileFilter();
		BuildSearch.AddRule(IncludeRule);
		BuildSearch.AddRule(ExcludeRule, FileFilterType.Exclude);
		
		foreach (FileReference CandidateBuild in BuildSearch.ApplyToDirectory(new DirectoryReference(SplitPath.Item1), false))
		{
			string BaseBuildPath = CandidateBuild.FullName.Replace(MetaDataFilename, "");
			Match Match = Pattern.Match(BaseBuildPath);
			if (Match.Success)
			{
				int MatchCL = int.Parse(Match.Result("${CL}"));
				if (IsValidCL(MatchCL, BuildType, LocalSync))
				{
					ValidBuilds.Add(new LooseSharedCookedBuild { CL = MatchCL, Path = new DirectoryReference(BaseBuildPath), Platform = TargetPlatform });
				}
			}
		}

		return ValidBuilds;
	}


	private static bool IsValidCL(int CL, SharedCookType BuildType, BuildVersion Version)
	{
		if (BuildType == SharedCookType.Exact && CL == Version.Changelist)
		{
			return true;
		}
		else if (BuildType == SharedCookType.Content && CL >= Version.EffectiveCompatibleChangelist && CL <= Version.Changelist)
		{
			return true;
		}
		else if (BuildType == SharedCookType.Any && CL <= Version.Changelist)
		{
			return true;
		}

		return false;
	}

	private static Regex RegexFromWildcards(string Path, BuildVersion Version, string TargetPlatform)
	{
		string Pattern = Path.Replace(@"\", @"\\");
		Pattern = Pattern.Replace("[BRANCHNAME]", Version.BranchName.Replace(@"+", @"\+"));
		Pattern = Pattern.Replace("[PLATFORM]", TargetPlatform);
		Pattern = Pattern.Replace("[CL]", @"(?<CL>\d+)");
		return new Regex(Pattern);
	}

	private static Tuple<string, string> SplitOnFixedPrefix(string Path)
	{
		int IndexOfFirstParam = Path.IndexOf("[");
		int PrefixStart = Path.LastIndexOf(@"\", IndexOfFirstParam);
		return new Tuple<string, string>(Path.Substring(0, PrefixStart), Path.Substring(PrefixStart));
	}

	public interface ISharedCookedBuild
	{
		int CL { get; set; }
		string Platform { get; set; }
		bool CopyBuild(DirectoryReference InstallPath);
	}

	private class ManifestSharedCookedBuild : ISharedCookedBuild
	{
		public int CL { get; set; }
		public string Platform { get; set; }
		public FileReference Manifest { get; set; }

		public bool CopyBuild(DirectoryReference InstallPath)
		{
			Logger.LogInformation("Installing shared cooked build from manifest: {Arg0} to {Arg1}", Manifest.FullName, InstallPath.FullName);

			DirectoryReference PlatformInstallPath = DirectoryReference.Combine(InstallPath, Platform.ToString());

			FileReference PreviousManifest = FileReference.Combine(PlatformInstallPath, ".build", "Current.manifest");

			FileReference BPTI = FileReference.Combine(Unreal.RootDirectory, "Engine", "Restricted", "NotForLicensees", "Binaries", "Win64", "BuildPatchToolInstaller.exe");
			if (!FileReference.Exists(BPTI))
			{
				Logger.LogInformation("Could not locate BuildPatchToolInstaller.exe");
				return false;
			}

			bool PreviousManifestExists = FileReference.Exists(PreviousManifest);
			if (!PreviousManifestExists && DirectoryReference.Exists(PlatformInstallPath))
			{
				DirectoryReference.Delete(PlatformInstallPath, true);
			}

			IProcessResult Result = CommandUtils.Run(BPTI.FullName, string.Format("-Manifest={0} -OutputDir={1} -stdout -GenericConsoleOutput", Manifest.FullName, PlatformInstallPath.FullName), null, CommandUtils.ERunOptions.Default);
			if (Result.ExitCode != 0)
			{
				Logger.LogWarning("Failed to install manifest {Arg0} to {Arg1}", Manifest.FullName, PlatformInstallPath.FullName);
				return false;
			}

			FileReference SyncedBuildFile = new FileReference(CommandUtils.CombinePaths(PlatformInstallPath.FullName, SyncedBuildFileName));
			FileReference.WriteAllLines(SyncedBuildFile, new string[] { CL.ToString(), Manifest.FullName });

			return true;
		}
	}

	private class LooseSharedCookedBuild : ISharedCookedBuild
	{
		public int CL { get; set; }
		public string Platform { get; set; }
		public DirectoryReference Path { get; set; }
		public bool CopyBuild(DirectoryReference InstallPath)
		{
			Logger.LogInformation("Copying shared cooked build from stage directory: {Arg0} to {Arg1}", Path.FullName, InstallPath.FullName);

			// Delete existing
			if (DirectoryReference.Exists(InstallPath))
			{
				DirectoryReference.Delete(InstallPath, true);
			}
			DirectoryReference.CreateDirectory(InstallPath);

			// Copy new
			if (!CommandUtils.CopyDirectory_NoExceptions(Path.FullName, InstallPath.FullName))
			{
				Logger.LogWarning("Failed to copy {Arg0} -> {Arg1}", Path.FullName, InstallPath.FullName);
				return false;
			}
			FileReference SyncedBuildFile = new FileReference(CommandUtils.CombinePaths(InstallPath.FullName, SyncedBuildFileName));
			FileReference.WriteAllLines(SyncedBuildFile, new string[] { CL.ToString(), Path.FullName });

			return true;
		}
	}

	private class ExistingSharedCookedBuild : ISharedCookedBuild
	{
		public int CL { get; set; }
		public string Platform { get; set; }

		public bool CopyBuild(DirectoryReference InstallPath)
		{
			Logger.LogInformation("Using previously synced shared cooked build");
			return true;
		}
	}
}
