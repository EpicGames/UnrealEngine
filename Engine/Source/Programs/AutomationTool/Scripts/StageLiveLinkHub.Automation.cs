// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using EpicGames.Core;
using Microsoft.Extensions.Logging;
using System.Collections.Generic;
using System.IO;
using UnrealBuildBase;
using UnrealBuildTool;
using static System.Net.WebRequestMethods;
using System.Diagnostics;
using System.Xml.Linq;

namespace AutomationTool
{
	// This will stage livelink hub files in 
	[Help("Command to stage livelink hub plugin dependencies for an installed build.")]
	[Help("Platform=<Platform>", "Target platform for which livelinkhub was built. (required)")]
	[Help("Verbose", "Whether to stage with verbose logging. (optional)")]
	class StageLiveLinkHub : BuildCommand
	{
		// Root output directory for the staged files.
		private DirectoryReference OutputDir;

		// Directory for plugins that should be staged but not discoverable by the editor.
		private DirectoryReference LiveLinkHubPluginsDirectory;

		// Whether to output with verbose logging.
		private bool Verbose = false;

		// LiveLinkHub staging config.
		private ConfigHierarchy Config;

		// Target build platform.
		private UnrealTargetPlatform TargetPlatform;

		/// <summary>
		/// Entry point for the commandlet
		/// </summary>
		public override void ExecuteBuild()
		{
			try
			{
				OutputDir = Unreal.RootDirectory;

				LiveLinkHubPluginsDirectory = DirectoryReference.Combine(Unreal.EngineDirectory, "Programs", "LiveLinkHub", "Staged", "Plugins");
				
				Verbose = ParseParam("Verbose");

				string PlatformsString = ParseParamValue("Platform");
				TargetPlatform = UnrealTargetPlatform.Parse(PlatformsString);

				Config = GetLiveLinkHubConfig();

				if (Config == null)
				{
					Log.Logger.LogInformation("Stage LiveLinkHub: Could not find livelink hub staging config");
					return;
				}

				StagePlugins();
				StageRemappedDependencies();
				StageDirectDependencies();

			}
			catch (Exception Ex)
			{
				Logger.LogInformation("Stage LiveLinkHub: Exception {Ex}", LogUtils.FormatException(Ex));
			}
		}

		// Stages restricted plugins content and config to a non-restricted location.
		private void StagePlugins()
		{
			List<string> PluginDependenciesPath;
			Config.GetArray("LiveLinkHub_InstalledBuild", "PluginDependencies", out PluginDependenciesPath);

			if (PluginDependenciesPath != null)
			{
				foreach (string PluginDependencyPath in PluginDependenciesPath)
				{
					StagePlugin(PluginDependencyPath);
				}
			}
			else
			{
				Log.Logger.LogInformation("Stage LiveLinkHub: Could not find livelink hub dependencies in the staging config");
			}
		}

		// These dependencies need to be copied from a restricted folder to a non-restricted location.
		private void StageRemappedDependencies()
		{
			List<string> RemappedDependencies;
            Config.GetArray("LiveLinkHub_InstalledBuild", "RemappedDependencies", out RemappedDependencies);

            if (RemappedDependencies != null)
            {
            	foreach (string RemappedDependencyPath in RemappedDependencies)
            	{
            		StageRemappedDependency(RemappedDependencyPath);
            	}
            }
		}

		// These dependencies need to be copied from a restricted folder to the binaries folder.
		private void StageDirectDependencies()
		{
			// For 5.4, we don't need to stage external dependencies except for windows.
			if (TargetPlatform == UnrealTargetPlatform.Win64)
			{
				List<string> DirectDependencies;
				Config.GetArray("LiveLinkHub_InstalledBuild", "DirectDependencies", out DirectDependencies);

				if (DirectDependencies != null)
				{
					foreach (string DirectDependencyPath in DirectDependencies)
					{
						StageDirectDependency(DirectDependencyPath);
					}
				}
			}
			else
			{
				Log.Logger.LogInformation($"Stage LiveLinkHub: Skipped direct dependencies for platform {TargetPlatform}");
			}
		}

		// Stages a restricted dependency to a non-restricted location.
		private void StageRemappedDependency(string RemappedDependencyPath)
		{
			FileReference SourceFile = FileReference.Combine(Unreal.RootDirectory, RemappedDependencyPath);
			FileReference Replaced = new FileReference(SourceFile.FullName.Replace("\\Restricted\\NotForLicensees", "").Replace("/Restricted/NotForLicensees", ""));
			DirectoryReference PluginDirectory = DirectoryReference.Combine(Unreal.EngineDirectory, "Plugins");

			FileReference RemappedFile = FileReference.Combine(LiveLinkHubPluginsDirectory, Replaced.FullName.Substring(PluginDirectory.FullName.Length + 1));

			if (Verbose)
			{
				Log.Logger.LogInformation($"Stage LiveLinkHub: Staging remapped dependency : {SourceFile} -> {RemappedFile}");
			}

			CopyFile_NoExceptions(SourceFile.FullName, RemappedFile.FullName);
			CodeSign.SignMultipleIfEXEOrDLL(this, new List<String> { RemappedFile.FullName });
		}

		// Stages a dependency to the binaries win64 folder.
		private void StageDirectDependency(string DirectDependencyPath)
		{
			FileReference SourceFile = FileReference.Combine(Unreal.RootDirectory, DirectDependencyPath);

			DirectoryReference DestinationDirectory = DirectoryReference.Combine(OutputDir, "Engine", "Binaries", TargetPlatform.ToString());

			FileReference DestinationFile = FileReference.Combine(DestinationDirectory, SourceFile.GetFileName());
		
			if (Verbose)
			{
				Log.Logger.LogInformation($"Stage LiveLinkHub: Staging direct dependency : {SourceFile} -> {DestinationFile}");
			}

			CopyFile_NoExceptions(SourceFile.FullName, DestinationFile.FullName);
			CodeSign.SignMultipleIfEXEOrDLL(this, new List<String> { DestinationFile.FullName });
		}

		// Fetches the livelinkhub staging configuration file.
		private ConfigHierarchy GetLiveLinkHubConfig()
		{
			DirectoryReference ConfigDirectory = DirectoryReference.Combine(Unreal.EngineDirectory, "Restricted", "NotForLicensees", "Programs", "LiveLinkHub");
			return ConfigCache.ReadHierarchy(ConfigHierarchyType.Engine, ConfigDirectory, BuildHostPlatform.Current.Platform);
		}

		// Stages a plugin folder, stripping away the source folder.
		private void StagePlugin(string PluginPath)
		{
			string[] Subfolders = { "Content", "Resources", "Config" };

			// Stage subfolders
			DirectoryReference FullPluginPath = DirectoryReference.Combine(Unreal.RootDirectory, PluginPath);

			DirectoryReference PluginWithoutRestricted = new DirectoryReference(FullPluginPath.FullName.Replace("\\Restricted\\NotForLicensees", "").Replace("/Restricted/NotForLicensees", ""));

			DirectoryReference PluginDirectory = DirectoryReference.Combine(Unreal.EngineDirectory, "Plugins");

			// Remove the path up until /engine
			DirectoryReference PluginRemappedFolder = DirectoryReference.Combine(LiveLinkHubPluginsDirectory, PluginWithoutRestricted.FullName.Substring(PluginDirectory.FullName.Length + 1));

			foreach (string Subfolder in Subfolders)
			{
				DirectoryReference PluginSubfolder = DirectoryReference.Combine(FullPluginPath, Subfolder);

				if (DirectoryReference.Exists(PluginSubfolder))
				{
					foreach (FileReference File in DirectoryReference.EnumerateFiles(PluginSubfolder, "*", SearchOption.AllDirectories))
					{
						FileReference DestinationPath = FileReference.Combine(PluginRemappedFolder, File.FullName.Substring(FullPluginPath.FullName.Length + 1));

						if (Verbose)
						{
							Log.Logger.LogInformation($"Stage LiveLinkHub: Staging plugin file {DestinationPath}");
						}

						CopyFile_NoExceptions(File.FullName, DestinationPath.FullName);
						CodeSign.SignMultipleIfEXEOrDLL(this, new List<String> { DestinationPath.FullName });
					}
				}
			}

			// Stage plugin manifest
			foreach (FileReference File in DirectoryReference.EnumerateFiles(FullPluginPath, "*.uplugin", SearchOption.AllDirectories))
			{
				FileReference DestinationPath = FileReference.Combine(PluginRemappedFolder, File.GetFileName());

				if (Verbose)
				{
					Log.Logger.LogInformation($"Stage LiveLinkHub: Staging manifest {DestinationPath}");
				}

				CopyFile_NoExceptions(File.FullName, DestinationPath.FullName);
				break;
			}
		}
	}
}
