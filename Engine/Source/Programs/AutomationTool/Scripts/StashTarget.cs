// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.IO;
using System.Linq;
using System.Text;
using System.Threading.Tasks;
using EpicGames.Core;
using UnrealBuildBase;
using UnrealBuildTool;
using Microsoft.Extensions.Logging;

namespace AutomationTool
{
	[Help("Copy all the binaries for a target into a different folder. Can be restored using the UnstashTarget command. Useful for A/B testing.")]
	[Help("-Name", "Name of the target")]
	[Help("-Platform", "Platform that the target was built for")]
	[Help("-Configuration", "Architecture that the target was built for")]
	[Help("-Architecture", "Architecture that the target was built for")]
	[Help("-Project", "Project file for the target")]
	[Help("-To", "Output directory to store the stashed binaries")]
	public class StashTarget : BuildCommand
	{
		public override void ExecuteBuild()
		{
			// Parse all the arguments
			string TargetName = ParseRequiredStringParam("Name");
			string PlatformName = ParseOptionalStringParam("Platform");
			UnrealTargetPlatform Platform;
			if (UnrealTargetPlatform.TryParse(PlatformName, out Platform))
			{
				Platform = HostPlatform.Current.HostEditorPlatform;
			}
			UnrealTargetConfiguration Configuration = ParseOptionalEnumParam<UnrealTargetConfiguration>("Configuration") ?? UnrealTargetConfiguration.Development;
			string ArchitectureString = ParseOptionalStringParam("Architecture");
			UnrealArchitectures Architecture = UnrealArchitectures.FromString(ArchitectureString, Platform);
			FileReference ProjectFile = ParseOptionalFileReferenceParam("Project");
			DirectoryReference ToDir = ParseRequiredDirectoryReferenceParam("To");

			// Read the receipt
			FileReference ReceiptFile = TargetReceipt.GetDefaultPath(DirectoryReference.FromFile(ProjectFile) ?? Unreal.EngineDirectory, TargetName, Platform, Configuration, Architecture);
			if(!FileReference.Exists(ReceiptFile))
			{
				throw new AutomationException("Unable to find '{0}'", ReceiptFile);
			}

			TargetReceipt Receipt = TargetReceipt.Read(ReceiptFile);

			// Enumerate all the files we want to move
			List<FileReference> FilesToMove = new List<FileReference>();
			FilesToMove.Add(ReceiptFile);
			FilesToMove.AddRange(Receipt.BuildProducts.Select(x => x.Path));

			// Move all the files to the output folder
			DirectoryReference.CreateDirectory(ToDir);
			CommandUtils.DeleteDirectoryContents(ToDir.FullName);
			foreach(FileReference SourceFile in FilesToMove)
			{
				FileReference TargetFile = FileReference.Combine(ToDir, SourceFile.MakeRelativeTo(Unreal.RootDirectory));
				Logger.LogInformation("Copying {SourceFile} to {TargetFile}", SourceFile, TargetFile);
				CommandUtils.CopyFile(SourceFile.FullName, TargetFile.FullName);
			}
		}
	}

	[Help("Copy all the binaries from a target back into the root directory. Use in combination with the StashTarget command.")]
	[Help("-From", "Directory to copy from")]
	public class UnstashTarget : BuildCommand
	{
		public override void ExecuteBuild()
		{
			// Parse the arguments
			DirectoryReference FromDir = ParseRequiredDirectoryReferenceParam("From");
			if(!DirectoryReference.Exists(FromDir))
			{
				throw new AutomationException("Source directory '{0}' does not exist", FromDir);
			}

			// Just copy all the files into place
			foreach(FileReference SourceFile in DirectoryReference.EnumerateFiles(FromDir, "*", SearchOption.AllDirectories))
			{
				FileReference TargetFile = FileReference.Combine(Unreal.RootDirectory, SourceFile.MakeRelativeTo(FromDir));
				Logger.LogInformation("Copying {SourceFile} to {TargetFile}", SourceFile, TargetFile);
				CopyFile(SourceFile.FullName, TargetFile.FullName);
			}
		}
	}
}
