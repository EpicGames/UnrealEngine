// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Concurrent;
using System.Collections.Generic;
using System.IO;
using System.Linq;
using System.Text;
using System.Text.RegularExpressions;
using System.Threading;
using System.Threading.Tasks;
using EpicGames.Core;

namespace AutomationTool
{
	[Help("Merge one or more remote DDC shares into a local share, taking files with the newest timestamps and keeping the size below a certain limit")]
	[Help("LocalDir=<Path>", "The local DDC directory to add/remove files from")]
	[Help("RemoteDir=<Path>", "The remote DDC directory to pull from. May be specified multiple times.")]
	[Help("MaxSize=<Size>", "Maximum size of the local DDC directory. TB/MB/GB/KB units are allowed.")]
	[Help("MaxDays=<Num>", "Only copies files with modified timestamps in the past number of days.")]
	[Help("TimeLimit=<Time>", "Maximum time to run for. h/m/s units are allowed.")]
	class SyncDDC : BuildCommand
	{
		/// <summary>
		/// Stores information about a file in the DDC
		/// </summary>
		class CacheFile
		{
			/// <summary>
			/// Relative path from the root of the DDC store
			/// </summary>
			public string RelativePath;

			/// <summary>
			/// Full path to the actual location on the store
			/// </summary>
			public string FullName;

			/// <summary>
			/// Size of the file
			/// </summary>
			public long Length;

			/// <summary>
			/// Last time the file was written to
			/// </summary>
			public DateTime LastWriteTimeUtc;

			/// <summary>
			/// Constructor
			/// </summary>
			/// <param name="RelativePath">Relative path for this file</param>
			/// <param name="FullName"></param>
			/// <param name="Length"></param>
			/// <param name="LastWriteTimeUtc"></param>
			public CacheFile(string RelativePath, string FullName, long Length, DateTime LastWriteTimeUtc)
			{
				this.RelativePath = RelativePath;
				this.FullName = FullName;
				this.Length = Length;
				this.LastWriteTimeUtc = LastWriteTimeUtc;
			}

			/// <summary>
			/// Format this object as a string for debugging
			/// </summary>
			/// <returns>Relative path to the file</returns>
			public override string ToString()
			{
				return RelativePath;
			}
		}

		/// <summary>
		/// Statistics for copying files
		/// </summary>
		class CopyStats
		{
			/// <summary>
			/// The number of files transferred
			/// </summary>
			public int NumFiles;

			/// <summary>
			/// The number of bytes transferred
			/// </summary>
			public long NumBytes;
		}

		/// <summary>
		/// Execute the command
		/// </summary>
		public override void ExecuteBuild()
		{
			Console.WriteLine();

			// Parse the command line arguments
			string LocalDirName = ParseParamValue("LocalDir", null);
			if(LocalDirName == null)
			{
				throw new AutomationException("Missing -LocalDir=... argument");
			}

			long MaxSize;
			if(!TryParseSize(ParseParamValue("MaxSize", "0mb"), out MaxSize))
			{
				throw new AutomationException("Invalid -MaxSize=... argument");
			}

			string[] RemoteDirNames = ParseParamValues("RemoteDir");
			if(RemoteDirNames.Length == 0)
			{
				throw new AutomationException("Missing -RemoteDir=... argument");
			}

			int MaxDays;
			if(!int.TryParse(ParseParamValue("MaxDays", "3"), out MaxDays))
			{
				throw new AutomationException("Invalid -MaxDays=... argument");
			}

			int TimeLimit;
			if(!TryParseTime(ParseParamValue("TimeLimit", "0m"), out TimeLimit))
			{
				throw new AutomationException("Invalid -TimeLimit=... argument");
			}

			bool bPreview = ParseParam("Preview");

			// Make sure the source directory exists
			List<DirectoryInfo> RemoteDirs = new List<DirectoryInfo>();
			foreach(string RemoteDirName in RemoteDirNames)
			{
				DirectoryInfo RemoteDir = new DirectoryInfo(RemoteDirName);
				if(!RemoteDir.Exists)
				{
					throw new AutomationException("Remote directory '{0}' does not exist", RemoteDirName);
				}
				RemoteDirs.Add(RemoteDir);
			}

			// Get the local directory
			DirectoryInfo LocalDir = new DirectoryInfo(LocalDirName);
			if(!LocalDir.Exists)
			{
				LocalDir.Create();
			}

			// Create all the base DDC directory names. These are three entries deep, each numbered 0-9.
			List<string> BasePathPrefixes = new List<string>();
			for(int IndexA = 0; IndexA <= 9; IndexA++)
			{
				for(int IndexB = 0; IndexB <= 9; IndexB++)
				{
					for(int IndexC = 0; IndexC <= 9; IndexC++)
					{
						BasePathPrefixes.Add(String.Format("{0}{3}{1}{3}{2}{3}", IndexA, IndexB, IndexC, Path.DirectorySeparatorChar));
					}
				}
			}

			// Find all the local files
			ConcurrentBag<CacheFile> LocalFiles = new ConcurrentBag<CacheFile>();
			Console.WriteLine("Enumerating local files from {0}", LocalDir.FullName);
			ForEach(BasePathPrefixes, (BasePath, Messages) => (() => EnumerateFiles(LocalDir, BasePath, LocalFiles)), "Enumerating files...");
			Console.WriteLine("Found {0} files, {1}mb.", LocalFiles.Count, LocalFiles.Sum(x => x.Length) / (1024 * 1024));
			Console.WriteLine();

			// Find all the remote files
			ConcurrentBag<CacheFile> RemoteFiles = new ConcurrentBag<CacheFile>();
			foreach(DirectoryInfo RemoteDir in RemoteDirs)
			{
				Console.WriteLine("Enumerating remote files from {0}", RemoteDir.FullName);
				ForEach(BasePathPrefixes, (BasePath, Messages) => (() => EnumerateFiles(RemoteDir, BasePath, RemoteFiles)), "Enumerating files...");
				Console.WriteLine("Found {0} files, {1}mb.", RemoteFiles.Count, RemoteFiles.Sum(x => x.Length) / (1024 * 1024));
				Console.WriteLine();
			}

			// Get the oldest file that we want to copy
			DateTime OldestLastWriteTimeUtc = DateTime.Now - TimeSpan.FromDays(MaxDays);

			// Build a lookup of remote files by name
			Dictionary<string, CacheFile> RelativePathToRemoteFile = new Dictionary<string, CacheFile>(StringComparer.InvariantCultureIgnoreCase);
			foreach(CacheFile RemoteFile in RemoteFiles)
			{
				if(RemoteFile.LastWriteTimeUtc > OldestLastWriteTimeUtc)
				{
					RelativePathToRemoteFile[RemoteFile.RelativePath] = RemoteFile;
				}
			}

			// Build a lookup of local files by name
			Dictionary<string, CacheFile> RelativePathToLocalFile = LocalFiles.ToDictionary(x => x.RelativePath, x => x, StringComparer.InvariantCultureIgnoreCase);

			// Build a list of target files that we want in the DDC
			long TotalSize = 0;
			Dictionary<string, CacheFile> RelativePathToTargetFile = new Dictionary<string, CacheFile>(StringComparer.InvariantCultureIgnoreCase);
			foreach(CacheFile TargetFile in Enumerable.Concat<CacheFile>(RelativePathToLocalFile.Values, RelativePathToRemoteFile.Values).OrderByDescending(x => x.LastWriteTimeUtc))
			{
				if(MaxSize > 0 && TotalSize + TargetFile.Length > MaxSize)
				{
					break;
				}
				if(!RelativePathToTargetFile.ContainsKey(TargetFile.RelativePath))
				{
					RelativePathToTargetFile.Add(TargetFile.RelativePath, TargetFile);
					TotalSize += TargetFile.Length;
				}
			}

			// Print measure of how coherent the cache is
			double CoherencyPct = RelativePathToTargetFile.Values.Count(x => RelativePathToLocalFile.ContainsKey(x.RelativePath)) * 100.0 / RelativePathToTargetFile.Count;
			Console.WriteLine("Cache is {0:0.0}% coherent with remote.", CoherencyPct);
			Console.WriteLine();

			// Remove any outdated files
			List<CacheFile> FilesToRemove = RelativePathToLocalFile.Values.Where(x => !RelativePathToTargetFile.ContainsKey(x.RelativePath)).ToList();
			if(bPreview)
			{
				Console.WriteLine("Sync would remove {0} files ({1}mb)", FilesToRemove.Count, FilesToRemove.Sum(x => x.Length) / (1024 * 1024));
			}
			else if(FilesToRemove.Count > 0)
			{
				Console.WriteLine("Deleting {0} files ({1}mb)...", FilesToRemove.Count, FilesToRemove.Sum(x => x.Length) / (1024 * 1024));
				ForEach(FilesToRemove, (File, Messages) => (() => RemoveFile(LocalDir, File.RelativePath, Messages)), "Deleting files");
				Console.WriteLine();
			}

			// Add any new files
			List<CacheFile> FilesToAdd = RelativePathToTargetFile.Values.Where(x => !RelativePathToLocalFile.ContainsKey(x.RelativePath)).ToList();
			if(bPreview)
			{
				Console.WriteLine("Sync would add {0} files ({1}mb)", FilesToAdd.Count, FilesToAdd.Sum(x => x.Length) / (1024 * 1024));
			}
			else if(FilesToAdd.Count > 0)
			{
				Console.WriteLine("Copying {0} files ({1}mb)...", FilesToAdd.Count, FilesToAdd.Sum(x => x.Length) / (1024 * 1024));

				CancellationTokenSource CancellationTokenSource = new CancellationTokenSource();
				if (TimeLimit > 0)
				{
					CancellationTokenSource.CancelAfter(TimeLimit * 1000);
				}

				DateTime StartTime = DateTime.UtcNow;
				CopyStats Stats = new CopyStats();
				ForEach(FilesToAdd, (File, Messages) => (() => CopyFile(File, LocalDir, Messages, Stats)), "Copying files...", CancellationTokenSource.Token);

				double TotalSizeMb = Stats.NumBytes / (1024.0 * 1024.0);
				Console.WriteLine("Copied {0} files totalling {1:0.0}mb ({2:0.00}mb/s).", Stats.NumFiles, TotalSizeMb, TotalSizeMb / (DateTime.UtcNow - StartTime).TotalSeconds);

				double FinalCoherencyPct = (RelativePathToTargetFile.Values.Count(x => RelativePathToLocalFile.ContainsKey(x.RelativePath)) + Stats.NumFiles) * 100.0 / RelativePathToTargetFile.Count;
				Console.WriteLine();
				Console.WriteLine("Final cache is {0:0.0}% coherent with remote.", FinalCoherencyPct);

				if (CancellationTokenSource.IsCancellationRequested)
				{
					Console.WriteLine("Halting due to expired time limit.");
				}
			}

			Console.WriteLine();
		}

		/// <summary>
		/// Execute a command for each item in the given list, printing progress messages and queued up error strings
		/// </summary>
		/// <typeparam name="T">Type of the item to process</typeparam>
		/// <param name="Items">List of items</param>
		/// <param name="CreateAction">Delegate which will create an action to execute for an item</param>
		/// <param name="Message">Prefix to add to progress messages</param>
		/// <param name="CancellationToken"></param>
		static void ForEach<T>(IList<T> Items, Func<T, ConcurrentQueue<string>, Action> CreateAction, string Message, CancellationToken? CancellationToken = null)
		{
			using (ThreadPoolWorkQueue Queue = new ThreadPoolWorkQueue())
			{
				ConcurrentQueue<string> Warnings = new ConcurrentQueue<string>();
				foreach(T Item in Items)
				{
					Action ThisAction = CreateAction(Item, Warnings);
					if(CancellationToken.HasValue)
					{
						Action OriginalAction = ThisAction;
						CancellationToken Token = CancellationToken.Value;
						ThisAction = () => { if (!Token.IsCancellationRequested) OriginalAction(); };
					}
					Queue.Enqueue(ThisAction);
				}

				DateTime StartTime = DateTime.UtcNow;
				DateTime NextUpdateTime = DateTime.UtcNow + TimeSpan.FromSeconds(0.5);
				for(;;)
				{
					bool bResult = Queue.Wait(2000);

					if (!CancellationToken.HasValue || !CancellationToken.Value.IsCancellationRequested)
					{
						DateTime CurrentTime = DateTime.UtcNow;
						if (CurrentTime >= NextUpdateTime || bResult)
						{
							int NumRemaining = Queue.NumRemaining;
							Console.WriteLine("{0} ({1}/{2}; {3}%)", Message, Items.Count - NumRemaining, Items.Count, (int)((Items.Count - NumRemaining) * 100.0f / Items.Count));
							NextUpdateTime = CurrentTime + TimeSpan.FromSeconds(10.0);
						}
					}

					if(bResult)
					{
						break;
					}
				}

				List<string> WarningsList = new List<string>(Warnings);
				if(WarningsList.Count > 0)
				{
					const int MaxWarnings = 50;
					if (WarningsList.Count > MaxWarnings)
					{
						Console.WriteLine("{0} warnings, showing first {1}:", WarningsList.Count, MaxWarnings);
					}
					else
					{
						Console.WriteLine("{0} {1}:", WarningsList.Count, (WarningsList.Count == 1) ? "warning" : "warnings");
					}
					for(int Idx = 0; Idx < WarningsList.Count && Idx < MaxWarnings; Idx++)
					{
						Console.WriteLine("  {0}", WarningsList[Idx]);
					}
				}
			}
		}

		/// <summary>
		/// Enumerates all the files in a given directory, and adds them to a bag
		/// </summary>
		/// <param name="BaseDir">Base directory to enumerate</param>
		/// <param name="PathPrefix">Path from the base directory containing files to enumerate</param>
		/// <param name="Files">Collection of found files</param>
		static void EnumerateFiles(DirectoryInfo BaseDir, string PathPrefix, ConcurrentBag<CacheFile> Files)
		{
			DirectoryInfo SearchDir = new DirectoryInfo(Path.Combine(BaseDir.FullName, PathPrefix));
			if(SearchDir.Exists)
			{
				foreach(FileInfo File in SearchDir.EnumerateFiles("*.udd"))
				{
					CacheFile CacheFile = new CacheFile(PathPrefix + File.Name, File.FullName, File.Length, File.LastAccessTimeUtc);
					Files.Add(CacheFile);
				}
			}
		}

		/// <summary>
		/// Copies a file from one DDC to another
		/// </summary>
		/// <param name="SourceFile">File to copy</param>
		/// <param name="TargetDir">Target DDC directory</param>
		/// <param name="Messages">Queue to receieve error messages</param>
		/// <param name="Stats">Stats for the files copied</param>
		static void CopyFile(CacheFile SourceFile, DirectoryInfo TargetDir, ConcurrentQueue<string> Messages, CopyStats Stats)
		{
			int NumRetries = 0;
			for (;;)
			{
				// Try to copy the file, and return if we succeed
				string Message;
				if (TryCopyFile(SourceFile, TargetDir, out Message))
				{
					Interlocked.Increment(ref Stats.NumFiles);
					Interlocked.Add(ref Stats.NumBytes, SourceFile.Length);
					break;
				}

				// Increment the number of retries
				NumRetries++;

				// Give up after retrying 5 times, and report the last error
				if (NumRetries >= 3)
				{
					// Check that the source file still actually exists. If it doesn't, we'll ignore the errors.
					try
					{
						if (!File.Exists(SourceFile.FullName))
						{
							Interlocked.Increment(ref Stats.NumFiles);
							break;
						}
					}
					catch
					{
					}

					// Otherwise queue up the error message
					Messages.Enqueue(Message);
					break;
				}
			}
		}

		/// <summary>
		/// Tries to copy a file from one DDC to another
		/// </summary>
		/// <param name="SourceFile">File to copy</param>
		/// <param name="TargetDir">Target DDC directory</param>
		/// <param name="Message">Queue to receieve error messages</param>
		/// <returns>True if the file was copied successfully, false otherwise</returns>
		static bool TryCopyFile(CacheFile SourceFile, DirectoryInfo TargetDir, out string Message)
		{
			string TargetFileName = Path.Combine(TargetDir.FullName, SourceFile.RelativePath);
			string IntermediateFileName = TargetFileName + ".temp";
			string IntermediateDir = Path.GetDirectoryName(IntermediateFileName);

			// Create the target directory
			try
			{
				Directory.CreateDirectory(IntermediateDir);
			}
			catch (Exception Ex)
			{
				Message = String.Format("Unable to create {0}: {1}", IntermediateDir, Ex.Message.TrimEnd());
				return false;
			}

			// Copy the file from the remote location to the intermediate file
			try
			{
				File.Copy(SourceFile.FullName, IntermediateFileName);
			}
			catch (Exception Ex)
			{
				Message = String.Format("Unable to copy {0} to {1}: {2}", SourceFile.FullName, IntermediateFileName, Ex.Message.TrimEnd());
				return false;
			}

			// Rename the file into place as a transactional operation
			try
			{
				File.Move(IntermediateFileName, TargetFileName);
			}
			catch (Exception Ex)
			{
				Message = String.Format("Unable to rename {0} to {1}: {2}", IntermediateFileName, TargetFileName, Ex.Message.TrimEnd());
				return false;
			}

			Message = null;
			return true;
		}

		/// <summary>
		/// Removes a file from the DDC
		/// </summary>
		/// <param name="BaseDir">Base directory for the DDC</param>
		/// <param name="RelativePath">Relative path for the file to remove</param>
		/// <param name="Messages">Queue to receieve error messages</param>
		static void RemoveFile(DirectoryInfo BaseDir, string RelativePath, ConcurrentQueue<string> Messages)
		{
			string FileName = Path.Combine(BaseDir.FullName, RelativePath);
			try
			{
				File.Delete(FileName);
			}
			catch(Exception Ex)
			{
				Messages.Enqueue(String.Format("Unable to delete {0}: {1}", FileName, Ex.Message.TrimEnd()));
			}
		}

		/// <summary>
		/// Parses a size argument as number of bytes, which may be specified in tb/gb/mb/kb units.
		/// </summary>
		/// <param name="Text">Text to parse</param>
		/// <param name="Size">Receives the parsed argument, in bytes</param>
		/// <returns>True if a size could be parsed</returns>
		static bool TryParseSize(string Text, out long Size)
		{
			Match Match = Regex.Match(Text, "^(\\d+)(tb|gb|mb|kb)$");
			if(!Match.Success)
			{
				Size = 0;
				return false;
			}

			Size = long.Parse(Match.Groups[1].Value);
			switch(Match.Groups[2].Value.ToLowerInvariant())
			{
				case "tb":
					Size *= 1024 * 1024 * 1024 * 1024L;
					break;
				case "gb":
					Size *= 1024 * 1024 * 1024L;
					break;
				case "mb":
					Size *= 1024 * 1024L;
					break;
				case "kb":
					Size *= 1024L;
					break;
			}
			return true;	
		}

		/// <summary>
		/// Parses a time argument as number of seconds, which may be specified in h/m/s units.
		/// </summary>
		/// <param name="Text">Text to parse</param>
		/// <param name="TimeSeconds">Receives the parsed argument, in seconds</param>
		/// <returns>True if a size could be parsed</returns>
		static bool TryParseTime(string Text, out int TimeSeconds)
		{
			Match Match = Regex.Match(Text, "^(\\d+)(h|m|s)$");
			if (!Match.Success)
			{
				TimeSeconds = 0;
				return false;
			}

			TimeSeconds = int.Parse(Match.Groups[1].Value);
			switch (Match.Groups[2].Value.ToLowerInvariant())
			{
				case "h":
					TimeSeconds *= 60 * 60;
					break;
				case "m":
					TimeSeconds *= 60;
					break;
			}
			return true;
		}
	}
}
