// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Linq;
using System.Text;
using System.Threading.Tasks;
using AutomationTool;
using UnrealBuildBase;

namespace AutomationTool
{
	[Help("Creates a temporary client and syncs a path from Perforce.")]
	[RequireP4]
	[DoesNotNeedP4CL]
	class SyncDepotPath : BuildCommand
	{
		public override void ExecuteBuild()
		{
			// Parse the parameters
			string DepotPath = ParseParamValue("DepotPath");
			if (DepotPath == null)
			{
				throw new AutomationException("Missing -DepotPath=... parameter");
			}

			string OutputDir = ParseParamValue("OutputDir");
			if (OutputDir == null)
			{
				throw new AutomationException("Missing -OutputDir=... parameter");
			}

			// Create a temporary client to sync down the folder
			string ClientName = String.Format("{0}_{1}_SyncDepotPath_Temp", P4Env.User, Unreal.MachineName);

			List<KeyValuePair<string, string>> RequiredView = new List<KeyValuePair<string, string>>();
			RequiredView.Add(new KeyValuePair<string, string>(DepotPath, "/..."));

			if(P4.DoesClientExist(ClientName))
			{
				P4.DeleteClient(ClientName);
			}

			P4ClientInfo Client = new P4ClientInfo();
			Client.Owner = P4Env.User;
			Client.Host = Unreal.MachineName;
			Client.RootPath = OutputDir;
			Client.Name = ClientName;
			Client.View = RequiredView;
			Client.Stream = null;
			Client.Options = P4ClientOption.NoAllWrite | P4ClientOption.Clobber | P4ClientOption.NoCompress | P4ClientOption.Unlocked | P4ClientOption.NoModTime | P4ClientOption.RmDir;
			Client.LineEnd = P4LineEnd.Local;
			P4.CreateClient(Client);

			// Sync the workspace, then delete the client
			try
			{
				P4Connection Perforce = new P4Connection(P4Env.User, ClientName);
				Perforce.Sync("-f //...");
			}
			finally
			{
				P4.DeleteClient(ClientName);
			}
		}
	}
}
