// Copyright Epic Games, Inc. All Rights Reserved.

using AutomationTool;
using System;
using System.Collections.Generic;
using System.Linq;
using System.IO;
using Microsoft.Extensions.Logging;
using EpicGames.Core;
using UnrealBuildBase;
using Amazon.S3.Model;


namespace BuildScripts.Automation
{
	/**
	 * Setup of the two servers:
	 * 
	 * Main (source) Server - the server where you do your main UE work
	 *   Clients:
	 *     CleanOutgoing - this should be a client that you NEVER work from, it is used solely to sync from p4, to a clean directory
	 * Target Server - the server that wants to mirror/sync changes from main UE
	 *   Streams/Clients:
	 *     Main - a normal Main stream, where you do work on the target server
	 *     Staging - a development stream, parent is Main, where conflicts between Source Server and Target Server work are managed. Do no work here other than resolving conflicts.
	 *     Incoming - a release stream, parent is Staging, with a client that maps to EXACT SAME directory on disk as CleanOutgoing on Source Server. Do no work here, not even a little.
	 */
	[RequireP4]
	[DoesNotNeedP4CL]
	[Help("Syncs a clean clientspec (should NOT be the client you are running this out of), and mirrors it into another p4 server via 3 streams. See the SyncPerforceServers.cs for details")]
	[Help("SourceClient=<client>", "p4 client on the source server (which is the active server that normal UAT p4 scripts would use")]
	[Help("Server=<server:port>", "p4 server for the target (in the format server:port). There needs to be a client mapped to same directory as SourceClient is mapped to")]
	[Help("User=<user>", "(optional) p4 username on the target server")]
	[Help("SyncList=<fileset+fileset+...>", "(optional) a + separated list of p4 file paths to sync when getting latest from source stream (//Server/UE5/Engine/...+//Server/UE5/Templates/...")]
	[Help("SyncListFile=<path>", "(optional) a text file that contains a list of p4 paths to sync, one per line. It can contain - prefixed lines to remove subpaths from being copied to target server")]
	[Help("Submit", "REQUIRED for this script to work (it's actually processed by different code, so it must be specified)")]
	[Help("HideSpew", "If specified, this command will not list the file outputs in each command")]
	[Help("SkipSync", "If specified, the sync from SourceClient will not be preformed")]
	[Help("SkipReconcile", "If specified, the reconcile into the target server's incoming stream will not be performed")]
	[Help("SkipMergeToStaging", "If specified, the merge from Incoming and Main streams into the Staging stream will not be performed")]
	[Help("SkipCopyToMain", "If specified, the final copy from Staging to Main will not be performed")]
	class SyncPerforceServers : BuildCommand
	{
		protected P4ClientInfo GetClientInfoForParent(P4Connection Connection, string User, string StreamName, string ClientDesc)
		{
			// find parent stream for the incoming client's stream
			P4StreamRecord CientStream = Connection.Streams(StreamName).FirstOrDefault();
			if (CientStream == null)
			{
				throw new AutomationException($"Unable to find a stream info for {StreamName}");
			}
			string ParentStreamName = CientStream.Parent;

			// now find a client for it
			Logger.LogInformation($"Looking for an {ClientDesc} client using parent stream {ParentStreamName}...");
			P4ClientInfo[] ParentClients = Connection.GetClientsForUser(UserName: User, AllowedStream: ParentStreamName);
			if (ParentClients.Length == 0)
			{
				throw new AutomationException($"Unable to find a {ClientDesc} client for stream {ParentStreamName}");
			}

			return ParentClients[0];
		}

		public override void ExecuteBuild()
		{
			string SourceClientName = ParseRequiredStringParam("SourceClient");
			string Server = ParseRequiredStringParam("Server");
			string FilterList = ParseOptionalStringParam("SyncList");
			string FilterFile = ParseOptionalStringParam("SyncListFile");
			string User = ParseOptionalStringParam("User") ?? P4Env.User;

			bool bHideSpew = ParseParam("HideSpew");
			bool bSkipSync = ParseParam("SkipSync");
			bool bSkipReconcile = ParseParam("SkipReconcile");
			bool bSkipMergeToStaging = ParseParam("SkipMergeToStaging");
			bool bSkipCopyToMain = ParseParam("SkipCopyToMain");

			if (!CommandUtils.AllowSubmit)
			{
				Logger.LogError("Submitting is not allowed, which is required for this script. Run again with -submit, which is currently the only way to allow submitting via P4.");
				return;
			}

			#region Source setup

			// get source client info
			Logger.LogInformation($"Looking up client {SourceClientName} for user {P4Env.User}...");
			P4ClientInfo SourceClient = P4.GetClientInfo(SourceClientName);
			if (SourceClient == null)
			{
				throw new AutomationException($"Unable to find client {SourceClientName} on source server.");
			}
			string CleanDir = SourceClient.RootPath;

			// make sure we aren't using the _current_ engine as source, as it can never be clean if we are running in it
			if (Unreal.EngineDirectory.IsUnderDirectory(new DirectoryReference(CleanDir)))
			{
				throw new AutomationException("You are running this from a UAT compiled in your clean directory! That means you have a non-clean directory to work from. You need to use another stream's UAT instance");
			}

			#endregion

			#region Incoming client setup

			Logger.LogInformation($"Looking for an incoming client mapped to '{CleanDir}' on target server '{Server}'...");
			
			// find an incoming client on target server that maps to the clean directory we will sync to
			P4Connection P4NoClient = new P4Connection(User, null, Server);
			// GetClientsForUser expects something UNDER the root, not the root dir itself
			P4ClientInfo[] IncomingClients = P4NoClient.GetClientsForUser(UserName: User, PathUnderClientRoot: System.IO.Path.Combine(CleanDir, "Engine"));
			if (IncomingClients.Length == 0)
			{
				throw new AutomationException($"Unable to find an incoming client on target server that is mapped to {CleanDir}");
			}
			if (IncomingClients.Length > 1)
			{
				throw new AutomationException($"Multiple clients are mapped to '{CleanDir}' on the target server, unable to determine the client to use. [{string.Join(", ", IncomingClients.Select(x => x.Name))}");
			}

			string IncomingClientName = IncomingClients[0].Name;
			string IncomingStreamName = IncomingClients[0].Stream;

			#endregion

			#region Staging client setup

			P4ClientInfo StagingClient = GetClientInfoForParent(P4NoClient, User, IncomingStreamName, "staging");
			string StagingStreamName = StagingClient.Stream;

			#endregion

			#region Main client setup

			P4ClientInfo MainClient = GetClientInfoForParent(P4NoClient, User, StagingStreamName, "main");
			string MainStreamName = MainClient.Stream;

			#endregion

			#region Filters setup

			HashSet<string> Filters = new();
			HashSet<string> Reverts = new();
			if (!string.IsNullOrEmpty(FilterList))
			{
				Filters.UnionWith(FilterList.Split('+'));
				Filters.Add($"{SourceClient.Stream}/*");
			}
			if (!string.IsNullOrEmpty(FilterFile))
			{
				string[] Lines = File.ReadAllLines(FilterFile);
				Filters.UnionWith(Lines.Where(x => !x.StartsWith("-")));
				Filters.Add($"{SourceClient.Stream}/*");
				Reverts.UnionWith(Lines.Where(x => x.StartsWith("-")).Select(x => x.Substring(1)));
			}
			if (Filters.Count == 0)
			{
				Filters.Add($"{SourceClient.Stream}/...");
			}

			// strip blanks
			Filters.RemoveWhere(x => string.IsNullOrWhiteSpace(x));

			#endregion

			#region Verify

			string SkipSync = bSkipSync ? "<skipped> " : "";
			string SkipReconcile = bSkipReconcile ? "<skipped> " : "";
			string SkipStaging = bSkipMergeToStaging ? "<skipped> " : "";
			string SkipMain = bSkipCopyToMain ? "<skipped> " : "";

			Logger.LogInformation("");
			Logger.LogInformation("");
			Logger.LogInformation($"Ready to begin with this flow:");
			Logger.LogInformation($"  {SkipSync}[SYNC]      Stream '{SourceClient.Stream}'");
			Logger.LogInformation($"  {SkipReconcile}[RECONCILE] Into Stream '{IncomingStreamName}'");
			Logger.LogInformation($"  {SkipStaging}[MERGE]     Stream '{MainStreamName}' into '{StagingStreamName}'");
			Logger.LogInformation($"  {SkipStaging}[MERGE]     Stream '{IncomingStreamName}' into '{StagingStreamName}'");
			Logger.LogInformation($"  {SkipMain}[COPY]      Stream '{StagingStreamName}' onto '{MainStreamName}'");
			Logger.LogInformation($"Clients to use are:");
			if (!bSkipSync)
			{
				Logger.LogInformation($"  {SourceClientName} -> {SourceClient.Stream}");
			}
			if (!bSkipReconcile || !bSkipMergeToStaging)
			{
				Logger.LogInformation($"  {IncomingClientName} -> {IncomingStreamName}");
			}
			if (!bSkipMergeToStaging || !bSkipCopyToMain)
			{
				Logger.LogInformation($"  {StagingClient.Name} -> {StagingStreamName}");
				Logger.LogInformation($"  {MainClient.Name} -> {MainStreamName}");
			}
			if (!bSkipSync)
			{
				Logger.LogInformation($"Files to sync:");
				foreach (string Filter in Filters)
				{
					Logger.LogInformation($"  {Filter}");
				}

				if (Reverts.Count > 0)
				{
					Logger.LogInformation($"Files to skip (technically, it will sync then 'remove from workspace'):");
					foreach (string Revert in Reverts)
					{
						Logger.LogInformation($"  {Revert}");
					}
				}
			}
			Logger.LogInformation("");
			if (!bSkipMergeToStaging)
			{
				Logger.LogInformation($"NOTE: The MERGE from {IncomingStreamName} to {StagingStreamName} is expected/likely to require manual resolving. This script will pause if needed, allowing you to resolve in p4v manually.");
			}

			Logger.LogInformation($"If everything looks good, press Enter to start! (Or Control-C to cancel)");
			Console.ReadLine();

			// setup various client connections
			P4Connection P4SourceClient = new P4Connection(null, SourceClientName);
			P4Connection P4IncomingClient = new P4Connection(User, IncomingClientName, Server);
			P4Connection P4StagingClient = new P4Connection(User, StagingClient.Name, Server);
			P4Connection P4MainClient = new P4Connection(User, MainClient.Name, Server);

			#endregion

			#region Sync clean files

			if (!bSkipSync)
			{
				foreach (string Filter in Filters)
				{
					Logger.LogInformation($"Syncing {Filter}");
					P4SourceClient.Sync(Filter, AllowSpew:!bHideSpew);
				}
				foreach (string Revert in Reverts)
				{
					Logger.LogInformation($"Removing {Revert}");
					P4SourceClient.Sync(Revert + "@0", AllowSpew: !bHideSpew);
				}
			}

			#endregion

			#region Reconcile into Incoming stream

			if (!bSkipReconcile)
			{
				// tell the incoming workspace that what is on disc is up to date - this is a lie if other people are also mirroring servers, because someone else may have updated this stream
				// without this clientspec knowing - however, because the sync from the source server (done above) is ground truth, we don't need to actually pull anything down. reconciling
				// will fail, however, if this client is out of date
				Logger.LogInformation($"Syncrhonizing server state...");
				P4IncomingClient.Sync($"-k {IncomingStreamName}/...", AllowSpew: !bHideSpew);

				Logger.LogInformation($"Reconciling changed files...");

				// reconcile files on the incoming stream
				int CL = P4IncomingClient.CreateChange(Description: $"Reconciling changes on target incoming stream from {P4Env.ServerAndPort} / {SourceClient.Stream} to {Server} / {IncomingStreamName}");
				P4IncomingClient.Reconcile(CL, $"-m {IncomingStreamName}/...", AllowSpew: !bHideSpew);

				// nothing should fail here, as it's an incoming only stream, so force the issue
				P4IncomingClient.Submit(CL, true);
			}

			#endregion

			#region Merge to Staging

			if (!bSkipMergeToStaging)
			{
				Logger.LogInformation($"Syncing Staging client {StagingStreamName}/...");
				P4StagingClient.Sync($"{IncomingStreamName}/...", AllowSpew: !bHideSpew);

				// first we merge down from Main into Staging.
				int CL = P4StagingClient.CreateChange(Description: $"Merging {MainStreamName} to {StagingStreamName}", AllowSpew: !bHideSpew);
				P4StagingClient.P4($"integrate -c {CL} {MainStreamName}/... {StagingStreamName}/...", AllowSpew: !bHideSpew);
				if (P4StagingClient.P4($"resolve -am -c {CL}", AllowSpew: !bHideSpew).ExitCode != 0)
				{
					Logger.LogError($"Failed to resolve the merge from {MainStreamName} to {StagingStreamName}. This is unexpected, and indicates that the staging directory was dirtied outside of this process. Resolve everything in changelist {CL}, then press enter to continue. \nIf you kill this script, you can run this script again with \"-skipsync -skipreconcile\".");
					Console.ReadLine();
				}

				// Nothing should have gone into Staging since the last time we did this, so this _should_ not fail, but if it does, alert the user and have them try again
				try
				{
					P4StagingClient.Submit(CL);
				}
				catch (P4Exception)
				{
					Logger.LogError($"Submit of changelist {CL} failed for unknown reason. Submit manually, then press enter to continue. \nIf you kill this script, you can run this script again with \"-skipsync -skipreconcile\" to get to this step faster.");
					Console.ReadLine();
				}

				CL = P4StagingClient.CreateChange(Description: $"Merging {IncomingStreamName} to {StagingStreamName}", AllowSpew: !bHideSpew);
				P4StagingClient.P4($"integrate -c {CL} {IncomingStreamName}/... {StagingStreamName}/...", AllowSpew: !bHideSpew);
				if (P4StagingClient.P4($"resolve -am -c {CL}", AllowSpew: !bHideSpew).ExitCode != 0)
				{
					Logger.LogError($"Manual resolves are needed when merging {IncomingStreamName} to {StagingStreamName}. This is expected. Resolve changelist {CL}, then press enter to continue. \nIf you kill this script, you can run this script again with \"-skipsync -skipreconcile\" to get to this step faster.");
					Console.ReadLine();
				}

				try
				{
					P4StagingClient.Submit(CL);
				}
				catch (P4Exception)
				{
					Logger.LogError($"Submit of changelist {CL} failed for unknown reason. Submit manually, then press enter to continue. \nIf you kill this script, you can run this script again with \"-skipsync -skipreconcile\" to get to this step faster.");
					Console.ReadLine();
				}
			}

			#endregion

			#region Copy to Main

			if (!bSkipCopyToMain)
			{
				int CL = P4MainClient.CreateChange(Description: $"Copying {StagingStreamName} to parent {MainStreamName}", AllowSpew: !bHideSpew);
				P4MainClient.P4($"copy -c {CL} -S {StagingStreamName}", AllowSpew: !bHideSpew);

				try
				{
					P4MainClient.Submit(CL);
				}
				catch (P4Exception)
				{
					Logger.LogError($"Submit of changelist {CL} failed for unknown reason. Submit manually, then press enter to continue. \nIf you kill this script, you can run this script again with \"-skipsync -skipreconcile -skipmergetostaging\" to get to this step faster.");
					Console.ReadLine();
				}
			}

			#endregion

		}
	}
}
