<Project xmlns="http://schemas.microsoft.com/developer/msbuild/2003">
  <ItemGroup>
    <ModulesToBuild Include="$(Modules)" />
  </ItemGroup>
  <PropertyGroup>
    <BuildInParallel Condition="'$(BuildInParallel)' == ''">true</BuildInParallel>
    <Configuration Condition="'$(Configuration)' == ''">Development</Configuration>
    <Verbosity>quiet</Verbosity>
    <Platform>AnyCPU</Platform>
    <TreatWarningsAsErrors>false</TreatWarningsAsErrors>
    <BuildProjectReferences>true</BuildProjectReferences>
	<NoWarn>1591,612,618,672</NoWarn>
  </PropertyGroup>
  <Target Name="Build">
    <Error 
	  Text="Unable to build Project %(ModulesToBuild.FullPath). Not a valid .csproj file."
      Condition="'%(ModulesToBuild.Extension)'!='.csproj'" />
	<Error 
	  Text="Unable to build Project %(ModulesToBuild.FullPath). Project file not found."
      Condition="!Exists('%(ModulesToBuild.FullPath)')" />
	<MSBuild
      Projects="@(ModulesToBuild)"
      BuildInParallel="$(BuildInParallel)"
	  Targets="Build"
      Properties="NoLogo=true;Verbosity=$(Verbosity);Configuration=$(Configuration);Platform=$(Platform);TreatWarningsAsErrors=$(TreatWarningsAsErrors);NoWarn=$(NoWarn);BuildProjectReferences=$(BuildProjectReferences)"/>
  </Target>
</Project>