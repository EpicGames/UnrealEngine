// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Linq;
using System.Text;
using AutomationTool;
using UnrealBuildTool;
using Microsoft.Extensions.Logging;

/// <summary>
/// Common UEBuild utilities
/// </summary>
public class UnrealBuildUtils : CommandUtils
{
	/// <summary>
	/// Builds BuildPatchTool for the specified platform.
	/// </summary>
	/// <param name="Command"></param>
	/// <param name="InPlatform"></param>
	public static void BuildBuildPatchTool(BuildCommand Command, UnrealBuildTool.UnrealTargetPlatform InPlatform)
	{
		BuildProduct(Command, new UnrealBuild.BuildTarget()
			{
				UprojectPath = null,
				TargetName = "BuildPatchTool",
				Platform = InPlatform,
				Config = UnrealBuildTool.UnrealTargetConfiguration.Shipping,
			});
	}

	private static void BuildProduct(BuildCommand Command, UnrealBuild.BuildTarget Target)
	{
		if (Target == null)
		{
			throw new AutomationException("Target is required when calling UnrealBuildUtils.BuildProduct");
		}

		Logger.LogInformation("Building {Arg0}", Target.TargetName);

		if (Command == null)
		{
			Command = new UnrealBuildUtilDummyBuildCommand();
		}

		var UnrealBuild = new UnrealBuild(Command);

		var Agenda = new UnrealBuild.BuildAgenda();
		Agenda.Targets.Add(Target);

		UnrealBuild.Build(Agenda, InDeleteBuildProducts: true, InUpdateVersionFiles: true);
		UnrealBuild.CheckBuildProducts(UnrealBuild.BuildProductFiles);
	}

	class UnrealBuildUtilDummyBuildCommand : BuildCommand
	{
		public override void ExecuteBuild()
		{
			// noop
		}
	}
}
