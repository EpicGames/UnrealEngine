// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Linq;
using System.Text;
using AutomationTool;
using UnrealBuildTool;
using Microsoft.Extensions.Logging;
using System.Diagnostics;
using System.IO;
using EpicGames.Core;

/// <summary>
/// Unsigned files violation check
/// </summary>
public class UnsignedFilesViolationCheck : BuildCommand
{
	/// <summary>
	/// Runs powershell command.
	/// </summary>
	/// <param name="Command"></param>
	/// <param name="WorkingDirectory"></param>
	public static void RunPowershellCommand(string Command, string WorkingDirectory)
	{
		string Powershell = "powershell.exe";
		string Arguments = $"-NoProfile -ExecutionPolicy Bypass -Command \"& {{Set-Location '{WorkingDirectory}'; {Command};}}\""; 
		int StatusCode = 0;
		RunAndLog(CmdEnv, Powershell, Arguments, null, StatusCode);
		if (StatusCode != 0)
		{
			Logger.LogError("Command returned status code: {StatusCode}", StatusCode);
		}
	}

	/// <summary>
	/// Checks expected word in file.
	/// </summary>
	/// <param name="OutputFile"></param>
	/// <param name="ExpectedWord"></param>
	static (int, List<string>) CheckWordInOutput(string OutputFile, string ExpectedWord)
	{
		string File = ReadAllText(OutputFile);
		Logger.LogInformation("Command output: {OutputFile}", OutputFile); 
		int WordCount = 0;
		List<string> ViolationFileNames = new List<string>();
		string[] words = File.Split(new[] { ' ', '\t', '\n', '\r' }, StringSplitOptions.RemoveEmptyEntries);

		for (int i = 0; i < words.Length; i++)
		{
			if (words[i] == ExpectedWord)
			{
				WordCount++;
				if (i + 1 < words.Length)
				{
					ViolationFileNames.Add(words[i + 1]);
				}
			}
		}
		return (WordCount, ViolationFileNames);
	}

	public override void ExecuteBuild()
	{
		string InstalledRootPath = ParseParamValue("Root");
		if (!Directory.Exists(InstalledRootPath))
		{
			Logger.LogError(InstalledRootPath);
			Environment.Exit(1);
		}

		string CommandExeSign = $"Get-ChildItem -Recurse *.exe | ForEach-object {{Get-AuthenticodeSignature $_}} | Out-File -FilePath {CmdEnv.LogFolder}\\exe_sign.txt";
		string CommandDllSign = $"Get-ChildItem -Recurse *.dll | ForEach-object {{Get-AuthenticodeSignature $_}} | Out-File -FilePath {CmdEnv.LogFolder}\\dll_sign.txt";
		string WordToCheck = "NotSigned";
		string ExeSignOutput = CombinePaths(CmdEnv.LogFolder, "exe_sign.txt");
		string DllSignOutput = CombinePaths(CmdEnv.LogFolder, "dll_sign.txt");

		RunPowershellCommand(CommandExeSign, InstalledRootPath);

		(int WordCount, List<string> ViolationFileNames) = CheckWordInOutput(ExeSignOutput, WordToCheck);
		if (WordCount != 0)
		{
			Logger.LogError("There are '{WordCount}' violations for exe files. The list of distributed files: '{Files}'", WordCount, string.Join(", ", ViolationFileNames));
		}
		else
		{
			Logger.LogInformation("Release of the Engine is not distributing unsigned exe files");
		}

		RunPowershellCommand(CommandDllSign, InstalledRootPath);

		(WordCount, ViolationFileNames) = CheckWordInOutput(DllSignOutput, WordToCheck);
		if (WordCount != 0)
		{
			Logger.LogError("There are '{WordCount}' violations for dll files. The list of distributed files: '{Files}'", WordCount, string.Join(", ", ViolationFileNames));
		}
		else
		{
			Logger.LogInformation("Release of the Engine is not distributing unsigned dll files");
		}

	}
}
