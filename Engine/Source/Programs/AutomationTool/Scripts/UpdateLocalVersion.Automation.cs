// Copyright Epic Games, Inc. All Rights Reserved.
using System;
using System.Collections.Generic;
using System.IO;
using System.Threading;
using System.Reflection;
using AutomationTool;
using UnrealBuildTool;

[Help("Updates your local versions based on your P4 sync")]
[Help("CL", "Overrides the automatically disovered changelist number with the specified one")]
[Help("CompatibleCL", "Overrides the changelist that the engine is API-compatible with")]
[Help("Promoted", "Value for whether this is a promoted build (defaults to 1).")]
[Help("Branch", "Overrides the branch string.")]
[Help("Licensee", "When updating version files, store the changelist number in licensee format")]
[RequireP4]
public class UpdateLocalVersion : BuildCommand
{
	public override void ExecuteBuild()
	{
		UnrealBuild UnrealBuild = new UnrealBuild(this);
		int? ChangelistOverride = ParseParamNullableInt("cl");
		int? CompatibleChangelistOverride = ParseParamNullableInt("compatiblecl");
  		// If override is not provided, try to read compatible CL value from p4 if enabled
		if (!CompatibleChangelistOverride.HasValue && P4Enabled)
		{
			CompatibleChangelistOverride = P4Env.CodeChangelist;
		}
		string Build = ParseParamValue("Build", null);
		UnrealBuild.UpdateVersionFiles(ChangelistNumberOverride: ChangelistOverride, CompatibleChangelistNumberOverride: CompatibleChangelistOverride, Build: Build);
	}
}
