// Copyright Epic Games, Inc. All Rights Reserved.

using AutomationTool;
using System;
using System.Collections.Generic;
using System.Linq;
using System.Text;
using System.Threading.Tasks;
using EpicGames.Core;
using UnrealBuildBase;
using System.Text.RegularExpressions;
using Microsoft.Extensions.Logging;

using static AutomationTool.CommandUtils;

namespace BuildScripts.Automation
{
	[Help("Tool for updating a gate for robomerge")]
	[Help("Stream=<Name>", "Stream that contains the gate")]
	[Help("Change=<value>", "Change to set in the gate")]
	[Help("File=<Path>", "Path to update (relative to Stream)")]
	[Help("Client=<Name>", "The Perforce client to use")]
	[Help("UpdateIfLater", "Only write the gate if the current time is later or equal to this (HH:MM, default 00:00)")]
	[Help("UpdateIfEarlier", "Only write the gate if the current time is earlier or equal to this (HH:MM, default 23:59:59)")]
	[RequireP4]
	class UpdateRoboMergeGate : BuildCommand
	{
		public override void ExecuteBuild()
		{
			string Stream = ParseOptionalStringParam("Stream");
			string File = ParseRequiredStringParam("File");
			string ClientName = ParseOptionalStringParam("Client");
			int Change = ParseParamInt("Change", P4Env.Changelist);

			// try to parse and begin/end. If there is a time value the date will be todays date

			string UpdateLaterParam = ParseParamValue("UpdateIfLater", "");
			string UpdateEarlierParam = ParseParamValue("UpdateIfEarlier", "");

			if (!string.IsNullOrEmpty(UpdateEarlierParam) && !string.IsNullOrEmpty(UpdateLaterParam))
			{
				DateTime UpdateIfLater = DateTime.Parse(UpdateLaterParam);
				DateTime UpdateIfEarlier = DateTime.Parse(UpdateEarlierParam);

				// times are EST
				DateTime CurrentTime = DateTime.UtcNow;
				TimeZoneInfo easternZone = TimeZoneInfo.FindSystemTimeZoneById("Eastern Standard Time");
				CurrentTime = TimeZoneInfo.ConvertTimeFromUtc(CurrentTime, easternZone);

				Logger.LogInformation("Checking current time {Arg0} against UpdateIfLater ({Arg1}) and UpdateIfEarlier ({Arg2})", CurrentTime.ToString("HH:mm"), UpdateIfLater.ToString("HH:mm"), UpdateIfEarlier.ToString("HH:mm"));

				if (CurrentTime < UpdateIfLater || CurrentTime > UpdateIfEarlier)
				{
					Logger.LogInformation("Current time is outside of range. Skipping gate update");
					return;
				}
			}
			else
			{
				Logger.LogInformation("No time constrains specified via -UpdateIfLater/-UpdateIfEarlier");
			}

			if (!AllowSubmit)
			{
				Logger.LogWarning("Skipping update due to submit being disabled.");
				return;
			}

			// Get the root directory for the new workspace
			DirectoryReference RootDir = DirectoryReference.Combine(Unreal.EngineDirectory, "Intermediate", "RobomergeGate");

			// Create a brand new workspace
			P4ClientInfo Client = new P4ClientInfo();
			Client.Owner = CommandUtils.P4Env.User;
			Client.Host = Unreal.MachineName;
			Client.Stream = Stream;
			Client.RootPath = RootDir.FullName;
			Client.Name = ClientName == null ? String.Format("{0}_Robomerge", Unreal.MachineName) : ClientName;
			Client.Options = P4ClientOption.NoAllWrite | P4ClientOption.Clobber | P4ClientOption.NoCompress | P4ClientOption.Unlocked | P4ClientOption.NoModTime | P4ClientOption.RmDir;
			Client.LineEnd = P4LineEnd.Local;
			CommandUtils.P4.CreateClient(Client, AllowSpew: false);

			// Get the connection that we're going to submit with
			P4Connection SubmitP4 = new P4Connection(Client.Owner, Client.Name);

			int NewCL = 0;
			try
			{
				// Try to update the file
				NewCL = SubmitP4.CreateChange(Description: String.Format("Updating RoboMerge gate to CL {0}", Change));
				for (int Attempts = 1; ; Attempts++)
				{
					FileReference LocalFile = new FileReference(RootDir.FullName + File);
					DirectoryReference.CreateDirectory(LocalFile.Directory);

					SubmitP4.Revert(String.Format("-k \"{0}\"", LocalFile.FullName));
					SubmitP4.Sync(String.Format("-f \"{0}\"", LocalFile.FullName), AllowSpew: false);

					if (FileReference.Exists(LocalFile))
					{
						JsonObject Object = JsonObject.Read(LocalFile);
						int ExistingChange = Object.GetIntegerField("Change");
						if (Change <= ExistingChange)
						{
							Logger.LogInformation("Skipping write because existing CL ({ExistingChange}) is greater or equal to new CL ({Change}).", ExistingChange, Change);
							SubmitP4.Revert("//...");
							SubmitP4.DeleteChange(NewCL);
							break;
						}
					}

					string Output;
					SubmitP4.LogP4Output(out Output, "", String.Format("opened -a \"{0}\"", LocalFile.FullName));
					bool FileOpenedBySomeoneElse = !Output.Contains("not opened anywhere");
					if (FileOpenedBySomeoneElse)
					{
						Logger.LogInformation("Waiting 1 second as the file ({Arg0}) is already checked out ({Output}).", LocalFile.FullName, Output);
						System.Threading.Thread.Sleep(1000);
					}
					else
					{

						SubmitP4.Add(NewCL, String.Format("\"{0}\"", LocalFile.FullName));
						SubmitP4.Edit(NewCL, String.Format("\"{0}\"", LocalFile.FullName));
						SubmitP4.P4(String.Format("reopen -t text+S8 \"{0}\"", LocalFile.FullName), AllowSpew: false);

						using (JsonWriter Writer = new JsonWriter(LocalFile))
						{
							Writer.WriteObjectStart();
							Writer.WriteValue("Change", Change.ToString());

							string JobId = Environment.GetEnvironmentVariable("UE_HORDE_JOBID");
							string HordeUrl = Environment.GetEnvironmentVariable("UE_HORDE_URL");
							if (!String.IsNullOrEmpty(JobId) && !String.IsNullOrEmpty(HordeUrl))
							{
								Writer.WriteValue("Url", $"{HordeUrl}job/{JobId}");
							}
							Writer.WriteObjectEnd();
						}

						try
						{
							int SubmittedCL;
							SubmitP4.Submit(NewCL, out SubmittedCL, Force: true);
							if (SubmittedCL > 0)
							{
								break;
							}
						}
						catch (P4Exception ex)
						{
							if (Regex.IsMatch(ex.Message, @"RoboMerge\/gates.*already locked on Commit Server by buildmachine"))
							{
								// If the file is already locked by buildmachine, ignore the failure.
								return;
							}
						}

					}
					if (Attempts >= 20)
					{
						throw new AutomationException("Unable to update Robomerge gate after {0} attempts", Attempts);
					}
				}
			}
			finally
			{
				CommandUtils.P4.Changes(out List<P4Connection.ChangeRecord> PendingChanges, $"-c {Client.Name} -s pending");
				foreach (P4Connection.ChangeRecord PendingChange in PendingChanges)
				{
					SubmitP4.DeleteChange(PendingChange.CL, true);
				}
				CommandUtils.P4.DeleteClient(Client.Name);
			}
		}
	}
}
