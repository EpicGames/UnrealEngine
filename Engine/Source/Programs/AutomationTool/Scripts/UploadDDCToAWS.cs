// Copyright Epic Games, Inc. All Rights Reserved.

using Amazon.Runtime;
using Amazon.Runtime.CredentialManagement;
using Amazon.S3;
using Amazon.S3.Model;
using AutomationTool;
using System;
using System.Collections.Concurrent;
using System.Collections.Generic;
using System.Diagnostics;
using System.IO;
using System.IO.Compression;
using System.Linq;
using System.Net;
using System.Security.Cryptography;
using System.Text;
using System.Threading;
using EpicGames.Core;
using UnrealBuildBase;
using Microsoft.Extensions.Logging;

using static AutomationTool.CommandUtils;

namespace AutomationTool
{
	class UploadDDCToAWS : BuildCommand
	{
		class DerivedDataFile
		{
			public FileInfo Info { get; }
			public string Key { get; }
			public ContentHash KeyHash { get; }

			public DerivedDataFile(FileInfo Info, string Key)
			{
				this.Info = Info;
				this.Key = Key;
				this.KeyHash = GetKeyHash(Key);
			}

			public static ContentHash GetKeyHash(string Key)
			{
				string UpperKey = Key.ToUpperInvariant();
				byte[] UpperKeyBytes = Encoding.UTF8.GetBytes(UpperKey);
				return ContentHash.SHA1(UpperKeyBytes);
			}
		}

		class BundleManifest
		{
			public class Entry
			{
				public string Name;
				public string ObjectKey;
				public DateTime Time;
				public int CompressedLength;
				public int UncompressedLength;
			}

			public List<Entry> Entries = new List<Entry>();

			public void Read(JsonObject Object)
			{
				JsonObject[] BundleObjects = Object.GetObjectArrayField(nameof(Entries));
				foreach (JsonObject BundleObject in BundleObjects)
				{
					Entry Bundle = new Entry();
					Bundle.Name = BundleObject.GetStringField(nameof(Bundle.Name));
					Bundle.ObjectKey = BundleObject.GetStringField(nameof(Bundle.ObjectKey));
					Bundle.Time = DateTime.Parse(BundleObject.GetStringField(nameof(Bundle.Time)));
					Bundle.CompressedLength = BundleObject.GetIntegerField(nameof(Bundle.CompressedLength));
					Bundle.UncompressedLength = BundleObject.GetIntegerField(nameof(Bundle.UncompressedLength));
					Entries.Add(Bundle);
				}
			}

			public void Save(FileReference File)
			{
				using (JsonWriter Writer = new JsonWriter(File))
				{
					Write(Writer);
				}
			}

			public void Write(JsonWriter Writer)
			{
				Writer.WriteObjectStart();
				Writer.WriteArrayStart(nameof(Entries));
				foreach (Entry Bundle in Entries)
				{
					Writer.WriteObjectStart();
					Writer.WriteValue(nameof(Bundle.Name), Bundle.Name);
					Writer.WriteValue(nameof(Bundle.ObjectKey), Bundle.ObjectKey);
					Writer.WriteValue(nameof(Bundle.Time), Bundle.Time.ToString("u"));
					Writer.WriteValue(nameof(Bundle.CompressedLength), Bundle.CompressedLength);
					Writer.WriteValue(nameof(Bundle.UncompressedLength), Bundle.UncompressedLength);
					Writer.WriteObjectEnd();
				}
				Writer.WriteArrayEnd();
				Writer.WriteObjectEnd();
			}
		}

		public class RootManifest
		{
			public class Entry
			{
				public DateTime CreateTime;
				public string Key;
			}

			public string AccessKey;
			public string SecretKey;
			public List<Entry> Entries = new List<Entry>();

			public void Read(JsonObject Object)
			{
				AccessKey = Object.GetStringField(nameof(AccessKey));
				SecretKey = Object.GetStringField(nameof(SecretKey));

				JsonObject[] ManifestObjects = Object.GetObjectArrayField(nameof(Entries));
				foreach (JsonObject ManifestObject in ManifestObjects)
				{
					Entry Entry = new Entry();
					Entry.CreateTime = DateTime.Parse(ManifestObject.GetStringField(nameof(Entry.CreateTime)));
					Entry.Key = ManifestObject.GetStringField(nameof(Entry.Key));
					Entries.Add(Entry);
				}
			}

			public void Save(FileReference File)
			{
				using (JsonWriter Writer = new JsonWriter(File))
				{
					Write(Writer);
				}
			}

			public void Write(JsonWriter Writer)
			{
				Writer.WriteObjectStart();
				Writer.WriteValue(nameof(AccessKey), AccessKey);
				Writer.WriteValue(nameof(SecretKey), SecretKey);
				Writer.WriteArrayStart(nameof(Entries));
				foreach (Entry Entry in Entries)
				{
					Writer.WriteObjectStart();
					Writer.WriteValue(nameof(Entry.CreateTime), Entry.CreateTime.ToString("u"));
					Writer.WriteValue(nameof(Entry.Key), Entry.Key);
					Writer.WriteObjectEnd();
				}
				Writer.WriteArrayEnd();
				Writer.WriteObjectEnd();
			}
		}

		protected class UploadState
		{
			public int NumFiles;
			public long NumBytes;
		}

		protected class DeleteState
		{
			public int NumFiles;
		}

		protected const uint BundleSignature = (uint)'D' | ((uint)'D' << 8) | ((uint)'B' << 16);
		protected const uint BundleSignatureV1 = BundleSignature | (1 << 24);
		protected const long MinBundleSize = 40 * 1024 * 1024; // Any bundles with this amount of usable data will be repacked
		protected const long MaxBundleSize = 100 * 1024 * 1024;

		static readonly Amazon.RegionEndpoint Region = Amazon.RegionEndpoint.USEast1;

		public override void ExecuteBuild()
		{
			bool bLocal = ParseParam("Local");
			string BucketName = !bLocal ? ParseRequiredStringParam("Bucket") : "";
			FileReference CredentialsFile = !bLocal ? ParseRequiredFileReferenceParam("CredentialsFile") : null;
			string CredentialsKey = !bLocal ? ParseRequiredStringParam("CredentialsKey") : "";
			DirectoryReference CacheDir = ParseRequiredDirectoryReferenceParam("CacheDir");
			DirectoryReference FilterDir = ParseRequiredDirectoryReferenceParam("FilterDir");
			int Days = ParseParamInt("Days", 7);
			int MaxFileSize = ParseParamInt("MaxFileSize", 0);
			string RootManifestPath = ParseRequiredStringParam("Manifest");
			string KeyPrefix = ParseParamValue("KeyPrefix", "");
			bool bReset = ParseParam("Reset");

			// Try to get the credentials by the key passed in from the script
			AWSCredentials Credentials = null;
			if (!bLocal)
			{
				CredentialProfileStoreChain CredentialsChain = new CredentialProfileStoreChain(CredentialsFile.FullName);
				if (!CredentialsChain.TryGetAWSCredentials(CredentialsKey, out Credentials))
				{
					throw new AutomationException("Unknown credentials key: {0}", CredentialsKey);
				}
			}

			// Create the new client
			using (AmazonS3Client Client = !bLocal ? new AmazonS3Client(Credentials, Region) : null)
			{
				using (SemaphoreSlim RequestSemaphore = new SemaphoreSlim(4))
				{
					// Read the filters
					HashSet<string> Paths = new HashSet<string>();
					foreach (FileInfo FilterFile in FilterDir.ToDirectoryInfo().EnumerateFiles("*.txt"))
					{
						TimeSpan Age = DateTime.UtcNow - FilterFile.LastWriteTimeUtc;
						if (Age < TimeSpan.FromDays(3))
						{
							Logger.LogInformation("Reading {Arg0}", FilterFile.FullName);

							string[] Lines = File.ReadAllLines(FilterFile.FullName);
							foreach (string Line in Lines)
							{
								string TrimLine = Line.Trim().Replace('\\', '/');
								if (TrimLine.Length > 0)
								{
									Paths.Add(TrimLine);
								}
							}
						}
						else if(Age > TimeSpan.FromDays(5))
						{
							try
							{
								Logger.LogInformation("Deleting {Arg0}", FilterFile.FullName);
								FilterFile.Delete();
							}
							catch (Exception Ex)
							{
								Logger.LogWarning("Unable to delete: {Arg0}", Ex.Message);
								Logger.LogDebug("{Text}", ExceptionUtils.FormatExceptionDetails(Ex));
							}
						}
					}
					Logger.LogInformation("Found {0:n0} files", Paths.Count);

					// Enumerate all the files that are in the network DDC
					Logger.LogInformation("");
					Logger.LogInformation("Filtering files in {CacheDir}...", CacheDir);
					List<DerivedDataFile> Files = ParallelExecute<string, DerivedDataFile>(Paths, (Path, FilesBag) => ReadFileInfo(CacheDir, Path, FilesBag));

					// Filter to the maximum size
					if (MaxFileSize != 0)
					{
						int NumRemovedMaxSize = Files.RemoveAll(x => x.Info.Length > MaxFileSize);
						Logger.LogInformation("");
						Logger.LogInformation("Removed {NumRemovedMaxSize} files above size limit ({1:n0} bytes)", NumRemovedMaxSize, MaxFileSize);
					}

					// Create the working directory
					DirectoryReference WorkingDir = DirectoryReference.Combine(Unreal.EngineDirectory, "Saved", "UploadDDC");
					DirectoryReference.CreateDirectory(WorkingDir);

					// Get the path to the manifest
					FileReference RootManifestFile = FileReference.Combine(Unreal.RootDirectory, RootManifestPath);

					// Read the old root manifest
					RootManifest OldRootManifest = new RootManifest();
					if (FileReference.Exists(RootManifestFile))
					{
						OldRootManifest.Read(JsonObject.Read(RootManifestFile));
					}

					// Read the old bundle manifest
					BundleManifest OldBundleManifest = new BundleManifest();
					if (OldRootManifest.Entries.Count > 0)
					{
						FileReference LocalBundleManifest = FileReference.Combine(WorkingDir, "OldBundleManifest.json");
						if (TryDownloadFile(Client, BucketName, OldRootManifest.Entries.Last().Key, LocalBundleManifest))
						{
							OldBundleManifest.Read(JsonObject.Read(LocalBundleManifest));
						}
					}

					// Create the new manifest
					BundleManifest NewBundleManifest = new BundleManifest();

					// Try to download the old manifest, and add all the bundles we want to keep to the new manifest
					if (!bReset)
					{
						foreach (BundleManifest.Entry Bundle in OldBundleManifest.Entries)
						{
							FileReference BundleFile = FileReference.Combine(WorkingDir, Bundle.Name);
							if (!FileReference.Exists(BundleFile))
							{
								Logger.LogInformation("Downloading {BundleFile}", BundleFile);

								FileReference TempCompressedFile = new FileReference(BundleFile.FullName + ".incoming.gz");
								if (!TryDownloadFile(Client, BucketName, Bundle.ObjectKey, TempCompressedFile))
								{
									Logger.LogWarning("Unable to download {Arg0}", Bundle.ObjectKey);
									continue;
								}

								FileReference TempUncompressedFile = new FileReference(BundleFile.FullName + ".incoming");
								try
								{
									DecompressFile(TempCompressedFile, TempUncompressedFile);
								}
								catch (Exception Ex)
								{
									Logger.LogWarning("Unable to uncompress {Arg0}: {Arg1}", Bundle.ObjectKey, Ex.ToString());
									continue;
								}

								FileReference.Move(TempUncompressedFile, BundleFile);
							}
							NewBundleManifest.Entries.Add(Bundle);
						}
					}

					// Figure out all the item digests that we already have
					Dictionary<BundleManifest.Entry, HashSet<ContentHash>> BundleToKeyHashes = new Dictionary<BundleManifest.Entry, HashSet<ContentHash>>();
					foreach (BundleManifest.Entry Bundle in NewBundleManifest.Entries)
					{
						HashSet<ContentHash> KeyHashes = new HashSet<ContentHash>();

						FileReference BundleFile = FileReference.Combine(WorkingDir, Bundle.Name);
						using (FileStream Stream = FileReference.Open(BundleFile, FileMode.Open, FileAccess.Read, FileShare.Read))
						{
							BinaryReader Reader = new BinaryReader(Stream);

							uint Signature = Reader.ReadUInt32();
							if (Signature != BundleSignatureV1)
							{
								throw new Exception(String.Format("Invalid signature for {0}", BundleFile));
							}

							int NumEntries = Reader.ReadInt32();
							for (int EntryIdx = 0; EntryIdx < NumEntries; EntryIdx++)
							{
								byte[] Digest = new byte[ContentHash.LengthSHA1];
								if (Reader.Read(Digest, 0, ContentHash.LengthSHA1) != ContentHash.LengthSHA1)
								{
									throw new Exception("Unexpected EOF");
								}
								KeyHashes.Add(new ContentHash(Digest));
								Stream.Seek(4, SeekOrigin.Current);
							}
						}

						BundleToKeyHashes[Bundle] = KeyHashes;
					}

					// Calculate the download size of the manifest
					long DownloadSize = NewBundleManifest.Entries.Sum(x => (long)x.CompressedLength);

					// Remove any bundles which have less than the minimum required size in valid data. We don't mark the manifest as dirty yet; these
					// files will only be rewritten if new content is added, to prevent the last bundle being rewritten multiple times.
					foreach (KeyValuePair<BundleManifest.Entry, HashSet<ContentHash>> Pair in BundleToKeyHashes)
					{
						long ValidBundleSize = Files.Where(x => Pair.Value.Contains(x.KeyHash)).Sum(x => (long)x.Info.Length);
						if (ValidBundleSize < MinBundleSize)
						{
							NewBundleManifest.Entries.Remove(Pair.Key);
						}
					}

					// Find all the valid digests
					HashSet<ContentHash> ReusedKeyHashes = new HashSet<ContentHash>();
					foreach (BundleManifest.Entry Bundle in NewBundleManifest.Entries)
					{
						ReusedKeyHashes.UnionWith(BundleToKeyHashes[Bundle]);
					}

					// Remove all the files which already exist
					int NumRemovedExist = Files.RemoveAll(x => ReusedKeyHashes.Contains(x.KeyHash));
					if (NumRemovedExist > 0)
					{
						Logger.LogInformation("");
						Logger.LogInformation("Removed {0:n0} files which already exist", NumRemovedExist);
					}

					// Read all the files we want to include
					List<Tuple<DerivedDataFile, byte[]>> FilesToInclude = new List<Tuple<DerivedDataFile, byte[]>>();
					if (Files.Count > 0)
					{
						Logger.LogInformation("");
						Logger.LogInformation("Reading remaining {0:n0} files into memory ({1:n1}mb)...", Files.Count, (float)Files.Sum(x => (long)x.Info.Length) / (1024 * 1024));
						FilesToInclude.AddRange(ParallelExecute<DerivedDataFile, Tuple<DerivedDataFile, byte[]>>(Files, (x, y) => ReadFileData(x, y)));
					}

					// Generate new data
					{
						// Flag for whether to update the manifest
						bool bUpdateManifest = false;

						// Upload the new bundle
						Logger.LogInformation("");
						if (FilesToInclude.Count == 0)
						{
							Logger.LogInformation("No new files to add.");
						}
						else
						{
							// Sort the files to include by creation time. This will bias towards grouping older, more "permanent", items together.
							Logger.LogInformation("Sorting input files");
							List<Tuple<DerivedDataFile, byte[]>> SortedFilesToInclude = FilesToInclude.OrderBy(x => x.Item1.Info.CreationTimeUtc).ToList();

							// Get the target bundle size
							long TotalSize = SortedFilesToInclude.Sum(x => (long)x.Item2.Length);
							int NumBundles = (int)((TotalSize + (MaxBundleSize - 1)) / MaxBundleSize);
							long TargetBundleSize = TotalSize / NumBundles;

							// Split the input data into bundles
							List<List<Tuple<DerivedDataFile, byte[]>>> BundleFilesToIncludeList = new List<List<Tuple<DerivedDataFile, byte[]>>>();
							long BundleSize = 0;
							for (int FileIdx = 0; FileIdx < SortedFilesToInclude.Count; BundleSize = BundleSize % TargetBundleSize)
							{
								List<Tuple<DerivedDataFile, byte[]>> BundleFilesToInclude = new List<Tuple<DerivedDataFile, byte[]>>();
								for (; BundleSize < TargetBundleSize && FileIdx < SortedFilesToInclude.Count; FileIdx++)
								{
									BundleFilesToInclude.Add(SortedFilesToInclude[FileIdx]);
									BundleSize += SortedFilesToInclude[FileIdx].Item2.Length;
								}
								BundleFilesToIncludeList.Add(BundleFilesToInclude);
							}

							// Upload each bundle
							DateTime NewBundleTime = DateTime.UtcNow;
							for (int BundleIdx = 0; BundleIdx < BundleFilesToIncludeList.Count; BundleIdx++)
							{
								List<Tuple<DerivedDataFile, byte[]>> BundleFilesToInclude = BundleFilesToIncludeList[BundleIdx];

								// Get the new bundle info
								string NewBundleSuffix = (BundleFilesToIncludeList.Count > 1) ? String.Format("-{0}_of_{1}", BundleIdx + 1, BundleFilesToIncludeList.Count) : "";
								string NewBundleName = String.Format("Bundle-{0:yyyy.MM.dd-HH.mm}{1}.ddb", NewBundleTime.ToLocalTime(), NewBundleSuffix);

								// Create a random number for the object key
								string NewBundleObjectKey = KeyPrefix + "bulk/" + CreateObjectName();

								// Create the bundle header
								byte[] Header;
								using (MemoryStream HeaderStream = new MemoryStream())
								{
									BinaryWriter Writer = new BinaryWriter(HeaderStream);
									Writer.Write(BundleSignatureV1);
									Writer.Write(BundleFilesToInclude.Count);

									foreach (Tuple<DerivedDataFile, byte[]> FileToInclude in BundleFilesToInclude)
									{
										Writer.Write(FileToInclude.Item1.KeyHash.Bytes, 0, ContentHash.LengthSHA1);
										Writer.Write((int)FileToInclude.Item2.Length);
									}

									Header = HeaderStream.ToArray();
								}

								// Create the output file
								FileReference NewBundleFile = FileReference.Combine(WorkingDir, NewBundleName + ".gz");
								Logger.LogInformation("Writing {NewBundleFile}", NewBundleFile);
								using (FileStream BundleStream = FileReference.Open(NewBundleFile, FileMode.Create, FileAccess.Write, FileShare.Read))
								{
									using (GZipStream ZipStream = new GZipStream(BundleStream, CompressionLevel.Optimal, true))
									{
										ZipStream.Write(Header, 0, Header.Length);
										foreach (Tuple<DerivedDataFile, byte[]> FileToInclude in BundleFilesToInclude)
										{
											ZipStream.Write(FileToInclude.Item2, 0, FileToInclude.Item2.Length);
										}
									}
								}

								// Upload the file
								long NewBundleCompressedLength = NewBundleFile.ToFileInfo().Length;
								long NewBundleUncompressedLength = Header.Length + BundleFilesToInclude.Sum(x => (long)x.Item2.Length);
								Logger.LogInformation("Uploading bundle to {NewBundleObjectKey} ({1:n1}mb)", NewBundleObjectKey, NewBundleCompressedLength / (1024.0f * 1024.0f));
								UploadFile(Client, BucketName, NewBundleFile, 0, NewBundleObjectKey, RequestSemaphore, null);

								// Add the bundle to the new manifest
								BundleManifest.Entry Bundle = new BundleManifest.Entry();
								Bundle.Name = NewBundleName;
								Bundle.ObjectKey = NewBundleObjectKey;
								Bundle.Time = NewBundleTime;
								Bundle.CompressedLength = (int)NewBundleCompressedLength;
								Bundle.UncompressedLength = (int)NewBundleUncompressedLength;
								NewBundleManifest.Entries.Add(Bundle);

								// Mark the manifest as requiring an update
								bUpdateManifest = true;
							}
						}

						// Update the manifest
						if (bUpdateManifest)
						{
							DateTime UtcNow = DateTime.UtcNow;
							DateTime RemoveBundleManifestsBefore = UtcNow - TimeSpan.FromDays(3.0);

							// Update the root manifest
							RootManifest NewRootManifest = new RootManifest();
							NewRootManifest.AccessKey = OldRootManifest.AccessKey;
							NewRootManifest.SecretKey = OldRootManifest.SecretKey;
							foreach (RootManifest.Entry Entry in OldRootManifest.Entries)
							{
								if (Entry.CreateTime >= RemoveBundleManifestsBefore)
								{
									NewRootManifest.Entries.Add(Entry);
								}
							}

							// Make sure there's an entry for the last 24h
							DateTime RequireBundleManifestAfter = UtcNow - TimeSpan.FromDays(1.0);
							if(!NewRootManifest.Entries.Any(x => x.CreateTime > RequireBundleManifestAfter))
							{
								RootManifest.Entry NewEntry = new RootManifest.Entry();
								NewEntry.CreateTime = UtcNow;
								NewEntry.Key = KeyPrefix + CreateObjectName();
								NewRootManifest.Entries.Add(NewEntry);
							}

							// Save out the new bundle manifest
							FileReference NewBundleManifestFile = FileReference.Combine(WorkingDir, "NewBundleManifest.json");
							NewBundleManifest.Save(NewBundleManifestFile);

							// Update all the bundle manifests still valid
							foreach (RootManifest.Entry Entry in NewRootManifest.Entries)
							{
								Logger.LogInformation("Uploading bundle manifest to {Arg0}", Entry.Key);
								UploadFile(Client, BucketName, NewBundleManifestFile, 0, Entry.Key, RequestSemaphore, null);
							}

							string LocalBundleManifestPath = String.Format("file://{0}", NewBundleManifestFile.FullName.Replace('\\', '/'));
							if (bLocal && NewRootManifest.Entries.LastOrDefault()?.Key != LocalBundleManifestPath)
							{
								RootManifest.Entry NewEntry = new RootManifest.Entry();
								NewEntry.CreateTime = UtcNow;
								NewEntry.Key = LocalBundleManifestPath;
								NewRootManifest.Entries.Add(NewEntry);
							}

							// Overwrite all the existing manifests
							int ChangeNumber = 0;
							if (AllowSubmit || (P4Enabled && bLocal))
							{
								List<string> ExistingFiles = P4.Files(CommandUtils.MakePathSafeToUseWithCommandLine(RootManifestFile.FullName));

								// Create a changelist containing the new manifest
								ChangeNumber = P4.CreateChange(Description: "Updating DDC bundle manifest\n#skipci");
								if (ExistingFiles.Count > 0)
								{
									P4.Edit(ChangeNumber, CommandUtils.MakePathSafeToUseWithCommandLine(RootManifestFile.FullName));
									NewRootManifest.Save(RootManifestFile);
								}
								else
								{
									NewRootManifest.Save(RootManifestFile);
									P4.Add(ChangeNumber, CommandUtils.MakePathSafeToUseWithCommandLine(RootManifestFile.FullName));
								}
							}
							else if (bLocal)
							{
								NewRootManifest.Save(RootManifestFile);
							}

							if (AllowSubmit)
							{
								// Submit it
								int SubmittedChangeNumber;
								P4.Submit(ChangeNumber, out SubmittedChangeNumber, true);

								if (SubmittedChangeNumber <= 0)
								{
									throw new AutomationException("Failed to submit change");
								}

								// Delete any bundles that are no longer referenced
								HashSet<string> KeepObjectKeys = new HashSet<string>(NewBundleManifest.Entries.Select(x => x.ObjectKey));
								foreach (BundleManifest.Entry OldEntry in OldBundleManifest.Entries)
								{
									if (!KeepObjectKeys.Contains(OldEntry.ObjectKey))
									{
										Logger.LogInformation("Deleting unreferenced bundle {Arg0}", OldEntry.ObjectKey);
										DeleteFile(Client, BucketName, OldEntry.ObjectKey, RequestSemaphore, null);
									}
								}

								// Delete any bundle manifests which are no longer referenced
								HashSet<string> KeepManifestKeys = new HashSet<string>(NewRootManifest.Entries.Select(x => x.Key));
								foreach (RootManifest.Entry OldEntry in OldRootManifest.Entries)
								{
									if (!KeepManifestKeys.Contains(OldEntry.Key))
									{
										Logger.LogInformation("Deleting unreferenced manifest {Arg0}", OldEntry.Key);
										DeleteFile(Client, BucketName, OldEntry.Key, RequestSemaphore, null);
									}
								}
							}
							else
							{
								// Skip submitting
								Logger.LogWarning("Skipping manifest submit due to missing -Submit argument.");
							}

							// Update the new download size
							DownloadSize = NewBundleManifest.Entries.Sum(x => (long)x.CompressedLength);
						}
					}

					// Print some stats about the final manifest
					Logger.LogInformation("");
					Logger.LogInformation("Total download size {0:n1}mb", DownloadSize / (1024.0 * 1024.0));
					Logger.LogInformation("");
				}
			}
		}

		private string CreateObjectName()
		{
			byte[] NameBytes = RandomNumberGenerator.GetBytes(40);
			return StringUtils.FormatHexString(NameBytes);
		}

		private bool TryDownloadFile(AmazonS3Client Client, string BucketName, string Key, FileReference OutputFile)
		{
			if (Client == null)
			{
				return false;
			}
			try
			{
				GetObjectRequest Request = new GetObjectRequest();
				Request.BucketName = BucketName;
				Request.Key = Key;

				using (GetObjectResponse Response = Client.GetObjectAsync(Request).Result)
				{
					if ((int)Response.HttpStatusCode >= 200 && (int)Response.HttpStatusCode <= 299)
					{
						Response.WriteResponseStreamToFileAsync(OutputFile.FullName, false, CancellationToken.None).Wait();
						return true;
					}
				}
				return false;
			}
			catch (Exception Ex)
			{
				AmazonS3Exception AmazonEx = Ex as AmazonS3Exception;
				if (AmazonEx == null || AmazonEx.StatusCode != HttpStatusCode.NotFound)
				{
					Logger.LogWarning("Exception while downloading {Key}: {Ex}", Key, Ex);
				}
				return false;
			}
		}

		private void DecompressFile(FileReference CompressedFile, FileReference UncompressedFile)
		{
			using (FileStream OutputStream = FileReference.Open(UncompressedFile, FileMode.Create, FileAccess.Write))
			{
				using (FileStream InputStream = FileReference.Open(CompressedFile, FileMode.Open, FileAccess.Read))
				{
					GZipStream ZipStream = new GZipStream(InputStream, CompressionMode.Decompress, true);
					ZipStream.CopyTo(OutputStream);
				}
			}
		}

		private void ReadFileInfo(DirectoryReference BaseDir, string Path, ConcurrentBag<DerivedDataFile> Files)
		{
			FileReference Location = FileReference.Combine(BaseDir, Path);
			try
			{
				FileInfo Info = Location.ToFileInfo();
				if (Info.Exists)
				{
					string Key = new FileReference(Info).MakeRelativeTo(BaseDir).Replace('\\', '/');
					Files.Add(new DerivedDataFile(Info, Key));
				}
			}
			catch (Exception Ex)
			{
				Logger.LogWarning("Unable to read file info {Location}: {Ex}", Location, Ex);
			}
		}

		private void ReadFileData(DerivedDataFile DerivedDataFile, ConcurrentBag<Tuple<DerivedDataFile, byte[]>> FilesToInclude)
		{
			try
			{
				byte[] Data = File.ReadAllBytes(DerivedDataFile.Info.FullName);
				FilesToInclude.Add(Tuple.Create(DerivedDataFile, Data));
			}
			catch(Exception Ex)
			{
				Logger.LogWarning("Unable to read file {Arg0}: {Ex}", DerivedDataFile.Info.FullName, Ex);
			}
		}

		protected void ParallelExecute(IEnumerable<Action> Actions)
		{
			using (ThreadPoolWorkQueue Queue = new ThreadPoolWorkQueue())
			{
				int TotalCount = 0;
				foreach (Action Action in Actions)
				{
					Queue.Enqueue(Action);
					TotalCount++;
				}

				for (; ; )
				{
					Queue.Wait(TimeSpan.FromSeconds(10.0));

					int NumRemaining = Queue.NumRemaining;
					Logger.LogInformation("Processed {0:n0} ({Arg1}%)", TotalCount - NumRemaining, ((TotalCount - NumRemaining) * 100) / TotalCount);

					if (NumRemaining == 0)
					{
						break;
					}
				}
			}
		}

		protected List<U> ParallelExecute<T, U>(IEnumerable<T> Source, Action<T, ConcurrentBag<U>> Transform)
		{
			ConcurrentBag<U> Output = new ConcurrentBag<U>();
			ParallelExecute(Source.Select(x => new Action(() => Transform(x, Output))));
			return Output.ToList();
		}

		protected void UploadFile(AmazonS3Client Client, string BucketName, FileReference File, long Length, string Key, SemaphoreSlim Semaphore, UploadState State)
		{
			if (Client == null)
			{
				return;
			}
			try
			{
				Retry(() => UploadFileInner(Client, BucketName, File, Length, Key, Semaphore, State), 10);
			}
			catch (Exception Ex)
			{
				Logger.LogWarning("Exception trying to upload {File}: {Arg1}", File, ExceptionUtils.FormatExceptionDetails(Ex));
			}
		}

		protected void UploadFileInner(AmazonS3Client Client, string BucketName, FileReference File, long Length, string Key, SemaphoreSlim Semaphore, UploadState State)
		{
			try
			{
				Semaphore.Wait();

				if (FileReference.Exists(File))
				{
					PutObjectRequest Request = new PutObjectRequest();
					Request.BucketName = BucketName;
					Request.Key = Key;
					Request.FilePath = File.FullName;
					Request.CannedACL = S3CannedACL.NoACL;
					Client.PutObjectAsync(Request).Wait();
				}

				if (State != null)
				{
					Interlocked.Increment(ref State.NumFiles);
					Interlocked.Add(ref State.NumBytes, Length);
				}
			}
			finally
			{
				Semaphore.Release();
			}
		}

		protected void DeleteFile(AmazonS3Client Client, string BucketName, string Key, SemaphoreSlim Semaphore, DeleteState State)
		{
			try
			{
				DeleteFileInner(Client, BucketName, Key, Semaphore, State);
			}
			catch (Exception Ex)
			{
				Logger.LogWarning("Exception trying to delete {Key}: {Arg1}", Key, ExceptionUtils.FormatExceptionDetails(Ex));
			}
		}

		protected void DeleteFileInner(AmazonS3Client Client, string BucketName, string Key, SemaphoreSlim Semaphore, DeleteState State)
		{
			try
			{
				Semaphore.Wait();

				DeleteObjectRequest Request = new DeleteObjectRequest();
				Request.BucketName = BucketName;
				Request.Key = Key;
				Client.DeleteObjectAsync(Request).Wait();
				if (State != null)
				{
					Interlocked.Increment(ref State.NumFiles);
				}
			}
			finally
			{
				Semaphore.Release();
			}
		}

		protected void Retry(Action ActionToRetry, int MaxAttempts)
		{
			Retry(() => { ActionToRetry(); return 1; }, MaxAttempts);
		}

		protected T Retry<T>(Func<T> FuncToRetry, int MaxAttempts)
		{
			int NumAttempts = 0;
			for (; ; )
			{
				try
				{
					NumAttempts++;
					return FuncToRetry();
				}
				catch (Exception Ex)
				{
					if (NumAttempts < MaxAttempts)
					{
						Logger.LogDebug("Attempt {NumAttempts}: {Arg1}", NumAttempts, ExceptionUtils.FormatExceptionDetails(Ex));
					}
					else
					{
						throw;
					}
				}
				Thread.Sleep(TimeSpan.FromSeconds(5.0));
			}
		}
	}
}
