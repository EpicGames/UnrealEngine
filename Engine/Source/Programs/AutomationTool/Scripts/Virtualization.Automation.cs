// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Linq;
using System.Text;
using System.Threading.Tasks;

using AutomationTool;
using EpicGames.Core;
using UnrealBuildBase;
using UnrealBuildTool;
using Microsoft.Extensions.Logging;

namespace AutomationScripts.Automation
{
	[RequireP4]
	// BuildCommand is the base class for all commands.
	public abstract class VirtualizationBase : BuildCommand
	{
		protected VirtualizationBase(string InCommandletName)
		{
			CommandletName = InCommandletName;
		}
		public override void ExecuteBuild()
		{
			try
			{
				Logger.LogInformation("********** {Arg0} COMMAND STARTED **********", CommandletName.ToUpper());

				bool NoBuild = ParseParam("nobuild");

				ParseProjectPerforceCommandline();

				ProjectParams Params = new ProjectParams
				(
					Command: this,

					// Shared
					Build: !NoBuild,
					RawProjectPath: ProjectPath

				);

				BuildEditor(Params);

				CreateChangelist(Params);
				RunCommandlet(Params);
				SubmitPackages();
			}
			catch (Exception ProcessEx)
			{
				Logger.LogInformation("********** {Arg0} COMMAND FAILED **********", CommandletName.ToUpper());
				Logger.LogInformation("Error message: {Arg0}", ProcessEx.Message);

				HandleFailure();

				throw;
			}
		}

		protected abstract void RunCommandlet(ProjectParams Params);
		private void BuildEditor(ProjectParams Params)
		{
			Logger.LogInformation("Running Step:- BuildEditor");
			Project.Build(this, Params, WorkingCL: -1, ProjectBuildTargets.Editor);
		}

		private void ParseProjectPerforceCommandline()
		{
			ProjectP4Client = ParseParamValue("ProjectP4Client");

			if(!string.IsNullOrEmpty(ProjectP4Client))
			{
				if (!P4.DoesClientExist(ProjectP4Client, true))
				{
					throw new AutomationException("Project P4 client '{0}' does not exist!", ProjectP4Client);
				}

				ProjectP4 = new P4Connection(P4Env.User, ProjectP4Client, P4Env.ServerAndPort);
			}
			else
			{
				ProjectP4Client = P4Env.Client;
				ProjectP4 = P4;
			}
		}

		private void CreateChangelist(ProjectParams Params)
		{
			Logger.LogInformation("Running Step:- CreateChangelist");

			string Description = String.Format("{0}: Running '{1} on the project project from engine changelist {2}\n#rb None", Params.ShortProjectName, CommandletName, P4Env.Changelist);
			WorkingCL = ProjectP4.CreateChange(ProjectP4Client, Description);
			Logger.LogInformation("Working in {WorkingCL}", WorkingCL);
		}

		private void SubmitPackages()
		{
			Logger.LogInformation("Running Step:- SubmitResavedPackages");

			// Check everything in!
			if (WorkingCL != -1)
			{
				Logger.LogInformation("{Text}", "Running Step:- Submitting CL " + WorkingCL);
				int SubmittedCL;
				ProjectP4.Submit(WorkingCL, out SubmittedCL, true, true);
				Logger.LogInformation("{Text}", "INFO: Packages successfully submitted in cl " + SubmittedCL.ToString());

				WorkingCL = -1;
			}
		}

		private void HandleFailure()
		{
			try
			{
				if (WorkingCL != -1)
				{
					ProjectP4.RevertAll(WorkingCL);
					ProjectP4.DeleteChange(WorkingCL);

					WorkingCL = -1;
				}
			}
			catch (P4Exception P4Ex)
			{
				Logger.LogError("{Text}", "Failed to clean up P4 changelist: " + P4Ex.Message);
			}
			catch (Exception SendMailEx)
			{
				Logger.LogError("{Text}", "Failed to notify that build succeeded: " + SendMailEx.Message);
			}
		}

		public virtual FileReference ProjectPath
		{
			get
			{
				if (ProjectFullPath == null)
				{
					ProjectFullPath = ParseProjectParam();

					if (ProjectFullPath == null)
					{
						throw new AutomationException("No project file specified. Use -project=<project>.");
					}
				}

				return ProjectFullPath;
			}
		}

		private string CommandletName;

		private P4Connection ProjectP4;

		protected string ProjectP4Client;

		private FileReference ProjectFullPath;

		protected int WorkingCL = -1;
	}

	public class RehydrateProject : VirtualizationBase
	{
		[Help("Rehydrates all eligible packages in the given project")]
		[Help("Usage: RehydrateProject -Project=\"???\" -ProjectP4Client=\"???\" -Submit")]
		[Help("-Project=???", "Path to the .uproject file of the project you want to rehydrate ")]
		[Help("-ProjectP4Client=???", "[Optional] The p4 client spec (workspace) to use to checkout & submit the packages.")]
		[Help("-Submit", "[Optional] Submit the changelist of rehydrated packages, this will always occur when run in the build machine environment")]
		public RehydrateProject()
			: base("Rehydrate Project")
		{

		}
		protected override void RunCommandlet(ProjectParams Params)
		{
			Logger.LogInformation("Running Step:- RunCommandlet");

			string EditorExe = HostPlatform.Current.GetUnrealExePath(Params.UnrealExe);
			if (!FileExists(EditorExe))
			{
				Logger.LogError("Missing " + EditorExe + " executable. Needs to be built first.");
				throw new AutomationException("Missing " + EditorExe + " executable. Needs to be built first.");
			}

			string CommandletParams = String.Format(" -SCCProvider={0} -P4Port={1} -P4User={2} -P4Client={3} -P4Changelist={4} -P4Passwd={5}", "Perforce", P4Env.ServerAndPort, P4Env.User, ProjectP4Client, WorkingCL.ToString(), P4.GetAuthenticationToken());

			RunCommandlet(Params.RawProjectPath, Params.UnrealExe, "VirtualizationEditor.RehydrateProject", CommandletParams);
		}
	}

	[Help("Virtualizes all eligible packages in the given project")]
	[Help("Usage: VirtualizeProject -Project=\"???\" -ProjectP4Client=\"???\" -Submit")]
	[Help("-Project=???", "Path to the .uproject file of the project you want to virtualize ")]
	[Help("-ProjectP4Client=???", "[Optional] The p4 client spec (workspace) to use to checkout & submit the packages.")]
	[Help("-Submit", "[Optional] Submit the changelist of virtualized packages, this will always occur when run in the build machine environment")]
	public class VirtualizeProject : VirtualizationBase
	{
		public VirtualizeProject()
			: base("Virtualize Project")
		{

		}
		protected override void RunCommandlet(ProjectParams Params)
		{
			Logger.LogInformation("Running Step:- RunCommandlet");

			string EditorExe = HostPlatform.Current.GetUnrealExePath(Params.UnrealExe);
			if (!FileExists(EditorExe))
			{
				Logger.LogError("Missing " + EditorExe + " executable. Needs to be built first.");
				throw new AutomationException("Missing " + EditorExe + " executable. Needs to be built first.");
			}

			string CommandletParams = String.Format(" -SCCProvider={0} -P4Port={1} -P4User={2} -P4Client={3} -P4Changelist={4} -P4Passwd={5}", "Perforce", P4Env.ServerAndPort, P4Env.User, ProjectP4Client, WorkingCL.ToString(), P4.GetAuthenticationToken());

			RunCommandlet(Params.RawProjectPath, Params.UnrealExe, "VirtualizationEditor.VirtualizeProject", CommandletParams);
		}
	}
} //namespace AutomationScripts.Automation
