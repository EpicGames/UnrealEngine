// Copyright Epic Games, Inc. All Rights Reserved.
using AutomationTool;
using EpicGames.Core;
using System;
using System.Diagnostics;
using System.Threading;
using Microsoft.Extensions.Logging;

namespace AutomationScripts.Automation
{
	[Help("Execute a World Partition builder")]
	[Help("Builder=<Name>", "Name of the builder to run")]
	[Help("CommandletArgs=<value>", "Arguments to provide to the builder commandlet")]
	[Help("Submit", "If the files modified by the builder should be submitted at the end of the process")]
	[Help("ShelveUser=<Name>", "If provided (along with -ShelveWorkspace, modified files will be shelved for the P4 User in the specified Workspace.")]
	[Help("ShelveWorkspace=<Name>", "If provided (along with -ShelveUser, modified files will be shelved for the P4 User in the specified Workspace.")]
	public class WorldPartitionBuilder : BuildCommand
	{
		public override void ExecuteBuild()
		{
			string Builder = ParseRequiredStringParam("Builder");
			string CommandletArgs = ParseOptionalStringParam("CommandletArgs");

			bool bSubmitResult = ParseParam("Submit");
			string SubmitTags = ParseOptionalStringParam("SubmitTags");

			string ShelveUser = ParseOptionalStringParam("ShelveUser");
			string ShelveWorkspace = ParseOptionalStringParam("ShelveWorkspace");
			bool bShelveResult = !String.IsNullOrEmpty(ShelveUser) && !String.IsNullOrEmpty(ShelveWorkspace);

			if (!P4Enabled && (bSubmitResult || bShelveResult))
			{
				Logger.LogError("P4 required to submit or shelve build results");
				return;
			}

			CommandletArgs = "-Builder=" + Builder + " " + CommandletArgs;

			if (bSubmitResult)
			{
				CommandletArgs += " -AutoSubmit";

				if (!String.IsNullOrEmpty(SubmitTags))
				{
					CommandletArgs += " -AutoSubmitTags=\"" + SubmitTags + "\"";
				}
			}

			FileReference ProjectPath = ParseProjectParam();

			// Execute the commandlet - Will throw an exception on failures
			RunCommandlet(ProjectPath, UnrealExe:null, "WorldPartitionBuilderCommandlet", CommandletArgs);
			
			if (bShelveResult)
			{
				Logger.LogInformation("### Shelving build results ###");

				// Create a new changelist and move all checked out files to it
				Logger.LogInformation("Creating pending changelist to shelve builder changes");
				int PendingCL = P4.CreateChange(P4Env.Client);
				P4.LogP4("", $"reopen -c {PendingCL} //...", AllowSpew: true);

				// Shelve changelist & revert changes
				Logger.LogInformation("Shelving changes...");
				P4.Shelve(PendingCL);
				Logger.LogInformation("Reverting local changes...");
				P4.Revert($"-w -c {PendingCL} //...");

				// Assign shelve to the provided user+workspace
				Logger.LogInformation("Changing ownership of CL {PendingCL} to user {ShelveUser}, workspace {ShelveWorkspace}", PendingCL, ShelveUser, ShelveWorkspace);
				P4.UpdateChange(PendingCL, ShelveUser, ShelveWorkspace, null, true);

				Logger.LogInformation("### Shelving completed ###");
			}
		}
	}
}
