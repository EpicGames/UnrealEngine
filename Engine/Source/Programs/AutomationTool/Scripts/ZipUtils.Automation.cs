// Copyright 2006-2016 Donya Labs AB. All Rights Reserved.

using System;
using System.IO;
using System.Collections.Generic;
using System.Linq;
using AutomationTool;
using EpicGames.Core;
using UnrealBuildBase;
using Microsoft.Extensions.Logging;
using System.IO.Compression;

[Help("ZipUtils is used to zip/unzip (i.e:RunUAT.bat ZipUtils -archive=D:/Content.zip -add=D:/UE/Pojects/SampleGame/Content/) or (i.e:RunUAT.bat ZipUtils -archive=D:/Content.zip -extract=D:/UE/Pojects/SampleGame/Content/)")]
[Help("archive=<PathToArchive>", "Path to folder that should be add to the archive.")]
[Help("add=<Path>", "Path to folder that should be add to the archive.")]
[Help("extract=<Path>", "Path to folder where the archive should be extracted")]
[Help("compression=<0..9>", "Compression Level 0: Copy, 1-3: Fastest, 4-6: Balanced, 7-9: Best Compression")]
public class ZipUtils : BuildCommand
{
	public override ExitCode Execute()
	{

		Logger.LogInformation("************************ STARTING ZIPUTILS ************************");

		if (Params.Length < 2)
		{
			Logger.LogError("Invalid number of arguments {Arg0}", Params.Length);
			return ExitCode.Error_Arguments;
		}

		string ZipFilePath = ParseParamValue("archive", "");
		var ExtractArgs = ParseParamValues("extract");
		var AddArgs = ParseParamValues("add");
		int CompressionLevel = ParseParamInt("compression", 5);


		if (string.IsNullOrWhiteSpace(ZipFilePath) || System.IO.Path.GetExtension(ZipFilePath) != ".zip")
		{
			Logger.LogError("No zip file specified");
			return ExitCode.Error_Arguments;
		}

		bool bUnzip = ExtractArgs.Count() > 0;
		bool bZip = AddArgs.Count() > 0;

		if (!bUnzip && !bZip)
		{
			Logger.LogError("Invalid arguments. Please specify -archive or -extract option.");
			return ExitCode.Error_Arguments;
		}

		//setup filter to include all files
		FileFilter Filter = new FileFilter();
		Filter.Include("*");

		if (bUnzip)
		{
			string OutputFolderPath = ExtractArgs[0];

			Logger.LogInformation("Running unzip : {Arg0} OuputFolder:{OutputFolderPath}", ZipFilePath.ToString(), OutputFolderPath);

			if (!System.IO.File.Exists(ZipFilePath))
			{
				Logger.LogError("Invalid zip file path.");
				return ExitCode.Error_Arguments;
			}

			var ExtractedFiles = InternalUnzipFiles(ZipFilePath, OutputFolderPath);

			if (ExtractedFiles.Count() == 0)
			{
				Logger.LogWarning("No files extracted from file zip.");
				return ExitCode.Error_Unknown;
			}
			else
			{
				Logger.LogInformation("List of files extracted:");
				foreach (var file in ExtractedFiles)
				{
					Logger.LogInformation("\t{file}", file);
				}
			}
		}
		else /*(bZip)*/
		{

			string ArchiveFolder = AddArgs[0];
			Logger.LogInformation("Compress level : {CompressionLevel}", CompressionLevel);
			Logger.LogInformation("Running zip : {Arg0}", ArchiveFolder.ToString());

			FileAttributes attr = File.GetAttributes(ArchiveFolder);

			if (!System.IO.Directory.Exists(ArchiveFolder) || !attr.HasFlag(FileAttributes.Directory))
			{
				Logger.LogError("Invalid zip file path.");
				return ExitCode.Error_Arguments;
			}

			InternalZipFiles(new FileReference(ZipFilePath), new DirectoryReference(ArchiveFolder), Filter, CompressionLevel);
		}

		Logger.LogInformation("************************ ZIPUTIL WORK COMPLETED ************************");

		return ExitCode.Success;
	}

	/// <summary>
	/// Creates a zip file containing the given input files
	/// </summary>
	/// <param name="ZipFileName">Filename for the zip</param>
	/// <param name="Filter">Filter which selects files to be included in the zip</param>
	/// <param name="CompressionValue"></param>
	/// <param name="BaseDirectory">Base directory to store relative paths in the zip file to</param>
	internal static void InternalZipFiles(FileReference ZipFileName, DirectoryReference BaseDirectory, FileFilter Filter, int CompressionValue = 0)
	{
		// Bucket the 1-9 parameter values into three equal sized buckets, as this is what System.IO.Compression supports.
		CompressionLevel Level = CompressionValue switch
		{
			0 => CompressionLevel.NoCompression,
			< 4 => CompressionLevel.Fastest,
			< 7 => CompressionLevel.Optimal,
			_ => CompressionLevel.SmallestSize,
		};

		CreateDirectory(Path.GetDirectoryName(ZipFileName.FullName));
		using ZipArchive Zip = ZipFile.Open(ZipFileName.FullName, ZipArchiveMode.Create);

		foreach (FileReference FilteredFile in Filter.ApplyToDirectory(BaseDirectory, true))
		{
			Zip.CreateEntryFromFile(FilteredFile.FullName, FilteredFile.MakeRelativeTo(BaseDirectory), Level);
		}
	}

	/// <summary>
	/// Extracts the contents of a zip file
	/// </summary>
	/// <param name="ZipFileName">Name of the zip file</param>
	/// <param name="BaseDirectory">Output directory</param>
	/// <returns>List of files written</returns>
	public static IEnumerable<string> InternalUnzipFiles(string ZipFileName, string BaseDirectory)
	{
		// Manually extract the files. Leaving this code as is to maintain the custom permission fixup.
		using ZipArchive Zip = ZipFile.OpenRead(ZipFileName);
		List<string> OutputFileNames = [];
		foreach (ZipArchiveEntry Entry in Zip.Entries.Where(e => !string.IsNullOrEmpty(e.Name)))
		{
			string OutputFileName = Path.Combine(BaseDirectory, Entry.FullName);
			Directory.CreateDirectory(Path.GetDirectoryName(OutputFileName));

			using (FileStream OutputStream = new(OutputFileName, FileMode.Create, FileAccess.Write))
			{
				using Stream EntryStream = Entry.Open();
				EntryStream.CopyTo(OutputStream);
			}

			if (!OperatingSystem.IsWindows() && IsProbablyAMacOrIOSExe(OutputFileName))
			{
				FixUnixFilePermissions(OutputFileName);
			}
			OutputFileNames.Add(OutputFileName);
		}
		return OutputFileNames;
	}
}



