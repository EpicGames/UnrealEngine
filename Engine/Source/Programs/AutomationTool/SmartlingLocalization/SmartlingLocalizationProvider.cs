// Copyright Epic Games, Inc. All Rights Reserved.
using System;
using System.Diagnostics;
using System.Collections.Generic;
using System.IO;
using System.Threading.Tasks;
using System.Text;
using System.Text.Json;
using System.Text.Json.Serialization;
using System.Net;
using System.Net.Http;
using System.Net.Http.Headers;
using AutomationTool;
using UnrealBuildTool;
using EpicGames.Localization;
using System.Threading;
using Microsoft.Extensions.Logging;

using static AutomationTool.CommandUtils;
using System.Linq;

#pragma warning disable SYSLIB0014

namespace EpicGames.SmartlingLocalization
{
	public class SmartlingConfig
	{
		public string ProjectId;
		public string UserId;
		public string UserSecret;
	}

	// A series of model classes to make deserialization of HTTP responses cleaner and easier 
	public class SmartlingResponseEnvelope<T>
	{
		public T Response { get; set; }
	}

	public class SmartlingDataEnvelope<T>
	{
		public T Data { get; set; }
	}

	public class SmartlingAuthenticationToken
	{
		public string AccessToken { get; set; }
		public int ExpiresIn { get; set; }
		public int RefreshExpiresIn { get; set; }
		public string RefreshToken { get; set; }
		public string TokenType { get; set; }
		public DateTime LastSuccessfulUpdateTime { get; set; }
	}

	public class SmartlingRequestError
	{
		public string Key { get; set; }
		public string Message { get; set; }
		
		public override string ToString()
		{
			return $"Key: {Key}\nMessage: {Message}";
		}
	}

	public class SmartlingRequestErrorsEnvelope
	{
		public List<SmartlingRequestError> Errors { get; set; }
	}

	public abstract class SmartlingLocalizationProvider : LocalizationProvider
	{
		public SmartlingLocalizationProvider(LocalizationProviderArgs InArgs)
			: base(InArgs)
		{
			if (String.IsNullOrEmpty(LocalizationBranchName))
			{
				Logger.LogWarning("LocalizationBranchName is null or empty. The branch name is used to create the correct file URIs for Smartling. Wrong files may be downloaded from Smartling and files may be clobbered in Smartling. Please append a value to LocalizationName as a command line argument to fix this.");
			}
			Config = new SmartlingConfig();
			Client = new HttpClient() { Timeout = TimeSpan.FromSeconds(1300) };
			JsonOptions = new JsonSerializerOptions();
			JsonOptions.PropertyNameCaseInsensitive = true;
			JsonOptions.DefaultIgnoreCondition = JsonIgnoreCondition.WhenWritingNull;
			AuthenticationToken = null;
		}

		// public override Task InitializeProjectWithLocalizationProvider(string ProjectName, ProjectImportExportInfo ProjectImportInfo)
		// {
		// 	// @TODOLocalization: Throw an exception if the config data is null/invalid 
		// }

		public async override Task DownloadProjectFromLocalizationProvider(string ProjectName, ProjectImportExportInfo ProjectImportInfo)
		{
			Logger.LogInformation($"Starting Smartling download for {ProjectName} project files.");
			Stopwatch Watch = Stopwatch.StartNew();
			List<Task> DownloadTasks = new List<Task>();
			// Get the latest files for each culture.
			foreach (var Culture in ProjectImportInfo.CulturesToGenerate)
			{
				// Skip the native culture as we don't need the original source from Smartling 
				if (Culture == ProjectImportInfo.NativeCulture)
				{
					continue;
				}

				DownloadTasks.Add(DownloadLatestPOFileAndLog(Culture, null, ProjectImportInfo));
				foreach (var Platform in ProjectImportInfo.SplitPlatformNames)
				{
					DownloadTasks.Add(DownloadLatestPOFileAndLog(Culture, Platform, ProjectImportInfo));
				}
			}
			await Task.WhenAll(DownloadTasks);
			Watch.Stop();
			Logger.LogInformation($"Completed Smartling download for {ProjectName} project files in {Watch.ElapsedMilliseconds / 1000.0:F2} seconds.");
		}

		private async Task DownloadLatestPOFileAndLog(string EpicLocale, string Platform, ProjectImportExportInfo ProjectImportInfo)
		{
			List<string> Logs = await DownloadLatestPOFile(EpicLocale, Platform, ProjectImportInfo);
			lock (LoggerLock)
			{
				foreach (string Log in Logs)
				{
					if (Log.StartsWith("[WARNING]"))
					{
						Logger.LogWarning(Log);
					}
					else
					{
						Logger.LogInformation(Log);
					}
				}
			}
		}

		private async Task<List<string>> DownloadLatestPOFile(string EpicLocale, string Platform, ProjectImportExportInfo ProjectImportInfo)
		{
			List<string> Logs = new List<string>();
			await GetAuthenticationToken(Logs);

			var DestinationDirectory = String.IsNullOrEmpty(Platform)
			? new DirectoryInfo(CommandUtils.CombinePaths(RootWorkingDirectory, ProjectImportInfo.DestinationPath))
			: new DirectoryInfo(CommandUtils.CombinePaths(RootWorkingDirectory, ProjectImportInfo.DestinationPath, ProjectImportExportInfo.PlatformLocalizationFolderName, Platform));
			var CultureDirectory = (ProjectImportInfo.bUseCultureDirectory) ? new DirectoryInfo(Path.Combine(DestinationDirectory.FullName, EpicLocale)) : DestinationDirectory;

			var ExportFile = new FileInfo(Path.Combine(CultureDirectory.FullName, ProjectImportInfo.PortableObjectName));
			// note that the Smartling Filename and Uri may be different 
			var SmartlingFilename = GetSmartlingFilename(ProjectImportInfo.PortableObjectName, Platform);

			Logs.Add($"Downloading: '{SmartlingFilename}' as '{ExportFile.FullName}' ({EpicLocale})");

			string SmartlingLocale = ConvertEpicLocaleToSmartlingLocale(EpicLocale);
			string DownloadEndpoint = $"https://api.smartling.com/files-api/v2/projects/{Config.ProjectId}/locales/{SmartlingLocale}/file";
			string SmartlingFileUri = GetSmartlingFileUri(SmartlingFilename);
			var DownloadUriBuilder = new UriBuilder(DownloadEndpoint);
			// Changing the retrieval type to pseudo is a good way to test downloads
			// Published here doesn't necessarily mean that the strings are done translating. The Smartling configuration allows publishing every step of the way in the pipeline. We are just approximating "pending" by leveraging the Smartling workflow istead of the API.
			string DownloadRetrievalType = "published";
			string DownloadQueryString = $"fileUri={SmartlingFileUri}&retrievalType={DownloadRetrievalType}&includeOriginalStrings=false";
			DownloadUriBuilder.Query = DownloadQueryString;

			HttpResponseMessage DownloadResponse = null;
			int MaxTries = 5;
			// 1 to account for the initial try 
			int CurrentTries = 1;
			// The base for which all the exponential back off will be derived from.
			int InitialTimeOut = 250;
			int CurrentTimeOut = CurrentTries * InitialTimeOut;
			// Measures how long it takes from sending a response to successfully getting a response 
			Stopwatch DownloadStopWatch = new Stopwatch();
			while (true)
			{
				CurrentTimeOut = CurrentTries * InitialTimeOut;
				Logs.Add($"Current time out for download response {CurrentTimeOut}s.");
				TimeSpan Timeout = TimeSpan.FromSeconds(CurrentTimeOut);
				using var CancellationToken = new CancellationTokenSource(Timeout);
				try
				{
					DownloadStopWatch.Start(); 
					DownloadResponse = await Client.GetAsync(DownloadUriBuilder.Uri, CancellationToken.Token);
					if (DownloadResponse.StatusCode == HttpStatusCode.Unauthorized)
					{
						++CurrentTries;
						if (CurrentTries > MaxTries)
						{
							DownloadStopWatch.Stop();
							Logs.Add($"[WARNING] Failed in downloading: '{ExportFile.FullName}' ({EpicLocale}) exhausted all retries after {DownloadStopWatch.ElapsedMilliseconds / 1000}.");
							return Logs;
						}
						Logs.Add($"Encountered HTTP Status Code 401. Authentication most likely expired. Retrying {CurrentTries}/{MaxTries} times with refreshed authentication token.");
						await GetAuthenticationToken(Logs);
						continue;
					}
					break;
				}
				catch (Exception Ex)
				{
					++CurrentTries;
					if (CurrentTries > MaxTries)
					{
						DownloadStopWatch.Stop();
						Logs.Add($"[WARNING] Failed in downloading: '{ExportFile.FullName}' ({EpicLocale}) exhausted all retries in {DownloadStopWatch.ElapsedMilliseconds / 1000.0:F2} seconds. - {Ex}");
						return Logs;
					}
					Logs.Add($"Failed to get download response. Retrying {CurrentTries}/{MaxTries} times.");
					// We need to retreive the authentication token again because after each timeout, we may exceed the validity of the authentication token 
					await GetAuthenticationToken(Logs);
				}
			}

			if (DownloadResponse.IsSuccessStatusCode)
			{
				if (!CultureDirectory.Exists)
				{
					Logger.LogInformation($"Culture directory for '{EpicLocale}' does not exist. Creating directory '{CultureDirectory.FullName}'");
					CultureDirectory.Create();
					CultureDirectory.Refresh();
				}
				// Do not initialize this to true. Further down we will do a check to see the previous state of the file and restore the read-only settings of the file.
				// Even if a file did not exist on disk yet and was downloaded by Smartling, fi this defaults to true, the file will be set to read-only
				// Newly downloaded files would be added by Perforce during commandlet execution. Files added by Perforce will NOT have their read-only status removed 
				// That only happens for perforce check outs. As such, adding a newly downloaded file that is read-only will cause commandlets that need to update the file to fail 
				bool ExportFileWasReadOnly = false;
				if (ExportFile.Exists)
				{
					// We're going to clobber the existing PO file, so make sure it's writable (it may be read-only if in Perforce)
					ExportFileWasReadOnly = ExportFile.IsReadOnly;
					ExportFile.IsReadOnly = false;
				}
				// Download the file 
				using (var DownloadStream = await DownloadResponse.Content.ReadAsStreamAsync())
				{
					using (var DownloadFileStream = ExportFile.OpenWrite())
					{
						await DownloadStream.CopyToAsync(DownloadFileStream);
					}
				}
				// We refresh to update the state of the ifle after it's downloaded 
				ExportFile.Refresh();
				DownloadStopWatch.Stop();
				Logs.Add($"[SUCCESS] Downloading: '{SmartlingFileUri}' as '{ExportFile.FullName}' ({EpicLocale}) size {ExportFile.Length / 1024.0:F2}kb in {DownloadStopWatch.ElapsedMilliseconds / 1000.0:F2} seconds.");
				// Reset the write status of the file
				if (ExportFileWasReadOnly)
				{
					ExportFile.IsReadOnly = true;
				}

				// Update the back-up copy so we can diff against what we got from Smartling, and what the gather commandlet produced
				// This can be used to verify if the Smartling download is corrupt or if our export file is corrupted.
				// Set the flag to true for debug. 
				bool bCreateBackupCopy = false;
				if (bCreateBackupCopy)
				{
					string ExportFileCopyPath = Path.Combine(ExportFile.DirectoryName, $"{Path.GetFileNameWithoutExtension(ExportFile.Name)}_FromSmartling{ExportFile.Extension}");
					Logs.Add($"Updating Smartling copy '{ExportFileCopyPath}'");
					var ExportFileCopy = new FileInfo(ExportFileCopyPath);

					var ExportFileCopyWasReadOnly = false;
					if (ExportFileCopy.Exists)
					{
						// We're going to clobber the existing PO file, so make sure it's writable (it may be read-only if in Perforce)
						ExportFileCopyWasReadOnly = ExportFileCopy.IsReadOnly;
						ExportFileCopy.IsReadOnly = false;
					}

					ExportFile.CopyTo(ExportFileCopy.FullName, true);
					// reset the read/write status 
					if (ExportFileCopyWasReadOnly)
					{
						ExportFileCopy.IsReadOnly = true;
					}
					// We don't upload the copy to p4 as it's no longer strictly necessary. Just use the file as a lcoal debug. 
				}
			}
			else
			{
				// The file may not currently exist in Smartling and will need to be uploaded first via the Upload step later on. 
				DownloadStopWatch.Stop();
				Logs.Add($"[FAILED] Downloading: '{ExportFile.FullName}' ({EpicLocale} in {DownloadStopWatch.ElapsedMilliseconds / 1000.0:F2} seconds. The file may need to be uploaded first.)");
				await AppendRequestErrorsToLog(DownloadResponse, Logs);
			}
			return Logs;
		}

		private async Task AppendRequestErrorsToLog(HttpResponseMessage Response, List<string> Logs)
		{
			string ResponseString = await Response.Content.ReadAsStringAsync();
			var ResponseEnvelope = JsonSerializer.Deserialize<SmartlingResponseEnvelope<SmartlingRequestErrorsEnvelope>>(ResponseString, JsonOptions);
			var RequestErrors = ResponseEnvelope.Response.Errors;
			foreach (SmartlingRequestError RequestError in RequestErrors)
			{
				Logs.Add($"Smartling Info:\n {RequestError.ToString()}");
			}
			
		}

		public async override Task UploadProjectToLocalizationProvider(string ProjectName, ProjectImportExportInfo ProjectExportInfo)
		{
			Logger.LogInformation($"Starting Smartling upload for {ProjectName} project files.");
			Stopwatch Watch = Stopwatch.StartNew();
			List<Task> UploadTasks = new List<Task>();
			// Upload the .po file for the native culture first
			UploadTasks.Add(UploadLatestPOFileAndLog(ProjectExportInfo.NativeCulture, null, ProjectExportInfo));

			foreach (var Platform in ProjectExportInfo.SplitPlatformNames)
			{
				UploadTasks.Add(UploadLatestPOFileAndLog(ProjectExportInfo.NativeCulture, Platform, ProjectExportInfo));
			}
			await Task.WhenAll(UploadTasks);

			// Uploading all cultures is a legacy behavior from Onesky that isn't currently needed within our workflow. We can support this if there is a need. 
			Watch.Stop();
			Logger.LogInformation($"Completed Smartling upload for {ProjectName} project files in {Watch.ElapsedMilliseconds / 1000.0:F2} seconds.");
		}

		private async Task UploadLatestPOFileAndLog(string EpicLocale, string Platform, ProjectImportExportInfo ProjectExportInfo)
		{
			List<string> Logs = await UploadLatestPOFile(EpicLocale, Platform, ProjectExportInfo);
			lock (LoggerLock)
			{
				foreach (string Log in Logs)
				{
					if (Log.StartsWith("[WARNING]"))
					{
						Logger.LogWarning(Log);
					}
					else
					{
						Logger.LogInformation(Log);
					}
					
				}
			}

		}

		private async Task<List<string>> UploadLatestPOFile(string EpicLocale, string Platform, ProjectImportExportInfo ProjectExportInfo)
		{
			List<string> Logs = new List<string>();
			await GetAuthenticationToken(Logs);

			var SourceDirectory = String.IsNullOrEmpty(Platform)
				? new DirectoryInfo(CommandUtils.CombinePaths(RootWorkingDirectory, ProjectExportInfo.DestinationPath))
				: new DirectoryInfo(CommandUtils.CombinePaths(RootWorkingDirectory, ProjectExportInfo.DestinationPath, ProjectImportExportInfo.PlatformLocalizationFolderName, Platform));
			var CultureDirectory = (ProjectExportInfo.bUseCultureDirectory) ? new DirectoryInfo(Path.Combine(SourceDirectory.FullName, EpicLocale)) : SourceDirectory;
			string FileToUploadPath = Path.Combine(CultureDirectory.FullName, ProjectExportInfo.PortableObjectName);
			var FileToUpload = new FileInfo(FileToUploadPath);
			if (!FileToUpload.Exists)
			{
				Logs.Add($"Unable to upload '{FileToUploadPath}'. File does not exist.");
				return Logs;
			}
			string SmartlingFilename = GetSmartlingFilename(ProjectExportInfo.PortableObjectName, Platform);
			string SmartlingFileUri = GetSmartlingFileUri(SmartlingFilename);
			bool bIsNative = EpicLocale == ProjectExportInfo.NativeCulture;

			Logs.Add($"Uploading: '{FileToUpload.FullName}' as '{SmartlingFileUri}' ({EpicLocale})");
			// For now we only upload the file in the native locale
			// It is dissuaded to import translations to Smartling. Perform all translations in the Smartling dashboard instead.

			string UploadEndpoint = $"https://api.smartling.com/files-api/v2/projects/{Config.ProjectId}/file";

			using var UploadMultipartFormDataContent = new MultipartFormDataContent();
				var UploadFileStreamContent = new StreamContent(File.OpenRead(FileToUpload.FullName));
			// generic binary stream we will simply consider the octet-stream mime type 
			UploadFileStreamContent.Headers.ContentType = new MediaTypeHeaderValue("application/octet-stream");
			UploadMultipartFormDataContent.Add(UploadFileStreamContent, "file", SmartlingFileUri);
			UploadMultipartFormDataContent.Add(new StringContent(SmartlingFileUri, Encoding.UTF8, "application/json"), "fileUri");
			UploadMultipartFormDataContent.Add(new StringContent("gettext", Encoding.UTF8, "application/json"), "fileType");
			// We introduce a Smartling namespace to leverage Smartling's string sharing feature
			// Following Smartling best practices, we make the Smartling namespace the same as the full file path of the file.
			// https://help.smartling.com/hc/en-us/articles/360008143833-String-Sharing-and-Namespaces-via-Smartling-API
			UploadMultipartFormDataContent.Add(new StringContent(SmartlingFilename, Encoding.UTF8, "application/json"), "smartling.namespace");
			// all placeholder values in the source files will raise warnings in Smartling which need to be manually reviewed.
			// This allows the warnings to be resolved. The regex designates anything within {} to be a placeholder value.
			// This accounts for the common FText formatted string placeholders of {0} or {MyPlaceholderVariable}
			UploadMultipartFormDataContent.Add(new StringContent("\\{([^}]+)\\}", Encoding.UTF8, "application/json"), "smartling.placeholder_format_custom");
			// Controls whether base characters ( > < & " ) are "escaped" into entities 
			// We set this as false so that we don't automatically escape and have Smartling think these are HTML tags or something.
			// https://help.smartling.com/hc/en-us/articles/360007894594-Gettext-PO-POT
			UploadMultipartFormDataContent.Add(new StringContent("false", Encoding.UTF8, "application/json"), "smartling.entity_escaping");

			HttpResponseMessage UploadResponse = null;
			int CurrentTries = 1;
			int MaxTries = 5;
			// @TODOLocalization: Doing retries with exponential backoff won't work great with uploads as we'll run into HTTP 402 errors where the resource is locked. Need to find a way around that.
			// For now, we don't run into issues of timing out or failure to authenticate, so we're fine. But this isn't correct  
			while (true)
			{
				try
				{
					UploadResponse = await Client.PostAsync(UploadEndpoint, UploadMultipartFormDataContent);
					if (UploadResponse.StatusCode == HttpStatusCode.Unauthorized)
					{
						++CurrentTries;
						if (CurrentTries > MaxTries)
						{
							Logs.Add($"[WARNING] Failed in uploading: '{FileToUpload.FullName}' ({EpicLocale}). Exhausted all retries.");
							return Logs;
						}
						Logs.Add($"Encountered HTTP Status Code 401. Authentication token most likely expired. Retrying {CurrentTries}/{MaxTries} times with refreshed authentication token.");
						continue;
					}
					break;
				}
				catch (Exception Ex)
				{
					Logs.Add($"[WARNING] Failed in uploading: '{FileToUpload.FullName}' ({EpicLocale}) - {Ex}");
					return Logs;
				}
			}

			if (UploadResponse.IsSuccessStatusCode)
			{
				Logs.Add($"[SUCCESS] Uploading: '{FileToUpload.FullName}' ({EpicLocale})");
			}
			else
			{
				Logs.Add($"[FAILED] Failed in uploading: '{FileToUpload.FullName}' ({EpicLocale})");
				await AppendRequestErrorsToLog(UploadResponse, Logs);
			}
			return Logs;
		}

		// Override in child classes if there is a different set of languages that you need translations for 
		virtual protected Dictionary<string,string> GetEpicLocaleToSmartlingLocaleDictionary()
		{
			// Smartling mostly uses the same language codes as Unreal, however there are some differences
			// Login to the Smartling dashboard and check the languages you are translating to for each project. 
			return new Dictionary<string, string>
			{
				{ "es",  "es-ES" },	// Spain Spanish
				{ "es-419",  "es-MX" },	// Mexico Spanish
				{ "zh-Hans", "zh-CN" },	// Simp Chinese
				{ "zh-Hant", "zh-TW" },	// Trad Chinese
			};
		}

		private string ConvertEpicLocaleToSmartlingLocale(string EpicLocale)
		{
			string SmartlingLocale = null;
			var CultureMappings = GetEpicLocaleToSmartlingLocaleDictionary();
			if (CultureMappings.TryGetValue(EpicLocale, out SmartlingLocale))
			{
				return SmartlingLocale;
			}
			return EpicLocale;
		}

		private string GetSmartlingFilename(string BaseFilename, string Platform)
		{
			string SmartlingFilename = BaseFilename;
			if (!String.IsNullOrEmpty(Platform))
			{
				// Apply the platform suffix.
				string SmartlingFilenameWithSuffix = Path.GetFileNameWithoutExtension(SmartlingFilename) + "_" + Platform + Path.GetExtension(SmartlingFilename);
				SmartlingFilename = SmartlingFilenameWithSuffix;
			}
			if (!String.IsNullOrEmpty(RemoteFilenamePrefix))
			{
				// Apply the prefix (this is used to avoid collisions with plugins that use the same name for their PO files)
				// E.g Say you have both PluginA and PluginB with a loc target called Plugin. This would cause the remote names to be PluginA_Plugin and PluginB_Plugin rather than conflict.
				SmartlingFilename = RemoteFilenamePrefix + "_" + SmartlingFilename;
			}
			return SmartlingFilename;
		}

		private string GetSmartlingFileUri(string SmartlingFilename)
		{
			// The LocalizationBranchName variable is populated by the LocalizationBranch command line argument 
			// It needs to be valid to ensure that the correct file is interacted with on Smartling. 
			// We already raise a warning in the constructor, but we dont' crash as we don't want the automation jobs to end with localization
			if (!String.IsNullOrEmpty(LocalizationBranchName))
			{
				// Apply the localization branch suffix.
				string SmartlingFilenameWithSuffix = Path.GetFileNameWithoutExtension(SmartlingFilename) + "_" + LocalizationBranchName + Path.GetExtension(SmartlingFilename);
				return SmartlingFilenameWithSuffix;
			}
			return SmartlingFilename;
		}

		private async Task GetAuthenticationToken(List<string> Logs)
		{
			// If the token is still valid, we just return. Otherwise we either refresh or authenticate again.

		
			// Any retreival or modification of the authentication token must be protected by this semaphore
			// Calls to RequestAuthenticationToken() and RefreshAuthenticationToken() are also protected by this semaphore.
			//  Do not call RequestAuthenticationToken() or RefreshAuthenticationToken() by themselves as they are not thread safe.
			await AuthenticationTokenSemaphore.WaitAsync();
			try
			{
				// If the last successful update + time to expire is still less than our current time accounting for some slack, we don't need to do anything to the token.
				// We account for slack as it makes little sense to try authenticating with only 1s of validity left on a busy network
				int Slack = 30;
				if (AuthenticationToken != null && AuthenticationToken.LastSuccessfulUpdateTime.AddSeconds(AuthenticationToken.ExpiresIn - Slack) > DateTime.UtcNow)
				{
					Logs.Add("Authentication token still valid. Using current authentication token.");
					return;
				}

				if (AuthenticationToken != null && AuthenticationToken.LastSuccessfulUpdateTime.AddSeconds(AuthenticationToken.RefreshExpiresIn) > DateTime.UtcNow)
				{
					try
					{
						await RefreshAuthenticationToken(Logs);
					}
					catch (Exception)
					{
						await RequestAuthenticationToken(Logs);
					}
				}
				else
				{
					await RequestAuthenticationToken(Logs);
				}
			}
			finally
			{
				AuthenticationTokenSemaphore.Release();
			}
		}

		/// <summary>
		/// Requests a new authentication token from the Smartling endpoints. This call is not threadsafe and should ONLY be used from GetAuthenticationToken() where the AuthenticationToken instane is protected by syncrhonization primitives. Do not call this function directly.
		/// </summary>
		/// <param name="Logs"></param>
		/// <returns></returns>
		private async Task RequestAuthenticationToken(List<string> Logs)
		{
			string AuthenticateEndpoint = "https://api.smartling.com/auth-api/v2/authenticate";
			// serialize from Dictionary as MultipartFormDataContent not supported as a content-type 
			var AuthenticateRequestContent = new Dictionary<string, string>()
			{
				{ "userIdentifier", Config.UserId},
				{ "userSecret", Config.UserSecret}
			};
			var AuthenticateRequestContentJson = JsonSerializer.Serialize(AuthenticateRequestContent, JsonOptions);
			var AuthenticateRequestStringContent = new StringContent(AuthenticateRequestContentJson, Encoding.UTF8, "application/json");
			try
			{
				var AuthenticateResponse = Client.PostAsync(AuthenticateEndpoint, AuthenticateRequestStringContent).Result;
				if (AuthenticateResponse.IsSuccessStatusCode)
				{
					string AuthenticateResponseString = await AuthenticateResponse.Content.ReadAsStringAsync();
					var AuthenticateResponseEnvelope = JsonSerializer.Deserialize<SmartlingResponseEnvelope<SmartlingDataEnvelope<SmartlingAuthenticationToken>>>(AuthenticateResponseString, JsonOptions);
					AuthenticationToken = AuthenticateResponseEnvelope.Response.Data;
					AuthenticationToken.LastSuccessfulUpdateTime = DateTime.UtcNow;
					Client.DefaultRequestHeaders.Authorization = new AuthenticationHeaderValue(AuthenticationToken.TokenType, AuthenticationToken.AccessToken);
					Logs.Add("Successfully retrieved authentication token!");
				}
				else
				{
					Logs.Add("Failed to retrieve authentication token.");
					await AppendRequestErrorsToLog(AuthenticateResponse, Logs);
				}
			}
			catch (Exception Ex)
			{
				Logs.Add($"Failed to retrieve authentication token. {Ex}");
			}
		}

		/// <summary>
		/// Refreshes the authentication token using the Smarglin endpoint. This function is not thread safe and should ONLY be called by GetAuthenticationToken() where synchronization is used to protect the AuthenticationToken instance. Do not call this function directly, use GetAuthenticationToken() isntead.
		/// </summary>
		/// <param name="Logs"></param>
		/// <returns></returns>
		private async Task RefreshAuthenticationToken(List<string> Logs)
		{
			string RefreshEndpoint = "https://api.smartling.com/auth-api/v2/authenticate/refresh"; 
			// serialize from Dictionary as MultipartFormDataContent not supported as a content-type 
			var RefreshRequestContent = new Dictionary<string, string>()
			{
				{ "refreshToken", AuthenticationToken.RefreshToken}
			};
			var RefreshRequestContentJson = JsonSerializer.Serialize(RefreshRequestContent, JsonOptions);
			var RefreshRequestStringContent = new StringContent(RefreshRequestContentJson, Encoding.UTF8, "application/json");
			try
			{
				var RefreshResponse = Client.PostAsync(RefreshEndpoint, RefreshRequestStringContent).Result;
				if (RefreshResponse.IsSuccessStatusCode)
				{
					string RefreshResponseString = await RefreshResponse.Content.ReadAsStringAsync();
					var RefreshResponseEnvelope = JsonSerializer.Deserialize<SmartlingResponseEnvelope<SmartlingDataEnvelope<SmartlingAuthenticationToken>>>(RefreshResponseString, JsonOptions);
					AuthenticationToken = RefreshResponseEnvelope.Response.Data;
					AuthenticationToken.LastSuccessfulUpdateTime = DateTime.UtcNow;
					Client.DefaultRequestHeaders.Authorization = new AuthenticationHeaderValue(AuthenticationToken.TokenType, AuthenticationToken.AccessToken);
					Logger.LogInformation("Successfully refreshed authentication token!");
				}
				else
				{
					Logs.Add("Failed to refresh authentication token.");
					await AppendRequestErrorsToLog(RefreshResponse, Logs);
				}
			}
			catch (Exception Ex)
			{
				Logs.Add($"Failed to refresh authentication token. {Ex}");
			}
		}

		protected SmartlingConfig Config;
		protected HttpClient Client;
		private JsonSerializerOptions JsonOptions;
		private SmartlingAuthenticationToken AuthenticationToken;
		private readonly SemaphoreSlim AuthenticationTokenSemaphore= new SemaphoreSlim(1, 1);
		private readonly Object LoggerLock = new object();
	}
}
