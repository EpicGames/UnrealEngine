// Copyright Epic Games, Inc. All Rights Reserved.
using System;
using System.Collections.Generic;
using System.Linq;
using System.IO;
using AutomationTool;
using UnrealBuildTool;
using EpicGames.Core;
using UnrealBuildBase;
using Microsoft.Extensions.Logging;

using static AutomationTool.CommandUtils;
using IdentityModel.OidcClient;

public static class SteamDeckSupport
{
	static string RSyncPath = Path.Combine(Unreal.RootDirectory.FullName, "Engine\\Extras\\ThirdPartyNotUE\\cwrsync\\bin\\rsync.exe");
	static string SSHPath = Path.Combine(Unreal.RootDirectory.FullName, "Engine\\Extras\\ThirdPartyNotUE\\cwrsync\\bin\\ssh.exe");
	static string DevKitRSAPath = Path.Combine(Environment.GetFolderPath(Environment.SpecialFolder.LocalApplicationData), @"steamos-devkit\steamos-devkit\devkit_rsa");
	static string KnownHostsPath = Path.Combine(Environment.GetFolderPath(Environment.SpecialFolder.UserProfile), @".ssh\known_hosts");
	static string SteamDeckScripts = Path.Combine(Unreal.RootDirectory.FullName, "Engine\\Build\\SteamDeck");
	static string DefaultUserName = "deck";
	
	#region Devices

	public static List<DeviceInfo> GetDevices(UnrealTargetPlatform RuntimePlatform)
	{
		List<DeviceInfo> Devices = new List<DeviceInfo>();

		// Look for any Steam Deck devices that are in the Engine ini files. If so lets add them as valid devices
		// This will be required for matching devices passed into the BuildCookRun command
		ConfigHierarchy EngineConfig = ConfigCache.ReadHierarchy(ConfigHierarchyType.Engine, null, UnrealTargetPlatform.Win64);

		List<string> SteamDeckDevices;
		if (EngineConfig.GetArray("SteamDeck", "SteamDeckDevice", out SteamDeckDevices))
		{
			// Expected ini format: +SteamDeckDevice=(IpAddr=10.1.33.19,Name=MySteamDeck,UserName=deck)
			foreach (string DeckDevice in SteamDeckDevices)
			{
				string IpAddr = ConfigHierarchy.GetStructEntry(DeckDevice, "IpAddr", false);
				string DeviceName = ConfigHierarchy.GetStructEntry(DeckDevice, "Name", false);
				string ConfigUserName = ConfigHierarchy.GetStructEntry(DeckDevice, "UserName", false);

				// As of SteamOS version 3.1, "deck" is the required UserName to be used when making a remote connection
				// to the device. Eventually it will be configurable, so we will allow users to set it via the config.
				string UserName = string.IsNullOrEmpty(ConfigUserName) ? DefaultUserName : ConfigUserName;

				if (RuntimePlatform == UnrealTargetPlatform.Win64)
				{
					DeviceName += " (Proton)";
				}
				else if (RuntimePlatform == UnrealTargetPlatform.Linux)
				{
					DeviceName += " (Native Linux)";
				}

				// Name is optional, if its empty/not found lets just use the IpAddr for the Name
				if (string.IsNullOrEmpty(DeviceName))
				{
					DeviceName = "[SteamDeck] " + IpAddr;
				}

				if (!string.IsNullOrEmpty(IpAddr))
				{
					// TODO Fix the usage of OSVersion here. We are abusing this and using MS OSVersion to allow Turnkey to be happy
					DeviceInfo SteamDeck = new DeviceInfo(RuntimePlatform, DeviceName, IpAddr,
						Environment.OSVersion.Version.ToString(), "SteamDeck", true, true);
					SteamDeck.PlatformValues["UserName"] = UserName;

					Devices.Add(SteamDeck);
				}
				else
				{
					Logger.LogError("You must specify the 'IpAddr' field to connect to a SteamDeck!");
				}
			}
		}

		return Devices;
	}

	/**
	 * Get the ipaddr and username of the device from the params and/or device registry
	 */
	public static bool GetDeviceInfo(UnrealTargetPlatform RuntimePlatform, ProjectParams Params, out string IpAddr, out string UserName)
	{
		if (Params.DeviceNames.Count != 1)
		{
			Logger.LogWarning("SteamDeck deployment requires 1, and only, device");
			IpAddr = UserName = null;
			return false;
		}

		// look up in the devices in the registry based on the params (commandline -device=)
		List<DeviceInfo> Devices = GetDevices(RuntimePlatform);
		// assume DeviceName is an Id (ie IpAddr)
		string DeviceId = Params.DeviceNames[0];
		DeviceInfo Device = Devices.FirstOrDefault(x => x.Id.Equals(DeviceId, StringComparison.InvariantCultureIgnoreCase));
		// if the id didn't match, fallback to trying name match
		if (Device == null)
		{
			Device = Devices.FirstOrDefault(x => x.Name.Equals(DeviceId, StringComparison.InvariantCultureIgnoreCase));
		}

		// if the device is still null, dont assume this is a valid steam deck device
		if (Device == null)
		{
			IpAddr = UserName = null;
			return false;
		}

		IpAddr = Device.Id;

		// if -deviceuser was specified, prefer that, otherwise use what's in the registry
		UserName = Params.DeviceUsername;
		if (string.IsNullOrEmpty(UserName))
		{
			UserName = Device?.PlatformValues["UserName"];
		}

		if (string.IsNullOrEmpty(UserName) || string.IsNullOrEmpty(IpAddr))
		{
			Logger.LogWarning("Unable to discover Ip Adress and Username for your Steamdeck. Use -device to specify device, and if that device is not in your Engine.ini, then you will need to specify -deviceuser= to set the username.");
			return false;
		}

		return true;
	}

	#endregion

	#region Deploying

	private static string GetRegisterGameScript(ProjectParams Params, UnrealTargetPlatform RuntimePlatform, string GameFolderPath, string GameRunArgs, string MsvsmonVersion)
	{
		FileReference ExePath = Params.GetProjectExeForPlatform(RuntimePlatform);
		string RelGameExePath = ExePath.MakeRelativeTo(DirectoryReference.Combine(ExePath.Directory, "../../..")).Replace('\\', '/');
		string GameId = $"{Params.ShortProjectName}_{RuntimePlatform}";

		// create a script that will be copied over to the SteamDeck and ran to register the game
		// TODO make these easier to customize, vs hard coding the settings. Assume debugging for now, requires the user to have uploaded the required msvsmom/remote debugging stuff
		// which is done through uploading any game with debugging enabled through the SteamOS Devkit Client
		string[] LinuxSettings =
		{
			$"\"steam_play\": \"0\"",
		};
		string[] ProtonSettings =
		{
			$"\"steam_play\": \"1\"",
			$"\"steam_play_debug\": \"1\"",
			$"\"steam_play_debug_version\": \"{MsvsmonVersion}\"",
		};
		string JoinedSettings = string.Join(", ", RuntimePlatform == UnrealTargetPlatform.Linux ? LinuxSettings : ProtonSettings);

		string[] Parms =
		{
			$"\"gameid\":\"{GameId}\"",
			$"\"directory\":\"{GameFolderPath}\"",
			$"\"argv\":[\"{RelGameExePath} {GameRunArgs}\"]",
			$"\"settings\": {{{JoinedSettings}}}",
		};
		string JoinedParams = string.Join(", ", Parms);

		// combine all the settings
		return $"#!/bin/bash\nrm ~/devkit-game/{GameId}-settings.json\npython3 ~/devkit-utils/steam-client-create-shortcut --parms '{{{JoinedParams}}}'";
	}

	private static string GetRunGameScript(ProjectParams Params, UnrealTargetPlatform RuntimePlatform)
	{
		string GameId = $"{Params.ShortProjectName}_{RuntimePlatform}";

		// NOTE this path is setup based on our steam-client-run-game script, if this changes other places need to change
		return $"#!/bin/bash\npython3 ~/epic-scripts/steam-client-run-game --parms '{{\"gameid\":\"{GameId}\"}}'";
	}

	// Rsync is a Cygwin utility, so convert windows paths to a path that Cygwin Rsync will understand
	// This will not work with UNC paths
	public static string FixupHostPath(string HostPath)
	{
		if (HostPlatform.Platform == UnrealTargetPlatform.Win64)
		{
			string CygdrivePath = "";

			if (!string.IsNullOrEmpty(HostPath))
			{
				string FullPath = Path.GetFullPath(HostPath);
				string RootPath = Path.GetPathRoot(FullPath);

				CygdrivePath = Path.Combine("/cygdrive", Char.ToLower(FullPath[0]).ToString(), FullPath.Substring(RootPath.Length));

				return CygdrivePath.Replace('\\', '/');
			}

			return CygdrivePath;
		}

		return HostPath;
	}

	private static IProcessResult Rsync(string SourceFolder, string DestFolder, string UserName, string IpAddr, string[] ExcludeList)
	{
		// make a standard set of options to pass to the rsync for auth's -e option
		string[] AuthOpts =
		{
			$"\"\"{SSHPath}\"\"",
			$"-o UserKnownHostsFile=\"\"{KnownHostsPath}\"\"",
			$"-o StrictHostKeyChecking=no",
			$"-i \"\"{DevKitRSAPath}\"\"",
		};
		
		string AuthOptions = $"-e \"{string.Join(" ", AuthOpts)}\"";

		// make a set of --exclude options for anything we want to exclude
		string ExcludeOptions = "";
		if (ExcludeList != null)
		{
			ExcludeOptions = string.Join(" ", ExcludeList.Select(x => $"--exclude={x}"));
		}

		string[] RsyncOpts =
		{
			// archive, verbose, human readable
			$"-avh",
			// ensure certain permissions on the files copied
			$"--chmod=Du=rwx,Dgo=rx,Fu=rwx,Fog=rx",
			// standard authorization options
			AuthOptions,
			// this is a trick to make sure the target path exists before running the remote rsync command
			$"--rsync-path=\"mkdir -p {DestFolder} && rsync\"",
			// copy/update files as needed
			$"--update",
			// delete destination files not in source
			$"--delete",
			// exclude some files
			ExcludeOptions,
			// source path, in cygwin-speak (/cygdrive/c/Program Files.....)
			$"'{FixupHostPath(SourceFolder)}/'",
			// destrination in standard 'user@ipaddr:destpath' format
			$"{UserName}@{IpAddr}:{DestFolder}",
		};

		// Exclude removing the Saved folders to preserve logs and crash data. Though note these will keep filling up with data
		return CommandUtils.Run(RSyncPath, string.Join(" ", RsyncOpts), "");
	}

	private static IProcessResult SSHCommand(string Command, string UserName, string IpAddr)
	{
		string SSHArgs = $"-i {DevKitRSAPath} {UserName}@{IpAddr}";
		return CommandUtils.Run(SSHPath, $"{SSHArgs} \"{Command}\"", "");
	}

	/* Deploying to a steam deck currently does 2 things
	 *
	 * 1) Generates a script CreateShortcutHelper.sh that will register the game on the SteamDeck once uploaded
	 * 2) Uploads the build using rsync to the devkit-game location. Once uploaded it runs the CreateShortcutHelper.sh generated before.
	 */
	public static void Deploy(UnrealTargetPlatform RuntimePlatform, ProjectParams Params, DeploymentContext SC)
	{
		string IpAddr, UserName;
		if (GetDeviceInfo(RuntimePlatform, Params, out IpAddr, out UserName) == false)
		{
			return;
		}

		string GameFolderPath = $"/home/{UserName}/devkit-game/{Params.ShortProjectName}_{RuntimePlatform}";
		string GameRunArgs = $"{SC.ProjectArgForCommandLines} {Params.StageCommandline} {Params.RunCommandline}".Replace("\"", "\\\"");

		if (!File.Exists(DevKitRSAPath))
		{
			Logger.LogWarning("Unable to find '{DevKitRSAPath}' rsa key needed to deploy to the steam deck. Make sure you've installed the SteamOS Devkit client", DevKitRSAPath);
			return;
		}

		// copy debugger support files if needed
		string MsvsmonVersion = "";
		if (RuntimePlatform == UnrealTargetPlatform.Win64)
		{
			if (DeployMsvsmon(IpAddr, UserName, out MsvsmonVersion) == false)
			{
				return;
			}
		}

		// write a create shortcut and run game script into the stage dir so it will be copied with out project. Note this is required due to the json
		// strings requiring a single quote, which our run command replaces with a double quote due to a dotnet limitation.
		string CreateShortcutScriptFileName = "CreateShortcutHelper.sh";
		string CreateShortcutScriptFile = Path.Combine(SC.StageDirectory.FullName, CreateShortcutScriptFileName);
		File.WriteAllText(CreateShortcutScriptFile, GetRegisterGameScript(Params, RuntimePlatform, GameFolderPath, GameRunArgs, MsvsmonVersion));

		string RunGameScriptFileName = "RunGameHelper.sh";
		string RunGameScriptFile = Path.Combine(SC.StageDirectory.FullName, RunGameScriptFileName);
		File.WriteAllText(RunGameScriptFile, GetRunGameScript(Params, RuntimePlatform));

		// copy the staged directory, skipping over huge debug files that we don't need on the target
		IProcessResult Result = Rsync(SC.StageDirectory.FullName, GameFolderPath, UserName, IpAddr, new string[] { /*"*.debug", */ "*.pdb", "Saved/" });

		if (Result.ExitCode > 0)
		{
			Logger.LogWarning("Failed to rsync the {Arg0} to {GameFolderPath}. Check connection on ip Check connection on ip {UserName}@{IpAddr}", SC.StageDirectory.FullName, GameFolderPath, UserName, IpAddr);
			return;
		}

		// copy any scripts we have SteamDeckScripts to our custom epic script location
		string DevkitUtilsPath = $"/home/{UserName}/epic-scripts/";
		Result = Rsync(SteamDeckScripts, DevkitUtilsPath, UserName, IpAddr, new string[] {});

		if (Result.ExitCode > 0)
		{
			Logger.LogWarning("Failed to rsync the {SteamDeckScripts} to {DevkitUtilsPath}. Check connection on ip Check connection on ip {UserName}@{IpAddr}", SteamDeckScripts, DevkitUtilsPath, UserName, IpAddr);
			return;
		}

		// Run the script to register the game with the Deck
		Result = SSHCommand($"chmod +x {GameFolderPath}/{CreateShortcutScriptFileName} && {GameFolderPath}/{CreateShortcutScriptFileName}", UserName, IpAddr);

		if (Result.ExitCode > 0)
		{
			Logger.LogWarning("Failed to run the {CreateShortcutScriptFileName}.sh script. Check connection on ip Check connection on ip {UserName}@{IpAddr}", CreateShortcutScriptFileName, UserName, IpAddr);
			return;
		}
	}

	private static bool DeployMsvsmon(string IpAddr, string UserName, out string MsvsmonVersion)
	{
		// get the path to some compiler's Msvsmon/RemoteDebugger support files to copy over
		string MsvsmonPath = null;
		MsvsmonVersion = "2022";
		IEnumerable<DirectoryReference> InstallDirs = WindowsExports.TryGetVSInstallDirs(WindowsCompiler.VisualStudio2022);
		if (InstallDirs != null)
		{
			MsvsmonPath = Path.Combine(InstallDirs.First().FullName, "Common7/IDE/Remote Debugger");
		}
		string TargetMsvsmonPath = $"/home/{UserName}/devkit-msvsmon/msvsmon{MsvsmonVersion}";

		IProcessResult Result = Rsync(MsvsmonPath, TargetMsvsmonPath, UserName, IpAddr, null);
		if (Result.ExitCode > 0)
		{
			Logger.LogWarning("Failed to rsync debugger support files to the SteamDeck. Check connection on ip {UserName}@{IpAddr}", UserName, IpAddr);
			return false;
		}

		return true;
	}

	#endregion

	#region Running

	public static IProcessResult RunClient(UnrealTargetPlatform RuntimePlatform, CommandUtils.ERunOptions ClientRunFlags, string ClientApp, string ClientCmdLine, ProjectParams Params)
	{
		string IpAddr, UserName;
		if (GetDeviceInfo(RuntimePlatform, Params, out IpAddr, out UserName) == false)
		{
			return null;
		}

		if (RuntimePlatform == UnrealTargetPlatform.Win64)
		{
			// Run msvsmoninstall.py in order to copy remote debuggers files in the right folders
			IProcessResult Result = SSHCommand("python3 ~/devkit-msvsmon/msvsmoninstall.py", UserName, IpAddr);

			if (Result.ExitCode > 0)
			{
				Logger.LogWarning("Failed to run msvsmoninstall.py");
				return Result;
			}
		}

		string GameFolderPath = $"/home/{UserName}/devkit-game/{Params.ShortProjectName}_{RuntimePlatform}";
		string RunGameScriptFileName = "RunGameHelper.sh";

		// TODO would be great if we could tail the log while running. Figure out how to cancel/exit app
		return SSHCommand($"chmod +x {GameFolderPath}/{RunGameScriptFileName} && {GameFolderPath}/{RunGameScriptFileName}", UserName, IpAddr);
	}

	#endregion
}
