// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Linq;
using System.Text;
using System.Threading.Tasks;

namespace Turnkey
{
	public class BuildSource
	{
 		public string Project;
		public CopyAndRun[] Sources;
		public string BuildEnumerationSuffix;
		// 		string[] Platforms;
		// 		string Project;


		internal void PostDeserialize()
		{
			if (Sources != null)
			{
				Array.ForEach(Sources, x => x.PostDeserialize());
			}
		}
	}
}
