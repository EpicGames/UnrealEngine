// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Linq;
using System.Text;
using System.Threading.Tasks;
using EpicGames.Core;
using UnrealBuildTool;
using UnrealBuildBase;

namespace Turnkey.Commands
{
	class ConfigTester : TurnkeyCommand
	{
		protected override CommandGroup Group => CommandGroup.Misc;

		protected override void Execute(string[] CommandOptions)
		{
			FileReference ProjectFile = TurnkeyUtils.GetProjectFromCommandLineOrUser(CommandOptions);
			List<UnrealTargetPlatform> Platforms = TurnkeyUtils.GetPlatformsFromCommandLineOrUser(CommandOptions, null);
			string BranchName = TurnkeyUtils.ParseParamValue("Branch", null, CommandOptions);
			string Section = TurnkeyUtils.ParseParamValue("Section", null, CommandOptions);
			string Key = TurnkeyUtils.ParseParamValue("Key", null, CommandOptions);

			ConfigHierarchyType Branch = Enum.Parse<ConfigHierarchyType>(BranchName);

			// get a list of builds from config
			foreach (UnrealTargetPlatform Platform in Platforms)
			{
				ConfigHierarchy Config = ConfigCache.ReadHierarchy(Branch, ProjectFile == null ? null : ProjectFile.Directory, Platform, IncludePluginsForTargetType:TargetType.Game);

				string Value;
				Config.GetString(Section, Key, out Value);

				TurnkeyUtils.Log($"Value[{Platform}]: {Value}");
			}
		}
	}
}
