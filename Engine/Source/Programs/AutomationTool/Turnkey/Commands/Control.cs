// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using AutomationTool;
using UnrealBuildTool;
using System.Linq;
using Gauntlet;

namespace Turnkey.Commands
{
	class Control : TurnkeyCommand
	{
		protected override CommandGroup Group => CommandGroup.Misc;

		private delegate void PerformOperation(ITargetDevice Device);

		protected override void Execute(string[] CommandOptions)
		{
			List<DeviceInfo> Devices = TurnkeyUtils.GetDevicesFromCommandLineOrUser(CommandOptions, null);
			if (Devices == null)
			{
				return;
			}

			Dictionary<string, PerformOperation> Ops = new Dictionary<string, PerformOperation>();
			Ops.Add("Connect", x => x.Connect());
			Ops.Add("Disconnect", x => x.Disconnect());
			Ops.Add("PowerOn", x => x.PowerOn());
			Ops.Add("PowerOff", x => x.PowerOff());
			Ops.Add("Reboot", x => x.Reboot());

			bool bWasScripted = false;
			// break if the GetGenericOption was scripted, otherwise we would loop forever
			while (!bWasScripted)
			{
				string Operation = TurnkeyUtils.GetGenericOption(CommandOptions, Ops.Keys.ToList(), "Operation", out bWasScripted);
				if (Operation == null)
				{
					return;
				}

				foreach (DeviceInfo Device in Devices)
				{
					TurnkeyUtils.Log($"Performing {Operation} on {Device.Platform}@{Device.Name}:");

					try
					{
						ITargetDevice GauntletDevice = TurnkeyGauntletUtils.GetGauntletDevice(Device);
						if (GauntletDevice != null)
						{
							Ops[Operation](GauntletDevice);
						}
					}
					catch (Exception Ex)
					{
						TurnkeyUtils.Log($"Operation {Operation} failed : {Ex.Message}\n");
					}
				}
			}
		}
	}
}
