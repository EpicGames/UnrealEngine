// Copyright Epic Games, Inc. All Rights Reserved.

using AutomationTool;
using System;
using System.Collections.Generic;
using System.Linq;
using System.IO;
using System.Text.RegularExpressions;
using EpicGames.Core;
using UnrealBuildTool;
using UnrealBuildBase;




// Here's how we manage commandlines for builds/configs/targets. It has complexity so writing down here for reference
//
// Involved options:
//		-client
//		-server
//			These denote Client or Server builds - both for executable and cooked data
//		-target=Foo 
//			Foo can be either say EngineTestClient, UnrealGame, etc. Note that UnrealGame is special for content only projects
//			This option has historically not been used, instead being calculated from the above options. However with 
//			more projects having multiple game/editor targets, it is useful to explicitly choose one of those targets
//		-clientconfig=[Debug|Development|Test|Shipping]
//			Used to choose a build configuration for client _and game_ executables
//		-serverconfig
//			Similar but for server executables
//
// In the case of conflicts (-client and -target=UnrealGame) UAT will fail, so we try to detect that earlier and disallow it
//
// Additionally, for code-based projects, 

// However, given how builds are likely to be triggered by the editor, which can let user change the target/config/etc, when
// ExecuteBuild handles the {ProjectPackagingSettings} replacement, it will first look at the existing commandline and not 
// spceify a conflicting mode (commandline overrides PackagingSettings). 
//  
//

namespace Turnkey.Commands
{
	internal class CreateBuild : TurnkeyCommand
	{
		protected override CommandGroup Group => CommandGroup.Builds;

		const string SettingsSection = "/Script/UnrealEd.ProjectPackagingSettings";

		internal static string[] SplitCommandLine(string CommandLine)
		{
			// handle -foo, -foo=bar, -foo="bar", and -foo="bar baz"
			Regex CommandLineRegex = new Regex("(-\\w*\\s+)|(-\\w*=\\w*\\s+)|(-\\w*=\"[^\"]*\"\\s +)");
			// split the args and clean up
			return CommandLineRegex.Split(CommandLine).Select(x => x.Trim()).Where(x => x != "").ToArray();
		}

		string GetArchiveDirectory(string[] CommandOptions, FileReference ProjectFile)
		{
			ConfigHierarchy Config = ConfigCache.ReadHierarchy(ConfigHierarchyType.Game, ProjectFile.Directory, BuildHostPlatform.Current.Platform);

			string StagingDirectory = Config.GetStructEntryForSetting(SettingsSection, "StagingDirectory", "Path");
			if (string.IsNullOrEmpty(StagingDirectory))
			{
				StagingDirectory = Path.Combine(ProjectFile.Directory.FullName, "Saved", "Archives");
			}

			string OutputDir = TurnkeyUtils.ParseParamValue("OutputDir", null, CommandOptions);
			if (OutputDir == null)
			{
				//if (OperatingSystem.IsWindows())
				//{
				//	string ChosenFile = null;
				//	System.Threading.Thread t = new System.Threading.Thread(x =>
				//	{
				//		ChosenFile = UnrealWindowsForms.Utils.ShowOpenFileDialogAndReturnFilename("Project Files (*.uproject)|*.uproject");
				//	});

				//	t.SetApartmentState(System.Threading.ApartmentState.STA);
				//	t.Start();
				//	t.Join();

				//	if (ChosenFile == null)
				//	{
				//		continue;
				//	}
				//}
				//else
				//{
				//	while (true)
				{
					OutputDir = TurnkeyUtils.ReadInput("Enter archival directory:", StagingDirectory);
				}
				//}

			}
			return OutputDir;
		}

		private static string GetPlatformSetting(ConfigHierarchy Config, UnrealTargetPlatform Platform, string PerPlatformKey, string GenericKey, string Default)
		{
			string IniPlatformName = ConfigHierarchy.GetIniPlatformName(Platform);
			string Value = Config.GetMapValueForSetting(SettingsSection, PerPlatformKey, Platform.ToString());
			if (Value == null && !string.IsNullOrEmpty(GenericKey))
			{
				Config.GetString(SettingsSection, GenericKey, out Value);
			}
			if (Value == null)
			{
				Value = Default;
			}

			return Value;
		}

		// hardcoded list of content-only, non-temp-target, targets
		static Dictionary<TargetType, string> UETargets = new Dictionary<TargetType, string>()
		{
			{ TargetType.Game, "UnrealGame" },
			{ TargetType.Client, "UnrealClient" },
			{ TargetType.Server, "UnrealServer" },
		};

		public static string MakeCommandLineFromPackagingSettings(UnrealTargetPlatform Platform, FileReference ProjectFile, string ExistingCommandLine)
		{
			ConfigHierarchy Config = ConfigCache.ReadHierarchy(ConfigHierarchyType.Game, ProjectFile.Directory, Platform);

			string[] ExistingOptions = SplitCommandLine(ExistingCommandLine);
			bool bBuild = CommandUtils.ParseParam(ExistingOptions, "-build");
			bool bCook = CommandUtils.ParseParam(ExistingOptions, "-cook") || CommandUtils.ParseParam(ExistingOptions, "-skipcook");
			bool bStage = CommandUtils.ParseParam(ExistingOptions, "-stage") || CommandUtils.ParseParam(ExistingOptions, "-skipstage");
			bool bPackage = CommandUtils.ParseParam(ExistingOptions, "-package") || CommandUtils.ParseParam(ExistingOptions, "-skippackage");
			bool bArchive = CommandUtils.ParseParam(ExistingOptions, "-archive");
			bool bClient = CommandUtils.ParseParam(ExistingOptions, "-client");
			bool bServer = CommandUtils.ParseParam(ExistingOptions, "-server");


			// for targetname and configuration, the priority is -overridetarget/config (coming from editor), then look in passed in
			// commandline, and finally look in the ini
			string IniTargetName = GetPlatformSetting(Config, Platform, "PerPlatformBuildTarget", "BuildTarget", null);
			string TargetName = CommandUtils.ParseParamValue(ExistingOptions, "-target", IniTargetName);
			TargetName = TurnkeyUtils.ParseParamValue("OverrideTarget", TargetName, new string[] { });

			ProjectProperties Props = ProjectUtils.GetProjectProperties(ProjectFile);
			List<SingleTargetProperties> DetectedTargets = Props.Targets;



			//if (CommandUtils.IsNullOrEmpty(DetectedTargets))
			//{
			//	DetectedTargets = new List<SingleTargetProperties>
			//	{
			//		new SingleTargetProperties { TargetName = "UnrealGame", Rules = new (new TargetInfo("UnrealGame", Platform, UnrealTargetConfiguration.Development, null)) }
			//	}
			//	ProjectParams Params = new ProjectParams(Command: TurnkeyUtils.CommandUtilHelper, RawProjectPath: ProjectFile, Client: true, DedicatedServer: true);
			//	DetectedTargets = Params.ProjectTargets;
			//}

			// figure out the target/targetype to use 
			TargetRules ChosenTarget = null;

			// first, handle the special case where the editor passed down a UETarget but UBT knows it needs a generated target - swap for the generated one
			if (Props.bWasGenerated && !string.IsNullOrEmpty(TargetName) && 
				UETargets.Values.Contains(TargetName, StringComparer.InvariantCultureIgnoreCase))
			{
				// generated projects likely only have one game target, so use it
				if (Props.Targets.Count == 1)
				{
					ChosenTarget = Props.Targets[0].Rules;
				}
				else
				{
					// get the type of the UE target
					TargetType UETargetType = UETargets.FirstOrDefault(x => x.Value.Equals(TargetName, StringComparison.InvariantCultureIgnoreCase)).Key;
					ChosenTarget = Props.Targets.FirstOrDefault(x => x.Rules.Type == UETargetType)?.Rules;
				}
			}

			if (ChosenTarget == null && !CommandUtils.IsNullOrEmpty(Props.Targets))
			{
				if (!string.IsNullOrEmpty(TargetName))
				{
					ChosenTarget = DetectedTargets.FirstOrDefault(x => x.TargetName.Equals(TargetName, StringComparison.InvariantCultureIgnoreCase))?.Rules;
					if (ChosenTarget == null)
					{
						throw new AutomationException($"The target specified on the commandline, {TargetName}, could not be found in project {ProjectFile}" +
							(Props.bWasGenerated ? $" Note that this project requires a temporary target, so {ProjectFile.GetFileNameWithoutAnyExtensions()} is probably the target you want." : ""));
					}
				}
				// if we don't have one now, find one based on params
				if (ChosenTarget == null)
				{
					ChosenTarget = DetectedTargets.FirstOrDefault(x => (bClient && x.Rules.Type == TargetType.Client) || (bServer && x.Rules.Type == TargetType.Server) || (!bClient && !bServer && x.Rules.Type == TargetType.Game))?.Rules;
				}
			}

			// now figure out the final type to use, with specified target, specified flags, and detected targets
			TargetType FinalTargetType = TargetType.Game;
			string FinalTargetName = "UnrealGame";
			if (ChosenTarget != null)
			{
				FinalTargetType = ChosenTarget.Type;
				FinalTargetName = ChosenTarget.Name;
			}
			else if (bClient || TargetName.Equals("UnrealClient", StringComparison.InvariantCultureIgnoreCase))
			{
				FinalTargetType = TargetType.Client;
				FinalTargetName = "UnrealClient";
			}
			else if (bServer || TargetName.Equals("UnrealServer", StringComparison.InvariantCultureIgnoreCase))
			{
				FinalTargetType = TargetType.Server;
				FinalTargetName = "UnrealServer";
			}

			// now that we have a target, we can use it to look on the commandline for server or clientconfig
			string IniBuildConfig = GetPlatformSetting(Config, Platform, "PerPlatformBuildConfig", "BuildConfiguration", "Development");
			string BuildConfig = CommandUtils.ParseParamValue(ExistingOptions, FinalTargetType == TargetType.Server ? "serverconfig" : "clientconfig", IniBuildConfig);
			BuildConfig = TurnkeyUtils.ParseParamValue("OverrideConfiguration", BuildConfig, new string[] { });
			BuildConfig = BuildConfig.Replace("PPBC_", "");

			string CommandLine = "";

			Func<string, bool> GetBool = (string Setting) =>
			{
				bool bValue;
				if (Config.GetBool(SettingsSection, Setting, out bValue))
				{
					return bValue;
				}
				return false;
			};

			Action<string, string> AddIfTrue = (string Setting, string ToAdd) =>
			{
				if (GetBool(Setting))
				{
					CommandLine += $" {ToAdd}";
				}
			};

			Action<string> AddIfNotAlready = (string ToAdd) =>
			{
				string[] Tokens = ToAdd.Split('=');
				// if we are a -option, look for mach, if we are a -foo= option, look for a -foo= option in the 
				if ((Tokens.Count() == 1 && !ExistingOptions.Contains(Tokens[0], StringComparer.InvariantCultureIgnoreCase)) ||
					(Tokens.Count() == 2 && !ExistingOptions.Any(x => x.StartsWith(Tokens[0] + "=", StringComparison.InvariantCultureIgnoreCase))))
				{
					CommandLine += $" {ToAdd}";
				}
			};


			switch (FinalTargetType)
			{
				case TargetType.Client:
					AddIfNotAlready($"-clientconfig={BuildConfig}");
					break;
				case TargetType.Server:
					AddIfNotAlready("-noclient");
					AddIfNotAlready($"-serverconfig={BuildConfig}");
					break;
				case TargetType.Game:
					AddIfNotAlready($"-clientconfig={BuildConfig}");
					break;
				default: throw new AutomationException($"TypeType {FinalTargetType} is not allowed for packaging at this time");
			}

			AddIfNotAlready($"-target={FinalTargetName}");

			if (bBuild)
			{
				AddIfTrue("FullRebuild", "-clean");
			}

			if (bCook)
			{
				if (GetBool("bUseIoStore"))
				{
					AddIfTrue("bUseZenStore", "-zenstore");
				}
				AddIfTrue("bSkipEditorContent", "-SkipCookingEditorContent");
			}
			
			if (bCook || bStage)
			{
				AddIfTrue("bGenerateChunks", "-manifests");
			}

			if (bCook || bStage || bPackage)
			{
				string CookFlavor = GetPlatformSetting(Config, Platform, "PerPlatformTargetFlavorName", null, null);
				CookFlavor = TurnkeyUtils.ParseParamValue("OverrideFlavor", CookFlavor, new string[] { });
				if (!string.IsNullOrEmpty(CookFlavor))
				{
					string IniPlatformName = ConfigHierarchy.GetIniPlatformName(Platform);
					// remove the Platform_ from the flavor if there, so Android_ASTC would become just Android
					CookFlavor = CookFlavor.Replace($"{IniPlatformName}_", "");
					AddIfNotAlready($"-cookflavor={CookFlavor}");
				}
			}

			if (bStage)
			{
				// zenstore precludes iostore when staging (it has already done the work)
				if (!GetBool("bUseZenStore"))
				{
					AddIfTrue("bUseIoStore", "-iostore -pak");
				}
				AddIfTrue("UsePakFile", "-pak");
				
				// more complex settings below
				if (Platform == UnrealTargetPlatform.Win64)
				{
					AddIfTrue("IncludePrerequisites", "-prereqs");

					//string AppLocalDirectory = ExecuteBuild.GetStructEntryForSetting(Config, SettingsSection, "ApplocalPrerequisitesDirectory", "Path");
					//if (!string.IsNullOrEmpty(AppLocalDirectory))
					//{
					//	CommandLine += $" -applocaldirectory=\"{AppLocalDirectory}";
					//}
					//else
					//{
					//	AddIfTrue("IncludeAppLocalPrerequisites", $" -applocaldirectory=\"$(EngineDir)/Binaries/ThirdParty/AppLocalDependencies\"");
					//}
				}

				DataDrivenPlatformInfo.ConfigDataDrivenPlatformInfo DDPI = DataDrivenPlatformInfo.GetDataDrivenInfoForPlatform(Platform);
				if (DDPI.bCanUseCrashReporter)
				{
					AddIfTrue("IncludeCrashReporter", "-CrashReporter");
				}

				bool bBuildHttpChunkInstallData;
				Config.GetBool(SettingsSection, "bBuildHttpChunkInstallData", out bBuildHttpChunkInstallData);
				if (bBuildHttpChunkInstallData)
				{
					string HttpChunkInstallDataDirectory = Config.GetStructEntryForSetting(SettingsSection, "HttpChunkInstallDataDirectory", "Path");
					string HttpChunkInstallDataVersion;
					Config.GetString(SettingsSection, "HttpChunkInstallDataVersion", out HttpChunkInstallDataVersion);

					CommandLine += $" -manifests -createchunkinstall -chunkinstalldirectory=\"{HttpChunkInstallDataDirectory}\" -chunkinstallversion={HttpChunkInstallDataVersion}";
				}
			}

			if (bStage || bPackage)
			{
				AddIfTrue("ForDistribution", "-distribution");
			}

			return CommandLine;
		}

		static string MakeCommandLineInteractively(List<UnrealTargetPlatform> Platforms, FileReference ProjectFile, string ExistingCommandLine, bool bBuild, bool bCook, bool bStage, bool bPackage, bool bArchive)
		{
			string CommandLine = "";

			if (bBuild || bStage || bPackage)
			{
				List<string> Configs = new List<string> { "Debug", "Development", "Test", "Shipping" };
				int BinaryConfigChoice = TurnkeyUtils.ReadInputInt("Select the compiled game configuration:", Configs, false, 1);
				string BuildConfig = Configs[BinaryConfigChoice];

				//switch (TargetType)
				//{
				//	case TargetType.Client: CommandLine += $" -client -clientconfig={BuildConfig}"; break;
				//	case TargetType.Server: CommandLine += $" -server -noclient -serverconfig={BuildConfig}"; break;
				//	case TargetType.Game: CommandLine += $" -clientconfig={BuildConfig}"; break;
				//	default: throw new AutomationException($"TypeType {TargetType} is not allowed for packaging at this time");
				//}
			}

			if (bBuild)
			{
				if (TurnkeyUtils.GetUserConfirmation("Would you like to perform a full rebuild of all programs? [-clean]", false))
				{
					CommandLine += " -clean";
				}
			}

			if (bCook)
			{
				if (TurnkeyUtils.GetUserConfirmation("Would you like to skip cooking editor content? [-SkipCookingEditorContent]", false))
				{
					CommandLine += " -SkipCookingEditorContent";
				}
			}

			if (bStage || bCook)
			{
				if (TurnkeyUtils.GetUserConfirmation("Would you like to split your data up into chunks? This requires a UAssetManager (???) subclass to manage chunk assignments [-manifests]", false))
				{
					CommandLine += " -manifests";

					//					if (TurnkeyUtils.GetUserConfirmation("Would you like to generate Http chunk install data? [-createchunkinstall]", false))
					//					{
					//						ConfigHierarchy Config = ConfigCache.ReadHierarchy(ConfigHierarchyType.Game, ProjectFile.Directory, BuildHostPlatform.Current.Platform);

					//						CommandLine += " -createchunkinstall";
					//						TurnkeyUtils.ReadInput("Entier the HttpChunkInstallDirectory")
					////						BuildCookRunParams += FString::Printf(TEXT(" -manifests -createchunkinstall -chunkinstalldirectory=\"%s\" -chunkinstallversion=%s"), *(PackagingSettings->HttpChunkInstallDataDirectory.Path), *(PackagingSettings->HttpChunkInstallDataVersion));
					//					}

				}


				int ContainerModeChoice = TurnkeyUtils.ReadInputInt("How would you like your files collected?", new List<string>
						{ "Optimized containers when staging [-iostore -pak]", "[Experimental] Optimized containers when cooking [-zenstore -pak]", "Left as fully loose files"}, false, 0);
				switch (ContainerModeChoice)
				{
					case 0: CommandLine += " -iostore -pak"; break;
					case 1: CommandLine += " -zenstore -pak"; break;
					case 2: CommandLine += ""; break;
				}
			}

			if (bStage)
			{
				bool bCanUseCrashReporter = false;
				foreach (UnrealTargetPlatform Platform in Platforms)
				{
					DataDrivenPlatformInfo.ConfigDataDrivenPlatformInfo DDPI = DataDrivenPlatformInfo.GetDataDrivenInfoForPlatform(Platform);
					bCanUseCrashReporter = bCanUseCrashReporter || DDPI.bCanUseCrashReporter;
				}
				if (bCanUseCrashReporter)
				{
					if (TurnkeyUtils.GetUserConfirmation("Would you like to include the crash reporter? [-CrashReporter]", true))
					{
						CommandLine += " -CrashReporter";
					}
				}

				if (Platforms.Contains(UnrealTargetPlatform.Win64))
				{
					if (TurnkeyUtils.GetUserConfirmation("Would you like to include the Windows Prerequisites installer? [-prereqs]", false))
					{
						CommandLine += " -prereqs";
					}
					//int ContainerModeChoice = TurnkeyUtils.ReadInputInt("How would you like your files collected?", new List<string>
					//	{ "Optimized containers when staging [-iostore -pak]", "[Experimental] Optimized containers when cooking [-zenstore -pak]", "Left as fully loose files"}, false, 0);
				}
			}

			if (bStage || bPackage)
			{
				if (TurnkeyUtils.GetUserConfirmation("Would you create build appropriate for distribution to end users? [-distribution]", false))
				{
					CommandLine += " -distribution";
				}
			}

			return CommandLine;
		}

		const string DividerLine = "================================================\n";

		protected override void Execute(string[] CommandOptions)
		{
			int BuildChoice = TurnkeyUtils.ReadInputInt("What kind of build would you like to create?", new List<string> { "One-off build to execute now", "Save a build to project settings to execute in Editor or Turnkey" }, true, 1);
			if (BuildChoice == 0)
			{
				return;
			}
			bool bMakingSavedBuild = BuildChoice == 2;

			int UserMode = TurnkeyUtils.ReadInputInt("Choose an interactive mode to use:", new List<string> { "Simple Mode: Running a game (locally, remote device, etc)", "Simple Mode: Making a packaged up build (install later, share with others, etc)", "Advanced mode: Walk through all questions" }, true, 2);
			if (UserMode == 0)
			{
				return;
			}
			bool bAskForDevice = UserMode == 1 || UserMode == 3;
			bool bAskForPackage = UserMode == 2 || UserMode == 3;

			// get the project, always needed
			FileReference ProjectFile = TurnkeyUtils.GetProjectFromCommandLineOrUser(CommandOptions);

			// we need a platform to execute
			List<UnrealTargetPlatform> Platforms = null;
			FileReference DefaultConfig = null;

			if (bMakingSavedBuild)
			{
				Platforms = new List<UnrealTargetPlatform> { HostPlatform.Platform };

				DefaultConfig = ConfigCache.GetDefaultConfigFileReference(ConfigHierarchyType.Game, ProjectFile.Directory);
				if (!FileReference.Exists(DefaultConfig))
				{
					TurnkeyUtils.Log($"Unable to find DefaultGame.ini for {ProjectFile.GetFileNameWithoutExtension()}. We will continue, but just print out the info at the end.");
					DefaultConfig = null;
				}
				else
				{
					// make sure the file is writable
					while (DefaultConfig != null && File.GetAttributes(DefaultConfig.FullName).HasFlag(FileAttributes.ReadOnly))
					{
						string ReadOnlyPrompt = "The project's DefaultGame.ini is read-only, unable to save. What would you like to do?";
						List<string> Options = new List<string> { "Try again", "Continue anyway, printout info at the end", "Make file writeable" };
						if (CommandUtils.P4Enabled)
						{
							Options.Add("Checkout with Perforce");
						}
						else
						{
							ReadOnlyPrompt += "\n(Try re-running UAT with -p4 to enable Perforce to allow Turnkey to checkout for you)";
						}

						int Choice = TurnkeyUtils.ReadInputInt(ReadOnlyPrompt, Options, true, 2);
						if (Choice == 0)
						{
							return;
						}
						else if (Choice == 2)
						{
							DefaultConfig = null;
							break;
						}
						else if (Choice == 3)
						{
							// clear read-only flag
							File.SetAttributes(DefaultConfig.FullName, File.GetAttributes(DefaultConfig.FullName) & ~FileAttributes.ReadOnly);
						}
						else if (Choice == 4)
						{
							TurnkeyUtils.Log("Attempting to create a p4 changelist and check out DefaultGame.ini...");
							try
							{
								int CL = CommandUtils.P4.CreateChange(CommandUtils.P4Env.Client, $"[AUTO-GENERATED] - Adding a custom project build to DefaultGame.ini");
								CommandUtils.P4.Edit(CL, CommandUtils.MakePathSafeToUseWithCommandLine(DefaultConfig.FullName));
								TurnkeyUtils.Log($"Created changelist {CL}. Use P4 to check in when ready!");
							}
							catch  (Exception Ex)
							{
								TurnkeyUtils.Log($"P4 error: {Ex.Message}");
							}
						}
					}
				}
			}
			else
			{
				Platforms = TurnkeyUtils.GetPlatformsFromCommandLineOrUser(CommandOptions, null);
			}

			if (Platforms.Count == 0 || ProjectFile == null)
			{
				return;
			}

			//if (TurnkeyUtils.ParseParam("testconfig", CommandOptions))
			//{
			//	ConfigCache.WriteSettingToDefaultConfig(ConfigHierarchyType.Game, ProjectFile.Directory, ConfigCache.ConfigDefaultUpdateType.SetValue, "SectionTestA", "InsertedValue", "True");
			//	ConfigCache.WriteSettingToDefaultConfig(ConfigHierarchyType.Game, ProjectFile.Directory, ConfigCache.ConfigDefaultUpdateType.SetValue, "SectionTestA", "Foo", "SetValue");
			//	ConfigCache.WriteSettingToDefaultConfig(ConfigHierarchyType.Game, ProjectFile.Directory, ConfigCache.ConfigDefaultUpdateType.AddArrayEntry, "SectionTestB", "Builds", "(Blah=123123)");
			//	ConfigCache.WriteSettingToDefaultConfig(ConfigHierarchyType.Game, ProjectFile.Directory, ConfigCache.ConfigDefaultUpdateType.SetValue, "SectionTest2", "InsertedValue", "True");
			//	ConfigCache.WriteSettingToDefaultConfig(ConfigHierarchyType.Game, ProjectFile.Directory, ConfigCache.ConfigDefaultUpdateType.AddArrayEntry, "SectionTest1", "Builds", "(Blah=123123)");
			//	ConfigCache.WriteSettingToDefaultConfig(ConfigHierarchyType.Game, ProjectFile.Directory, ConfigCache.ConfigDefaultUpdateType.SetValue, "SectionTest1", "Foo2", "SetValue2");
			//	return;
			//}
			 
			bool bBuild = false, bCook = false, bStage = false, bDeploy = false, bRun = false, bPackage = false, bArchive = false;
			string TargetType = "Game";
			string TargetName = null;
//			string BuildConfig = "Development";

			if (UserMode == 1)
			{
				bBuild = true;
				bCook = true;
				bStage = true;
				bDeploy = true;
				bRun = true;
			}
			else if (UserMode == 2)
			{
				bBuild = true;
				bCook = true;
				bStage = true;
				bPackage = true;
				bArchive = TurnkeyUtils.GetUserConfirmation("Would you like to copy the output to an archival location? [-archive]", false);
			}
			else
			{
				ProjectProperties Props = ProjectUtils.GetProjectProperties(ProjectFile);
//				int TargetTypeChoice = TurnkeyUtils.ReadInputInt("Select your target type:", new List<string> { "Game (Game + Server)", "Client (No Server)", "Server (No Game)" }, false, 0);
//				TargetType = TargetTypeChoice == 0 ? "Game" : TargetTypeChoice == 1 ? "Client" : "Server";

				int TargetTypeChoice = TurnkeyUtils.ReadInputInt("Select your target:", Props.Targets.Select(x => x.TargetName).ToList(), true, Props.Targets.IndexOf(Props.Targets.First(x => x.Rules.Type == UnrealBuildTool.TargetType.Game)) + 1);
				if (TargetTypeChoice == 0)
				{
					return;
				}
				TargetName = 
				TargetType = Props.Targets[TargetTypeChoice - 1].Rules.Type.ToString();

				bBuild = TurnkeyUtils.GetUserConfirmation("Would you like to compile necessary programs? [-build]", true);
				bCook = TurnkeyUtils.GetUserConfirmation("Would you like to cook your project's assets? [-cook]", true);
				bStage = TurnkeyUtils.GetUserConfirmation("Would you like gather all files into a staging directory? [-stage]", true);

				bPackage = TurnkeyUtils.GetUserConfirmation("Would you like to generate a final platform-specific package? [-package]", true);
				bArchive = TurnkeyUtils.GetUserConfirmation("Would you like to copy the output to an archival location? [-archive]", false);

				bDeploy = TurnkeyUtils.GetUserConfirmation("Would you like to copy the gathered files to a device? [-deploy]", !bPackage);
				bRun = TurnkeyUtils.GetUserConfirmation("Would you like to run the game? [-run]", bDeploy);
			}

			string BCRParams = "";
			if (!bMakingSavedBuild)
			{
				BCRParams = $"-project=\"{ProjectFile}\"";
			}

			BCRParams +=
				  (bBuild ? " -build" : "")
				+ (bCook ? " -cook" : " -skipcook")
				+ (bStage ? " -stage" : " -skipstage")
				+ (bPackage ? " -package" : " -skippackage")
				+ (bArchive ? " -archive" : "")
				+ (bDeploy ? " -deploy" : "")
				+ (bRun ? " -run" : "")
				;

			if (bArchive)
			{
				// when saving, use a placeholder to ask later
				BCRParams += " -archivedirectory=" + (bMakingSavedBuild ? "{BrowseForDir}" : GetArchiveDirectory(CommandOptions, ProjectFile));
			}

			string PPSettings = MakeCommandLineFromPackagingSettings(HostPlatform.Platform, ProjectFile, BCRParams);
			string PPPrompt = DividerLine + $"The options so far are:\n  {BCRParams}\nThe project's packaging settings (for {Platforms[0]}) are:\n  {PPSettings}\n" +
				"How would you like to use the packaging settings? Not all may apply to this build and will be ignored.";
			if (bMakingSavedBuild)
			{
//				PPPrompt += "\nNote that different platforms "
			}
			List<string> PPOptions = new List<string>
			{
				"Do not add any of the packaging settings",
				"Interactively choose packaging settings",
				"Use the packaging settings shown above",
			};
			if (bMakingSavedBuild)
			{
				PPOptions.Add("Use what the settings will be when this command is run later");
			}
			int ExtendedParamsChoice = TurnkeyUtils.ReadInputInt(PPPrompt, PPOptions, false, PPOptions.Count - 1);
			//bool bUsePackagingSettings = TurnkeyUtils.GetUserConfirmation("Would you like to further control the build with your project's packaging settings? [-xyz]", true);

			string ExtendedParams = "";
			if (ExtendedParamsChoice == 1)
			{
				ExtendedParams = MakeCommandLineInteractively(Platforms, ProjectFile, BCRParams, bBuild, bCook, bStage, bPackage, bArchive);
			}
			else  if (ExtendedParamsChoice == 3)
			{
				// encode the targettype into the tag for later use
				ExtendedParams = " {ProjectPackagingSettings}";
			}

			string LastPlatformParams = "";
			string AddCmdLine = "";
			foreach (UnrealTargetPlatform Platform in Platforms)
			{
				string PlatformParams = BCRParams;

				// if we are making a saved build, then leave the {Platform} tag in the string, otherwise, put in the actual platform
				if (bMakingSavedBuild)
				{
					PlatformParams += " -platform={Platform}";
				}
				else
				{
					PlatformParams += " -platform=" + Platform;
				}

				if (ExtendedParamsChoice == 2)
				{
					ExtendedParams = MakeCommandLineFromPackagingSettings(Platform, ProjectFile, BCRParams + ExtendedParams + PlatformParams);
				}


				PlatformParams += ExtendedParams;
				if (TurnkeyUtils.GetUserConfirmation($"Would you like to add anymore options to the build commandline for {Platform}?\nCurrent commandline is:\n   BuildCookRun {PlatformParams}", false))
				{
					LastPlatformParams = TurnkeyUtils.ReadInput("Enter extra options:", LastPlatformParams);
					PlatformParams += " " + LastPlatformParams;
				}

				// the commandline may be saved to a commandline.txt, or we may be running with it drectly (See jira UE-99467 for upcoming work that will affect this)
				if (bStage || bRun)
				{
					AddCmdLine = TurnkeyUtils.ReadInput($"Enter any commandline options you want to add to the game [-addcmdline]", AddCmdLine);
					if (!string.IsNullOrEmpty(AddCmdLine))
					{
						PlatformParams += $" -addcmdline=\"{AddCmdLine}\"";
					}
				}

				if (bDeploy || bRun)
				{
					if (bMakingSavedBuild)
					{
						PlatformParams += " -device={DeviceId}";
					}
					else
					{
						List<DeviceInfo> Devices = TurnkeyUtils.GetDevicesFromCommandLineOrUser(CommandOptions, Platform);
						if (Devices == null || Devices.Count == 0)
						{
							// we need a device here, if canceled, quit out
							return;
						}
						PlatformParams += " -device=" + string.Join("+", Devices.Select(x => x.Id));
					}
				}

				TurnkeyUtils.Log($"Final Command:\nRunUAT BuildCookRun {PlatformParams}");

//				if (TurnkeyUtils.GetUserConfirmation("Do you want to save this to DefaultGame.ini for use in the editor?", false))
				if (bMakingSavedBuild)
				{
					TurnkeyUtils.Log(DividerLine + $"We are now ready to save this build to DefaultGame.ini!");

					List<string> ExistingBuilds;
					ConfigHierarchy Config = ConfigCache.ReadHierarchy(ConfigHierarchyType.Game, ProjectFile.Directory, BuildHostPlatform.Current.Platform);
					Config.GetArray(SettingsSection, "ProjectCustomBuilds", out ExistingBuilds);

					string Name;
					if (ExistingBuilds != null && ExistingBuilds.Count > 0)
					{
						TurnkeyUtils.Log($"Here are the existing builds your project already has (and their descrptions):");
						foreach (string Build in ExistingBuilds)
						{
							string BuildName = ConfigHierarchy.GetStructEntry(Build, "Name", false);
							string BuildHelp = ConfigHierarchy.GetStructEntry(Build, "HelpText", false);
							TurnkeyUtils.Log($"  {BuildName} - {BuildHelp}");
						}
						Name = TurnkeyUtils.ReadInput("Enter unique name for this build, it must be different than the above names", "");
					}
					else
					{
						Name = TurnkeyUtils.ReadInput("Enter a name for this build", "");
					}

					if (!string.IsNullOrEmpty(Name))
					{
						string HelpText = TurnkeyUtils.ReadInput("Enter the help/description for this build", "");
						string RestrictedPlatforms = TurnkeyUtils.ReadInput("If you want to restrict this to certain plaforms, enter a comma separated list of platform names:", "");

						if (!string.IsNullOrEmpty(RestrictedPlatforms))
						{
							IEnumerable<string> PlatformNames = RestrictedPlatforms.Split(", +".ToCharArray()).ToList()
								// fixup windows, and wrap in quotes
								.Select(x => (x.Equals("Win64", StringComparison.InvariantCultureIgnoreCase) ? "Windows" : x))
								.Select(x => $"\"{x}\"");
							RestrictedPlatforms = "(" + string.Join(",", PlatformNames) + ")";
						}

						//// fix up the commandline to be appropriate (remove certain options, like project, platform, and archive directory)
						//Regex CommandLineRegex = new Regex("(-\\w*\\s+)|(-\\w*=\\w*\\s+)|(-\\w*=\"[^\"]*\"\\s +)");
						//// split the args and clean up
						//List<string> IndividualArgs = CommandLineRegex.Split(PlatformParams).Select(x => x.Trim()).Where(x => x != "").ToList();

						//// remove some known bad params
						//IndividualArgs = IndividualArgs.Where(x => !x.StartsWith("-project=", StringComparison.InvariantCultureIgnoreCase)).ToList();

						//// update some params to have variables
						//for (int Index = 0; Index < IndividualArgs.Count; Index++)
						//{
						//	if (IndividualArgs[Index].StartsWith("-platform=", StringComparison.InvariantCultureIgnoreCase))
						//	{
						//		IndividualArgs[Index] = "-platform={Platform}";
						//	}
						//	else if (IndividualArgs[Index].StartsWith("-archivedirectory=", StringComparison.InvariantCultureIgnoreCase))
						//	{
						//		IndividualArgs[Index] = "-archivedirectory={BrowseForDir}";
						//	}
						//}


						// generate struct string for ini file
						string ValueText = $"(Name=\"{Name}\",HelpText=\"{HelpText}\",SpecificPlatforms={RestrictedPlatforms},BuildCookRunParams=\"{PlatformParams}\")";

						// just print out the text to put into the DefaultGame.ini later
						if (DefaultConfig == null)
						{
							TurnkeyUtils.Log($"Since DefaultGame.ini isn't writeable, here is the line you can manually add to DefaultGame.ini in the [{SettingsSection}] section to add the build to editor and Turnkey's ExecuteBuild command:\n  {ValueText}");
						}
						else
						{
							//string SavedCommandLine = string.Join(" ", IndividualArgs);
							TurnkeyUtils.Log($"Saving reusable commandline to DefaultGame.ini:\n  {PlatformParams}");
							TurnkeyUtils.Log($"You can now run the command via RunUAT Turnkey -command=ExecuteBuild -build=\"{Name}\" or run the editor and find it in the Platforms menu.");
							TurnkeyUtils.Log("Further, you can edit this in the Packaging Settings in the editor as needed");

							// add this setting in to the ProjectSettings section
							ConfigCache.WriteSettingToDefaultConfig(ConfigHierarchyType.Game, ProjectFile.Directory, ConfigCache.ConfigDefaultUpdateType.AddArrayEntry, SettingsSection, "ProjectCustomBuilds", ValueText, Log.Logger);
							ConfigCache.InvalidateCaches();
						}
					}
				}
				else
				{
					if (TurnkeyUtils.GetUserConfirmation($"Ready to execute the follwing command?\n  BuildCookRun {PlatformParams}", true))
					{
						ExecuteBuild.ExecuteBuildCookRun(PlatformParams);
					}
				}
			}
		}
	}
}
