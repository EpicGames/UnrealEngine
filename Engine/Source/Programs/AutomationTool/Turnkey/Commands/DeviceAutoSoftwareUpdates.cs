// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using AutomationTool;
using UnrealBuildTool;

namespace Turnkey.Commands
{
	class DeviceAutoSoftwareUpdates : TurnkeyCommand
	{
		protected override CommandGroup Group => CommandGroup.Sdk;

		protected override void Execute(string[] CommandOptions)
		{
			// Are we enabling or disabling auto updates on the devices?
			bool bUnattended = TurnkeyUtils.ParseParam("Unattended", CommandOptions);
			string EnableArg = TurnkeyUtils.ParseParamValue("Enable", null, CommandOptions);

			bool bEnableAutoSoftwareUpdates = false;
			if (string.IsNullOrEmpty(EnableArg))
			{
				if (bUnattended)
				{
					TurnkeyUtils.Log("Error: Desired state of device auto software updates must be specified via -Enable=true/false");
					return;
				}
				else
				{
					bEnableAutoSoftwareUpdates = TurnkeyUtils.ReadInputInt("Enable device auto software updates?", new List<string> { "Disable", "Enable" }, false) != 0;
				}
			}
			else
			{
				bEnableAutoSoftwareUpdates = EnableArg.Equals("true", StringComparison.InvariantCultureIgnoreCase);
			}

			// Get list of devices
			List<UnrealTargetPlatform> PlatformsWithSdks = TurnkeyManifest.GetPlatformsWithSdks();
			List<DeviceInfo> ChosenDevices = TurnkeyUtils.GetDevicesFromCommandLineOrUser(CommandOptions, PlatformsWithSdks);


			// Set the auto updates mode on each device
			foreach (DeviceInfo Device in ChosenDevices)
			{
				AutomationTool.Platform AutomationPlatform = Platform.GetPlatform(Device.Platform);
				AutomationPlatform.SetDeviceAutoSoftwareUpdateMode(Device, bEnableAutoSoftwareUpdates);
			}
		}
	}
}
