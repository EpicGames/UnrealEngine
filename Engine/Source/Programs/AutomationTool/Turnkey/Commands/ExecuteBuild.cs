// Copyright Epic Games, Inc. All Rights Reserved.

using AutomationTool;
using System;
using System.Collections.Generic;
using System.Linq;
using System.IO;
using System.Text.RegularExpressions;
using EpicGames.Core;
using UnrealBuildTool;

namespace Turnkey.Commands
{
	class ExecuteBuild : TurnkeyCommand
	{
		protected override CommandGroup Group => CommandGroup.Builds;

		protected override void Execute(string[] CommandOptions)
		{
			// we need a platform to execute
			FileReference ProjectFile = TurnkeyUtils.GetProjectFromCommandLineOrUser(CommandOptions);
			List<UnrealTargetPlatform> Platforms = TurnkeyUtils.GetPlatformsFromCommandLineOrUser(CommandOptions, null);
			string DesiredBuild = TurnkeyUtils.ParseParamValue("Build", null, CommandOptions);
			string ExtraOptions = TurnkeyUtils.ParseParamValue("ExtraOptions", "", CommandOptions);
			string OutputDir = TurnkeyUtils.ParseParamValue("OutputDir", null, CommandOptions);
			bool bPrintCommandOnly = TurnkeyUtils.ParseParam("PrintOnly", CommandOptions);
			bool bInteractive = TurnkeyUtils.ParseParam("Interactive", CommandOptions);

			// we need a project file, so if canceled, abore this command
			if (ProjectFile == null)
			{
				return;
			}

			// get a list of builds from config
			foreach (UnrealTargetPlatform Platform in Platforms)
			{
				ConfigHierarchy GameConfig = ConfigCache.ReadHierarchy(ConfigHierarchyType.Game, ProjectFile.Directory, Platform);

				List<string> EngineBuilds;
				List<string> ProjectBuilds;
				GameConfig.GetArray("/Script/UnrealEd.ProjectPackagingSettings", "EngineCustomBuilds", out EngineBuilds);
				GameConfig.GetArray("/Script/UnrealEd.ProjectPackagingSettings", "ProjectCustomBuilds", out ProjectBuilds);

				List<string> Builds = new List<string>();
				if (EngineBuilds != null)
				{
					Builds.AddRange(EngineBuilds);
				}
				if (ProjectBuilds != null)
				{
					Builds.AddRange(ProjectBuilds);
				}

				Dictionary<string, Tuple<string, string>> BuildCommands = new Dictionary<string, Tuple<string, string>>(StringComparer.InvariantCultureIgnoreCase);
				if (Builds != null)
				{
					foreach (string Build in Builds)
					{
						string Name = ConfigHierarchy.GetStructEntry(Build, "Name", false);
						string Help = ConfigHierarchy.GetStructEntry(Build, "HelpText", false);
						string SpecificPlatforms = ConfigHierarchy.GetStructEntry(Build, "SpecificPlatforms", true);
						string Params = ConfigHierarchy.GetStructEntry(Build, "BuildCookRunParams", false);

						// make sure required entries are there
						if (Name == null || Params == null)
						{
							continue;
						}

						// if platforms are specified, and this platform isn't one of them, skip it
						if (!string.IsNullOrEmpty(SpecificPlatforms))
						{
							string IniPlatformName = ConfigHierarchy.GetIniPlatformName(Platform);
							string[] PlatformList = SpecificPlatforms.Split(",\"".ToCharArray(), StringSplitOptions.RemoveEmptyEntries);
							// case insensitive Contains
							if (PlatformList.Length > 0 && !PlatformList.Any(x => x.Equals(IniPlatformName, StringComparison.OrdinalIgnoreCase)))
							{
								continue;
							}
						}

						// add to list of commands
						BuildCommands.Add(Name, new Tuple<string, string>(Help, Params));
					}
				}

				if (BuildCommands.Count == 0)
				{
					TurnkeyUtils.Log("Unable to find a build for platform {0} and project {1}", Platform, ProjectFile.GetFileNameWithoutAnyExtensions());
					continue;
				}

				string FinalParams;
				if (!string.IsNullOrEmpty(DesiredBuild) && BuildCommands.ContainsKey(DesiredBuild))
				{
					FinalParams = BuildCommands[DesiredBuild].Item2;
				}
				else
				{
					List<string> BuildNames = BuildCommands.Keys.ToList();
					List<string> BuildItems = BuildNames.Select(x => x + (string.IsNullOrEmpty(BuildCommands[x].Item1) ? "" : $" [{BuildCommands[x].Item1}]")).ToList();
					int Choice = TurnkeyUtils.ReadInputInt("Choose a build to execute", BuildItems, true);
					if (Choice == 0)
					{
						continue;
					}

					FinalParams = BuildCommands[BuildNames[Choice - 1]].Item2;
				}

				// make sure a project option is specified
				if (!FinalParams.Contains("-project=", StringComparison.InvariantCultureIgnoreCase))
				{
					FinalParams = "-project={Project} " + FinalParams;
				}

				FinalParams = FinalParams.Replace("{Project}", "\"" + ProjectFile.FullName + "\"", StringComparison.InvariantCultureIgnoreCase);
				FinalParams = FinalParams.Replace("{Platform}", Platform.ToString(), StringComparison.InvariantCultureIgnoreCase);
				FinalParams = FinalParams.Replace("{ProjectPackagingSettings}", CreateBuild.MakeCommandLineFromPackagingSettings(Platform, ProjectFile, FinalParams));
				FinalParams = PerformIniReplacements(FinalParams, ProjectFile.Directory, Platform);

				// if there is a {DeviceId} param, get it from the commandline or user
				if (FinalParams.Contains("{DeviceId}", StringComparison.InvariantCultureIgnoreCase))
				{
					List<DeviceInfo> Devices = TurnkeyUtils.GetDevicesFromCommandLineOrUser(CommandOptions, Platform);
					string DeviceIds = string.Join("+", Devices.Select(x => x.Id));
					FinalParams = FinalParams.Replace("{DeviceId}", DeviceIds, StringComparison.InvariantCultureIgnoreCase);
				}

				FinalParams += " " + ExtraOptions;

				if (bPrintCommandOnly)
				{
					TurnkeyUtils.Log("To execute this build manually, run:");
					TurnkeyUtils.Log("");
					TurnkeyUtils.Report($"RunUAT BuildCookRun {FinalParams}");
					TurnkeyUtils.Log("");
					return;
				}

				// handle browsing for BrowseForDir after printing out, as it doesn't make much sense to ask for a directory or print out one
				if (FinalParams.Contains("{BrowseForDir}"))
				{
					// if the -outputdir option was specified, use that, otherwise ask user for directory
					if (OutputDir == null)
					{
						//if (OperatingSystem.IsWindows())
						//{
						//	string ChosenFile = null;
						//	System.Threading.Thread t = new System.Threading.Thread(x =>
						//	{
						//		ChosenFile = UnrealWindowsForms.Utils.ShowOpenFileDialogAndReturnFilename("Project Files (*.uproject)|*.uproject");
						//	});

						//	t.SetApartmentState(System.Threading.ApartmentState.STA);
						//	t.Start();
						//	t.Join();

						//	if (ChosenFile == null)
						//	{
						//		continue;
						//	}
						//}
						//else
						//{
						//	while (true)
							{
								OutputDir = TurnkeyUtils.ReadInput("Enter output directory:");
							}
						//}

					}
					if (string.IsNullOrEmpty(OutputDir))
					{
						TurnkeyUtils.Log("Cancelling...");
						return;
					}
					// handle {BrowseForDir} with or without quotes already around it (if already has quotes, then replace with the path, and if
					// without quotes, replace with path wrapped in quotes
					FinalParams = FinalParams.Replace("\"{BrowseForDir}\"", OutputDir);
					FinalParams = FinalParams.Replace("{BrowseForDir}", $"\"{OutputDir}\"");
				}

				TurnkeyUtils.Log("Executing '{0}'...", FinalParams);
				ExecuteBuildCookRun(FinalParams);
			}
		}

		internal static void ExecuteBuildCookRun(string Params)
		{
			// split the params on whitespace not inside quotes (see https://stackoverflow.com/questions/4780728/regex-split-string-preserving-quotes/4780801#4780801 to explain the regex)
			Regex Matcher = new Regex("(?<=^[^\"]*(?:\"[^\"]*\"[^\"]*)*)\\s(?=(?:[^\"]*\"[^\"]*\")*[^\"]*$)");
			// split the string, removing empty results
			List<string> Arguments = Matcher.Split(Params).Where(x => x != "").ToList();

			UnrealBuildBase.CommandInfo BCRCommand = new UnrealBuildBase.CommandInfo("BuildCookRun");
			// chop off the first - character in all the commands (see Automation.ParseParam)
			BCRCommand.Arguments = Arguments.Select(x => x.Substring(1)).ToList();

			// use the BCR's exitcode as Turnkey's exitcode
			TurnkeyUtils.ExitCode = Automation.ExecuteAsync(new List<UnrealBuildBase.CommandInfo>() { BCRCommand }, ScriptManager.Commands).Result;
		}

		string GetIniSetting(string Spec, DirectoryReference ProjectDir, UnrealTargetPlatform Platform)
		{
			// handle these cases:
			//  iniif:-option:Engine:/Script/Module.Class:bUseOption
			//  iniif:-option:bUseOption   [convenience for ProjectPackagingSettings setting]
			//  inivalue:Engine:/Script/Module.Class:SomeSetting
			//  inivalue:SomeSetting       [convenience for ProjectPackagingSettings setting]

			string[] CommandAndModifiers = Spec.Split("|".ToCharArray(), StringSplitOptions.RemoveEmptyEntries);
			string[] Tokens = CommandAndModifiers[0].Split(":".ToCharArray());

			string ConfigName;
			string SectionName;
			string Key;
			string IniIfValue = "";
			if (Tokens[0].Equals("iniif", StringComparison.InvariantCultureIgnoreCase))
			{
				if (Tokens.Length == 3)
				{
					ConfigName = "Game";
					SectionName = "/Script/UnrealEd.ProjectPackagingSettings";
					Key = Tokens[2];
					IniIfValue = Tokens[1];
				}
				else if (Tokens.Length == 5)
				{
					ConfigName = Tokens[2];
					SectionName = Tokens[3];
					Key = Tokens[4];
					IniIfValue = Tokens[1];
				}
				else
				{
					TurnkeyUtils.Log("Found a bad iniif spec: {0}", Spec);
					return "";
				}
			}
			else if (Tokens[0].Equals("inivalue", StringComparison.InvariantCultureIgnoreCase))
			{
				if (Tokens.Length == 2)
				{
					ConfigName = "Game";
					SectionName = "/Script/UnrealEd.ProjectPackagingSettings";
					Key = Tokens[1];
				}
				else if (Tokens.Length == 4)
				{
					ConfigName = Tokens[1];
					SectionName = Tokens[2];
					Key = Tokens[3];
				}
				else
				{
					TurnkeyUtils.Log("Found a bad inivalue spec: {0}", Spec);
					return "";
				}
			}
			else
			{
				TurnkeyUtils.Log("Found a bad ini spec: {0}", Spec);
				return "";
			}

			// get the value, if it exists (or empty string if not)
			ConfigHierarchyType ConfigType;
			if (!ConfigHierarchyType.TryParse(ConfigName, out ConfigType))
			{
				TurnkeyUtils.Log("Found a bad config name {0} in spec {1}", ConfigName, Spec);
				return "";
			}
			ConfigHierarchy Config = ConfigCache.ReadHierarchy(ConfigType, ProjectDir, Platform);
			string FoundValue;
			Config.GetString(SectionName, Key, out FoundValue);


			if (Tokens[0].Equals("iniif", StringComparison.InvariantCultureIgnoreCase))
			{
				bool bIsTrue;
				if (bool.TryParse(FoundValue, out bIsTrue) && bIsTrue)
				{
					FoundValue = IniIfValue;
				}
				else
				{
					return "";
				}
			}

			// look to see if we have a replace modifier to update the ini value, and apply it if so
			if (CommandAndModifiers.Length > 1)
			{
				string[] SearchAndReplace = CommandAndModifiers[1].Split("=".ToCharArray());
				if (SearchAndReplace.Length != 2)
				{
					TurnkeyUtils.Log("Found a search/replace modifier {0} in spec {1}", CommandAndModifiers, Spec);
					return "";
				}
				FoundValue = FoundValue.Replace(SearchAndReplace[0], SearchAndReplace[1]);
			}

			return FoundValue;
		}

		private string PerformIniReplacements(string Params, DirectoryReference ProjectDir, UnrealTargetPlatform Platform)
		{
			Regex IniMatch = new Regex("({(ini.*?)})+");
			foreach (Match Match in IniMatch.Matches(Params))
			{
				if (Match.Success)
				{
					// group[1] is {ini.....}, groups[2] is the same but without the {}
					Params = Params.Replace(Match.Groups[1].Value, GetIniSetting(Match.Groups[2].Value, ProjectDir, Platform));
				}
			}

			return Params;
		}
	}
}
