// Copyright Epic Games, Inc. All Rights Reserved.

using AutomationTool;
using UnrealBuildTool;
using System;
using System.Collections.Generic;
using System.Linq;
using System.Text;

namespace Turnkey.Commands
{
	class Help : TurnkeyCommand
	{
		protected override CommandGroup Group => CommandGroup.Informational;

		protected override void Execute(string[] CommandOptions)
		{
			bool bShowUsingHelp = TurnkeyUtils.ParseParam("Using", CommandOptions);
			bool bShowStudioHelp = TurnkeyUtils.ParseParam("Studio", CommandOptions);
			bool bShowAllHelp = TurnkeyUtils.ParseParam("Studio", CommandOptions);
			string PlatformString = TurnkeyUtils.ParseParamValue("Platform", null, CommandOptions);

			// if no type was chosen, ask for type
			bool bAskForType = !bShowUsingHelp && !bShowStudioHelp && !bShowAllHelp;

			if (bAskForType)
			{
				List<string> Options = new List<string> { "All - show all available help", "Using - end user help", "Studio - show how to set up Turnkey in your Studio" };
				
				int Choice = TurnkeyUtils.ReadInputInt("What type of help would you like to see?", Options, false);

				switch (Choice)
				{
					case 0: bShowAllHelp = true; break;
					case 1: bShowUsingHelp = true; break;
					case 2: bShowStudioHelp = true; break;
				}
			}

			if (bShowUsingHelp || bShowAllHelp)
			{
				TurnkeyUtils.Log("To use Turnkey, you can just run it like you are now, and follow prompts to complete actions, or you can add commandline options to script as much as possible. For instance, you could see this message directly with:");
				TurnkeyUtils.Log("\tRunUAT Turnkey -command=Help -Using");
			}

			if (bShowStudioHelp || bShowAllHelp)
			{
				TurnkeyUtils.Log("To set up Turnkey in your studio, you will need to configure the location of SDK installers and other components for the rest of your developers/QA/etc to access. This is accomplished with one or more chained TurnkeyManifest.xml files.");
				TurnkeyUtils.Log("The discovery starts with Engine/Build/Turnkey/TurnkeyManifest.xml, in the <AdditionalManifests> section. Note that more manifests can keep adding more into the chain, and they can be located on servers, not local storage.");

				TurnkeyUtils.Log(" ... fill in more here ...");

				TurnkeyUtils.Log("For information on how to setup a FileSource for a given platform, choose from the list below:");

				// find the platforms with help text
				List<UnrealTargetPlatform> PossiblePlatforms = UnrealTargetPlatform.GetValidPlatforms().ToList().FindAll(x => !string.IsNullOrEmpty(AutomationTool.Platform.GetPlatform(x).GetSDKCreationHelp()));
				// get or ask user for platforms
				List<UnrealTargetPlatform> ChosenPlatforms = TurnkeyUtils.GetPlatformsFromCommandLineOrUser(CommandOptions, PossiblePlatforms);

				if (ChosenPlatforms != null)
				{
					ChosenPlatforms.ForEach(x => TurnkeyUtils.Log(AutomationTool.Platform.GetPlatform(x).GetSDKCreationHelp()));
				}
			}
		}
	}
}
