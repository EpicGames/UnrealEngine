// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Linq;
using Gauntlet;
using UnrealBuildTool;
using AutomationTool;

namespace Turnkey.Commands
{
	class InstallBuild : TurnkeyCommand
	{
		protected override CommandGroup Group => CommandGroup.Builds;

		protected override void Execute(string[] CommandOptions)
		{
			// get distinct project names
			List<string> AllProjects = TurnkeyManifest.GetProjectsWithBuilds();

			int Choice = TurnkeyUtils.ReadInputInt("Select a project:", AllProjects, true);
			if (Choice == 0)
			{
				return;
			}

			string Project = AllProjects[Choice - 1];
			
			// go over sources for this project
			List<FileSource> ProjectSources = TurnkeyManifest.FilterDiscoveredBuilds(Project);

//			List<UnrealTargetPlatform> PossiblePlatforms = ProjectSources.SelectMany(x => x.GetPlatforms()).Distinct().ToList();
			List<string> PossibleVersions = ProjectSources.Select(x => x.Version).Distinct().ToList();

			Choice = TurnkeyUtils.ReadInputInt("Choose a build version", PossibleVersions, true);
			if (Choice == 0)
			{
				return;
			}

			FileSource OriginalSource = ProjectSources[Choice - 1];

			// now find platforms under it
			List<List<string>> PlatformExpansion = new List<List<string>>();
			string[] PlatformSources = CopyProvider.ExecuteEnumerate(OriginalSource.GetCopySourceOperation() + OriginalSource.BuildPlatformEnumerationSuffix, PlatformExpansion);

			if (PlatformSources.Length == 0)
			{
				TurnkeyUtils.Log("No platform builds found in {0}!", OriginalSource.GetCopySourceOperation());
				return;
			}

			List<string> CollapsedPlatformNames = PlatformExpansion.Select(x => x[0]).ToList();
			Choice = TurnkeyUtils.ReadInputInt("Choose a platform", CollapsedPlatformNames, true);
			if (Choice == 0)
			{
				return;
			}

			string LocalPath = CopyProvider.ExecuteCopy(PlatformSources[Choice - 1]);

			// @todo turnkey this could be improved i'm sure
			string TargetPlatformString = CollapsedPlatformNames[Choice - 1];
			UnrealTargetPlatform Platform;
			if (!UnrealTargetPlatform.TryParse(TargetPlatformString, out Platform))
			{
				TargetPlatformString = TargetPlatformString.Replace("Editor", "");
				TargetPlatformString = TargetPlatformString.Replace("Client", "");
				TargetPlatformString = TargetPlatformString.Replace("Server", "");

				// Windows hack
				if (TargetPlatformString == "Windows")
				{
					TargetPlatformString = "Win64";
				}

				if (!UnrealTargetPlatform.TryParse(TargetPlatformString, out Platform))
				{
					TurnkeyUtils.Log("Unable to figure out the UnrealTargetPlatform from {0}!", CollapsedPlatformNames[Choice - 1]);
					return;
				}
			}

			// find all build sources that can be created a folder path
			IEnumerable<IFolderBuildSource> BuildSources = Gauntlet.Utils.InterfaceHelpers.FindImplementations<IFolderBuildSource>();
			List<IBuild> Builds = BuildSources.Where(S => S.CanSupportPlatform(Platform)).SelectMany(S => S.GetBuildsAtPath(Project, LocalPath)).ToList();
			if (Builds.Count() == 0)
			{
				throw new AutomationException("No builds for {0} found at {1}", Platform, LocalPath);
			}

			// finally choose a build
			List<string> Options = Builds.Select(x => string.Format("{0} {1} {2} {3}", x.Platform, x.Configuration, x.Flags, x.GetType())).ToList();
			Choice = TurnkeyUtils.ReadInputInt("Choose a build", Options, true);
			if (Choice == 0)
			{
				return;
			}

			IBuild Build = Builds[Choice - 1];

			// install the build and run it
			UnrealAppConfig Config = new UnrealAppConfig();

			if (Build.Flags.HasFlag(BuildFlags.CanReplaceCommandLine))
			{
				Config.CommandLine = TurnkeyUtils.ReadInput("Enter replacement commandline. Leave blank for original, or space to wipe it out", "");
			}
			Config.Build = Build;
			Config.ProjectName = Project;

			List<DeviceInfo> TurnkeyDevices = TurnkeyUtils.GetDevicesFromCommandLineOrUser(CommandOptions, Platform);
			if (TurnkeyDevices == null)
			{
				TurnkeyUtils.Log("Could not find a device to install on!");
				return;
			}
			
			List<ITargetDevice> GauntletDevices = TurnkeyGauntletUtils.GetGauntletDevices(TurnkeyDevices);
			if (GauntletDevices == null)
			{
				TurnkeyUtils.Log("Could not find a device to install on!");
				return;
			}

			foreach (ITargetDevice GauntletDevice in GauntletDevices)
			{
				if (GauntletDevice != null)
				{
					IAppInstall Install = GauntletDevice.InstallApplication(Config);
					GauntletDevice.Run(Install);
				}
			}
		}
	}
}
