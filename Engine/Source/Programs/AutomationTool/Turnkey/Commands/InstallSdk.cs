// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Text;
using UnrealBuildTool;
using AutomationTool;
using EpicGames.Core;
using System.Linq;

namespace Turnkey.Commands
{
	class InstallSdk : TurnkeyCommand
	{
		protected override CommandGroup Group => CommandGroup.Sdk;

		protected override Dictionary<string, string[]> GetExtendedCommandsWithOptions()
		{
			return new Dictionary<string, string[]>()
			{
				{ "Auto Install All Needed Sdks", new string[] { "-platform=All", "-NeededOnly", "-BestAvailable" } },
				{ "Auto Update Installed Sdks", new string[] { "-platform=All", "-UpdateOnly", "-BestAvailable" } },
			};
		}


		protected override void Execute(string[] CommandOptions)
		{
			string DeviceName = TurnkeyUtils.ParseParamValue("Device", null, CommandOptions);
			string SdkTypeString = TurnkeyUtils.ParseParamValue("SdkType", null, CommandOptions);

			bool bUnattended = TurnkeyUtils.ParseParam("Unattended", CommandOptions);
			bool bBestAvailable = TurnkeyUtils.ParseParam("BestAvailable", CommandOptions);
			bool bUpdateOnly = TurnkeyUtils.ParseParam("UpdateOnly", CommandOptions);

			// best available installation requires valid and needed Sdks
			bool bValidOnly = bBestAvailable || !TurnkeyUtils.ParseParam("AllowInvalid", CommandOptions);
			bool bNeededOnly = bBestAvailable || TurnkeyUtils.ParseParam("NeededOnly", CommandOptions);


			FileSource.SourceType? DesiredType = null;
			if (SdkTypeString != null)
			{
				FileSource.SourceType OutType;
				if (!Enum.TryParse(SdkTypeString, out OutType))
				{
					TurnkeyUtils.Log("Invalid SdkType given with -SdkType={0}", SdkTypeString);
					return;
				}
				DesiredType = OutType;
			}
			// if a device is specified, assume Flash type
			else if (!string.IsNullOrEmpty(DeviceName))
			{
				DesiredType = FileSource.SourceType.Flash;
			}
			// otherwise ask
			else if (!bUnattended)
			{
				List<string> Options = new List<string>() { "Full or Auto Sdk", "Full Sdk", "AutoSdk", "Device Software / Flash" };
				int Choice = TurnkeyUtils.ReadInputInt("Choose a type of Sdk to install:", Options, true, 1);

				if (Choice == 0)
				{
					return;
				}
				// Choice 1 means two types, so we use null DesiredType to mean Full|AutoSdk
				if (Choice == 2)
				{
					DesiredType = FileSource.SourceType.Full;
				}
				else if (Choice == 3)
				{
					DesiredType = FileSource.SourceType.AutoSdk;
				}
				else if (Choice == 4)
				{
					DesiredType = FileSource.SourceType.Flash;
				}
			}

			// we need all sdks we can find
			List<UnrealTargetPlatform> PlatformsWithSdks = TurnkeyManifest.GetPlatformsWithSdks();

			List<DeviceInfo> ChosenDevices = null;
			List<UnrealTargetPlatform> PlatformsLeftToInstall;

			if (DesiredType == FileSource.SourceType.Flash)
			{
				TurnkeyUtils.GetPlatformsAndDevicesFromCommandLineOrUser(CommandOptions, false, out PlatformsLeftToInstall, out ChosenDevices, PlatformsWithSdks);
				
				if (PlatformsLeftToInstall.Count == 0 || ChosenDevices.Count == 0)
				{
					return;
				}
			}
			else
			{
				PlatformsLeftToInstall = TurnkeyUtils.GetPlatformsFromCommandLineOrUser(CommandOptions, PlatformsWithSdks);

				if (PlatformsLeftToInstall == null || PlatformsLeftToInstall.Count == 0)
				{
					return;
				}
			}


			TurnkeyContextImpl TurnkeyContext = new TurnkeyContextImpl();

			List<FileSource> SdksAlreadyInstalled = new List<FileSource>();

			// keep going while we have Sdks left to install
			while (PlatformsLeftToInstall.Count > 0)
			{
				UnrealTargetPlatform Platform = PlatformsLeftToInstall[0];

				// remove the platform (we may remove more later if an Sdk supports multiple platforms)
				PlatformsLeftToInstall.RemoveAt(0);

				// cache the automation platform object
				AutomationTool.Platform AutomationPlatform = AutomationTool.Platform.GetPlatform(Platform);
				UEBuildPlatformSDK SDK = UEBuildPlatformSDK.GetSDKForPlatform(Platform.ToString());

				// filter the Sdks if a platform was given, and type if it was given
				List<FileSource> Sdks;
				if (DesiredType == null)
				{
					// get Full and Auto together
					Sdks = TurnkeyManifest.FilterDiscoveredFileSources(Platform, FileSource.SourceType.Full);
					Sdks.AddRange(TurnkeyManifest.FilterDiscoveredFileSources(Platform, FileSource.SourceType.AutoSdk));
				}
				else
				{
					Sdks = TurnkeyManifest.FilterDiscoveredFileSources(Platform, DesiredType);
				}

				DeviceInfo InstallDevice = null;
				if (DesiredType == FileSource.SourceType.Flash)
				{
					InstallDevice = ChosenDevices.First();
 					DeviceName = InstallDevice.Name;
				}

				// skip Misc FileSources, as we dont know how they are sued
				Sdks = Sdks.FindAll(x => x.IsSdkType());

				if (bUpdateOnly)
				{
					// strip out Sdks where there is no Sdk installed yet
					Sdks = Sdks.FindAll(x => !string.IsNullOrEmpty(SDK.GetInstalledVersion()));
				}

				// strip out Sdks not in the allowed range of the platform
				bool bHadInvalidOptions = false;
				if (bValidOnly)
				{
					int BeforeCount = Sdks.Count;
					Sdks = Sdks.FindAll(x => x.IsVersionValid(Platform, InstallDevice));
					
					// track if we removed some invalid ones
					bHadInvalidOptions = Sdks.Count != BeforeCount;
				}

				//// strip out Sdks for platforms that are not needed to be updated
				//if (bNeededOnly)
				//{
				//	Sdks = Sdks.FindAll(x => x.IsNeeded(Platform, DeviceName));
				//}

				Sdks.Sort((a,b) => 
				{
					int Val = a.Type.CompareTo(b.Type);
					if (Val != 0)
					{
						return Val;
					}

					UInt64 VersionA, VersionB;
					if (SDK.TryConvertVersionToInt(a.Version, out VersionA) && SDK.TryConvertVersionToInt(b.Version, out VersionB))
					{
						Val = VersionA.CompareTo(VersionB);
						if (Val != 0)
						{
							return Val;
						}
						if (a.AllowedFlashDeviceTypes != null && b.AllowedFlashDeviceTypes != null)
						{
							return a.AllowedFlashDeviceTypes.CompareTo(b.AllowedFlashDeviceTypes);
						}
					}

					return 0;
				});


				// loop through Sdks that are left, and get the best one for all types
				Dictionary<FileSource.SourceType, FileSource> BestByType = new Dictionary<FileSource.SourceType, FileSource>();
				foreach (FileSource.SourceType Type in (FileSource.SourceType[])Enum.GetValues(typeof(FileSource.SourceType)))
				{
					BestByType[Type] = FileSource.ChooseBest(Sdks.Where(x => x.Type == Type).ToList(), SDK);
				}
				BestByType = BestByType.Where(x => x.Value != null).ToDictionary(x => x.Key, x => x.Value);

				if (bBestAvailable && Sdks.Count > 0)
				{
					// get the best for all types
					Sdks.Clear();
					Sdks.AddRange(BestByType.Values.Where(x => x != null));
 				}
				// if we are not doing best available Sdks, then let used choose one
				else
				{
					bool bAddShowMoreOption = bHadInvalidOptions && bValidOnly;

					if (Sdks.Count == 0 && !bAddShowMoreOption)
					{
						TurnkeyUtils.Log("No Sdks found for platform {0}. Skipping", Platform);
						//if (bStrippedDevices)
						//{
						//	TurnkeyUtils.Log("NOTE: Some Flash Sdks were removed because no devices were found");
						//}
						continue;
					}

					List<string> Options = new List<string>();
					foreach (FileSource Sdk in Sdks)
					{
						Options.Add(string.Format("[{0}] {1}", string.Join(",", Sdk.GetPlatforms()), Sdk.Name));
					}

					if (bAddShowMoreOption)
					{
						Options.Add("Show Invalid Sdks");
					}

					int BestVersionIndex = Sdks.Any() ? Sdks.IndexOf(BestByType.First().Value) : -1;

					SDKCollection SDKInfo;
					if (DesiredType == FileSource.SourceType.Flash)
					{
						DeviceInfo Device = GetDevice(AutomationPlatform, DeviceName);
						if (Device == null)
						{
							TurnkeyUtils.Log("Failed to get device (name: {0}) for platform {1}. Skipping", DeviceName == null ? "<null>/default" : DeviceName, Platform);
							continue;
						}
						SDKInfo = SDK.GetAllSoftwareInfo(Device.Type, Device.SoftwareVersion);
					}
					else
					{
						SDKInfo = SDK.GetAllSDKInfo();
					}
					//if (bStrippedDevices)
					//{
					//	Prompt += "\nNOTE: Some Flash Sdks were removed because no devices were found";
					//}

					string Prompt;
					string TypeString = DesiredType == FileSource.SourceType.Flash ? "Flash" : "Sdk";
					
//					Prompt = $"Select an {TypeString} to install [Current Version: {Current}, Allowed: {ValidVersions}]";
					Prompt = $"Select an {TypeString} to install\n{SDKInfo.ToString("Allowed", "")}";
					if (DesiredType != FileSource.SourceType.Flash)
					{
						Prompt += $"\n(Preferred: {SDK.GetMainVersion()}";
					}
					//}
					//else
					//{
					//	if (DesiredType == FileSource.SourceType.Flash)
					//	{
					//		Prompt = $"Select an {TypeString} to install [Current: {Current}, Valid Range: {MinVersion} - {MaxVersion}]";
					//	}
					//	else
					//	{
					//		Prompt = $"Select an {TypeString} to install [Current: {Current}, Main: {SDK.GetMainVersion()}, Valid Range: {MinVersion} - {MaxVersion}]";
					//	}
					//}


					int Choice = TurnkeyUtils.ReadInputInt(Prompt, Options, true, BestVersionIndex >= 0 ?  (BestVersionIndex + 1) : -1);

					if (Choice == 0)
					{
						continue;
					}

					// if they chose the show more option, then add thius platform back in, and loop with bAllowInvaid to be true
					if (bAddShowMoreOption && Choice == Options.Count)
					{
						PlatformsLeftToInstall.Insert(0, Platform);
						bValidOnly = false;
						continue;
					}

					// only install the chosen one
					FileSource ChosenSdk = Sdks[Choice - 1];
					Sdks = new List<FileSource>() { ChosenSdk };
				}

				foreach (FileSource Sdk in Sdks)
				{
					// because some Sdks can target muiltiple Sdks, if this one was already installed, don't need to reinstall it for other platforms
					if (SdksAlreadyInstalled.Contains(Sdk))
					{
						continue;
					}
					SdksAlreadyInstalled.Add(Sdk);

					// make sure prerequisites are in good shape
					if (AutomationPlatform.UpdateHostPrerequisites(TurnkeyUtils.CommandUtilHelper, TurnkeyContext, false) == false)
					{
						TurnkeyContext.ReportError("Failed to update host prerequisites");
						continue;
					}
					if (InstallDevice != null && 
						AutomationPlatform.UpdateDevicePrerequisites(InstallDevice, TurnkeyUtils.CommandUtilHelper, TurnkeyContext, false) == false)
					{
						TurnkeyContext.ReportError("Failed to update device prerequisites");
						continue;
					}

					bool bSdkAlreadyInstalled = (InstallDevice == null) && (Sdk.Type == FileSource.SourceType.Full) && SDK.GetAllInstalledSDKVersions().Contains(Sdk.Version);

					Sdk.DownloadOrInstall(Platform, TurnkeyContext, InstallDevice, false, bSdkAlreadyInstalled);
				}
			}
		}


		private DeviceInfo GetDevice(AutomationTool.Platform Platform, string DeviceName)
		{
			DeviceInfo[] Devices = Platform.GetDevices();
			if (Devices != null)
			{
				return Array.Find(Platform.GetDevices(), y => (DeviceName == null && y.bIsDefault) || (DeviceName != null && string.Compare(y.Name, DeviceName, true) == 0));
			}
			return null;
		}

	}
}
