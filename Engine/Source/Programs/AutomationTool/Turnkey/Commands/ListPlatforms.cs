// Copyright Epic Games, Inc. All Rights Reserved.

using System.Collections.Generic;
using UnrealBuildTool;
using AutomationTool;
using EpicGames.Core;

namespace Turnkey.Commands
{
	class ListPlatforms : TurnkeyCommand
	{
		protected override CommandGroup Group => CommandGroup.Informational;

		protected override void Execute(string[] CommandOptions)
		{
			TurnkeyUtils.Log("");
			TurnkeyUtils.Log("Platform Information:");

			List<UnrealTargetPlatform> ChosenPlatforms = TurnkeyUtils.GetPlatformsFromCommandLineOrUser(CommandOptions, null);
			foreach (UnrealTargetPlatform TargetPlatform in ChosenPlatforms)
			{
				Platform Platform = Platform.Platforms[new TargetPlatformDescriptor(TargetPlatform)];
				UEBuildPlatformSDK SDK = UEBuildPlatformSDK.GetSDKForPlatform(TargetPlatform.ToString());
				if (SDK == null || !SDK.bIsSdkAllowedOnHost)
				{
					continue;
				}

				SDKCollection SDKInfo = SDK.GetAllSDKInfo();
				SDKCollection SoftwareInfo = SDK.GetAllSoftwareInfo();

				string AutoSDKVersion = SDKInfo.AutoSdk?.Current;
				TurnkeyUtils.Log("  Platform: {0}", TargetPlatform.ToString());
				TurnkeyUtils.Log("  Installed Manual Sdk: {0}", SDKInfo.ToString());
				TurnkeyUtils.Log("  Installed Auto Sdk: {0}", AutoSDKVersion ?? "");
				foreach (SDKDescriptor Desc in SDKInfo.FullSdks)
				{
					TurnkeyUtils.Log($"  Allowed {Desc.Name} Range: {Desc.Min}-{Desc.Max}");
				}
				TurnkeyUtils.Log("  Valid Manual SDK(s) Installed? {0}", SDKInfo.AreAllManualSDKsValid());
				TurnkeyUtils.Log("  Valid Auto SDK Installed? {0}", AutoSDKVersion != null);

				string[] AllVersions = SDK.GetAllInstalledSDKVersions();
				TurnkeyUtils.Log("  AllVersions Installed: {0}", string.Join(",", AllVersions));

				// look for available sdks
				List<FileSource> MatchingFullSdks = TurnkeyManifest.FilterDiscoveredFileSources(TargetPlatform, FileSource.SourceType.Full);
				if (MatchingFullSdks == null || MatchingFullSdks.Count == 0)
				{
					TurnkeyUtils.Log("    NO MATCHING FULL SDK FOUND!");
				}
				else
				{
					TurnkeyUtils.Log("    Possible Full Sdks that could be installed:");

					foreach (FileSource Sdk in MatchingFullSdks)
					{
						TurnkeyUtils.Log(Sdk.ToString(4));
					}
				}

	
				TurnkeyUtils.Log("  Allowed Device Software Range: {0}", SoftwareInfo.ToString(""));
				TurnkeyUtils.Log("  Devices: ");

				DeviceInfo[] Devices = Platform.GetDevices();
				if (Devices == null || Devices.Length == 0)
				{
					TurnkeyUtils.Log("    NO DEVICES FOUND!");
				}
				else
				{
					foreach (DeviceInfo Device in Devices)
					{
						bool bIsSoftwareValid = SDK.IsSoftwareVersionValid(Device.SoftwareVersion, Device.Type);

						TurnkeyUtils.Log("    Name: {0}{1}", Device.Name, Device.bIsDefault ? "*" : "");
						TurnkeyUtils.Log("      Id: {0}", Device.Id);
						TurnkeyUtils.Log("      Type: {0}", Device.Type);
						foreach (KeyValuePair<string, string> Pair in Device.PlatformValues)
						{
							TurnkeyUtils.Log("      {0}: {1}", Pair.Key, Pair.Value);
						}
						TurnkeyUtils.Log("      Installed Software Version: {0}", Device.SoftwareVersion);
						TurnkeyUtils.Log("      Valid Software Installed?: {0}", bIsSoftwareValid);

						if (!bIsSoftwareValid)
						{
							// look for available flash
							List<FileSource> MatchingFlashSdks = TurnkeyManifest.FilterDiscoveredFileSources(TargetPlatform, FileSource.SourceType.Flash).FindAll(x => x.IsVersionValid(TargetPlatform, Device));
							if (MatchingFlashSdks == null || MatchingFlashSdks.Count == 0)
							{
								TurnkeyUtils.Log("      NO MATCHING FLASH SDK FOUND!");
							}
							else
							{
								TurnkeyUtils.Log("      Possible Flash Sdks that could be installed:");

								foreach (FileSource Sdk in MatchingFlashSdks)
								{
									TurnkeyUtils.Log(Sdk.ToString(6));
								}
							}
						}
					}
				}
				TurnkeyUtils.Log("");
			}

		}
	}
}
