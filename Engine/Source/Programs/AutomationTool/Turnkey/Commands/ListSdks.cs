// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using UnrealBuildTool;
using AutomationTool;
using System.Linq;

namespace Turnkey.Commands
{
	class ListSdks : TurnkeyCommand
	{
		protected override CommandGroup Group => CommandGroup.Informational;

		protected override void Execute(string[] CommandOptions)
		{
			TurnkeyUtils.Log("");
			TurnkeyUtils.Log("Available Installers:");

			string TypeString = TurnkeyUtils.ParseParamValue("Type", null, CommandOptions);

			List<UnrealTargetPlatform> Platforms = TurnkeyUtils.GetPlatformsFromCommandLineOrUser(CommandOptions, null);

			FileSource.SourceType? OptionalType = null;

			if (TypeString != null)
			{
				FileSource.SourceType Type;
				if (Enum.TryParse(TypeString, out Type))
				{
					OptionalType = Type;
				}
			}

			List<FileSource> Sdks;
			if (Platforms == null)
			{
				Sdks = TurnkeyManifest.FilterDiscoveredFileSources(null, OptionalType);
			}
			else
			{
				Sdks = Platforms.SelectMany(x => TurnkeyManifest.FilterDiscoveredFileSources(x, OptionalType)).ToList();
			}

			foreach (FileSource Sdk in Sdks)
			{
//				TurnkeyUtils.Log(Sdk.ToString(2));
				TurnkeyUtils.Log(Sdk.ToString());
			}
		}
	}
}
