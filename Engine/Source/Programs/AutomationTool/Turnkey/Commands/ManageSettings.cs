// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Linq;
using System.Text;
using System.Threading.Tasks;

namespace Turnkey.Commands
{
	class ManageSettings : TurnkeyCommand
	{
		protected override CommandGroup Group => CommandGroup.Settings;

		protected override Dictionary<string, string[]> GetExtendedCommandsWithOptions()
		{
			return new Dictionary<string, string[]>()
			{
				{ "List Settings", new string[] { "-ListVars" } },
			};
		}

		protected override void Execute(string[] CommandOptions)
		{
			UserSetting[] UserSettings = TurnkeySettings.AllUserSettings;
			UserSetting[] StudioSettings = TurnkeySettings.AllStudioSettings;

			// need to load the manifests to get StudioSettings!
			TurnkeyManifest.DiscoverManifests();

			if (TurnkeyUtils.ParseParam("ListVars", CommandOptions))
			{
				TurnkeyUtils.Log("All Settings:");
				foreach (UserSetting Setting in UserSettings.Union(StudioSettings))
				{
					TurnkeyUtils.Log("  {0} = '{1}'", Setting.VariableName, TurnkeyUtils.GetVariableValue(Setting.VariableName));
					TurnkeyUtils.Log("    [{0}]", Setting.Description);
				}

				return;
			}


			string Variable = TurnkeyUtils.ParseParamValue("Variable", null, CommandOptions);
			string Value = TurnkeyUtils.ParseParamValue("Value", null, CommandOptions);

			if (Variable != null)
			{
				if (Value == null)
				{
					TurnkeyUtils.Log("Error: When using -Variable=, you must also specify a value with -Value=", Variable);
					return;
				}

				// make sure it's a valid variable
				if (!Array.Exists(UserSettings, x => x.VariableName == Variable))
				{
					TurnkeyUtils.Log("Error: {0} is not a valid variable (variables are case sensitive, and Studio settings cannot be set)", Variable);
					return;
				}

				// finally set the value and done!
				string Previous = TurnkeySettings.SetUserSetting(Variable, Value);
				TurnkeyUtils.Log("Setting {0} to '{1}' (was '{2}')", Variable, Value, Previous);

				return;
			}

			// if nothing already handled, list all variables and let user enter a vale

			List<string> Options = new List<string>();


			foreach (UserSetting Setting in UserSettings)
			{
				Options.Add(string.Format("{0} - Currently '{1}'\n    [{2}]", Setting.VariableName, TurnkeyUtils.GetVariableValue(Setting.VariableName), Setting.Description));
			}

			int Choice = TurnkeyUtils.ReadInputInt("Choose a variable to set:", Options, true);
			if (Choice > 0)
			{
				string NewValue = TurnkeyUtils.ReadInput(string.Format("Enter value for {0} [Currently '{1}']", UserSettings[Choice - 1].VariableName, TurnkeyUtils.GetVariableValue(UserSettings[Choice - 1].VariableName)));
				if (NewValue == "")
				{
					int EmptyChoice = TurnkeyUtils.ReadInputInt("Empty response was given - what did you want to do?", new List<string> { "Delete setting", "Set to blank", "Leave existing value alone" }, false);
					if (EmptyChoice == 0)
					{
						TurnkeyUtils.Log("Clearing Settings[{0}].{1}", Choice - 1, UserSettings[Choice - 1].VariableName);
						TurnkeySettings.SetUserSetting(UserSettings[Choice - 1].VariableName, null);
						TurnkeySettings.Save();
					}
					else
					{
						NewValue = EmptyChoice == 1 ? "" : null;
					}
				}

				if (NewValue != null)
				{
					TurnkeyUtils.Log("Setting Settings[{0}].{1} = {2}", Choice - 1, UserSettings[Choice - 1].VariableName, NewValue);
					TurnkeySettings.SetUserSetting(UserSettings[Choice - 1].VariableName, NewValue);
					TurnkeySettings.Save();
				}
			}
		}
	}
}
