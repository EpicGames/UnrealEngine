// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using UnrealBuildTool;
using System.Linq;
using EpicGames.Core;

namespace Turnkey.Commands
{
	class QuickSwitchSdk : TurnkeyCommand
	{
		protected override CommandGroup Group => CommandGroup.Sdk;

		protected override void Execute(string[] CommandOptions)
		{
			// get the list of platforms to use
			IEnumerable<UnrealTargetPlatform> SelectedPlatforms;
			bool HasSuppliedCommand = TurnkeyUtils.ParseParamValue("Command", null, CommandOptions) != null;
			bool HasSuppliedPlatform = TurnkeyUtils.ParseParamValue("Platform", null, CommandOptions) != null;
			if (!HasSuppliedCommand || HasSuppliedPlatform)
			{
				SelectedPlatforms = TurnkeyUtils.GetPlatformsFromCommandLineOrUser(CommandOptions, null);
			}
			else
			{
				// if the user is running this command directly but has not specified any platforms, default to all valid platforms for convenience (RunUAT Turnkey -command=QuickSwitchSdk)
				SelectedPlatforms = UnrealTargetPlatform.GetValidPlatforms().Where(x => UEBuildPlatformSDK.GetSDKForPlatform(x.ToString()) != null);
			}

			// keep going while we have Sdks left to install
			TurnkeyUtils.StartTrackingExternalEnvVarChanges();
			foreach (UnrealTargetPlatform Platform in SelectedPlatforms)
			{
				UEBuildPlatformSDK PlatformSDK = UEBuildPlatformSDK.GetSDKForPlatform(Platform.ToString());

				// find the highest valid Sdk version & see if we can to take action
				string SelectedSDK = PlatformSDK.GetAllInstalledSDKVersions()
					.Where(x => PlatformSDK.IsVersionValid(x, "Sdk"))
					.OrderByDescending(x => PlatformSDK.TryConvertVersionToInt(x, out UInt64 Version) ? Version : 0)
					.FirstOrDefault();

				string CurrentSDK = PlatformSDK.GetInstalledVersion("Sdk");
				if (SelectedSDK == null && CurrentSDK == null)
				{
					TurnkeyUtils.Log("{0} - No suitable SDKs installed", Platform);
					continue;
				}
				if (SelectedSDK == null || SelectedSDK == CurrentSDK)
				{
					TurnkeyUtils.Log("{0} - Best SDK for is already installed ({1})", Platform, SelectedSDK??CurrentSDK);
					continue;
				}

				// attempt to quick switch to the selected SDK version
				bool bWasSwitched = PlatformSDK.SwitchToAlternateSDK(SelectedSDK, false );
				if (bWasSwitched == true)
				{
					TurnkeyUtils.Log("{0} - Quick-switched to already-installed version ({1})", Platform, SelectedSDK);
				}
				else
				{
					TurnkeyUtils.Log("{0} - Failed to quick switch to already-installed version ({1})", Platform, SelectedSDK);
				}
			}
			TurnkeyUtils.EndTrackingExternalEnvVarChanges();
		}
	}
}
