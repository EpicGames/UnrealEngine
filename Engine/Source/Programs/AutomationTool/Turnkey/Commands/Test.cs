// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Linq;
using System.Text;
using System.Threading.Tasks;

namespace Turnkey.Commands
{
	class Test : TurnkeyCommand
	{
		protected override CommandGroup Group => CommandGroup.Misc;

		protected override void Execute(string[] CommandOptions)
		{
			TurnkeyUtils.Log("Testing:");
			TurnkeyUtils.Log("");


			string[] TestEnumerates =
			{
				@"googledrive:/SdkInstalls/*/DummyInstall.zip",
				@"googledrive:/SdkInstalls/*/*.xml",
				@"googledrive:'1O6V1JO-LNZYxcB6wIqCiHjQ44mz4tv7S'/*",
				@"file:D:\Engine\*\Build\*.xml",
				@"file:D:\Engine\*\*",
				@"perforce://UE5/Main/*/Build/*.xml",
			};

			foreach (string Test in TestEnumerates)
			{
				TurnkeyUtils.Log("Enumerating {0}:", Test);
				string[] Results = CopyProvider.ExecuteEnumerate(Test);

				if (Results != null)
				{
					foreach (string Result in Results)
					{
						TurnkeyUtils.Log("  {0}", Result);
					}
				}
			}


			string[] TestCopies =
			{
				@"googledrive:/SdkInstalls/Installers/DummyInstall.zip",
				@"googledrive:/SdkInstalls/*/DummyInstall.zip",
				@"googledrive:/SdkInstalls/*",
			};

			foreach (string Test in TestCopies)
			{
				TurnkeyUtils.Log("Copying {0}:", Test);
				string Result = CopyProvider.ExecuteCopy(Test);

				TurnkeyUtils.Log("  Copied to {0}", Result);
			}
		}
	}
}
