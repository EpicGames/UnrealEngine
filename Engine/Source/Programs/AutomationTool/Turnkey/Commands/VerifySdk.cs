// Copyright Epic Games, Inc. All Rights Reserved.

using AutomationTool;
using Gauntlet;
using System;
using System.Collections.Generic;
using System.IO;
using System.Linq;
using EpicGames.Core;
using UnrealBuildTool;
using UnrealBuildBase;

namespace Turnkey.Commands
{
	class VerifySdk : TurnkeyCommand
	{
		protected override CommandGroup Group => CommandGroup.Sdk;

		protected override void Execute(string[] CommandOptions)
		{
			bool bUnattended = TurnkeyUtils.ParseParam("Unattended", CommandOptions);
			bool bPreferFullSdk = TurnkeyUtils.ParseParam("PreferFull", CommandOptions);
			bool bForceSdkInstall = TurnkeyUtils.ParseParam("ForceSdkInstall", CommandOptions);
			bool bForceDeviceInstall = TurnkeyUtils.ParseParam("ForceDeviceInstall", CommandOptions);
			bool bUpdateIfNeeded = bForceSdkInstall || bForceDeviceInstall || TurnkeyUtils.ParseParam("UpdateIfNeeded", CommandOptions);
			bool bSkipPlatformCheck = TurnkeyUtils.ParseParam("SkipPlatform", CommandOptions);
			bool bAutoChooseBest = bUnattended || bUpdateIfNeeded;

			List<UnrealTargetPlatform> PlatformsToCheck;
			List<DeviceInfo> DevicesToCheck;
			TurnkeyUtils.GetPlatformsAndDevicesFromCommandLineOrUser(CommandOptions, true, out PlatformsToCheck, out DevicesToCheck);

			// if we got no devices, requested some, and platforms were not specified, then we don't want to continue.
			// if -device and -platform were specified, and no devices found, we will still continue with the platforms
			if (DevicesToCheck.Count == 0 && TurnkeyUtils.ParseParamValue("Device", null, CommandOptions) != null && TurnkeyUtils.ParseParamValue("Platform", null, CommandOptions) == null)
			{
				TurnkeyUtils.Log("Devices were requested, but none of them were found. Since -platforms was not specified, exiting command...");
			}

			if (PlatformsToCheck.Count == 0 && !bSkipPlatformCheck)
			{
				TurnkeyUtils.Log("Platform(s) and/or device(s) needed for VerifySdk command. Check parameters or selections.");
				return;
			}

			TurnkeyUtils.Log("Installed Sdk validity:");
			TurnkeyUtils.ExitCode = ExitCode.Success;

			TurnkeyContextImpl TurnkeyContext = new TurnkeyContextImpl();

			TurnkeyUtils.StartTrackingExternalEnvVarChanges();

			// check all the platforms
			foreach (UnrealTargetPlatform Platform in PlatformsToCheck)
			{
				UEBuildPlatformSDK PlatformSDK = UEBuildPlatformSDK.GetSDKForPlatform(Platform.ToString());

				// get the platform object
				AutomationTool.Platform AutomationPlatform = AutomationTool.Platform.GetPlatform(Platform);

				// reset the errors for each device
				TurnkeyContext.ErrorMessages.Clear();

				// checking availability may generate errors, if prerequisites are failing
				SdkUtils.LocalAvailability LocalState = SdkUtils.GetLocalAvailability(AutomationPlatform, bUpdateIfNeeded, TurnkeyContext);


				SdkUtils.LocalAvailability ReportedState = LocalState;
				string StatusString;
				bool bHasOutOfDateSDK = false;

				if ((LocalState & SdkUtils.LocalAvailability.Platform_ValidHostPrerequisites) == 0)
				{
					StatusString = "Invalid";
				}
				else if ((LocalState & (SdkUtils.LocalAvailability.AutoSdk_ValidVersionExists | SdkUtils.LocalAvailability.InstalledSdk_ValidVersionExists)) == 0)
				{
					StatusString = "Invalid";
					bHasOutOfDateSDK = true;
				}
				else
				{
					StatusString = "Valid";
					ReportedState &= (SdkUtils.LocalAvailability.AutoSdk_ValidVersionExists | SdkUtils.LocalAvailability.InstalledSdk_ValidVersionExists | SdkUtils.LocalAvailability.Support_FullSdk | SdkUtils.LocalAvailability.Support_AutoSdk | SdkUtils.LocalAvailability.Sdk_HasBestVersion);
				}


				// gather list of tags/values to write out
				SDKCollection SDKInfo = null;
				List<string> PlatformProperties = new List<string>();
				{
					// overall status (valid/invalid)
					PlatformProperties.Add($"Status={StatusString}");

					if (PlatformSDK != null)
					{
						SDKInfo = PlatformSDK.GetAllSDKInfo();
						if (SDKInfo.FullSdks.Count() > 1)
						{
							PlatformProperties.Add($"SDKs=\"{string.Join(",", SDKInfo.FullSdks.Select(x => x.Name))}\"");
						}
						// get all the Min/Max versions and insert them right in, they will be in the proper format for parsing
						PlatformProperties.Add(SDKInfo.ToString("Allowed", ""));
					}

					PlatformProperties.Add($"Flags=\"{ReportedState}\"");

					if (TurnkeyContext.ErrorMessages.Count() > 0)
					{
						PlatformProperties.Add($"Error=\"{string.Join("|", TurnkeyContext.ErrorMessages)}\"");
					}
				}

				// write out all of the properties in unreal struct format: "Platform: (Props,...)"
				TurnkeyUtils.Report("{0}: ({1})", Platform, string.Join(", ", PlatformProperties));

				if (PlatformSDK == null)
				{
					continue;
				}

				// install if out of date, or if forcing it
				if (bForceSdkInstall || (bUpdateIfNeeded && bHasOutOfDateSDK))
				{
					// gather the SDKs that we need to install
					List<FileSource> SdksToInstall = new List<FileSource>();
					if (!bPreferFullSdk)
					{
						// attempt to install Autosdk
						FileSource AutoSdk = FileSource.FindMatchingSdk(AutomationPlatform, new FileSource.SourceType[] { FileSource.SourceType.AutoSdk }, bSelectBest: bAutoChooseBest);

						if (AutoSdk != null)
						{
							SdksToInstall.Add(AutoSdk);
						}
					}

					// if no autosdk was chosen, look for manual sdk(s)
					if (SdksToInstall.Count == 0)
					{
						// find manual versions that need to be installed (or all if forceing)
						IEnumerable<SDKDescriptor> NeededManualSdks = SDKInfo.FullSdks.Where(x => (bForceSdkInstall || x.Validity == SDKStatus.Invalid));
						List<string> MissingInstallers = new List<string>();

						foreach (SDKDescriptor Desc in NeededManualSdks)
						{
							// if we only have one manual sdk (normal), then don't specify a type
							string SpecificType = NeededManualSdks.Count() == 1 ? null : Desc.Name;
							FileSource SdkSource = FileSource.FindMatchingSdk(AutomationPlatform, new FileSource.SourceType[] { FileSource.SourceType.Full }, bSelectBest: bAutoChooseBest, SpecificType: SpecificType );

							if (SdkSource != null)
							{
								SdksToInstall.Add(SdkSource);
							}
							else
							{
								MissingInstallers.Add(Desc.Name);
							}
						}

						if (NeededManualSdks.Count() > 1 && MissingInstallers.Count > 0)
						{
							TurnkeyUtils.Log($"ERROR: {Platform}: Unable to find all Sdk installers. Missing '{string.Join(", ", MissingInstallers)}'. Will the ones that were found...");
						}
					}

					// if we went manual first, and still don't have any sdks, try for autosdk now
					if (bPreferFullSdk && SdksToInstall.Count == 0)
					{
						// attempt to install Autosdk
						FileSource AutoSdk = FileSource.FindMatchingSdk(AutomationPlatform, new FileSource.SourceType[] { FileSource.SourceType.AutoSdk }, bSelectBest: bAutoChooseBest);

						if (AutoSdk != null)
						{
							SdksToInstall.Add(AutoSdk);
						}
					}


					//FileSource BestSdk = null;
					//// find the best Sdk, prioritizing as request
					//if (bPreferFullSdk)
					//{
					//	BestSdk = FileSource.FindMatchingSdk(AutomationPlatform, new FileSource.SourceType[] { FileSource.SourceType.Full, FileSource.SourceType.BuildOnly, FileSource.SourceType.AutoSdk }, bSelectBest: bAutoChooseBest);
					//}
					//else
					//{
					//	BestSdk = FileSource.FindMatchingSdk(AutomationPlatform, new FileSource.SourceType[] { FileSource.SourceType.AutoSdk, FileSource.SourceType.BuildOnly, FileSource.SourceType.Full }, bSelectBest: bAutoChooseBest);
					//}

					if (SdksToInstall == null)
					{
						TurnkeyUtils.Log("ERROR: {0}: Unable to find any Sdks that could be installed", Platform);
						TurnkeyUtils.ExitCode = ExitCode.Error_SDKNotFound;
					}
					else
					{
						TurnkeyUtils.Log("Will install '{0}'", string.Join(", ", SdksToInstall.Select(x => x.Name)));

						foreach (FileSource Sdk in SdksToInstall)
						{
							bool bSdkAlreadyInstalled = Sdk.Type == FileSource.SourceType.Full && PlatformSDK.GetAllInstalledSDKVersions().Contains(Sdk.Version);

							// attempt to fast switch to the best one if it's already fully installed, unless we are forcing a reinstall
							if (!bForceSdkInstall && bSdkAlreadyInstalled)
							{
								bool bWasSwitched = PlatformSDK.SwitchToAlternateSDK(Sdk.Version, false);
								if (bWasSwitched == true)
								{
									TurnkeyUtils.Log("Fast-switched to already-installed version {0}", Sdk.Version);

									// if SwitchToAlternateSDK returns true, then we are good to go!
									continue;
								}
							}

							if (Sdk.DownloadOrInstall(Platform, TurnkeyContext, null, bUnattended, bSdkAlreadyInstalled) == false)
							{
								TurnkeyUtils.Log("Failed to install {0}", Sdk.Name);
								TurnkeyUtils.ExitCode = ExitCode.Error_SDKInstallFailed;
							}
						}

						// update LocalState
						//					LocalState = SdkUtils.GetLocalAvailability(AutomationPlatform, false);

						// @todo turnkey: validate!
					}
				}

				// now check software version of each device
				if (DevicesToCheck != null)
				{
					foreach (DeviceInfo Device in DevicesToCheck.Where(x => x.Platform == Platform))
					{
						// reset the errors for each device
						TurnkeyContext.ErrorMessages.Clear();

						bool bArePrerequisitesValid = AutomationPlatform.UpdateDevicePrerequisites(Device, TurnkeyUtils.CommandUtilHelper, TurnkeyContext, !bUpdateIfNeeded);

						// get the min/max versions from PlatformSDK
						SDKCollection SoftwareInfo = PlatformSDK.GetAllSoftwareInfo(Device.Type, Device.SoftwareVersion);
						if (!SoftwareInfo.Sdks.Any())
						{
							// if we found no supported software versions for the specific device type, query for device type independent versions
							SoftwareInfo = PlatformSDK.GetAllSoftwareInfo(null, Device.SoftwareVersion);
						}

						SdkUtils.LocalAvailability DeviceState = SdkUtils.LocalAvailability.None;
						if (!bArePrerequisitesValid)
						{
							StatusString = "Invalid";
							DeviceState |= SdkUtils.LocalAvailability.Device_InvalidPrerequisites;
						}
						else if (SoftwareInfo.AreAllManualSDKsValid())
						{
							StatusString = "Valid";
							DeviceState |= SdkUtils.LocalAvailability.Device_InstallSoftwareValid;
						}
						else
						{
							StatusString = "Invalid";
							DeviceState |= SdkUtils.LocalAvailability.Device_InstallSoftwareInvalid;
						}

						if (Device.bCanConnect == false)
						{
							DeviceState |= SdkUtils.LocalAvailability.Device_CannotConnect;
						}

						if (Device.AutoSoftwareUpdates == AutomationTool.DeviceInfo.AutoSoftwareUpdateMode.Disabled)
						{
							DeviceState |= SdkUtils.LocalAvailability.Device_AutoSoftwareUpdates_Disabled;
						}
						else if (Device.AutoSoftwareUpdates == AutomationTool.DeviceInfo.AutoSoftwareUpdateMode.Enabled)
						{
							DeviceState |= SdkUtils.LocalAvailability.Device_AutoSoftwareUpdates_Enabled;
						}

						List<string> DeviceProperties = new List<string>()
							{
								$"Name={Device.Name}",
								$"Type={Device.Type}",
								$"Status={StatusString}",
								SoftwareInfo.ToString("Allowed", ""),
								$"Flags=\"{DeviceState}\"",
							};
						
						foreach (KeyValuePair<string, string> Pair in Device.PlatformValues)
						{
							DeviceProperties.Add($"{Pair.Key}=\"{Pair.Value}\"");
						}

						if (TurnkeyContext.ErrorMessages.Count() > 0)
						{
							DeviceProperties.Add(string.Format("Error=\"{0}\"", string.Join("|", TurnkeyContext.ErrorMessages)));
						}

						// write out all of the properties in unreal struct format: "Platform: (Props,...)"
						TurnkeyUtils.Report("{0}@{1}: ({2})", Platform, Device.Id, string.Join(", ", DeviceProperties));

						//TurnkeyUtils.Report("{0}@{1}: (Name={2}, Status={3}, Installed={4}, MinAllowed={5}, MaxAllowed={6}, Flags=\"{7}\"{8})", Platform, Device.Id, Device.Name, StatusString, Device.SoftwareVersion,
						//	MinSoftwareAllowedVersion, MaxSoftwareAllowedVersion, DeviceState.ToString(), DeviceErrorString);

						if (bForceDeviceInstall || !SoftwareInfo.AreAllManualSDKsValid())
						{
							if (bUpdateIfNeeded)
							{
								if (Device.bCanConnect)
								{
									FileSource MatchingInstallableSdk = FileSource.FindMatchingSdk(AutomationPlatform, new FileSource.SourceType[] { FileSource.SourceType.Flash }, bSelectBest: bUnattended, DeviceType: Device.Type, CurrentSdk: Device.SoftwareVersion);

									if (MatchingInstallableSdk == null)
									{
										TurnkeyUtils.Log("ERROR: {0}: Unable to find any Sdks that could be installed on {1}", Platform, Device.Name);
										TurnkeyUtils.ExitCode = ExitCode.Error_SDKNotFound;
									}
									else
									{
										if (MatchingInstallableSdk.DownloadOrInstall(Platform, TurnkeyContext, Device, bUnattended, false) == false)
										{
											TurnkeyUtils.Log("Failed to update Device '{0}' with '{0}'", Device.Name, MatchingInstallableSdk.Name);
											TurnkeyUtils.ExitCode = ExitCode.Error_DeviceUpdateFailed;
										}
									}
								}
								else
								{
									TurnkeyUtils.Log("Skipping device {0} because it cannot connect.", Device.Name);
								}
							}
						}
					}
				}
			}

			TurnkeyUtils.EndTrackingExternalEnvVarChanges();
		}
	}
}
