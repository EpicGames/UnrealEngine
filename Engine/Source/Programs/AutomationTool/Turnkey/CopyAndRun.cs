// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Xml.Serialization;
using System.IO;
using UnrealBuildTool;
using AutomationTool;
using EpicGames.Core;
using System.Diagnostics;
using System.ComponentModel;

namespace Turnkey
{
	[XmlRoot(ElementName = "Source")]
	public class CopyAndRun
	{
		// @todo turnkey: for some reason the TypeConverter stuff setup in UnrealTargetPlatform isn't kicking in to convert from string to UTP, so do it a manual way
		[XmlElement("HostPlatform")]
		public string PlatformString = null;

		[XmlIgnore]
		public UnrealTargetPlatform? Platform = null;

		public string Copy = null;

		public string CommandPath = null;

		public string CommandLine = null;

		/// <summary>
		/// Needs a parameterless constructor for Xml deserialization
		/// </summary>
		CopyAndRun()
		{ }

		/// <summary>
		/// Create a one-of local object with just a Copy operation
		/// </summary>
		/// <param name="CopyOperation"></param>
		public CopyAndRun(string CopyOperation)
		{
			Copy = TurnkeyUtils.ExpandVariables(CopyOperation);
		}
		public CopyAndRun(CopyAndRun Other)
		{
			PlatformString = Other.PlatformString;
			Platform = Other.Platform;
			Copy = Other.Copy;
			CommandPath = Other.CommandPath;
			CommandLine = Other.CommandLine;
		}

		internal void PostDeserialize()
		{
			if (!string.IsNullOrEmpty(PlatformString))
			{
				Platform = UnrealTargetPlatform.Parse(PlatformString);
			}

			// perform early expansion, important for $(ThisManifestDir) which is valid only during deserialization
			// but don't use any other variables yet, because UAT could have bad values in Environment
			Copy = TurnkeyUtils.ExpandVariables(Copy, bUseOnlyTurnkeyVariables: true);
			CommandPath = TurnkeyUtils.ExpandVariables(CommandPath, bUseOnlyTurnkeyVariables: true);
			CommandLine = TurnkeyUtils.ExpandVariables(CommandLine, bUseOnlyTurnkeyVariables: true);
		}

		public static int RunExternalCommand(string CommandPath, string CommandLine, ITurnkeyContext TurnkeyContext, bool bUnattended, bool bRequiresPrivilegeElevation, bool bCreateWindow)
		{
			string FixedCommandPath = TurnkeyUtils.ExpandVariables(CommandPath);
			string FixedCommandLine = TurnkeyUtils.ExpandVariables(CommandLine);
			FixedCommandPath = FixedCommandPath.Replace(Path.AltDirectorySeparatorChar, Path.DirectorySeparatorChar);

			string PreviousCWD = Environment.CurrentDirectory;
			// if a directory was included in the command path, then run from there
			string CommandWorkingDir = Path.GetDirectoryName(FixedCommandPath);

			// if we run UseShellExecute with true, StartInfo.WorkingDirectory won't set the directory as expected
			if (!string.IsNullOrEmpty(CommandWorkingDir))
			{
				Environment.CurrentDirectory = CommandWorkingDir;
			}


			TurnkeyUtils.StartTrackingExternalEnvVarChanges();

			// run installer as administrator, as some need it
			Process InstallProcess = new Process();
			InstallProcess.StartInfo.UseShellExecute = bCreateWindow;
			InstallProcess.StartInfo.FileName = FixedCommandPath;
			InstallProcess.StartInfo.Arguments = FixedCommandLine;
			InstallProcess.StartInfo.WindowStyle = bCreateWindow ? ProcessWindowStyle.Normal : ProcessWindowStyle.Hidden;

			InstallProcess.OutputDataReceived += (Sender, Args) => { if (Args != null && Args.Data != null) TurnkeyUtils.Log(Args.Data.TrimEnd()); };
			InstallProcess.ErrorDataReceived += (Sender, Args) => { if (Args != null && Args.Data != null) TurnkeyUtils.Log("Error: {0}", Args.Data.TrimEnd()); };

			//installers may require administrator access to succeed. so run as an admmin.

			// run in a loop in case of a failure
			bool bDone = false;
			int ExitCode = -1;

			while (!bDone)
			{
				try
				{
					if (bRequiresPrivilegeElevation && InstallProcess.StartInfo.Verb != "runas")
					{
						TurnkeyUtils.Log("The installer {0} requires elevated permissions, trying with Admin privileges (output may be hidden)", FixedCommandPath);

						InstallProcess.StartInfo.UseShellExecute = true;
						InstallProcess.StartInfo.Verb = "runas";
						InstallProcess.StartInfo.WindowStyle = ProcessWindowStyle.Normal;
					}

					InstallProcess.Start();
					InstallProcess.WaitForExit();
					ExitCode = InstallProcess.ExitCode;
				}
				catch (Exception Ex)
				{
					// native error in a Win32Exception, of 740, means the process needs elevation, so we need to runas. However,
					// this will not allow capturing stdout, so run with window as Normal
					if (InstallProcess.StartInfo.UseShellExecute == false && Ex is Win32Exception && ((Win32Exception)Ex).NativeErrorCode == 740)
					{
						bRequiresPrivilegeElevation = true;
						continue;
					}

					TurnkeyContext.ReportError($"Error: {FixedCommandPath} caused an exception: {Ex.Message}");
					ExitCode = -1;
				}

				if (ExitCode != 0)
				{
					TurnkeyUtils.Log("");
					TurnkeyContext.ReportError($"Command {FixedCommandPath} {FixedCommandLine} failed [Exit code {ExitCode}, working dir = {CommandWorkingDir}]");
					TurnkeyUtils.Log("");

					if (!bUnattended)
					{
						bool bResponse = TurnkeyUtils.GetUserConfirmation("Do you want to attempt again?", false);
						if (bResponse == false)
						{
							bDone = true;
						}
					}
					else
					{
						bDone = true;
					}
				}
				else
				{
					bDone = true;
				}
			}

			TurnkeyUtils.EndTrackingExternalEnvVarChanges();

			Environment.CurrentDirectory = PreviousCWD;

			return ExitCode;
		}
	}
}
