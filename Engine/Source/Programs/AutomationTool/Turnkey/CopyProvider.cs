// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Linq;
using System.Reflection;
using System.Text;
using System.Threading.Tasks;
using System.IO;
using AutomationTool;
using UnrealBuildTool;
using EpicGames.Core;
using System.IO.Compression;

namespace Turnkey
{
	[Flags]
	public enum CopyExecuteSpecialMode
	{
		None = 0,
		UsePermanentStorage = 1,
		DownloadOnly = 2,
	}

	abstract class CopyProvider
	{
		/// <summary>
		/// Unique provider token to direct a copy operation ("perforce://UE5/Main/...)
		/// </summary>
		public abstract string ProviderToken { get; }

		/// <summary>
		/// Perform the copy operation
		/// </summary>
		/// <param name="Operation">Description for the operation</param>
		/// <param name="SpecialMode"></param>
		/// <param name="SpecialModeHint"></param>
		/// <returns>The output path of the copied file, or a directory that contains all of the wildcards in the operation ("perforce://depot/CarefullyRedist/.../Windows/*" would return something like "d:\\sdks")</returns>
		public abstract string Execute(string Operation, CopyExecuteSpecialMode SpecialMode, string SpecialModeHint);

		/// <summary>
		/// Uses the CopyProvider to return a list of files and directories (a directory MUST end with a / or \ character to denote directory since 
		/// we can't test locally what type it is
		/// </summary>
		/// <param name="Operation">Description for the operation, including a ProviderToken</param>
		/// <param name="Expansions">A list of what *'s expanded to, one entry for each result</param>
		/// <returns></returns>
		public abstract string[] Enumerate(string Operation, List<List<string>> Expansions);




		private static Dictionary<string, CopyProvider> CachedProviders = new Dictionary<string, CopyProvider>(StringComparer.OrdinalIgnoreCase);

		static CopyProvider()
		{
			// look for all subclasses, and cache by their ProviderToken
			foreach (Type AssemType in Assembly.GetExecutingAssembly().GetTypes())
			{
				if (typeof(CopyProvider).IsAssignableFrom(AssemType) && AssemType != typeof(CopyProvider))
				{
					CopyProvider Provider = (CopyProvider)Activator.CreateInstance(AssemType);
					CachedProviders[Provider.ProviderToken] = Provider;
				}
			}
		}

		private static bool ParseOperation(string Operation, out CopyProvider Provider, out string ProviderParam, bool bCanFail)
		{
			Operation = TurnkeyUtils.ExpandVariables(Operation);

			Provider = null;
			ProviderParam = null;

			int ColonLocation = Operation.IndexOf(':');
			if (ColonLocation < 0)
			{
				if (bCanFail)
				{
					return false;
				}
				throw new AutomationException("Malformed copy operation: {0}", Operation);
			}
			// get the token before the :
			string Token = Operation.Substring(0, ColonLocation);

			if (!CachedProviders.TryGetValue(Token, out Provider))
			{
				if (bCanFail)
				{
					return false;
				}
				throw new AutomationException("Unable to find a CopyProvider for copy type {0}", Token);
			}

			ProviderParam = Operation.Substring(ColonLocation + 1);
			return true;
		}

		/// <summary>
		/// Runs a Copy command, and returns the local path (either a directory or a file, depending on the operation
		/// </summary>
		/// <param name="CopyOperation"></param>
		/// <param name="SpecialMode"></param>
		/// <param name="SpecialModeHint"></param>
		/// <returns>Output path, which could then be used as $(OutputPath) in later operations</returns>
		public static string ExecuteCopy(string CopyOperation, CopyExecuteSpecialMode SpecialMode = CopyExecuteSpecialMode.None, string SpecialModeHint = null)
		{
			TurnkeyUtils.ClearVariable("CopyOutputPath");

			CopyProvider Provider;
			string ProviderParam;

			ParseOperation(CopyOperation, out Provider, out ProviderParam, false);

			string OperationExt = Path.GetExtension(ProviderParam).ToLower();
			bool bWillDecompressToLocation = (OperationExt == ".zip" || OperationExt == ".7z") && SpecialMode == CopyExecuteSpecialMode.UsePermanentStorage;

			// execute what comes after the colon (if we guess we are going to decompress, don't pass in a specified location, we will decompress to it after)
			string OutputPath = bWillDecompressToLocation ? Provider.Execute(ProviderParam, CopyExecuteSpecialMode.None, null) : 
				Provider.Execute(ProviderParam, SpecialMode, SpecialModeHint);

			// always unzip the file into a temp directory (or downloadpath if it's a permanent download), and make that directory be the outputpath
			if (OutputPath != null)
			{
				OutputPath = OutputPath.Replace(Path.AltDirectorySeparatorChar, Path.DirectorySeparatorChar);
				string Ext = Path.GetExtension(OutputPath).ToLower();
				if (Ext == ".zip" || Ext == ".7z")
				{
					// check if this file already has been unzipped - if the provider output to a new temp location but the same file, 
					// we can't trust it, so we re-decompress, or if the source file's date changed
					string FileVersion = File.GetLastWriteTimeUtc(OutputPath).ToString();
					string Tag = OutputPath + "_Decompressed";

					string CachedLocation = LocalCache.GetCachedPathByTag(Tag, FileVersion);

					// if it was there, use it directly
					if (CachedLocation != null)
					{
						OutputPath = CachedLocation;
					}
					else
					{
						// make a random temp directory, or use the download directory for permanent downloads
						string DecompressLocation;
						if (SpecialMode == CopyExecuteSpecialMode.UsePermanentStorage)
						{
							DecompressLocation = SpecialModeHint; // Path.Combine(Path.GetDirectoryName(OutputPath), Path.GetFileNameWithoutExtension(OutputPath) + "_Uncompressed");
						}
						else
						{
							DecompressLocation = LocalCache.CreateTempDirectory();
						}

						bool bFailed = false;
						if (Ext == ".zip")
						{
							try
							{
								TurnkeyUtils.Log("Unzipping {0} to {1}...", OutputPath, DecompressLocation);
								ZipFile.ExtractToDirectory(OutputPath, DecompressLocation, overwriteFiles: true);
							}
							catch (Exception Ex)
							{
								TurnkeyUtils.Log("Unzip failed: {0}", Ex);
								bFailed = true;
							}
						}
						else if (Ext == ".7z")
						{
							TurnkeyUtils.Log("7zip decompressing {0} to {1}...", OutputPath, DecompressLocation);

							int ExitCode = UnrealBuildTool.Utils.RunLocalProcessAndLogOutput(TurnkeyUtils.ExpandVariables("$(EngineDir)/Restricted/NotForLicensees/Extras/ThirdPartyNotUE/7-Zip/7z.exe"),
								string.Format("x -o{0} {1}", DecompressLocation, OutputPath), Log.Logger);
							if (ExitCode != 0)
							{
								TurnkeyUtils.Log("Failed to uncompress a .7z file {0}", OutputPath);
								bFailed = true;
							}
						}

						// the temp dir is now the outputpatb to return to later installation steps
						if (!bFailed)
						{
							OutputPath = DecompressLocation;
							LocalCache.CacheLocationByTag(Tag, OutputPath, FileVersion);
						}
						else
						{
							// return null on a failure
							OutputPath = null;
						}

						// @todo turnkey: let the CopyProvider delete the file (p4 needs to perform deletes of synced files)
					}
				}
			}

			TurnkeyUtils.SetVariable("CopyOutputPath", OutputPath);

			return OutputPath;
		}

		public static string[] ExecuteEnumerate(string CopyOperation, List<List<string>> Expansions=null)
		{
			CopyProvider Provider;
			string ProviderParam;

			// we allow this to fail (like an unknown variable, etc)
			if (!ParseOperation(CopyOperation, out Provider, out ProviderParam, true))
			{
				return null;
			}

			return Provider.Enumerate(ProviderParam, Expansions);
		}
	}

	class TurnkeyContextImpl : ITurnkeyContext
	{
		public List<string> ErrorMessages = new List<string>();

		public string RetrieveFileSource(object HintObject)
		{
			FileSource Sdk = (FileSource)HintObject;

			// run the first copy for the host platform`
			foreach (CopySource Copy in Sdk.Sources)
			{
				if (Copy.ShouldExecute())
				{
					return CopyProvider.ExecuteCopy(Copy.GetOperation());
				}
			}

			return null;
		}

		public int RunExternalCommand(string Command, string Params, bool bRequiresPrivilegeElevation, bool bUnattended, bool bCreateWindow)
		{
			TurnkeyUtils.Log("----------------------------------------------");
			TurnkeyUtils.Log("Running '{0} {1}'", TurnkeyUtils.ExpandVariables(Command), TurnkeyUtils.ExpandVariables(Params));

			TurnkeyUtils.Log("----------------------------------------------", Command);

			int ExitCode = CopyAndRun.RunExternalCommand(Command, Params, this, bUnattended, bRequiresPrivilegeElevation, bCreateWindow);

			TurnkeyUtils.Log("----------------------------------------------");
			TurnkeyUtils.Log($"Finished with {ExitCode}");

			TurnkeyUtils.Log("----------------------------------------------", Command);

			return ExitCode;
		}

		public string RetrieveFileSource(string Name, string InType, string InPlatform, string SubType)
		{
			UnrealTargetPlatform? Platform = null;
			FileSource.SourceType? Type = null;

			if (InPlatform != null)
			{
				// let this throw exception on failure, as it's a setup error
				Platform = UnrealTargetPlatform.Parse(InPlatform);
			}

			if (InType != null)
			{
				// let this throw exception on failure, as it's a setup error
				Type = (FileSource.SourceType)Enum.Parse(typeof(FileSource.SourceType), InType);
			}

			List<FileSource> Sources = TurnkeyManifest.FilterDiscoveredFileSources(Platform, Type);

			Sources = Sources.FindAll(x =>
			{
				if (Name.StartsWith("regex:"))
				{
					return TurnkeyUtils.IsValueValid(x.Name, Name, null);
				}
				// this will handle the case of x.Name starting with regex: or just doing a case insensitive string comparison of tag and CustomSdkId
				// range: is not supported, at least yet - we would have to check Tag with range: above, and also support range without a Platform (or pass in a platform somehow?)
				return TurnkeyUtils.IsValueValid(Name, x.Name, null);
			});

			if (Sources.Count == 0)
			{
				return null;
			}

			// @todo turnkey: If > 1 in Sources, warn user

			// execute the first found one
			return CopyProvider.ExecuteCopy(Sources[0].GetCopySourceOperation());
		}

		public string GetVariable(string VariableName)
		{
			return TurnkeyUtils.GetVariableValue(VariableName);
		}

		public void Log(string Message)
		{
			TurnkeyUtils.Log(Message);
		}

		public void ReportError(string Message)
		{
			Log("ERROR: " + Message);
			ErrorMessages.Add(Message);
		}

		public void PauseForUser(string Message)
		{
			TurnkeyUtils.PauseForUser(Message);
		}

		public int ReadInputInt(string Prompt, List<string> Options, bool bIsCancellable, int DefaultValue)
		{
			return TurnkeyUtils.ReadInputInt(Prompt, Options, bIsCancellable, DefaultValue);
		}

	}
}
