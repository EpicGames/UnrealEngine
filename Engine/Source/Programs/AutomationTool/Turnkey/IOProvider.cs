// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Linq;
using System.Text;
using System.Threading.Tasks;

namespace Turnkey
{
	abstract class IOProvider
	{
		public abstract void Log(string Message, bool bAppendNewLine);
		
		public virtual void Report(string Message, bool bAppendNewLine)
		{
			Log(Message, bAppendNewLine);
		}

		public abstract void PauseForUser(string Message, bool bAppendNewLine);

		public abstract string ReadInput(string Prompt, string DefaultValue, bool bAppendNewLine);
		
		public abstract int ReadInputInt(string Prompt, List<string> Options, bool bIsCancellable, int DefaultValue, bool bAppendNewLine);

		public abstract bool GetUserConfirmation(string Message, bool bDefaultValue, bool bAppendNewLine);
	}
}
