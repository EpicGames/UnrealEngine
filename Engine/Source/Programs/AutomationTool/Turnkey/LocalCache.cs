// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.IO;
using System.Text.Json;

namespace Turnkey
{
	[Serializable]
	public class CacheData
	{
		public string LocalPath;
		public string Version;
		public bool bIsFile;
		public Dictionary<string, long> FileToTimeCache = new Dictionary<string, long>();

		public CacheData()
		{

		}

		public CacheData(string LocalPath, string Version)
		{
			this.LocalPath = LocalPath;
			this.Version = Version;

			// check if it's a file, and if it is, add itself to the date cache
			if (File.Exists(LocalPath))
			{
				bIsFile = true;
				FileToTimeCache.Add("", File.GetLastWriteTimeUtc(LocalPath).ToFileTimeUtc());
			}
			else
			{
				bIsFile = false;
				string PathToRemove = LocalPath + Path.DirectorySeparatorChar;
				// gather the files under the localpath and remember dates, for later cleaning
				foreach (string Filename in Directory.EnumerateFiles(LocalPath, "*", SearchOption.AllDirectories))
				{
					FileToTimeCache.Add(Filename.Replace(PathToRemove, "").Replace("\\", "/"), File.GetLastWriteTimeUtc(Filename).ToFileTimeUtc());
				}
			}
		}

	}

	[Serializable]
	public class SavedCache
	{
		public Dictionary<string, CacheData> Cache = new Dictionary<string, CacheData>();
		public SavedCache()
		{

		}
	}



	static class LocalCache
	{
		public static string CreateTempDirectory()
		{
			string TempDir = Path.Combine(TempCacheLocation, Path.GetRandomFileName());
			Directory.CreateDirectory(TempDir);

			// queue this to clean up at the end
			TurnkeyUtils.AddPathToCleanup(TempDir);

			return TempDir;
		}


		public static void CacheLocationByTag(string Tag, string CachePath, string VersionMatch = "")
		{
			// Xml serialization doesn't work well with \ in strings in dictionaries
			Tag = Tag.Replace("\\", "/");
			if (CachePath.StartsWith(TempCacheLocation))
			{
				TagToTempCache.Cache[Tag] = new CacheData(CachePath, VersionMatch);
			}
			else
			{
				TagToDownloadCache.Cache[Tag] = new CacheData(CachePath, VersionMatch);
				SerializeObject(TagToDownloadCache, TagCacheFile);
			}
		}

		public static string GetCachedPathByTag(string Tag, string VersionMatch = "")
		{
			// Xml serialization doesn't work well with \ in strings in dictionaries
			Tag = Tag.Replace("\\", "/");

			// @todo turnkey - this assumes a tag can't be in both

			// look to see if the tag is known
			CacheData Data;
			if (TagToDownloadCache.Cache.TryGetValue(Tag, out Data) || TagToTempCache.Cache.TryGetValue(Tag, out Data))
			{
				// last minute double-check before using it, that it exists
				bool bExists = (Data.bIsFile && File.Exists(Data.LocalPath)) || (!Data.bIsFile && Directory.Exists(Data.LocalPath));
				if (!bExists)
				{
					TagToDownloadCache.Cache.Remove(Tag);
					TagToTempCache.Cache.Remove(Tag);
					return null;
				}

				// if the version doesn't match, return it, but let the called know it's out of date
				if (Data.Version != VersionMatch)
				{
					CleanCacheByTag(Tag);
					return null;
				}

				// if the cersion matches, then we can use it!
				return Data.LocalPath;
			}

			return null;
		}

		public static string CreateDownloadCacheDirectory()
		{
			return Path.Combine(DownloadCacheLocation, Path.GetRandomFileName());
		}

		public static string GetInstallCacheDirectory()
		{
			// this will prompt user if needed
			return TurnkeySettings.GetUserSetting("User_QuickSwitchSdkLocation");
		}





		// these temp files 
		public static string TempCacheLocation = Path.Combine(Path.GetTempPath(), "Turnkey", "TempFiles");

		public static string DownloadCacheLocation = Path.Combine(Path.GetTempPath(), "Turnkey", "DownloadCache");



		static void SerializeObject(SavedCache Object, string Filename)
		{
			string Str = JsonSerializer.Serialize(Object);
			Directory.CreateDirectory(Path.GetDirectoryName(Filename));
			File.WriteAllText(Filename, Str);

// 			using (FileStream Stream = new FileStream(Filename, FileMode.Create))
// 			{
// 				new BinaryFormatter().Serialize(Stream, Object);
// 			}
		}

		static SavedCache DeserializeObject(string Filename)
		{
			if (!File.Exists(Filename))
			{
				return null;
			}

			try
			{
				string Str = File.ReadAllText(Filename);
				if (!string.IsNullOrEmpty(Str))
				{
					return JsonSerializer.Deserialize<SavedCache>(Str);
				}

// 				if (File.Exists(Filename))
// 				{
// 					using (FileStream Stream = new FileStream(Filename, FileMode.Open))
// 					{
// 						return (SavedCache)new BinaryFormatter().Deserialize(Stream);
// 					}
// 				}
			}
			catch(Exception Ex)
			{
				TurnkeyUtils.Log("Exception: {0}", Ex.ToString());
			}

			// for any error, just return null
			return null;
		}

		// @todo turnkey - expire/cache dates/something to be able to check - likely the caller of this would need to do it, like GoogleDrive
		// checking versions of each file, while enumerating, but only download if newer
		static SavedCache TagToDownloadCache;
		static string TagCacheFile = Path.Combine(DownloadCacheLocation, "TagCache.xml");

		// the Temp cache is for during-run temp files, we don't serialize this cache out
		static SavedCache TagToTempCache = new SavedCache();

		static LocalCache()
		{
			TurnkeyUtils.Log("Loading cache bin {0}", TagCacheFile);
			TagToDownloadCache = DeserializeObject(TagCacheFile);
			TurnkeyUtils.Log("Loaded: {0}", TagToDownloadCache);

			if (TagToDownloadCache == null)
			{
				TagToDownloadCache = new SavedCache();
			}
			
			if (TagToDownloadCache.Cache.Count > 0)
			{
				TurnkeyUtils.Log("Cleaning old download cache...");

				List<string> EntriesToDelete = new List<string>();
				foreach (var Pair in TagToDownloadCache.Cache)
				{
					CacheData Data = Pair.Value;
					foreach (var FileWithDate in Data.FileToTimeCache)
					{
						string Filename = Path.Combine(Data.LocalPath, FileWithDate.Key);
						long FileTime = File.GetLastWriteTimeUtc(Filename).ToFileTimeUtc();
						long SavedTime = FileWithDate.Value;
						if (!File.Exists(Filename) || FileTime != SavedTime)
						{
							TurnkeyUtils.Log("Cleaning old download cache...");
							// remove this entry from the cache (delayed after the iterator)
							EntriesToDelete.Add(Pair.Key);

							break;
						}
					}
				}

				// clean any we deleted above (even if delete failed, we remove from the cache so we will download again)
				EntriesToDelete.ForEach(x => CleanCacheByTag(x));
			}

			// @todo turnkey: now delete directories that aren't represented in the cache, or are too old
		}

		public static void CleanCacheByTag(string Tag)
		{
			// Xml serialization doesn't work well with \ in strings in dictionaries
			Tag = Tag.Replace("\\", "/");

			TurnkeyUtils.Log("Cleaning tag {0}...", Tag);

			// if any files were bad, then we just delete the whole pile
			CacheData Data = TagToDownloadCache.Cache[Tag];
			TagToDownloadCache.Cache.Remove(Tag);

			if (Data.bIsFile)
			{
				AutomationTool.InternalUtils.SafeDeleteFile(Data.LocalPath);
			}
			else
			{
				AutomationTool.InternalUtils.SafeDeleteDirectory(Data.LocalPath);
			}
			SerializeObject(TagToDownloadCache, TagCacheFile);
		}
	}
}
