// Copyright Epic Games, Inc. All Rights Reserved.

using System.Reflection;
using System.Runtime.CompilerServices;
using System.Runtime.InteropServices;

// General Information about an assembly is controlled through the following 
// set of attributes. Change these attribute values to modify the information
// associated with an assembly.
[assembly: AssemblyTitle("Turnkey.Automation")]
[assembly: AssemblyDescription("")]
[assembly: AssemblyConfiguration("")]
