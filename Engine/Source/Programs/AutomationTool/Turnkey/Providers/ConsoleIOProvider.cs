// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Text;
using EpicGames.Core;

namespace Turnkey
{
	class ConsoleIOProvider : IOProvider	
	{
		public override void Log(string Message, bool bAppendNewLine)
		{
			if (bAppendNewLine)
			{
				Console.WriteLine(Message);
			}
			else
			{
				Console.Write(Message);
			}
		}

		public override void PauseForUser(string Message, bool bAppendNewLine)
		{
			Log(Message, bAppendNewLine);
			Log("Press Enter to continue", bAppendNewLine);
			Console.ReadLine();
		}

		public override string ReadInput(string Prompt, string Default, bool bAppendNewLine)
		{
			TurnkeyUtils.Log("");

			Log(Prompt, bAppendNewLine);
			if (!string.IsNullOrEmpty(Default))
			{
				Log("[Default: " + Default + "]", true);
			}

			string Response = Console.ReadLine();
			if (string.IsNullOrWhiteSpace(Response))
			{
				Response = Default;
			}
			return Response;
		}

		public override int ReadInputInt(string Prompt, List<string> Options, bool bIsCancellable, int DefaultValue, bool bAppendNewLine)
		{
			TurnkeyUtils.Log("");

			StringBuilder FullPromptBuilder  = new StringBuilder();

			// start with given prompt
			FullPromptBuilder.Append(Prompt);
			if (bAppendNewLine)
			{
				FullPromptBuilder.AppendLine("");
			}

			// now add options
			int Index = 0;
			if (bIsCancellable)
			{
				FullPromptBuilder.AppendLine(" {0}{1}{2} Cancel", DefaultValue == Index ? "[" : " ", Index, DefaultValue == Index++ ? "]" : ")");
			}

			// now add the options given
			foreach (string Option in Options)
			{
				// don't make an index for commented lines
				if (Option.StartsWith(";"))
				{
					FullPromptBuilder.AppendLine(" {0}", Option.Substring(1));
				}
				else
				{
					FullPromptBuilder.AppendLine(" {0}{1}{2} {3}", DefaultValue == Index ? "[" : " ", Index, DefaultValue == Index++ ? "]" : ")", Option);
				}
			}

			string FullPrompt = FullPromptBuilder.ToString();

			// go until good choice
			while (true)
			{
				string ChoiceString = ReadInput(FullPrompt, DefaultValue >= 0 ? DefaultValue.ToString() : null, false);

				int Choice;
				if (Int32.TryParse(ChoiceString, out Choice) == false || Choice < 0 || Choice >= Options.Count + (bIsCancellable ? 1 : 0))
				{
					TurnkeyUtils.Log("Invalid choice");
				}
				else
				{
					return Choice;
				}
			}
		}

		public override bool GetUserConfirmation(string Message, bool bDefaultValue, bool bAppendNewLine)
		{
			string Prompt = $"{Message} [{(bDefaultValue ? 'Y' : 'y')}/{(bDefaultValue ? 'n' : 'N')}]";
			string Result = ReadInput(Prompt, bDefaultValue ? "Y" : "N", bAppendNewLine);
			if (string.IsNullOrEmpty(Result))
			{
				return bDefaultValue;
			}
			else
			{
				return string.Equals( Result.Trim(), "Y", StringComparison.OrdinalIgnoreCase );
			}
		}
	}
}
