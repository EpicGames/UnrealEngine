// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Linq;
using System.Text;
using System.IO;
using DriveHelper;
using System.Text.RegularExpressions;

namespace Turnkey
{
	class GoogleDriveCopyProvider : CopyProvider
	{
		public override string ProviderToken { get { return "googledrive"; } }


		private static DriveServiceHelper ServiceHelper = null;
		private Dictionary<string, Google.Apis.Drive.v3.Data.File> PathToFileCache;// = new Dictionary<string, Google.Apis.Drive.v3.Data.File>();


		public override string Execute(string Operation, CopyExecuteSpecialMode SpecialMode, string SpecialModeHint)
		{
			if (!Prepare())
			{
				return null;
			}

			// operations that end in / must have an * appended, so the wildcard matching works properlty
			if (Operation.EndsWith("/"))
			{
				Operation += "*";
			}

			bool bHasWildcard = Operation.Contains("*");

			Dictionary<Google.Apis.Drive.v3.Data.File, string> IdsToCopyToFilename = new Dictionary<Google.Apis.Drive.v3.Data.File, string>();

			if (bHasWildcard)
			{
				// calculate the part of the input path to remove when making relative directories in the output
				string PathBeforeFirstStar = "";
				int StarLocation = Operation.IndexOf('*');
				int PrevSlash = Operation.LastIndexOf('/', StarLocation);
				PathBeforeFirstStar = Operation.Substring(0, PrevSlash + 1);

				Dictionary<string, Tuple<Google.Apis.Drive.v3.Data.File, List<string>>> Output = new Dictionary<string, Tuple<Google.Apis.Drive.v3.Data.File, List<string>>>();
				ExpandWildcards("", Operation, Output, null);

				foreach (var Pair in Output)
				{
					IdsToCopyToFilename.Add(Pair.Value.Item1, Pair.Key.Replace(PathBeforeFirstStar, ""));
				}
			}
			else
			{
				Google.Apis.Drive.v3.Data.File File = GetFileForPath(Operation);
				if (File != null)
				{
					IdsToCopyToFilename.Add(File, File.Name);
				}
				else
				{
					TurnkeyUtils.Log("ERROR: Unable to find GoogleDrive file {0}", Operation);
				}
			}


			string TagExtras = "";
			// allow the` special mode to modify the tag 
			if (SpecialMode == CopyExecuteSpecialMode.UsePermanentStorage)
			{
				TagExtras = "perm:" + SpecialModeHint;
			}
			string OperationTag = string.Format("googledrive_op:{0}:{1}", TagExtras, Operation);

			string CachedOperationLocation = LocalCache.GetCachedPathByTag(OperationTag);
			string OutputPath = CachedOperationLocation;

			// if there was a cached location, we have to check if all the files we want to download are still good
			if (CachedOperationLocation != null && IdsToCopyToFilename.Count > 0)
			{
				// now make sure each file is up to date, and only download out of date ones
				Dictionary<Google.Apis.Drive.v3.Data.File, string> NewIdsToCopyToFilename = new Dictionary<Google.Apis.Drive.v3.Data.File, string>();
				foreach (var Pair in IdsToCopyToFilename)
				{
					// check the cache status and version
					// @todo turnkey: we don't have per-version files under a directory tag cache entry, so we tag each file, although it will be duplicated with the op cache tag above
					string ExistingPath = LocalCache.GetCachedPathByTag(Pair.Key.Id, Pair.Key.Version.ToString());

					if (ExistingPath == null)
					{
						// still need to download this one
						NewIdsToCopyToFilename.Add(Pair.Key, Pair.Value);
					}
					// if we are removing our one and only file from the list to be deleted, we need to change our OutputPath to be that single file
					// since we won't be going through the lower loop
					else if (!bHasWildcard)
					{
						OutputPath = ExistingPath;
					}
				}
				IdsToCopyToFilename = NewIdsToCopyToFilename;
			}

			// download 1 or more files (try to use the cached operation location if we had one)
			string DownloadDirectory = CachedOperationLocation;

			if (IdsToCopyToFilename.Count > 0)
			{
				// if we are just downloading a quick switch SDK, or similar, then we are given a Hint for a location to download to
				if (SpecialMode == CopyExecuteSpecialMode.UsePermanentStorage && CachedOperationLocation == null)
				{
					DownloadDirectory = SpecialModeHint;// Path.Combine(LocalCache.GetInstallCacheDirectory(), SpecialModeHint);
					AutomationTool.InternalUtils.SafeDeleteDirectory(DownloadDirectory);
				}

				// if we didn't have a cached directory at all, then make a new one
				if (DownloadDirectory == null)
				{
					DownloadDirectory = LocalCache.CreateDownloadCacheDirectory();
				}

				OutputPath = DownloadDirectory;

				// we know we need to download all of these files, no need for more cache checking
				foreach (var Pair in IdsToCopyToFilename)
				{
					// finally, download it!
					string FinalPathname = Path.Combine(OutputPath, Pair.Value);
					DownloadFile(Pair.Key, FinalPathname);

					// and now, store off the cache for use later
					LocalCache.CacheLocationByTag(Pair.Key.Id, FinalPathname, Pair.Key.Version.ToString());

					// if we only downloaded a single file/folder, we want to use the file as the output
					if (!bHasWildcard)
					{
						OutputPath = FinalPathname;
					}
				}

				// to complete, remember the download directory as the cache location for this operation, if it wasn't already there
				if (CachedOperationLocation == null)
				{
					LocalCache.CacheLocationByTag(OperationTag, DownloadDirectory);
				}
			}

			return OutputPath;
		}


		public override string[] Enumerate(string Operation, List<List<string>> Expansions)
		{
			// if we have no wildcards, there's no need to waste time touching google drive, just return the spec
			if (!Operation.Contains("*"))
			{
				return new string[] { ProviderToken + ":" + Operation };
			}

			if (!Prepare())
			{
				return null;
			}

			TurnkeyUtils.Log("Enumerating GoogleDrive spec: {0}", Operation);

			Dictionary<string, Tuple<Google.Apis.Drive.v3.Data.File, List<string>>> Output = new Dictionary<string, Tuple<Google.Apis.Drive.v3.Data.File, List<string>>>();

			List<string> ExpansionSet = new List<string>();
			ExpandWildcards("", Operation, Output, ExpansionSet);

			// convert them into a nicer-for-Turnkey "'ParentId'/Filename" format
			List<string> FixedPaths = new List<string>();
			foreach (var Pair in Output)
			{
				if (Pair.Value.Item1.Parents.Count != 1)
				{
					TurnkeyUtils.Log("File {0} did not have 1 parent as expected. GoogleDriveCopyProvider.ExpandWildcards will probably have to change to remember the parent for each object as it goes down");
					continue;
				}

				string FixedPath = string.Format("'{0}'/{1}", Pair.Value.Item1.Parents[0], Pair.Value.Item1.Name);

				// cache this new path for faster Execute, etc
				if (!PathToFileCache.ContainsKey(FixedPath))
				{
					PathToFileCache.Add(FixedPath, Pair.Value.Item1);
				}

				// if it's a directory, then append a slash
				if (Pair.Value.Item1.MimeType == MIMETypes.GoogleDriveFolder.ToMimeString())
				{
					FixedPath += "/";
				}

				FixedPaths.Add(ProviderToken + ":" + FixedPath);
				if (Expansions != null)
				{
					Expansions.Add(Pair.Value.Item2);
				}
			}

			return FixedPaths.ToArray();
		}


		private Google.Apis.Drive.v3.Data.File GetFileForId(string Id)
		{
			// reverse dictionary lookup to see if we have cached this id
			var FoundPair = PathToFileCache.FirstOrDefault(x => x.Value.Id == Id);
			if (FoundPair.Key != null)
			{
				return FoundPair.Value;
			}

			// ask GoogleDrive for the file object
			Google.Apis.Drive.v3.Data.File Result = ServiceHelper.GetFile(Id, new List<string> { "id", "name", "parents", "mimeType", "version" });
			return Result;
		}

		private bool Prepare()
		{
			if (ServiceHelper == null)
			{
				string GoogleDriveCredentials = TurnkeyUtils.GetVariableValue("Studio_GoogleDriveCredentials");
				string GoogleDriveAppName = TurnkeyUtils.GetVariableValue("Studio_GoogleDriveAppName");

				if (string.IsNullOrEmpty(GoogleDriveCredentials) || string.IsNullOrEmpty(GoogleDriveAppName))
				{
					TurnkeyUtils.Log("ERROR: Unable to use GoogleDrive without 'Studio_GoogleDriveCredentials' and 'Studio_GoogleDriveAppName' being set first!");
					return false;
				}

				string SecretsFile = CopyProvider.ExecuteCopy(GoogleDriveCredentials);
				if (SecretsFile == null)
				{
					TurnkeyUtils.Log("ERROR: Unable to get GoogleDrive secrets/credentials file from {0}", GoogleDriveCredentials);
					return false;
				}

				TurnkeyUtils.Log("Connecting to GoogleDrive app '{0}'. If this pauses here, check your web browser for an authentication tab. This is required to be able to connect to Google Drive", GoogleDriveAppName);
				try
				{
					ServiceHelper = new DriveServiceHelper(GoogleDriveAppName, SecretsFile, Path.GetDirectoryName(SecretsFile));
					PathToFileCache = new Dictionary<string, Google.Apis.Drive.v3.Data.File>();
				}
				catch(Exception ex)
				{
					TurnkeyUtils.Log("ERROR: Unable to access GoogleDrive: {0}", ex.Message);
					return false;
				}
			}

			return true;
		}		
		
// 		private string GetHighestCachedDirectoryIdForPath(string DrivePath, out string RemainingPathAfterId)
// 		{
// 			string PathToCheck = DrivePath.TrimEnd("/".ToCharArray());
// 
// 			while (true)
// 			{
// 				// do we have the current path already cached to an id?
// 				Google.Apis.Drive.v3.Data.File FoundFile;
// 				if (PathToFileCache.TryGetValue(PathToCheck, out FoundFile))
// 				{
// 					RemainingPathAfterId = DrivePath.Replace(PathToCheck, "");
// 					return FoundFile.Id;
// 				}
// 
// 				// if not, then go back a component
// 				int SlashLocation = PathToCheck.LastIndexOf('/');
// 				if (SlashLocation < 0)
// 				{
// 					break;
// 				}
// 				
// 				PathToCheck = PathToCheck.Substring(0, SlashLocation);
// 			}
// 
// 			// at this point, nothing was found, so return no Id, and set remaining to the input
// 			RemainingPathAfterId = DrivePath;
// 			return null;
// 		}

		private Google.Apis.Drive.v3.Data.File GetFileForPath(string DrivePath)
		{
			if (DrivePath.Contains("*"))
			{
				throw new AutomationTool.AutomationException("GetFileForPath cannot be used with wildcards");
			}

			// if the file is already cached, return it
			if (PathToFileCache.ContainsKey(DrivePath))
			{
				return PathToFileCache[DrivePath];
			}

			// split up the remaining path into components
			string[] Components = DrivePath.Split("/".ToCharArray(), StringSplitOptions.RemoveEmptyEntries);
			int CurrentComponentIndex = 0;

			string PathBuildup = "";

			Google.Apis.Drive.v3.Data.File ParentFile = null;
			while (CurrentComponentIndex < Components.Length)
			{
				string CurrentComponent = Components[CurrentComponentIndex];

				// build up the path (adding a slash if we have a path already, or we don't but the Drive path did start with one)
				if (PathBuildup != "" || DrivePath.StartsWith("/"))
				{
					PathBuildup += "/";
				}
				PathBuildup += CurrentComponent;

				// do we already ahve this thing cached
				Google.Apis.Drive.v3.Data.File ComponentFile;
				if (PathToFileCache.TryGetValue(PathBuildup, out ComponentFile))
				{
					// if so, just remember it and move on
					ParentFile = ComponentFile;
				}
				else
				{
					bool bProcessComponent = true;
					// none are found yet, so find the root folder, or MyDrive if there's isn't one
					if (ParentFile == null)
					{
						if (DrivePath.StartsWith("/"))
						{
							Google.Apis.Drive.v3.Data.TeamDrive Drive = ServiceHelper.SearchForDriveByName(CurrentComponent);

							if (Drive == null)
							{
								TurnkeyUtils.Log("Unable to find Drive named {0}", CurrentComponent);
								return null;
							}
							
							// the drive id is also a folder id
							ParentFile = GetFileForId(Drive.Id);

							// move on to the next component since this one is found
							bProcessComponent = false;
						}
						else if (CurrentComponent.StartsWith("'"))
						{
							if (!CurrentComponent.EndsWith("'"))
							{
								TurnkeyUtils.Log("Expected matching ' symbols in {0}", DrivePath);
								return null;
							}

							// use the id between the ticks
							ParentFile = GetFileForId(CurrentComponent.Substring(1, CurrentComponent.Length - 2));

							// move on to the next component since this one is found
							bProcessComponent = false;
						}
						else 
						{
							// get the folder for my-drive!!
						}

						if (ParentFile == null)
						{
							TurnkeyUtils.Log("Unable to figure out root directory for {0}", DrivePath);
							return null;
						}
					}

					// otherwise, this is a normal component
					if (bProcessComponent)
					{
						List<Google.Apis.Drive.v3.Data.File> Files = ServiceHelper.SearchFiles(string.Format("name = '{0}' and '{1}' in parents and trashed = false", CurrentComponent, ParentFile.Id));
						if (Files.Count > 1)
						{
							TurnkeyUtils.Log("Found more than one item with the name {0}", CurrentComponent);
							return null;
						}
						if (Files.Count == 0)
						{
							// if there were no files, then this path doesn't exist, so just silently end
							return null;
						}

						// get the file with matching name
						ParentFile = Files[0];
					}

					// cache the path
					PathToFileCache.Add(PathBuildup, ParentFile);
				}

				CurrentComponentIndex++;
			}

			// at this point the ParentId is the id we cared about (file or folder)
			return ParentFile;
		}



		private void DownloadFile(Google.Apis.Drive.v3.Data.File File, string LocalPath)
		{
			if (File.MimeType != MIMETypes.GoogleDriveFolder.ToMimeString())
			{
				Directory.CreateDirectory(Path.GetDirectoryName(LocalPath));
				ServiceHelper.DownloadFile(File.Id, LocalPath);
			}
			else
			{
				List<Google.Apis.Drive.v3.Data.File> FilesInDirectory = ServiceHelper.SearchFiles(string.Format("'{0}' in parents and trashed = false", File.Id));
				foreach (Google.Apis.Drive.v3.Data.File FileInDir in FilesInDirectory)
				{
					DownloadFile(FileInDir, Path.Combine(LocalPath, FileInDir.Name));
				}
			}
		}

		private void ExpandWildcards(string Prefix, string PathString, Dictionary<string, Tuple<Google.Apis.Drive.v3.Data.File, List<string>>> Output, List<string> ExpansionSet)
		{
			char Slash = '/';

			// look through for *'s
			int StarLocation = PathString.IndexOf('*');

			// if this has no wildcard, it's a set file, just use it directly
			if (StarLocation == -1)
			{
				// see if the file exists
				Google.Apis.Drive.v3.Data.File FinalFile = GetFileForPath(Prefix + PathString);
				if (FinalFile != null)
				{
					Output.Add(Prefix + PathString, new Tuple<Google.Apis.Drive.v3.Data.File, List<string>>(FinalFile, ExpansionSet));
				}
				return;
			}

			// now go backwards looking for a Slash
			int PrevSlash = PathString.LastIndexOf(Slash, StarLocation);

			// current wildcard is the path segment up to next slash or the end
			int NextSlash = PathString.IndexOf(Slash, StarLocation);

			// if this are no more slashes, then this is the final expansion, and we can add to the result, and look for files
			bool bIsLastComponent = NextSlash == -1 || NextSlash == PathString.Length - 1;

			// get the wildcard path component
			string FullPathBeforeWildcard = Prefix + (PrevSlash >= 0 ? PathString.Substring(0, PrevSlash) : "");

			Google.Apis.Drive.v3.Data.File FolderBeforeWildcard = GetFileForPath(FullPathBeforeWildcard);
			if (FolderBeforeWildcard != null)
			{
				string Wildcard = (NextSlash == -1) ? PathString.Substring(PrevSlash + 1) : PathString.Substring(PrevSlash + 1, (NextSlash - PrevSlash) - 1);

				// convert to a wildcard with capturing to get what the matches are
				Regex Regex = new Regex(string.Format("^{0}$", Wildcard.Replace("*", "(.+?)")), RegexOptions.IgnoreCase);

				// get files that could match the wildcard and are not trashed
				string Query = string.Format("'{0}' in parents and trashed = false", FolderBeforeWildcard.Id);

				List<Google.Apis.Drive.v3.Data.File> Results = ServiceHelper.SearchFiles(Query);

				foreach (Google.Apis.Drive.v3.Data.File Result in Results)
				{
					// make sure we fit the filter
					Match Match = Regex.Match(Result.Name);
					if (!Match.Success)
					{
						continue;
					}

					List<string> NewExpansionSet = null;
					if (ExpansionSet != null)
					{
						NewExpansionSet = new List<string>(ExpansionSet);
						for (int GroupIndex = 1; GroupIndex < Match.Groups.Count; GroupIndex++)
						{
							NewExpansionSet.Add(Match.Groups[GroupIndex].Value);
						}
					}

					// cache the file (it may have already been cached tho from a previous test)
					string PathToFile = FullPathBeforeWildcard + "/" + Result.Name;
					if (!PathToFileCache.ContainsKey(PathToFile))
					{
						PathToFileCache.Add(PathToFile, Result);
					}

					if (bIsLastComponent)
					{
						// always add directories, but only add files if the path didn't end in / (/Sdks/*/ would only want to return directories)
						if (Result.MimeType == MIMETypes.GoogleDriveFolder.ToMimeString() || NextSlash == -1)
						{
							Output.Add(PathToFile, new Tuple<Google.Apis.Drive.v3.Data.File, List<string>>(Result, NewExpansionSet));
						}
					}
					else
					{
						// recurse with parts after the wildcard component
						// @todo turnkey
						string NewPrefix = (FullPathBeforeWildcard != "" ? (FullPathBeforeWildcard + Slash) : "") + Result.Name + Slash;
						ExpandWildcards(NewPrefix, PathString.Substring(NextSlash + 1), Output, NewExpansionSet);
					}
				}
			}
		}
	}
}

