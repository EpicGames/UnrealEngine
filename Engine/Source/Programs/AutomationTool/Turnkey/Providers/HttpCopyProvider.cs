// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.IO;
using System.Collections.Generic;
using System.Linq;
using System.Text.RegularExpressions;
using EpicGames.Core;
using System.Net;

#pragma warning disable SYSLIB0014

namespace Turnkey
{
	// there is an Https subclass below - so the operation will supply http: or https: and this class will handle both, by using the ProviderToken for Uris
	class HttpCopyProvider : CopyProvider
	{
		public override string ProviderToken { get { return "http"; } }

		public override string Execute(string Operation, CopyExecuteSpecialMode SpecialMode, string SpecialModeHint)
		{
			// because of how http requests can request a path on the client side, but return an actual file, we first make the request,
			// then deal with potential caching once we have a final filename

			// we have pulled http: from the specified operation (http://foo.com/bar), so prepend with http (or https - the ProviderToken) to
			// make the full URL
			Uri OpUri = new Uri($"{ProviderToken}:{Operation}");
			HttpWebResponse Response;
			try
			{
				// create the request and get an initial response (without downloading the target)
				HttpWebRequest Request = (HttpWebRequest)WebRequest.Create(OpUri);
				Response = (HttpWebResponse)Request.GetResponse();
			}
			catch (Exception Ex)
			{
				TurnkeyUtils.Log($"Http request {OpUri} failed to get a response: {Ex.Message}");
				return null;
			}

			// return on fail
			if (Response.StatusCode != HttpStatusCode.OK)
			{
				TurnkeyUtils.Log($"Received a failed response for request {OpUri}, code = {Response.StatusCode}");
				return null;
			}

			string TagExtras = "";
			// allow the special mode to modify the tag 
			if (SpecialMode == CopyExecuteSpecialMode.UsePermanentStorage)
			{
				TagExtras = "perm:" + SpecialModeHint;
			}

			// use the response to create a cache tag, including the LastModified time so updated files will be redownloaded
			string OperationTag = $"{ProviderToken}_op:{TagExtras}:{Response.ResponseUri.LocalPath}:{Response.LastModified}";

			string CachedOperationLocation = LocalCache.GetCachedPathByTag(OperationTag);
			string OutputPath = CachedOperationLocation;

			if (OutputPath == null)
			{
				string DownloadDirectory;
				// if we are just downloading a quick switch SDK, or similar, then we are given a Hint for a location to download to
				if (SpecialMode == CopyExecuteSpecialMode.UsePermanentStorage && CachedOperationLocation == null)
				{
					DownloadDirectory = SpecialModeHint;// Path.Combine(LocalCache.GetInstallCacheDirectory(), SpecialModeHint);
					AutomationTool.InternalUtils.SafeDeleteDirectory(DownloadDirectory);
				}
				else
				{
					DownloadDirectory = LocalCache.CreateDownloadCacheDirectory();
				}

				string Filename = Path.GetFileName(Response.ResponseUri.LocalPath);
				// if the response didn't get a filename with an extension, log a message that it may not be what is expected
				if (string.IsNullOrEmpty(Path.GetExtension(Filename)))
				{
					Filename = "http_" + Path.GetRandomFileName();
					TurnkeyUtils.Log($"The response ({Response.ResponseUri.LocalPath}) for the Http request ({OpUri}) didn't contain a filename. Saving to a temp filename ({DownloadDirectory}/{Filename}), but it may not work as expected...");
				}

				// put it together
				OutputPath = Path.Combine(DownloadDirectory, Filename);

				TurnkeyUtils.Log($"Downloading from {OpUri} to {OutputPath}...");
				Directory.CreateDirectory(DownloadDirectory);

				using (Stream ReadStream = Response.GetResponseStream())
				using (FileStream WriteStream = new FileStream(OutputPath, FileMode.Create))
				{
					byte[] buffer = new byte[4096];
					int bytesRead;
					int Total = 0;
					DateTime Start = DateTime.UtcNow;
					UInt64 LastTimeTier = 0;
					while ((bytesRead = ReadStream.Read(buffer, 0, buffer.Length)) > 0)
					{
						WriteStream.Write(buffer, 0, bytesRead);
						Total += bytesRead;

						UInt64 CurrentTier = (UInt64)((System.DateTime.UtcNow - Start).TotalSeconds) / 1;
						if (LastTimeTier != CurrentTier)
						{
							LastTimeTier = CurrentTier;
							TurnkeyUtils.Log($"   ... {Total} / {Response.ContentLength}");
						}
					}
				}


				// update the cache
				LocalCache.CacheLocationByTag(OperationTag, OutputPath);
			}

			return OutputPath;
		}


		public override string[] Enumerate(string Operation, List<List<string>> Expansions)
		{
			if (Operation.Contains("*"))
			{
				TurnkeyUtils.Log("http: operations don't allow for expansion '{0}'", Operation);
				return null;
			}

			// we don't allow any expansion with Http, so just return the input
			return new string[] { Operation };
		}
	}

	class HttpsCopyProvider : HttpCopyProvider
	{
		public override string ProviderToken { get { return "https"; } }

	}
}