// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Text;
using System.Text.RegularExpressions;
using EpicGames.Core;
using System.Diagnostics;

#pragma warning disable SYSLIB0014

namespace Turnkey
{
	class HybridIOProvider : ConsoleIOProvider
	{
		static HybridIOProvider()
		{
		}

		private string ShowMacDialog(string Prompt, string Default)
		{
			string Params = string.Format("-e 'display dialog \"{0}\" with title \"Turnkey Input\" default answer \"{1}\"'", Prompt.Replace("\n", "\\n").Replace("\t", "\\t"), Default);
			string OSAOutput = UnrealBuildTool.Utils.RunLocalProcessAndReturnStdOut("osascript", Params);

			// blank string means user canceled, which goes to stderr
			if (OSAOutput == "")
			{
				return null;
			}

			// regex the result
			Match Match = Regex.Match(OSAOutput, "text returned:(.*)$");
			if (!Match.Success)
			{
				return null;
			}

			// return the text in the dialog box
			return Match.Groups[1].Value;
		}

		private string ShowDialog(string Prompt, string Default, bool bIsList)
		{
			string Result = "";

			if (OperatingSystem.IsWindows())
			{
				System.Threading.Thread t = new System.Threading.Thread(x =>
				{
					Debug.Assert(OperatingSystem.IsWindowsVersionAtLeast(7));
					Result = UnrealWindowsForms.TurnkeyDialog.ShowDialogAndReturnResult(Prompt, Default);
				});

				t.SetApartmentState(System.Threading.ApartmentState.STA);
				t.Start();
				t.Join();
			}
			else if (OperatingSystem.IsMacOS())
			{
				Result = ShowMacDialog(Prompt, Default);
			}
			else
			{
				throw new NotImplementedException("Linux dialog not implemented");
			}

			if (string.IsNullOrEmpty(Result) && bIsList)
			{
				return "0";
			}

			return Result;
		}

		public override void PauseForUser(string Message, bool bAppendNewLine)
		{
			ShowDialog(Message, "", false);
		}

		public override string ReadInput(string Prompt, string Default, bool bAppendNewLine)
		{
			return ShowDialog(Prompt, Default, false);
		}

		public override int ReadInputInt(string Prompt, List<string> Options, bool bIsCancellable, int DefaultValue, bool bAppendNewLine)
		{
			StringBuilder FullPromptBuilder = new StringBuilder();

			// start with given prompt
			FullPromptBuilder.Append(Prompt);
			if (bAppendNewLine)
			{
				FullPromptBuilder.AppendLine("");
			}

			// now add the options given
			int Index = 1;
			foreach (string Option in Options)
			{
				if (Option.StartsWith(";"))
				{
					FullPromptBuilder.AppendLine(" {0}", Option.Substring(1));
				}
				else
				{
					FullPromptBuilder.AppendLine(" ({0}) {1}", Index++, Option);
				}
			}

			string FullPrompt = FullPromptBuilder.ToString();

			// go until good choice
			while (true)
			{
				string ChoiceString = ShowDialog(FullPrompt, DefaultValue >= 0 ? DefaultValue.ToString() : null, true);

				if (ChoiceString == null)
				{
					return bIsCancellable ? 0 : -1;
				}

				int Choice;
				if (Int32.TryParse(ChoiceString, out Choice) == false || Choice < 0 || Choice >= Options.Count + (bIsCancellable ? 1 : 0))
				{
					if (Choice < 0 && bIsCancellable)
					{
						return 0;
					}
					TurnkeyUtils.Log("Invalid choice");
				}
				else
				{
					return Choice;
				}
			}
		}
	}
}
