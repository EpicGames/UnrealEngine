// Copyright Epic Games, Inc. All Rights Reserved.

using System.IO;
using System.Collections.Generic;
using System.Linq;
using System.Text.RegularExpressions;
using EpicGames.Core;
using System.Security.AccessControl;
using System.Security.Principal;
using System;

using static AutomationTool.CommandUtils;
using Microsoft.Extensions.Logging;

#pragma warning disable SYSLIB0014

namespace Turnkey
{
	class NullCopyProvider : CopyProvider
	{
		public override string ProviderToken { get { return "file"; } }

		private void FixupOperation(ref string Operation)
		{
			Operation = Operation.Replace(Path.AltDirectorySeparatorChar, Path.DirectorySeparatorChar);

			if (UnrealBuildTool.BuildHostPlatform.Current.Platform == UnrealBuildTool.UnrealTargetPlatform.Mac && Operation.StartsWith("smb://"))
			{
				// match the form smb://server.net/Foo/Bar/Baz and retrieve the Foo, which will be the name in /Volumes/Foo
				Match Match = Regex.Match(Operation, @"^(smb:\/\/.+?)\/(.+?)\/(.*)$");

				// make sure regex matched
				if (!Match.Success)
				{
					return;
				}

				// mount the drive
				UnrealBuildTool.Utils.RunLocalProcessAndReturnStdOut("osascript", string.Format("-e 'mount volume \"{0}/{1}\"'", Match.Groups[1].Value, Match.Groups[2].Value));

				// convert smb://server.net/Foo/Bar/Baz to /Volumes/Foo/Bar/Baz
				TurnkeyUtils.Log("SMB Before: {0}", Operation);
				Operation = string.Format("/Volumes/{0}/{1}", Match.Groups[2].Value, Match.Groups[3].Value);
				TurnkeyUtils.Log("SMB After: {0}", Operation);
			}
		}

		public override string Execute(string Operation, CopyExecuteSpecialMode SpecialMode, string SpecialModeHint)
		{
			FixupOperation(ref Operation);

			// this provider can use the file directly, so just return the input after expanding variables, if it exists
			string OutputPath = TurnkeyUtils.ExpandVariables(Operation);

			int WildcardLocation = OutputPath.IndexOf('*');
			if (WildcardLocation >= 0)
			{
				// chop down to the last / before the wildcard
				int LastSlashLocation = OutputPath.Substring(0, WildcardLocation).LastIndexOf(Path.DirectorySeparatorChar);
				OutputPath = OutputPath.Substring(0, LastSlashLocation);
			}

			if (File.Exists(OutputPath) == false && Directory.Exists(OutputPath) == false)
			{
				TurnkeyUtils.Log("Reqeusted local path {0} does not exist!", OutputPath);
				return null;
			}

			// accessing large remote files direcly can be crazy slow, so we cache large-ish files locally before using
			bool bIsRemotePath = false;

			if (UnrealBuildTool.BuildHostPlatform.Current.Platform == UnrealBuildTool.UnrealTargetPlatform.Mac ||
				UnrealBuildTool.BuildHostPlatform.Current.Platform == UnrealBuildTool.UnrealTargetPlatform.Linux)
			{
				// https://github.com/mono/mono/issues/12157
				// resolved in newer versions of mono but the current bundled version is crashing when trying to create a new DriveInfo
				bIsRemotePath = false;
			}
			else
			{
				bIsRemotePath = (new System.Uri(OutputPath).IsUnc) || (new DriveInfo(OutputPath).DriveType == DriveType.Network);
			}

			bool bIsLargeFile = File.Exists(OutputPath) && new FileInfo(OutputPath).Length > 5 * 1024 * 1024;
			bool bIsDirectory = Directory.Exists(OutputPath);

			if ((bIsRemotePath && (bIsLargeFile || bIsDirectory)) || (SpecialMode == CopyExecuteSpecialMode.UsePermanentStorage))
			{
				// look in out local cache
				string OperationTag = string.Format("file_op:{0}", Operation);
				string CachedLocation = LocalCache.GetCachedPathByTag(OperationTag);

				if (CachedLocation != null)
				{
					return CachedLocation;
				}

				string CopyLocation = (SpecialMode == CopyExecuteSpecialMode.UsePermanentStorage) ? SpecialModeHint : Path.Combine(LocalCache.CreateTempDirectory(), Path.GetFileName(OutputPath));

				if (bIsDirectory)
				{
					TurnkeyUtils.Log("Copying remote directory structure to local temp location {0}...", CopyLocation);
					if (AutomationTool.CommandUtils.CopyDirectory_NoExceptions(OutputPath, CopyLocation) == false)
					{
						TurnkeyUtils.Log("Copy failed, unable to continue...");
						return null;
					}

				}
				else
				{
					TurnkeyUtils.Log("Copying large remote file to local temp location {0}...", CopyLocation);
					if (AutomationTool.CommandUtils.CopyFile_NoExceptions(OutputPath, CopyLocation) == false)
					{
						TurnkeyUtils.Log("Copy failed, unable to continue...");
						return null;
					}
				}

				LocalCache.CacheLocationByTag(OperationTag, CopyLocation);
				TurnkeyUtils.Log("Done!");

				return CopyLocation;
			}

			return OutputPath;
		}

		
		private bool VerifyPathAccess( string PathString )
		{
			if (!OperatingSystem.IsWindows())
			{
				return true;
			}

			bool bResult = false;

			DirectoryInfo DirInfo = new DirectoryInfo(PathString);
			if (!DirInfo.Exists)
			{
				return true;
			}

			while (DirInfo != null)
			{
				try
				{
					WindowsIdentity Identity = WindowsIdentity.GetCurrent();
					DirectorySecurity AccessControl = DirInfo.GetAccessControl(AccessControlSections.Access);
					foreach (FileSystemAccessRule AccessRule in AccessControl.GetAccessRules(true, true, typeof(SecurityIdentifier)))
					{
						if (Identity.Owner == AccessRule.IdentityReference || Identity.Groups.Contains(AccessRule.IdentityReference))
						{
							bResult |= AccessRule.FileSystemRights.HasFlag(FileSystemRights.ReadData|FileSystemRights.ListDirectory);
						}
					}

					break;
				}
				catch(System.UnauthorizedAccessException)
				{
					bResult = false;
					break;
				}
				catch(System.Exception)
				{
					// something went wrong - look further up the path to check permissions on the parent folder
					DirInfo = DirInfo.Parent;
				}
			}

			if (!bResult)
			{
				Logger.LogWarning("You do not have permission to access {PathString}", PathString);
			}
			return bResult;
		}

		private void ExpandWildcards(string Prefix, string PathString, Dictionary<string, List<string>> Output, List<string> ExpansionSet)
		{
			char Slash = Path.DirectorySeparatorChar;

			// look through for *'s
			int StarLocation = PathString.IndexOf('*');

			// if this has no wildcard, it's a set file, just use it directly
			if (StarLocation == -1)
			{
				PathString = Prefix + PathString;
				if (!VerifyPathAccess(PathString))
				{
					return;
				}
				if (Directory.Exists(PathString))
				{
					// make sure we end with a single slash
					Output.Add(ProviderToken + ":" + PathString.TrimEnd("/\\".ToCharArray()) + Slash, ExpansionSet);
				}
				else if (File.Exists(PathString))
				{
					Output.Add(ProviderToken + ":" + PathString, ExpansionSet);
				}
				return;
			}

			// now go backwards looking for a Slash
			int PrevSlash = PathString.LastIndexOf(Slash, StarLocation);

			// current wildcard is the path segment up to next slash or the end
			int NextSlash = PathString.IndexOf(Slash, StarLocation);

			// if this are no more slashes, then this is the final expansion, and we can add to the result, and look for files
			bool bIsLastComponent = NextSlash == -1 || NextSlash == PathString.Length - 1;

			// get the wildcard path component
			string FullPathBeforeWildcard = Prefix + (PrevSlash >= 0 ? PathString.Substring(0, PrevSlash) : "");

			if (!VerifyPathAccess(FullPathBeforeWildcard))
			{
				return;
			}
			if (Directory.Exists(FullPathBeforeWildcard))
			{
				// get the path component that has a wildcard
				string Wildcard = (NextSlash == -1) ? PathString.Substring(PrevSlash + 1) : PathString.Substring(PrevSlash + 1, (NextSlash - PrevSlash) - 1);

				// replace * with a non-greedy capture so *-foo-*.zip can get the best values for the *'s
				Regex Regex = new Regex(string.Format("^{0}$", Wildcard.Replace("+", "\\+").Replace(".", "\\.").Replace("*", "(.+?)")), RegexOptions.IgnoreCase);

				// track what's before and after the * to return what it expanded to
				int StarLoc = Wildcard.IndexOf('*');
				int PrefixLen = StarLoc;
				int SuffixLen = Wildcard.Length - (StarLoc + 1);
				foreach (string Dirname in Directory.EnumerateDirectories(FullPathBeforeWildcard, Wildcard))
				{
					List<string> NewExpansionSet = null;
					if (ExpansionSet != null)
					{
						NewExpansionSet = new List<string>(ExpansionSet);
						string PathComponent = Path.GetFileName(Dirname);

						Match Match = Regex.Match(PathComponent);
						if (!Match.Success)
						{
							continue;
						}

						for (int GroupIndex = 1; GroupIndex < Match.Groups.Count; GroupIndex++)
						{
							NewExpansionSet.Add(Match.Groups[GroupIndex].Value);
						}

//						// the match is the part of the filename that was not the *, so removing that will give us what we wanted to match
//						NewExpansionSet.Add(PathComponent.Substring(PrefixLen, PathComponent.Length - (PrefixLen + SuffixLen)));
					}

					if (bIsLastComponent)
					{
						// indicate directories with a slash at the end
						Output.Add(ProviderToken + ":" + Dirname + Slash, NewExpansionSet);
					}
					// recurse
					else
					{
						ExpandWildcards(Dirname + Slash, PathString.Substring(NextSlash + 1), Output, NewExpansionSet);
					}
				}

				// if the path ends with a slash, then we are only looking for directories (D:\\Sdks\*\ would only want to return directories)
				if (bIsLastComponent && NextSlash == -1)
				{
					foreach (string Filename in Directory.EnumerateFiles(FullPathBeforeWildcard))//, Wildcard))
					{
						string PathComponent = Path.GetFileName(Filename);
						Match Match = Regex.Match(PathComponent);
						if (!Match.Success)
						{
							continue;
						}

						List<string> NewExpansionSet = null;
						if (ExpansionSet != null)
						{
							NewExpansionSet = new List<string>(ExpansionSet);

							for (int GroupIndex = 1; GroupIndex < Match.Groups.Count; GroupIndex++)
							{
								NewExpansionSet.Add(Match.Groups[GroupIndex].Value);
							}

							// the match is the part of the filename that was not the *, so removing that will give us what we wanted to match
//							NewExpansionSet.Add(PathComponent.Substring(PrefixLen, PathComponent.Length - (PrefixLen + SuffixLen)));
						}

						Output.Add(ProviderToken + ":" + Filename, NewExpansionSet);
					}
				}
			}
		}

		public override string[] Enumerate(string Operation, List<List<string>> Expansions)
		{
			Dictionary<string, List<string>> Output = new Dictionary<string, List<string>>();

			FixupOperation(ref Operation);

			ExpandWildcards("", Operation, Output, Expansions == null ? null : new List<string>());

			if (Expansions != null)
			{
				Expansions.AddRange(Output.Values);
			}
			return Output.Keys.ToArray();
		}
	}
}
