// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Linq;
using EpicGames.Core;
using AutomationTool;
using System.Text.RegularExpressions;
using System.IO;
using UnrealBuildBase;

namespace Turnkey
{
	class PerforceCopyProvider : CopyProvider
	{
		public override string ProviderToken { get { return "perforce"; } }


		public override string Execute(string Operation, CopyExecuteSpecialMode SpecialMode, string SpecialModeHint)
		{
			if (!PrepareForOperation(Operation))
			{
				return null;
			}

			// find the output path using p4 where because a depot mapping could be strange
			string OutputPath = null;
			IProcessResult P4Result = PerforceConnection.P4(string.Format("where {0}", Operation), AllowSpew:false);
			if (P4Result.ExitCode == 0)
			{
				// sadly, this doesn't set an ExitCode, so we have to look at output
				if (!P4Result.Output.Trim().EndsWith("not in client view."))
				{
					// p4 where returns a trio of Depot, Client and Local paths, separated by spaces and potentially quoted.
					// more complicated mappings may have multiple trios so walk them to find one that matches
					string[] Items = SharedUtils.ParseCommandLine(P4Result.Output);
					if ((Items.Length % 3) == 0)
					{
						for (int Item = 0; Item < Items.Length; Item += 3)
						{
							// find a matching Depot path
							if (Items[Item].Equals(Operation, StringComparison.OrdinalIgnoreCase))
							{
								// use the Local path and stop searching
								OutputPath = Items[Item + 2];
								break;
							}
						}
					}
				}
			}

			// no luck with p4 where
			if (OutputPath == null)
			{
				OutputPath = Operation.Replace(ConnectedRoot, PerforceClient.RootPath);
				TurnkeyUtils.Log("Unable to discover local path using p4 where {0}. Falling back to best guess {1}", Operation, OutputPath);
			}

			// turn //a/b/c/foo*/... to //a/b/c
			OutputPath = GetDirectoryBeforeWildcard(OutputPath);

			OutputPath = OutputPath.Replace(Path.AltDirectorySeparatorChar, Path.DirectorySeparatorChar);

			TurnkeyUtils.Log("Syncing '{0}' to '{1}'", Operation, OutputPath);

			// now do the actual sync
			PerforceConnection.Sync(Operation, AllowSpew: false);

			return OutputPath;
		}

		public override string[] Enumerate(string Operation, List<List<string>> Expansions)
		{
			// if we have no wildcards, there's no need to waste time touching p4, just return the spec
			if (!Operation.Contains("*") && !Operation.Contains("..."))
			{
				return new string[] { ProviderToken + ":" + Operation };
			}

			// connect to the stream
			if (!PrepareForOperation(Operation))
			{
				return null;
			}

			// now get the file list that matches
			List<string> Results = PerforceConnection.Files(Operation);

			if (Expansions != null)
			{
				// figure out what the *'s turned into (note we use case-insensitive because p4 has an annoying habit of being case-preserving of directories
				// on a per-file basis (//depot/UE4/Foo/* might return //depot/UE4/Foo/A and //Depot/ue4/FOO/B)
				Regex Expression = new Regex(Operation.Replace("/", "\\/").Replace("*", "([^\\/]*)").Replace("...", ".*"), RegexOptions.IgnoreCase);

				// run the regex on each result
				foreach (string Result in Results)
				{
					List<string> ResultExpansions = new List<string>();
					Expansions.Add(ResultExpansions);

					Match ResultMatch = Expression.Match(Result);
					for (int Index = 1; Index < ResultMatch.Groups.Count; Index++)
					{
						ResultExpansions.Add(ResultMatch.Groups[Index].Value);
					}
				}
			}

			return Results.Select(x => ProviderToken + ":" + x).ToArray();
		}




		private string GetDirectoryBeforeWildcard(string Input)
		{
			// find the first wildcard and that's the lowest down outputpath we can use
			int DotsLocation = Input.IndexOf("...");
			int StarLocation = Input.IndexOf("*");
			// hjandle various combos of -1 and non -1
			int WildcardLocation = (DotsLocation >= 0 && StarLocation >= 0) ? Math.Min(DotsLocation, StarLocation) : Math.Max(DotsLocation, StarLocation);
			if (WildcardLocation != -1)
			{
				// chop down to the last / before the wildcard
				int LastSlashLocation = Input.Substring(0, WildcardLocation).Replace("\\", "/").LastIndexOf("/");
				Input = Input.Substring(0, LastSlashLocation);
			}

			return Input;
		}


		static private P4Connection PerforceConnection = null;
		static private P4ClientInfo PerforceClient = null;
		// only non-null if we are connected
		static private string ConnectedRoot = null;

		private P4ClientInfo DetectClientForStream(string Stream, string Username, string Hostname)
		{

			// find clients for this user in the given stream
			P4ClientInfo[] StreamClients = PerforceConnection.GetClientsForUser(Username, null, Stream);

			if (TurnkeySettings.HasSetUserSetting("User_LastPerforceClient"))
			{
				string ClientName = TurnkeySettings.GetUserSetting("User_LastPerforceClient");
				P4ClientInfo Client = StreamClients.FirstOrDefault(Client => string.Compare(Client.Name, ClientName, StringComparison.InvariantCultureIgnoreCase) == 0);
				if (Client != null && (string.IsNullOrEmpty(Client.Host) || string.Compare(Client.Host, Hostname) == 0))
				{
					return Client;
				}
			}

			// find the first one usable on this host
			foreach (P4ClientInfo Client in StreamClients)
			{
				if (string.IsNullOrEmpty(Client.Host) || string.Compare(Client.Host, Hostname) == 0)
				{
					TurnkeySettings.SetUserSetting("User_LastPerforceClient", Client.Name);
					return Client;
				}
			}

			// if no clients at all, return null
			return null;
		}

		private P4ClientInfo CanClientHandleOperation(string ClientName, string Operation, string Hostname)
		{
			// make sure the Operation can be supported by the client (Tokens[1] is the clientspec name)
			IProcessResult P4Result = PerforceConnection.P4(string.Format("-c {0}", ClientName), string.Format("where {0}", Operation), Input: null, AllowSpew: false, WithClient: false);

			if (P4Result.ExitCode == 0)
			{
				// sadly, this doesn't set an ExitCode, so we have to look at output
				if (!P4Result.Output.Trim().EndsWith("not in client view."))
				{
					P4ClientInfo ClientInfo = PerforceConnection.GetClientInfo(ClientName, true);
					// make sure it's usable on this computer
					if (string.IsNullOrEmpty(ClientInfo.Host) || string.Compare(ClientInfo.Host, Hostname) == 0)
					{
						return ClientInfo;
					}
				}
			}

			return null;
		}

		private P4ClientInfo DetectClientForDepot(string Operation, string Username, string Hostname)
		{
			// use the bits up to a wildcard
			Operation = GetDirectoryBeforeWildcard(Operation);

			P4ClientInfo Client;
			if (TurnkeySettings.HasSetUserSetting("User_LastPerforceClient"))
			{
				Client = CanClientHandleOperation(TurnkeySettings.GetUserSetting("User_LastPerforceClient"), Operation, Hostname);
				if (Client != null)
				{
					return Client;
				}
			}

			// Get all clients for this user
			string P4Command = String.Format("clients -u {0}", Username);

			// @todo turnkey: sort the results on third token (date)
			var P4Result = PerforceConnection.P4(string.Format("clients -u {0}", Username), AllowSpew: false, WithClient: false);
			if (P4Result.ExitCode != 0)
			{
				return null;
			}

			// Parse output.
			var Lines = P4Result.Output.Split(new string[] { Environment.NewLine }, StringSplitOptions.RemoveEmptyEntries);
			foreach (string Line in Lines)
			{
				var Tokens = Line.Split(new char[] { ' ' }, StringSplitOptions.RemoveEmptyEntries);
				if (Tokens[0] == "Client")
				{
					Client = CanClientHandleOperation(Tokens[1], Operation, Hostname);
					if (Client != null)
					{
						// remember this for next time
						TurnkeySettings.SetUserSetting("User_LastPerforceClient", Tokens[1]);

						return Client;
					}
				}
			}

			// if no clients at all, return null
			return null;
		}

		private bool PrepareForOperation(string Operation)
		{
			Match StreamMatch = new Regex(@"(\/\/\w*\/\w*)\/.*").Match(Operation);
			if (!StreamMatch.Success)
			{
				throw new AutomationException("Unable to find stream spec in perforce operation {0}", Operation);
			}

			string SpecRoot = StreamMatch.Groups[1].ToString();
			string DepotName = SpecRoot.Substring(0, SpecRoot.LastIndexOf('/'));
			string Hostname = Unreal.MachineName;

			// @todo turnkey - for depot types, we can't totally safely check DepotName, we need to make sure the recent Client can handle the Operation - which we could with tricky View parsing
			if (ConnectedRoot != SpecRoot || PerforceConnection == null)
			{
				TurnkeyUtils.Log("Finding clientspec usable with {0}...", Operation);

				PerforceConnection = new P4Connection(null, null);
				string Username = P4Environment.DetectUserName(PerforceConnection);

				// make sure it's a stream
				var P4Result = PerforceConnection.P4("stream -o " + SpecRoot, AllowSpew: false);
				bool bIsStream = P4Result.ExitCode == 0;

				// hunt down a client that can be used
				PerforceClient = bIsStream ? DetectClientForStream(SpecRoot, Username, Hostname) : DetectClientForDepot(Operation, Username, Hostname);

				if (PerforceClient != null)
				{
					TurnkeyUtils.Log("Using client {0}", PerforceClient.Name);
				}
				else
				{
					TurnkeyUtils.Log("Unable to find a clientspec for the perforce operation {0}, looking for a depot client", Operation);

					bool bResponse = TurnkeyUtils.GetUserConfirmation("Unable to find an existing Perforce Workspace for the Turnkey AutoSDK.\nWould you like to create one?", false);
					if (bResponse == false)
					{
						// make sure to try again next time
						PerforceConnection = null;
						TurnkeyUtils.Log("Skipping operation");
						return false;
					}

					string ClientName = "";
					int Index = 0;
					string BaseClientName = $"{Username}_{Hostname}_sdks";
					while (ClientName == "")
					{
						// get clientspec name
						string DefaultName = BaseClientName + (Index == 0 ? "" : $"_{Index}");
						string TestClientName = TurnkeyUtils.ReadInput("Enter clientspec name:", DefaultName);

						// make sure the clientname doesn't already exist (unlikely with hostname in it, but just to be sure)
						if (PerforceConnection.GetClientInfo(TestClientName, true) != null)
						{
							TurnkeyUtils.Log("Client {0} is already in use, please choose another name", TestClientName);
							// if the user picked a name, use that as the new base when testing, and for appending index to
							if (TestClientName != DefaultName)
							{
								BaseClientName = TestClientName;
							}
							Index++;
						}
						else
						{
							ClientName = TestClientName;
						}
					}

					// get local pathname
					// if we had UE_SDKS_ROOT already set, we use it as a default
					string AutoSdksDir = Environment.GetEnvironmentVariable("UE_SDKS_ROOT");
					string DefaultDir = Path.Combine(Environment.GetFolderPath(Environment.SpecialFolder.Personal), "AutoSdks");
					if (!string.IsNullOrEmpty(AutoSdksDir))
					{
						// @todo turnkey - for a deeper directory structure, we should match directory names to the perforce Operation, to figure it out properly
						DefaultDir = Path.GetDirectoryName(AutoSdksDir);
					}
					string LocalPath = TurnkeyUtils.ReadInput(string.Format("Enter where to map {0} on your computer:", bIsStream ? SpecRoot : DepotName), DefaultDir);

					// create a client from the input settings
					P4ClientInfo NewClient = new P4ClientInfo();
					NewClient.Name = ClientName;
					NewClient.Owner = Username;
					NewClient.Host = Hostname;
					NewClient.RootPath = LocalPath;
					NewClient.Options |= P4ClientOption.RmDir;
					if (bIsStream)
					{
						NewClient.Stream = SpecRoot;
					}
					else
					{
						// set up mapping (CreateClient prepends the Key with //<clientname>)
						NewClient.View.Add(new KeyValuePair<string, string>(DepotName + "/...", "/..."));
					}

					PerforceClient = PerforceConnection.CreateClient(NewClient);
				}

				if (PerforceClient == null)
				{
					TurnkeyUtils.Log("Unable to find or create a client, will not perform perforce copy operation {0}", Operation);
					return false;
				}

				// now connect to that 
				PerforceConnection = new P4Connection(Username, PerforceClient.Name);

				// @todo turnkey: how to check that for errors?

				ConnectedRoot = bIsStream ? SpecRoot : DepotName;
			}

			return true;
		}


	}
}
