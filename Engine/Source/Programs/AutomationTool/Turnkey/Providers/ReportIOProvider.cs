// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Linq;
using System.Text;
using System.Threading.Tasks;

namespace Turnkey
{
	class ReportIOProvider : ConsoleIOProvider
	{
		private System.IO.StreamWriter Out;

		public ReportIOProvider(string ReportFilename)
		{
			Out = new System.IO.StreamWriter(ReportFilename);
		}

		public override void Report(string Message, bool bAppendNewLine)
		{
			if (bAppendNewLine)
			{
				Out.WriteLine(Message);
			}
			else
			{
				Out.Write(Message);
			}
			Out.Flush();
		}

		public override string ReadInput(string Prompt, string Default, bool bAppendNewLine)
		{
			throw new AutomationTool.AutomationException("ReportIOProvider is unable to ask for input. The Turnkey commandline likely didn't specify enough information.");
		}

		public override int ReadInputInt(string Prompt, List<string> Options, bool bIsCancellable, int DefaultValue, bool bAppendNewLine)
		{
			throw new AutomationTool.AutomationException("ReportIOProvider is unable to ask for input. The Turnkey commandline likely didn't specify enough information.");
		}
	}
}
