// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Net.Sockets;
using System.Net;
using System.Text;
using EpicGames.Core;
using System.Linq;
using System.IO;

namespace Turnkey
{
	class EditorIOClient : IOProvider	
	{
		readonly int Version = 1; //keep in sync with .cpp

		int Port;
		Socket IOSocket;
		Lazy<IOProvider> FallbackIOProvider;
		bool bVerbose = false;

		public EditorIOClient( int InPort )
		{
			Port = InPort;
			FallbackIOProvider = new Lazy<IOProvider>(() => { return new HybridIOProvider(); } );
		}

		~EditorIOClient()
		{
			Disconnect();
		}

		private void VerboseMessage( string Message )
		{
			if(bVerbose)
			{
				Console.WriteLine(Message);
			}
		}

		public override void Log(string Message, bool bAppendNewLine)
		{
			// no socket needed for this - editor will be logging stdout anyway
			if (bAppendNewLine)
			{
				Console.WriteLine(Message);
			}
			else
			{
				Console.Write(Message);
			}
		}

		public override void PauseForUser(string Message, bool bAppendNewLine)
		{
			if (Connect())
			{
				// send the action definition
				if (SendAction( "PauseForUser", (Json) =>
				{
					Json.WriteValue("Message", Message);
				}))
				{					
					// wait for any response
					JsonObject ReceivedMessage = ReceiveMessage();
					Disconnect();

					// make sure there was a response, otherwise use the fallback
					if (ReceivedMessage != null)
					{
						return;
					}
				}
			}

			// failed to connect - use the fallback instead
			FallbackIOProvider.Value.PauseForUser(Message, bAppendNewLine);
		}

		public override string ReadInput(string Prompt, string Default, bool bAppendNewLine)
		{
			if (Connect())
			{
				// send the action message
				if (SendAction( "ReadInput", (Json) =>
				{
					Json.WriteValue("Prompt",  Prompt);
					Json.WriteValue("Default", Default);
				}))
				{
					// wait for the response
					JsonObject ReceivedMessage = ReceiveMessage();
					Disconnect();

					// parse the result
					if (ReceivedMessage != null)
					{
						string Result;
						if (ReceivedMessage.TryGetStringField("Result", out Result))
						{
							return Result;
						}
						return Default;
					}
				}
			}

			// failed to connect - use the fallback instead
			return FallbackIOProvider.Value.ReadInput(Prompt, Default, bAppendNewLine);
		}

		public override int ReadInputInt(string Prompt, List<string> Options, bool bIsCancellable, int DefaultValue, bool bAppendNewLine)
		{
			if (Connect())
			{
				// send the action message
				if (SendAction( "ReadInputInt", (Json) =>
				{
					Json.WriteValue("Prompt",             Prompt);
					Json.WriteValue("IsCancellable",      bIsCancellable);
					Json.WriteValue("DefaultValue",       DefaultValue);
					Json.WriteStringArrayField("Options", Options);
				}))
				{
					// wait for response
					JsonObject ReceivedMessage = ReceiveMessage();
					Disconnect();

					// parse the response
					if (ReceivedMessage != null)
					{
						int Choice;
						if (ReceivedMessage.TryGetIntegerField("Result", out Choice) == false || Choice < 0 || Choice >= Options.Count + (bIsCancellable ? 1 : 0))
						{
							TurnkeyUtils.Log("Invalid choice");
							return DefaultValue;
						}
						else
						{
							return Choice;
						}
					}
				}
			}

			// failed to connect - use the fallback instead
			return FallbackIOProvider.Value.ReadInputInt(Prompt, Options, bIsCancellable, DefaultValue, bAppendNewLine);
		}

		public override bool GetUserConfirmation(string Message, bool bDefaultValue, bool bAppendNewLine)
		{
			if (Connect())
			{
				// send the action message
				if (SendAction( "GetConfirmation", (Json) =>
				{
					Json.WriteValue("Message",      Message);
					Json.WriteValue("DefaultValue", bDefaultValue);
				}))
				{
					// wait for the response
					JsonObject ReceivedMessage = ReceiveMessage();
					Disconnect();

					// parse the result
					if (ReceivedMessage != null)
					{
						bool bResult;
						if (ReceivedMessage.TryGetBoolField("Result", out bResult))
						{
							return bResult;
						}
						return bDefaultValue;
					}
				}
			}

			// failed to connect - use the fallback instead
			return FallbackIOProvider.Value.GetUserConfirmation(Message, bDefaultValue, bAppendNewLine);

		}


		#region socket functions

		bool Connect()
		{
			try
			{
				// create a IPv4 blocking TCP socket & connect to the given port on localhost
				IOSocket = new Socket(AddressFamily.InterNetwork, SocketType.Stream, ProtocolType.Tcp );
				IOSocket.Blocking = true;

				IPEndPoint EndPoint = new IPEndPoint(IPAddress.Loopback, Port );
				IOSocket.Connect(EndPoint);
				return true;
			}
			catch( Exception e )
			{
				Console.WriteLine($"Error: Failed to connect to Turnkey IO server on port {Port}. {e}");
			}
			return false;
		}

		void Disconnect()
		{
			if (IOSocket != null)
			{
				IOSocket.Disconnect(false);
				IOSocket.Shutdown(SocketShutdown.Both);
				IOSocket.Close();
			}
		}

		protected JsonObject ReceiveMessage()
		{
			VerboseMessage($"TurnkeySocketIO waiting for server response");

			JsonObject Result = null;

			// read the message
			List<byte> Msg = new List<byte>();
			try
			{
				byte[] ReceiveBuffer = new byte[10240];
				int MessageTerminator = -1;
				do
				{
					int BytesReceived = IOSocket.Receive(ReceiveBuffer);
					if (BytesReceived > 0)
					{
						Msg.AddRange(ReceiveBuffer.Take(BytesReceived));
						MessageTerminator = Msg.IndexOf(0);
					}
				} while(MessageTerminator < 0);

				//remove the terminator. not expecting more than one message so this isn't handled
				Msg.RemoveRange( MessageTerminator, Msg.Count - MessageTerminator );
			}
			catch( Exception e )
			{
				Console.WriteLine($"Error: TurnkeySocketIO read failed {e}");
				return null;
			}
			SendAck();

			// parse the json
			try
			{
				string JsonString = Encoding.ASCII.GetString(Msg.ToArray());
				Result = JsonObject.Parse(JsonString);
			}
			catch( Exception e )
			{
				Console.WriteLine($"Error: TurnkeySocketIO parse failed {e}");
				return null;
			}

			// check for error message
			string ErrorMessage;
			if (Result.TryGetStringField("Error", out ErrorMessage) )
			{
				Console.WriteLine($"Error: TurnkeySocketIO server returned {ErrorMessage}");
				return null;
			}

			// all done
			return Result;
		}



		protected bool SendMessage( Action<JsonWriter> MessageFunc )
		{
			// build message
			StringWriter JsonMessage = new StringWriter();
			JsonWriter Json = new JsonWriter(JsonMessage, style:JsonWriterStyle.Readable );
			Json.WriteObjectStart();
				Json.WriteValue("version", Version );
				MessageFunc(Json);
			Json.WriteObjectEnd();

			// send the message with a null terminator
			string Message = JsonMessage.ToString() + '\0';
			byte[] MessageBytes = Encoding.ASCII.GetBytes(Message);
			try
			{
				int BytesSent = IOSocket.Send(MessageBytes, SocketFlags.None);
			}
			catch( Exception e )
			{
				Console.WriteLine($"Error: TurnkeySocketIO write failed {e}");
				return false;
			}

			return true;
		}

		protected bool SendAck()
		{
			return SendMessage( (Json) =>
			{
				Json.WriteValue("Reply", "done");
			});
		}

		protected bool SendAction( string Type, Action<JsonWriter> ActionFunc )
		{
			VerboseMessage($"TurnkeySocketIO sending action {Type}");

			return SendMessage( (Json) =>
			{
				Json.WriteObjectStart("Action");
					Json.WriteValue("Type", Type);
					ActionFunc(Json);
				Json.WriteObjectEnd();
			});
		}

		#endregion
	}
}
