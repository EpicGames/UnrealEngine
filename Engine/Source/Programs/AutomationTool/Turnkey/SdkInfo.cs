// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Text;
using System.Xml.Serialization;
using System.IO;
using EpicGames.Core;
using UnrealBuildTool;
using AutomationTool;
using System.Linq;

namespace Turnkey
{
	static class SdkUtils
	{
		[Flags]
		public enum LocalAvailability
		{
			None = 0,
			AutoSdk_VariableExists					= (1 << 0),
			AutoSdk_ValidVersionExists				= (1 << 1),
			AutoSdk_InvalidVersionExists			= (1 << 2),
			
			InstalledSdk_ValidInactiveVersionExists	= (1 << 3),
			InstalledSdk_ValidVersionExists			= (1 << 4),
			InstalledSdk_InvalidVersionExists		= (1 << 5),

			Platform_ValidHostPrerequisites			= (1 << 6),
			Platform_InvalidHostPrerequisites		= (1 << 7),

			Device_InvalidPrerequisites				= (1 << 8),
			Device_InstallSoftwareValid				= (1 << 9),
			Device_InstallSoftwareInvalid			= (1 << 10),
			Device_CannotConnect					= (1 << 11),

			Support_FullSdk							= (1 << 12),
			Support_AutoSdk							= (1 << 13),

			Sdk_HasBestVersion						= (1 << 14),

			Device_AutoSoftwareUpdates_Disabled		= (1 << 15),
			Device_AutoSoftwareUpdates_Enabled		= (1 << 16),

			Host_Unsupported						= (1 << 17),
		}

		static public LocalAvailability GetLocalAvailability(AutomationTool.Platform AutomationPlatform, bool bAllowUpdatingPrerequisites, TurnkeyContextImpl TurnkeyContext)
		{
			LocalAvailability Result = LocalAvailability.None;

			// for some legacy NDA platforms, we could have an UnrealTargetPlatform but no registered SDK
			UEBuildPlatformSDK SDK = UEBuildPlatformSDK.GetSDKForPlatform(AutomationPlatform.PlatformType.ToString());
			if (SDK == null)
			{
				return Result;
			}

			if (!SDK.bIsSdkAllowedOnHost)
			{
				return LocalAvailability.Host_Unsupported;
			}

			if (AutomationPlatform.UpdateHostPrerequisites(TurnkeyUtils.CommandUtilHelper, TurnkeyContext, !bAllowUpdatingPrerequisites))
			{
				Result |= LocalAvailability.Platform_ValidHostPrerequisites;
			}
			else
			{
				Result |= LocalAvailability.Platform_InvalidHostPrerequisites;
			}

			// get all the SDK info we can
			SDKCollection SDKInfo = SDK.GetAllSDKInfo();

			// if we don't have an AutoSDK in good shape, look for others
			if (SDKInfo.AutoSdk?.Current == null)
			{
				// look to see if other versions are around
				string AutoSdkVar = Environment.GetEnvironmentVariable("UE_SDKS_ROOT");
				if (AutoSdkVar != null)
				{
					// no matter what, remember we have the variable set
					Result |= LocalAvailability.AutoSdk_VariableExists;

					// get platform subdirectory
					string AutoSubdir = string.Format("Host{0}/{1}", HostPlatform.Current.HostEditorPlatform.ToString(), SDK.GetAutoSDKPlatformName());
					DirectoryInfo PlatformDir = new DirectoryInfo(Path.Combine(AutoSdkVar, AutoSubdir));
					if (PlatformDir.Exists)
					{
						foreach (DirectoryInfo Dir in PlatformDir.EnumerateDirectories())
						{
							// look to see if other versions have been synced, but are bad version (otherwise, we would have had AutoSDKVersion above!)
							if (File.Exists(Path.Combine(Dir.FullName, "setup.bat")) || File.Exists(Path.Combine(Dir.FullName, "setup.sh")))
							{
								Result |= LocalAvailability.AutoSdk_InvalidVersionExists;
								break;
							}
						}
					}
				}
			}
			else
			{
				Result |= LocalAvailability.AutoSdk_ValidVersionExists | LocalAvailability.AutoSdk_VariableExists;
			}

			// if all manual SDKs are valid, then we are good
			if (SDKInfo.AreAllManualSDKsValid())
			{
				Result |= LocalAvailability.InstalledSdk_ValidVersionExists;
			}
			// but if we have any manual SDKs that have a Current version, then we have invalid version installed
			else if (SDKInfo.Sdks.Where(x => string.Compare(x.Name, "AutoSDK", true) != 0 && x.Current != null).Count() > 0)
			{ 
				Result |= LocalAvailability.InstalledSdk_InvalidVersionExists;
			}

			// look for other, inactive, versions
			foreach (string AlternateVersion in SDK.GetAllInstalledSDKVersions())
			{
				// @todo turnkey: do we want to deal with multiple installed sdk for platforms with multiple SDK types???
				if (SDK.IsVersionValid(AlternateVersion, "Sdk"))
				{
					Result |= LocalAvailability.InstalledSdk_ValidInactiveVersionExists;
				}
			}

			// see if we have the best version available (note: this is an approximation of FileSource.ChooseBest() without hitting the file sources)
			if ((Result & (LocalAvailability.InstalledSdk_ValidVersionExists | LocalAvailability.AutoSdk_ValidVersionExists)) != 0)
			{
				string MainVersion = SDK.GetMainVersion();
				if (SDKInfo.Sdks.All(x => (x.Current == null) || (x.Current == MainVersion) || (x.Current == x.Max)))
				{
					// only set this when all valid Current sdks are either MainVersion or MaxVersion
					Result |= LocalAvailability.Sdk_HasBestVersion;
				}
			}


			string FullSupportedPlatforms = TurnkeyUtils.GetVariableValue("Studio_FullInstallPlatforms");
			string AutoSdkSupportedPlatforms = TurnkeyUtils.GetVariableValue("Studio_AutoSdkPlatforms");
			if (!string.IsNullOrEmpty(FullSupportedPlatforms))
			{
				if (FullSupportedPlatforms.ToLower() == "all" || FullSupportedPlatforms.Split(",".ToCharArray()).Contains(AutomationPlatform.PlatformType.ToString(), StringComparer.InvariantCultureIgnoreCase))
				{
					Result |= LocalAvailability.Support_FullSdk;
				}
			}
			if (!string.IsNullOrEmpty(AutoSdkSupportedPlatforms))
			{
				if (AutoSdkSupportedPlatforms.ToLower() == "all" || AutoSdkSupportedPlatforms.Split(",".ToCharArray()).Contains(AutomationPlatform.PlatformType.ToString(), StringComparer.InvariantCultureIgnoreCase))
				{
					Result |= LocalAvailability.Support_AutoSdk;
				}
			}

			return Result;
		}


		public static bool SetupAutoSdk(FileSource Source, ITurnkeyContext TurnkeyContext, UnrealTargetPlatform Platform, bool bUnattended)
		{
			AutomationTool.Platform AutomationPlatform = AutomationTool.Platform.GetPlatform(Platform);

			bool bSetupEnvVarAfterInstall = false;
			if (Environment.GetEnvironmentVariable("UE_SDKS_ROOT") == null)
			{
				if (bUnattended)
				{
					TurnkeyContext.ReportError($"Unable to install an AutoSDK without UE_SDKS_ROOT setup (can use Turnkey interactively to set it up)");
					return false;
				}

				bool bResponse = TurnkeyUtils.GetUserConfirmation("The AutoSdk system is not setup on this machine. Would you like to set it up now?", true);
				if (bResponse)
				{
					bSetupEnvVarAfterInstall = true;
				}
			}
			else
			{
				TurnkeyUtils.Log("{0}: AutoSdk is setup on this computer, will look for available AutoSdk to download", Platform);
			}


			// make sure this is unset so that we can know if it worked or not after install
			TurnkeyUtils.ClearVariable("CopyOutputPath");

			// now download it (AutoSdks don't "install") on download
			// @todo turnkey: handle errors, handle p4 going to wrong location, handle one Sdk for multiple platforms
			string CopyOperation = Source.GetCopySourceOperation();
			if (CopyOperation == null)
			{
				TurnkeyContext.ReportError($"Unable to find AutoSDK FileSource fopr {Platform}. Your Studio's TurnkeyManifest.xml file(s) may need to be fixed.");
				return false;
			}

			string DownloadedRoot;

			// perforce is special case in that it will setup UE_SDKS_ROOT after we download, because p4 already has a mapping to a certain location
			// and we don't need to ask user (if perforce needs to ask, it will)
			if (!CopyOperation.StartsWith(new PerforceCopyProvider().ProviderToken))
			{
				if (bSetupEnvVarAfterInstall)
				{
					string Response = TurnkeyUtils.ReadInput("Enter directory to use for root of AutoSDKs", Path.Combine(System.Environment.GetFolderPath(Environment.SpecialFolder.MyDocuments), "AutoSDK"));
					if (string.IsNullOrEmpty(Response))
					{
						return false;
					}

					// set the env var, globally
					TurnkeyUtils.StartTrackingExternalEnvVarChanges();
					Environment.SetEnvironmentVariable("UE_SDKS_ROOT", Response);
					Environment.SetEnvironmentVariable("UE_SDKS_ROOT", Response, EnvironmentVariableTarget.User);
					TurnkeyUtils.EndTrackingExternalEnvVarChanges();
					AutomationPlatform.PersistSdkRootVar();

					bSetupEnvVarAfterInstall = false;
				}

				// download the AutoSDK, to the UE_SDKS_ROOT dir
				string AutoSdkPlatformName = Platform.ToString();
				UEBuildPlatformSDK PlatformSDK = UEBuildPlatformSDK.GetSDKForPlatform(Platform.ToString());
				if (PlatformSDK != null)
				{
					AutoSdkPlatformName = PlatformSDK.GetAutoSDKPlatformName();
				}
				string AutoSDKDownloadedRoot = Path.Combine(Environment.GetEnvironmentVariable("UE_SDKS_ROOT"), $"Host{HostPlatform.Platform}", AutoSdkPlatformName, Source.Version);
				// it should return what we pass into ig
				DownloadedRoot = CopyProvider.ExecuteCopy(CopyOperation, CopyExecuteSpecialMode.UsePermanentStorage, AutoSDKDownloadedRoot);
				if (DownloadedRoot != AutoSDKDownloadedRoot)
				{
					throw new BuildException($"Turnkey did not download to the expected location, the copy operation failed: {CopyOperation} did not respect CopyExecuteSpecialMode.UsePermanentStorage with {AutoSDKDownloadedRoot}, it copied to {DownloadedRoot} instead");
				}
			}
			else
			{
				// download the AutoSDK using perforce, where it doesn't need a Hint passed to it
				DownloadedRoot = CopyProvider.ExecuteCopy(CopyOperation);
			}

			if (string.IsNullOrEmpty(DownloadedRoot))
			{
				TurnkeyContext.ReportError($"Unable to download the AutoSDK for {Platform}. Your Studio's TurnkeyManifest.xml file(s) may need to be fixed.");
				return false;
			}

			if (bSetupEnvVarAfterInstall)
			{
				// walk up to one above Host* directory
				DirectoryInfo AutoSdkSearch;
				if (Directory.Exists(DownloadedRoot))
				{
					AutoSdkSearch = new DirectoryInfo(DownloadedRoot);
				}
				else
				{
					AutoSdkSearch = new FileInfo(DownloadedRoot).Directory;
				}
				while (AutoSdkSearch.Name != "Host" + HostPlatform.Current.HostEditorPlatform.ToString())
				{
					AutoSdkSearch = AutoSdkSearch.Parent;
				}

				// now go one up to the parent of Host
				AutoSdkSearch = AutoSdkSearch.Parent;

				// we don't even ask the user in the p4 case because it doesn't make sense to use anything but where we downloaded to
				string AutoSdkDir = AutoSdkSearch.FullName;

				// set the env var, globally
				TurnkeyUtils.StartTrackingExternalEnvVarChanges();
				Environment.SetEnvironmentVariable("UE_SDKS_ROOT", AutoSdkDir);
				Environment.SetEnvironmentVariable("UE_SDKS_ROOT", AutoSdkDir, EnvironmentVariableTarget.User);
				TurnkeyUtils.EndTrackingExternalEnvVarChanges();
				AutomationPlatform.PersistSdkRootVar();

			}

			// and now activate it in case we need it this run
			TurnkeyUtils.Log("Re-activating AutoSDK '{0}'...", Source.Name);

			UEBuildPlatformSDK.GetSDKForPlatform(Platform.ToString()).ReactivateAutoSDK();

			return AutomationPlatform.PostSDKSetup(TurnkeyContext, bUnattended);
		}
	}
}
