// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using EpicGames.Core;
using AutomationTool;
using System.Threading;
using UnrealBuildBase;

namespace Turnkey
{
	class Turnkey : BuildCommand
	{
		public override ExitCode Execute()
		{
			IOProvider IOProvider;
			if (ParseParam("EditorIOPort"))
			{
				int Port = ParseParamInt("EditorIOPort");
				IOProvider = new EditorIOClient(Port);
			}
			else if (ParseParam("EditorIO"))
			{
				IOProvider = new HybridIOProvider();
			}
			else
			{
				string ReportFilename = ParseParamValue("ReportFilename");
				if (!string.IsNullOrEmpty(ReportFilename))
				{
					IOProvider = new ReportIOProvider(ReportFilename);
				}
				else
				{
					IOProvider = new ConsoleIOProvider();
				}
			}

			return Turnkey.Execute(IOProvider, this);
		}

		static bool bHasBeenInitialized = false;
		public static ExitCode Execute(IOProvider IOProvider, BuildCommand CommandUtilHelper)
		{
			if (!bHasBeenInitialized)
			{
				SetupVisuals();

				// cache some settings for other classes
				TurnkeyUtils.SetVariable("EngineDir", Unreal.EngineDirectory.FullName);
				TurnkeyUtils.SetVariable("Project", CommandUtilHelper.ParseParamValue("Project="));

				TurnkeySettings.Initialize();

				//			Microsoft.Win32.SystemEvents.UserPreferenceChanged += new Microsoft.Win32.UserPreferenceChangedEventHandler((sender, args) => TurnkeyUtils.Log("Got a change! {0}", args.Category));

				Console.CancelKeyPress += delegate
				{
					TurnkeyUtils.CleanupPaths();

					TurnkeyUtils.Log("");
					TurnkeyUtils.Log("If you installed an SDK, you should NOT \"Terminate batch job\"!");
					TurnkeyUtils.Log("");
					TurnkeyUtils.ExitCode = ExitCode.Success;
				};

				bHasBeenInitialized = true;
			}

			// IOProvider could change between multiple executions
			TurnkeyUtils.Initialize(IOProvider, CommandUtilHelper);


			try
			{
				// supplied command
				string SuppliedCommand = TurnkeyUtils.CommandUtilHelper.ParseParamValue("Command", null);
				if (SuppliedCommand != null)
				{
					TurnkeyCommand.ExecuteCommand(SuppliedCommand);
				}
				else
				{
					// no command will prompt
					while (TurnkeyCommand.ExecuteCommand())
					{
						TurnkeyUtils.Log("");
					}
				}
			}
			catch (Exception Ex)
			{
				TurnkeyUtils.Log("Turnkey exception: {0}", Ex.ToString());
			}
			finally
			{
				TurnkeyUtils.CleanupPaths();
			}

			return TurnkeyUtils.ExitCode;
		}

		#region Visuals

		private static void SetupVisuals()
		{
			if (OperatingSystem.IsWindowsVersionAtLeast(7))
			{
				UnrealWindowsForms.Utils.SetupVisuals();
			}
		}

		#endregion

	}
}
