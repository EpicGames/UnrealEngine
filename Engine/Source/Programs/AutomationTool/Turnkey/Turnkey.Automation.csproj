<Project Sdk="Microsoft.NET.Sdk">
  <Import Project="..\..\Shared\UnrealEngine.csproj.props" />
  
  <PropertyGroup>
    <OutputType>Library</OutputType>
    <GenerateAssemblyInfo>false</GenerateAssemblyInfo>
    <GenerateTargetFrameworkAttribute>false</GenerateTargetFrameworkAttribute>
    <RootNamespace>Turnkey.Automation</RootNamespace>
    <AssemblyName>Turnkey.Automation</AssemblyName>
    <WarningsNotAsErrors>612,618</WarningsNotAsErrors>
    <OutputPath Condition="'$(ArtifactsPath)' == ''">..\..\..\..\Binaries\DotNET\AutomationTool\AutomationScripts\Turnkey\</OutputPath>
    <TreatWarningsAsErrors>true</TreatWarningsAsErrors>
    <CopyLocalLockFileAssemblies>true</CopyLocalLockFileAssemblies>
    <SatelliteResourceLanguages>en</SatelliteResourceLanguages>    <!-- remove non english resource languages -->
</PropertyGroup>

  <ItemGroup>
    <Reference Include="UnrealWindowsForms">
      <HintPath>..\..\..\..\Binaries\DotNET\UnrealWindowsForms.dll</HintPath>
    </Reference>
  </ItemGroup>

  <ItemGroup>
    <Compile Include="..\..\Shared\MetaData.cs">
      <Link>Properties\MetaData.cs</Link>
    </Compile>
  </ItemGroup>
  <ItemGroup>
    <ProjectReference Include="..\..\Shared\EpicGames.Build\EpicGames.Build.csproj" PrivateAssets="All"><Private>false</Private></ProjectReference>
    <ProjectReference Include="..\..\Shared\EpicGames.Core\EpicGames.Core.csproj" PrivateAssets="All"><Private>false</Private></ProjectReference>
    <ProjectReference Include="..\..\UnrealBuildTool\UnrealBuildTool.csproj" PrivateAssets="All"><Private>false</Private></ProjectReference>
    <ProjectReference Include="..\AutomationUtils\AutomationUtils.Automation.csproj" PrivateAssets="All"><Private>false</Private></ProjectReference>
    <ProjectReference Include="..\Gauntlet\Gauntlet.Automation.csproj" PrivateAssets="All"><Private>false</Private></ProjectReference>
  </ItemGroup>
  <ItemGroup>
    <PackageReference Include="Google.Apis.Drive.v3" Version="1.33.1.1232" PrivateAssets="all" />
    <!-- Unused, but required to resolve version conflicts from transitive dependencies. -->
    <PackageReference Include="Newtonsoft.Json" Version="13.0.4" PrivateAssets="all" />
    <PackageReference Include="System.IO.FileSystem.AccessControl" Version="5.0.0" />
  </ItemGroup>
</Project>
