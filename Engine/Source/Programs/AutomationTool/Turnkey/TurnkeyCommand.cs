// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Reflection;
using System.Linq;
using System.Text;
using EpicGames.Core;

namespace Turnkey
{
	class CommandInfo
	{
		public TurnkeyCommand Command;
		public string DisplayString;
		public string[] Options; // this is in commandline format, like -opt=val

		public CommandInfo(TurnkeyCommand InCommand)
		{
			Command = InCommand;
			DisplayString = InCommand.GetType().Name;
			Options = new string[0];
		}
		public CommandInfo(TurnkeyCommand InCommand, string InDisplayString, string[] InOptions)
		{
			Command = InCommand;
			DisplayString = string.Format("{0} [{1}]", InDisplayString, string.Join(" ", InOptions));
			Options = InOptions;
		}
	}

	abstract class TurnkeyCommand
	{
		protected enum CommandGroup
		{
			Sdk,
			Builds,
			Informational,
			Settings,
			Misc,
		}

		protected abstract CommandGroup Group { get; }
		protected abstract void Execute(string[] CommandOptions);
		protected virtual Dictionary<string, string[]> GetExtendedCommandsWithOptions()
		{
			return null;
		}
		internal void InternalExecute(string[] CommandOptions)
		{
			Execute(CommandOptions);
		}


		// cached commands
		private static Dictionary<string, TurnkeyCommand> CachedCommandsByName = new Dictionary<string, TurnkeyCommand>(StringComparer.OrdinalIgnoreCase);
		private static List<CommandInfo> CachedCommandsByIndex = new List<CommandInfo>();
		private static Dictionary<CommandGroup, List<CommandInfo>> CachedCommandsByGroup = new Dictionary<CommandGroup, List<CommandInfo>>();

		static TurnkeyCommand()
		{
			List<TurnkeyCommand> BaseCommands = new List<TurnkeyCommand>();
			// look for all subclasses, and cache by their ProviderToken
			foreach (Type AssemType in Assembly.GetExecutingAssembly().GetTypes())
			{
				if (typeof(TurnkeyCommand).IsAssignableFrom(AssemType) && AssemType != typeof(TurnkeyCommand))
				{
					TurnkeyCommand Provider = (TurnkeyCommand)Activator.CreateInstance(AssemType);
					BaseCommands.Add(Provider);
				}
			}

			BaseCommands.SortBy(x => x.GetType().Name);

			foreach (CommandGroup Group in Enum.GetValues(typeof(CommandGroup)))
			{
				CachedCommandsByGroup[Group] = new List<CommandInfo>();
			}
			foreach (TurnkeyCommand Command in BaseCommands)
			{
				CachedCommandsByName[Command.GetType().Name] = Command;

				// always add base option
				CachedCommandsByGroup[Command.Group].Add(new CommandInfo(Command));
				Dictionary<string, string[]> ExtendedCommands = Command.GetExtendedCommandsWithOptions();
				if (ExtendedCommands != null)
				{
					foreach (var Pair in ExtendedCommands)
					{
						CachedCommandsByGroup[Command.Group].Add(new CommandInfo(Command, Pair.Key, Pair.Value));
					}
				}
			}

			// now flatten the lists to match the choices in ChooseCommand
			foreach (var Pair in CachedCommandsByGroup)
			{
				foreach (CommandInfo Command in Pair.Value)
				{
					CachedCommandsByIndex.Add(Command);
				}
			}
		}

		static bool bExitAfterCommand = false;
		public static void ExitTurnkeyLoopAfterCommand()
		{
			bExitAfterCommand = true;
		}

		public static bool ExecuteCommand(string CommandString=null)
		{
			TurnkeyCommand Command;
			string[] CommandOptions;
			if (CommandString == null)
			{
				int CommandIndex = PromptForCommand();

				// zero means to exit
				if (CommandIndex == 0)
				{
					return false;
				}
				// we know this is okay because PromptForCommand() will validate the input
				Command = CachedCommandsByIndex[CommandIndex - 1].Command;
				CommandOptions = CachedCommandsByIndex[CommandIndex - 1].Options;
			}
			else
			{
				if (!CachedCommandsByName.TryGetValue(CommandString, out Command))
				{
					TurnkeyUtils.Log("Invalid command");
					return false;
				}

				// no extra options, everything would be on the commandline
				CommandOptions = new string[0];
			}

			try
			{
				TurnkeyUtils.Log("");

				// run the command!
				Command.Execute(CommandOptions);

				TurnkeyUtils.Log("");
			}
			catch (System.Exception)
			{
				throw;
			}

			// true if we should keep going
			return !bExitAfterCommand;
		}

		private static int PromptForCommand()
		{
			// build up categorized list
			List<string> Options = new List<string>();
			foreach (var Pair in CachedCommandsByGroup)
			{
				CommandGroup Group = Pair.Key;
				List<CommandInfo> Commands = Pair.Value;

				if (Commands.Count > 0)
				{
					Options.Add($";{Group}:");
					foreach (CommandInfo Command in Commands)
					{
						Options.Add($"  {Command.DisplayString}");
					}
				}
			}
			return TurnkeyUtils.ReadInputInt("Enter command", Options, true);
		}
	}
}
