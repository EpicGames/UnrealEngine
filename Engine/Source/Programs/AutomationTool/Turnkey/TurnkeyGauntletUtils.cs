// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Linq;
using AutomationTool;
using UnrealBuildTool;
using Gauntlet;

namespace Turnkey
{
	static class TurnkeyGauntletUtils
	{

		static public ITargetDevice GetGauntletDevice(UnrealTargetPlatform Platform, string DeviceName)
		{
			IDeviceFactory Factory = Gauntlet.Utils.InterfaceHelpers.FindImplementations<IDeviceFactory>()
				.Where(F => F.CanSupportPlatform(Platform))
				.FirstOrDefault();

			if (Factory == null)
			{
				return null;
			}

			return Factory.CreateDevice(DeviceName, null);
		}

		static public ITargetDevice GetGauntletDevice(DeviceInfo Device)
		{
			IDeviceFactory Factory = Gauntlet.Utils.InterfaceHelpers.FindImplementations<IDeviceFactory>()
				.Where(F => F.CanSupportPlatform(Device.Platform))
				.FirstOrDefault();

			if (Factory == null)
			{
				return null;
			}

			return Factory.CreateDevice(Device.Name, null);
		}

		static public List<ITargetDevice> GetGauntletDevices(UnrealTargetPlatform Platform, List<string> DeviceNames)
		{
			IDeviceFactory Factory = Gauntlet.Utils.InterfaceHelpers.FindImplementations<IDeviceFactory>()
				.Where(F => F.CanSupportPlatform(Platform))
				.FirstOrDefault();

			if (Factory == null)
			{
				return new List<ITargetDevice>();
			}

			return DeviceNames.Select(x => Factory.CreateDevice(x, null)).ToList();
		}

		static public List<ITargetDevice> GetGauntletDevices(List<DeviceInfo> Devices)
		{
			return Devices.Select(x => GetGauntletDevice(x)).Where(x => x != null).ToList();
		}

		static public List<ITargetDevice> GetDefaultGauntletDevices(UnrealTargetPlatform Platform)
		{
			IDefaultDeviceSource DeviceSource = Gauntlet.Utils.InterfaceHelpers.FindImplementations<IDefaultDeviceSource>()
				.Where(S => S.CanSupportPlatform(Platform))
				.FirstOrDefault();

			if (DeviceSource == null)
			{
				return new List<ITargetDevice>();
			}

			return DeviceSource.GetDefaultDevices().ToList();
		}


	}
}
