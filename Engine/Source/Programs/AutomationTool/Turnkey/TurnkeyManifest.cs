// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.Text;
using System.Xml.Serialization;
using System.IO;
using System.Linq;
using System.Text.RegularExpressions;
using EpicGames.Core;
using UnrealBuildTool;
using AutomationTool;

namespace Turnkey
{
	public class TurnkeyManifest
	{
		static private string StandardManifestName = "TurnkeyManifest.xml";

		[XmlElement("FileSource")]
		public FileSource[] FileSources = null;

		[XmlArrayItem(ElementName = "Manifest")]
		public string[] AdditionalManifests = null;

		public SavedSetting[] SavedSettings = null;

		internal void PostDeserialize()
		{
			// load any settings set in the xml
			if (SavedSettings != null)
			{
				Array.ForEach(SavedSettings, x => TurnkeyUtils.SetVariable(x.Variable, TurnkeyUtils.ExpandVariables(x.Value)));
			}

			if (FileSources != null)
			{
				Array.ForEach(FileSources, x => x.PostDeserialize());

				// look for list expansion and fixup file sources (filexpansion will come later, on demand, to improve speed)
				List<FileSource> ExpandedSources = new List<FileSource>();
				List<FileSource> NonExpandedSources = new List<FileSource>();

				foreach (FileSource Source in FileSources)
				{
					List<FileSource> Expansions = Source.ConditionalExpandLists();
					if (Expansions != null)
					{
						ExpandedSources.AddRange(Expansions);
					}
					else
					{
						// add to new list, instead of removing from FileSources since we are iterating
						NonExpandedSources.Add(Source);
					}
				}

				// now combine them and replace FileSources
				NonExpandedSources.AddRange(ExpandedSources);
				FileSources = NonExpandedSources.ToArray();
			}
		}

		static List<FileSource> DiscoveredFileSources = null;

		public static List<UnrealTargetPlatform> GetPlatformsWithSdks()
		{
			DiscoverManifests();

			// this can handle FileSources with pending Expansions, so get the unique set of Platforms represented by pre or post expansion sources
			// skip over non-Sdk types, since this wants just Sdks
			List<UnrealTargetPlatform> Platforms = new List<UnrealTargetPlatform>();
			DiscoveredFileSources.FindAll(x => x.IsSdkType()).ForEach(x => Platforms.AddRange(x.GetPlatforms()));

			return Platforms.Distinct().ToList();
		}

		public static List<string> GetProjectsWithBuilds()
		{
			DiscoverManifests();

			// this can handle FileSources with pending Expansions, so get the unique set of Platforms represented by pre or post expansion sources
			// skip over non-Sdk types, since this wants just Sdks
			List<string> Projects = new List<string>();
			DiscoveredFileSources.FindAll(x => x.Type == FileSource.SourceType.Build).ForEach(x => Projects.Add(x.Project));

			return Projects.Distinct(StringComparer.InvariantCultureIgnoreCase).ToList();
		}


		public static List<FileSource> GetAllDiscoveredFileSources()
		{
			return FilterDiscoveredFileSources(null, null);
		}
		
		private static List<FileSource> ExpandFilteredSources(List<FileSource> Sources)
		{
			// get the set that needs to expand
			List<FileSource> NeedsExpansion = Sources.FindAll(x => x.NeedsFileExpansion());
			List<FileSource> NotNeedsExpansion = Sources.FindAll(x => !x.NeedsFileExpansion());


			// remove the ones that we are going to expand below
			DiscoveredFileSources = DiscoveredFileSources.FindAll(x => !NeedsExpansion.Contains(x));

			foreach (FileSource Source in NeedsExpansion)
			{
				List<FileSource> ExpansionResult = Source.ExpandCopySource();

				// add them to the full set of sources, and our filtered sources
				NotNeedsExpansion.AddRange(ExpansionResult);
				DiscoveredFileSources.AddRange(ExpansionResult);
			}

			// NotNeedsExpansion now contains the expanded sources, and the sources that didn't need to be expanded

			// now filter out platforms that are disabled on this host
			NotNeedsExpansion = NotNeedsExpansion.FindAll(x => x.GetPlatforms().Any(y => UEBuildPlatformSDK.GetSDKForPlatform(y.ToString()).bIsSdkAllowedOnHost));

			return NotNeedsExpansion;
		}

		public static List<FileSource> FilterDiscoveredFileSources(UnrealTargetPlatform? Platform, FileSource.SourceType? Type)
		{
			// hunt down manifests if needed
			DiscoverManifests();

			List<FileSource> Matching;
			if (Platform == null && Type == null)
			{
				Matching = DiscoveredFileSources;
			}
			else
			{
				// if the platform is from expansion (possible with AutoSDKs in particular), then we need to expand it in case this platform will be matched
				Matching = DiscoveredFileSources.FindAll(x => (Platform == null || (x.PlatformString?.StartsWith("$(")).GetValueOrDefault() || x.SupportsPlatform(Platform.Value)) && (Type == null || x.Type == Type.Value));
			}

			return ExpandFilteredSources(Matching);
		}

		public static List<FileSource> FilterDiscoveredBuilds(string Project)
		{
			// hunt down manifests if needed
			DiscoverManifests();

			List<FileSource> Filtered = DiscoveredFileSources.FindAll(x => x.Type == FileSource.SourceType.Build && x.Project.Equals(Project, StringComparison.InvariantCultureIgnoreCase));

			return ExpandFilteredSources(Filtered);
		}

		public static void DiscoverManifests()
		{
			// this is the indicator that we haven't run yet
			if (DiscoveredFileSources == null)
			{
				DiscoveredFileSources = new List<FileSource>();

				// known location to branch from, this will include a few other locations
				string RootOperation = "file:$(EngineDir)/Build/Turnkey/TurnkeyManifest.xml";

				LoadManifestsFromProvider(RootOperation).ForEach(x =>
				{
					if (x.FileSources != null)
					{
						DiscoveredFileSources.AddRange(x.FileSources);
					}
				});

				// also manually create local "FileSource" objects specified via code
				foreach (UnrealTargetPlatform Platform in UnrealTargetPlatform.GetValidPlatforms())
				{
					DataDrivenPlatformInfo.ConfigDataDrivenPlatformInfo Info = DataDrivenPlatformInfo.GetDataDrivenInfoForPlatform(Platform.ToString());
					if (Info != null && Info.bIsEnabled)
					{
						// these are usually going to be empty
						foreach (string SpecifiedVersion in AutomationTool.Platform.GetPlatform(Platform).GetCodeSpecifiedSdkVersions())
						{
							FileSource CodeSource = FileSource.CreateCodeSpecifiedSource($"{Platform} SDK {SpecifiedVersion}", SpecifiedVersion, Platform);
							DiscoveredFileSources.Add(CodeSource);
						}

						foreach (string SpecifiedVersion in AutomationTool.Platform.GetPlatform(Platform).GetCodeSpecifiedDeviceSoftwareUpdateVersions())
						{
							FileSource CodeSource = FileSource.CreateCodeSpecifiedSource($"{Platform} Device Software/Flash {SpecifiedVersion}", SpecifiedVersion, Platform);
							CodeSource.Type = FileSource.SourceType.Flash;
							DiscoveredFileSources.Add(CodeSource);
						}
					}
				}
			}
		}

		public static List<TurnkeyManifest> LoadManifestsFromProvider(string EnumerationOperation)
		{
			List<TurnkeyManifest> Manifests = new List<TurnkeyManifest>();

			string[] EnumeratedSources = CopyProvider.ExecuteEnumerate(EnumerationOperation);
			if (EnumeratedSources != null)
			{
				foreach (string Source in EnumeratedSources)
				{
					// retrieve the actual file locally
					string LocalManifestPath = CopyProvider.ExecuteCopy(Source);

					// if it doesn't exist, that's okay
					if (LocalManifestPath == null)
					{
						continue;
					}

					// chop off the last path component to get the dir of the ManifestFile source
					int LastSlash = Source.LastIndexOf('/');
					int LastBackSlash = Source.LastIndexOf('\\');
					string ThisManifestDir = Source.Substring(0, Math.Max(LastSlash, LastBackSlash));

					// if a directory is returned, look for the standardized manifest name, as we have not much else we can do with a directory
					if (LocalManifestPath.EndsWith("/") || LocalManifestPath.EndsWith("\\"))
					{
						LocalManifestPath = LocalManifestPath + StandardManifestName;
						if (!File.Exists(LocalManifestPath))
						{
							continue;
						}

						// if we had a directory, then ThisManifestDir above would have the parent of this directory, not this directory itself, fix it
						ThisManifestDir = Source;
					}

// 					Console.WriteLine("Info: Setting ManifestDir to {0}", ThisManifestDir);

					// allow this manifest's directory to be used in further copy, but needs to be in a stack
					string PrevManifestDir = TurnkeyUtils.SetVariable("ThisManifestDir", ThisManifestDir);

//					TurnkeyUtils.Log("Info: Processing manifest: {0}", LocalManifestPath);

					// read in the .xml
					TurnkeyManifest Manifest = Utils.ReadClass<TurnkeyManifest>(LocalManifestPath, Log.Logger);
					Manifest.PostDeserialize();

					Manifests.Add(Manifest);

					// now process any more referenced Sdks
					if (Manifest.AdditionalManifests != null)
					{
						foreach (string ManifestPath in Manifest.AdditionalManifests)
						{
							// now recurse on the extra manifests
							Manifests.AddRange(LoadManifestsFromProvider(ManifestPath));
						}
					}
					TurnkeyUtils.SetVariable("ThisManifestDir", PrevManifestDir);
// 					TurnkeyUtils.Log("Info: Resetting ManifestDir to {0}", PrevManifestDir);
				}
			}

			return Manifests;
		}

		public void Write(string Path)
		{
			Utils.WriteClass<TurnkeyManifest>(this, Path, "", Log.Logger);
		}
	}
}
