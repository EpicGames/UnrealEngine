// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.IO;
using System.Collections.Generic;
using System.Linq;
using System.Text;
using System.Xml.Serialization;

namespace Turnkey
{
	public struct SavedSetting
	{
		public string Variable;
		public string Value;
	}

	public class UserSetting
	{
		public string VariableName;
		public string DefaultValue;
		public string Description;

		public UserSetting(string VariableName, string DefaultValue, string Description)
		{
			this.VariableName = VariableName;
			this.DefaultValue = DefaultValue;
			this.Description = Description;
		}
	}


	static public class TurnkeySettings
	{
		public static string UserSettingManifestLocation = Path.Combine(Environment.GetFolderPath(Environment.SpecialFolder.LocalApplicationData), "Unreal Engine", "UnrealTurnkey", "UserSettingsManifest.xml");

		public static UserSetting[] AllUserSettings =
		{
			new UserSetting("User_QuickSwitchSdkLocation", null, "Location for downloaded Sdks to be stored (currently just for Google Drive, perforce will sync per clientspec)"),
			new UserSetting("User_AppleDevCenterUsername", null, "Sets the username to use when logging in to DevCenter for iOS/tvOS/macOS. (Supercedes the DevCenterUsername .ini setting)"),
			new UserSetting("User_AppleDevCenterTeamID", null, "Sets the teamid to use when logging in to DevCenter for iOS/tvOS/macOS. (Supercedes the IOSTeamID .ini setting)"),
			new UserSetting("User_IOSProvisioningProfile", null, "Sets the provisioning profile to use when setting up iOS signing. (Supercedes the MobileProvision .ini setting)"),
			new UserSetting("User_LastPerforceClient", null, "Perforce depot clientspec to look at first for finding matching clientspecs. Will be set automatically when one is found, but this can be used to force a particular client"),
			new UserSetting("User_ManualProjects", null, "List of manually entered projects, separated by semicolons. Set automatically, but can be edited to clean it up."),
			new UserSetting("User_LastSelectedProject", null, "The most recent project selected. Set automatically."),
			new UserSetting("User_LastSelectedPlatform", null, "The most recent platform selected. Set automatically."),
		};

		public static UserSetting[] AllStudioSettings =
		{
			new UserSetting("Studio_AppleSigningCertPassword", null, "A shared password that is used across Apple Signing Certificates"),
			new UserSetting("Studio_GoogleDriveCredentials", null, "The location of credentials file needed for GoogleDrive integration. The contents should start with: '{\"installed\":{\"client_id\":'"),
			new UserSetting("Studio_GoogleDriveAppName", null, "The name of the application your studio needs to create (see documentation for help)"),
			new UserSetting("Studio_FullInstallPlatforms", null, "The list of platforms your studio has support for Full Sdk installation. Note that this generally affects UI exposing of options, you can still use Turnkey commandline interface to install Sdks for unlisted platforms. This can be a comma separated list, or All."),
			new UserSetting("Studio_AutoSdkPlatforms", null, "The list of platforms your studio has support for AutoSdk. Note that this generally affects UI exposing of options, you can still use Turnkey commandline interface to install Sdks for unlisted platforms. This can be a comma separated list, or All."),
		};

		// basically same as Turnkey variables, but this only contains ones that were loaded so we can write them back out
		static Dictionary<string, string> SetUserSettings = new Dictionary<string, string>();


		public static void Initialize()
		{
			// then load any saved settings
			TurnkeyManifest.LoadManifestsFromProvider("file:" + TurnkeySettings.UserSettingManifestLocation);
			TurnkeyManifest.LoadManifestsFromProvider("file:$(EngineDir)/Build/Turnkey/TurnkeyStudioSettings.xml");

			// now pull out anything that was set and unset it from Turnkey
			// Studio settings aren't special like UserSettings (read-only) so we don't have to unset them
			foreach (UserSetting PossibleSetting in AllUserSettings)
			{
				string Value = TurnkeyUtils.GetVariableValue(PossibleSetting.VariableName);
				if (Value != null)
				{
					SetUserSettings.Add(PossibleSetting.VariableName, Value);
					TurnkeyUtils.ClearVariable(PossibleSetting.VariableName);
				}
			}
		}

		public static bool HasSetUserSetting(string VariableName)
		{
			return SetUserSettings.ContainsKey(VariableName);
		}
		public static Dictionary<string, string> GetSetUserSettings()
		{
			return SetUserSettings;
		}

		public static string GetUserSetting(string VariableName)
		{
			// handle the simple case
			string Value;
			if (SetUserSettings.TryGetValue(VariableName, out Value))
			{
				return Value;
			}

			UserSetting Setting = Array.Find(AllUserSettings, x => x.VariableName == VariableName);

			// make sure it's valid, if not, abort
			if (Setting == null)
			{
				return null;
			}

			// if it does exist, but there is a default, then use it
			if (Setting.DefaultValue != null)
			{
				return Setting.DefaultValue;
			}

			// if there was no default, ask the user for a value
			Value = TurnkeyUtils.ReadInput(string.Format("A value for {0} is needed, but has not been set. Enter a value for this variable.\n  {1}", Setting.VariableName, Setting.Description));

			// set it
			SetUserSetting(VariableName, Value);

			return Value;
		}

		public static string GetUserSettingIfSet(string VariableName, string DefaultIfNotSet)
		{
			return HasSetUserSetting(VariableName) ? GetUserSetting(VariableName) : DefaultIfNotSet;
		}

		public static string SetUserSetting(string VariableName, string Value)
		{
			string PreviousValue = null;

			SetUserSettings.TryGetValue(VariableName, out PreviousValue);
			if (Value == null)
			{
				SetUserSettings.Remove(VariableName);
			}
			SetUserSettings[VariableName] = Value;
			Save();

			return PreviousValue;
		}


		public static void Save()
		{
			// mutate to SavedSettings
			IEnumerable<SavedSetting> Settings = SetUserSettings.Select(x => new SavedSetting() { Variable = x.Key, Value = x.Value });

			if (Settings.Count() > 0)
			{
				// make sure we have anything worth saving
				TurnkeyManifest UserSettingsManifest = new TurnkeyManifest();
				UserSettingsManifest.SavedSettings = Settings.ToArray();

				UserSettingsManifest.Write(UserSettingManifestLocation);
			}
		}
	}
}
