// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections;
using System.Collections.Generic;
using System.IO;
using System.Linq;
using System.Text;
using System.Text.RegularExpressions;
using System.Globalization;
using EpicGames.Core;
using AutomationTool;
using UnrealBuildTool;
using UnrealBuildBase;
using System.Diagnostics;

namespace Turnkey
{
	static class TurnkeyUtils
	{
		// object that commands, etc can use to access UAT functionality
		static public BuildCommand CommandUtilHelper;

		// replacement for Environment.ExitCode
		static public ExitCode ExitCode = ExitCode.Success;

		static public void Initialize(IOProvider InIOProvider, BuildCommand InCommandUtilHelper)
		{
			IOProvider = InIOProvider;
			CommandUtilHelper = InCommandUtilHelper;

			// set up some lists
			SetVariable("AllPlatforms", string.Join(",", UnrealTargetPlatform.GetValidPlatformNames()));

			// walk over all the SDKs and get their AutoSDK string
			IEnumerable<string> AutoSDKPlatforms = UEBuildPlatformSDK.AllPlatformSDKObjects.Select(x => x.GetAutoSDKPlatformName()).Distinct();
			SetVariable("AutoSDKPlatforms", string.Join(",", AutoSDKPlatforms));

			// 			TurnkeyUtils.Log("AllPlatforms = {0}", GetVariableValue("AllPlatforms"));
			// 			TurnkeyUtils.Log("AutoSDKPlatforms = {0}", GetVariableValue("AutoSDKPlatforms"));

			SetVariable("HOST_PLATFORM_NAME", HostPlatform.Current.HostEditorPlatform.ToString());
		}

#region Turnkey Variables

		static Dictionary<string, string> TurnkeyVariables = new Dictionary<string, string>();

		public static string SetVariable(string Key, string Value)
		{
			string Previous;
			TurnkeyVariables.TryGetValue(Key, out Previous);
			TurnkeyVariables[Key] = Value;
			return Previous;
		}
		public static string GetVariableValue(string Key)
		{
			string Value;
			if (TurnkeyVariables.TryGetValue(Key, out Value) || TurnkeySettings.GetSetUserSettings().TryGetValue(Key, out Value))
			{
				return Value;
			}
			return null;
		}

		public static void ClearVariable(string Key)
		{
			TurnkeyVariables.Remove(Key);
		}
		public static bool HasVariable(string Key)
		{
			return TurnkeyVariables.ContainsKey(Key);
		}

		public static string ExpandVariables(string Str, bool bUseOnlyTurnkeyVariables = false)
		{
			// don't crash on null
			if (Str == null)
			{
				return null;
			}

			string ExpandedUserVariables = UnrealBuildTool.Utils.ExpandVariables(Str, TurnkeySettings.GetSetUserSettings(), true);
			return UnrealBuildTool.Utils.ExpandVariables(ExpandedUserVariables, TurnkeyVariables, bUseOnlyTurnkeyVariables);
		}

#endregion

#region Commandline Handling

		public static bool ParseParam(string Param, string[] ExtraOptions)
		{
			// our internal extraoptions still have - in front, but CommandUtilHelper won't have the dashes
			return CommandUtils.ParseParam(ExtraOptions, "-" + Param) || CommandUtilHelper.ParseParam(Param);
		}

		public static string ParseParamValue(string Param, string Default, string[] ExtraOptions)
		{
			// our internal extraoptions still have - in front, but CommandUtilHelper won't have the dashes
			string Value = CommandUtils.ParseParamValue(ExtraOptions, "-" + Param, null);
			if (Value == null)
			{
				Value = CommandUtilHelper.ParseParamValue(Param, Default);
			}

			return Value == null ? Default : Value;
		}

		private static List<UnrealTargetPlatform> GetAllValidPlatforms(List<UnrealTargetPlatform> SourcePlatforms=null)
		{
			if (SourcePlatforms == null)
			{
				SourcePlatforms = UnrealTargetPlatform.GetValidPlatforms().ToList();
			}

			return SourcePlatforms.Where(x => UEBuildPlatformSDK.GetSDKForPlatform(x.ToString()) != null).ToList();
		}

		public static List<UnrealTargetPlatform> GetPlatformsFromCommandLineOrUser(string[] CommandOptions, List<UnrealTargetPlatform> PossiblePlatforms)
		{
			string PlatformString = TurnkeyUtils.ParseParamValue("Platform", null, CommandOptions);
			bool bUnattended = TurnkeyUtils.ParseParam("Unattended", CommandOptions);

			// Remove known bad platforms
			PossiblePlatforms = GetAllValidPlatforms(PossiblePlatforms);

			// sort by name
			PossiblePlatforms.Sort((x, y) => string.Compare(x.ToString(), y.ToString()));

			List<UnrealTargetPlatform> Platforms = new List<UnrealTargetPlatform>();
			// prompt user for a platform
			if (PlatformString == null)
			{
				if (bUnattended)
				{
					// can't ask
					return null;
				}

				// default to last platform chosen
				string LastPlatform = TurnkeySettings.GetUserSettingIfSet("User_LastSelectedPlatform", "");

				List<string> PlatformOptions = PossiblePlatforms.ConvertAll(x => x.ToString());
				PlatformOptions.Add("All of the Above");

				int Default = PlatformOptions.FindIndex(x => x.Equals(LastPlatform, StringComparison.OrdinalIgnoreCase));
				int PlatformChoice = TurnkeyUtils.ReadInputInt("Choose a platform:", PlatformOptions, true, Default == -1 ? -1 : Default + 1);

				if (PlatformChoice == 0)
				{
					return null;
				}
				// All platforms means to install every platform with an installer
				if (PlatformChoice == PlatformOptions.Count)
				{
					Platforms = PossiblePlatforms;
				}
				else
				{
					Platforms.Add(PossiblePlatforms[PlatformChoice - 1]);
					// remember for next time
					TurnkeySettings.SetUserSetting("User_LastSelectedPlatform", PossiblePlatforms[PlatformChoice - 1].ToString());
				}
			}
			else if (PlatformString.ToLower() == "all")
			{
				Platforms = PossiblePlatforms;
			}
			else
			{
				string[] Tokens = PlatformString.Split("+".ToCharArray(), StringSplitOptions.RemoveEmptyEntries);

				foreach (string Token in Tokens)
				{
					UnrealTargetPlatform Platform;
					if (!UnrealTargetPlatform.TryParse(Token, out Platform))
					{
						TurnkeyUtils.Log("Platform {0} is unknown", Token);
						continue;
					}

					// if the platform isn't in the possible list, then don't add it
					if (PossiblePlatforms.Contains(Platform))
					{
						Platforms.Add(Platform);
					}
				}
			}

			return Platforms.OrderBy(x => x.ToString()).ToList();
		}

		private enum ProjectMode
		{
			Normal,
			Templates,
			Sandbox,
			Misc,
			Samples,
			Collaboration,
			Personal,
			Recent,
		}

		public static FileReference GetProjectFromCommandLineOrUser(string[] CommandOptions)
		{
			ProjectMode CurrentMode = ProjectMode.Normal;

			string Project = TurnkeyUtils.GetVariableValue("Project");
			string LastProject = TurnkeySettings.GetUserSettingIfSet("User_LastSelectedProject", "");

			// create and initialize a dictionary of type to project list
			Dictionary<ProjectMode, List<FileReference>> ProjectsByType = new Dictionary<ProjectMode, List<FileReference>>();
			foreach (ProjectMode Mode in Enum.GetValues(typeof(ProjectMode)))
			{
				ProjectsByType.Add(Mode, new List<FileReference>());
			}

			// look up old manual projects
			string ManualProjectString = TurnkeySettings.GetUserSettingIfSet("User_ManualProjects", "");
			List<string> ManualProjects = ManualProjectString.Split(";", StringSplitOptions.RemoveEmptyEntries).ToList();
			ProjectsByType[ProjectMode.Recent].AddRange(ManualProjects.Select(x => new FileReference(x)).ToList());
			// start on Recent page if last one was a manual project
			if (ProjectsByType[ProjectMode.Recent].Any(x => x.FullName.Equals(LastProject, StringComparison.OrdinalIgnoreCase)))
			{
				CurrentMode = ProjectMode.Recent;
			}

			// sort the discovered projects
			foreach (FileReference NativeProject in NativeProjects.EnumerateProjectFiles(EpicGames.Core.Log.Logger))
			{
				DirectoryReference ProjectDir = NativeProject.Directory;
				ProjectMode Mode = ProjectMode.Normal;

				if (ProjectDir.ContainsName("StarterContent", 0) || ProjectDir.ContainsName("ContentExamples", 0))
				{
					Mode = ProjectMode.Misc;
				}
				else if (ProjectDir.ContainsName("Sandbox", 0))
				{
					Mode = ProjectMode.Sandbox;
				}
				else if (ProjectDir.ContainsName("Templates", 0))
				{
					Mode = ProjectMode.Templates;
				}
				else if (ProjectDir.ContainsName("Samples", 0))
				{
					Mode = ProjectMode.Samples;
				}
				else if (ProjectDir.ContainsName("Collaboration", 0))
				{
					Mode = ProjectMode.Collaboration;
				}

				// if the last project was this project, then use this as the starting type
				if (LastProject != null && NativeProject.GetFileNameWithoutAnyExtensions().Equals(LastProject, StringComparison.OrdinalIgnoreCase))
				{
					CurrentMode = Mode;
				}
				ProjectsByType[Mode].Add(NativeProject);
			}

			// now look for the My Documents location
			DirectoryReference PersonalProjects = new DirectoryReference(Path.Combine(Environment.GetFolderPath(Environment.SpecialFolder.Personal), "Unreal Projects"));
			if (DirectoryReference.Exists(PersonalProjects))
			{
				foreach (DirectoryReference PersonalProjectDir in DirectoryReference.EnumerateDirectories(PersonalProjects))
				{
					FileReference PersonalProjectFile = DirectoryReference.EnumerateFiles(PersonalProjectDir, "*.uproject").FirstOrDefault();
					if (PersonalProjectFile != null)
					{
						ProjectsByType[ProjectMode.Personal].Add(PersonalProjectFile);
					}
				}
			}


			while (string.IsNullOrEmpty(Project))
			{
				// get list of project names from the dictionary
				List<string> ProjectNames = new List<string>();

				// add options to switch mode
				Dictionary<int, ProjectMode> IndexToMode = new Dictionary<int, ProjectMode>();
				foreach (ProjectMode Mode in Enum.GetValues(typeof(ProjectMode)))
				{
					if (Mode != CurrentMode)
					{
						IndexToMode.Add(ProjectNames.Count, Mode);
						ProjectNames.Add(string.Format("[Show {0} Projects]", Mode.ToString()));
					}
				}

				// now show the projects in this mode
				ProjectNames.AddRange(ProjectsByType[CurrentMode].Select(x => (CurrentMode == ProjectMode.Recent) ? x.FullName : x.GetFileNameWithoutAnyExtensions()));

				// and finally a manual entry option
				if (OperatingSystem.IsWindows())
				{
					ProjectNames.Add("[Browse...]");
				}
				else
				{
					ProjectNames.Add("[Enter Path To .uproject...]");
				}

				int Default = LastProject == null ? -1 : ProjectNames.FindIndex(x => x.Equals(LastProject, StringComparison.OrdinalIgnoreCase));
				int Choice = TurnkeyUtils.ReadInputInt(string.Format("Choose a {0} project to execute, or select a set of projects to list. (You can skip this by specifying -project=XXX on the commandline)", CurrentMode), ProjectNames, true, Default == -1 ? -1 : Default + 1);
				if (Choice == 0)
				{
					return null;
				}
				// skip cancel
				Choice = Choice - 1;

				// look for manual entry option
				if (Choice == ProjectNames.Count - 1)
				{
					if (OperatingSystem.IsWindows())
					{
						string ChosenFile = null;
						System.Threading.Thread t = new System.Threading.Thread(x =>
						{
							Debug.Assert(OperatingSystem.IsWindowsVersionAtLeast(7));
							ChosenFile = UnrealWindowsForms.Utils.ShowOpenFileDialogAndReturnFilename("Project Files (*.uproject)|*.uproject");
						});

						t.SetApartmentState(System.Threading.ApartmentState.STA);
						t.Start();
						t.Join();

						if (ChosenFile == null)
						{
							continue;
						}
					}
					else
					{
						while (true)
						{
							string ProjectChoice = TurnkeyUtils.ReadInput("Enter path to .uproject file:");
							if (!File.Exists(ProjectChoice))
							{
								bool bResponse = TurnkeyUtils.GetUserConfirmation(string.Format("'{0}' doesn't exist. Would you like to enter another path?", ProjectChoice), false);
								if (bResponse == false)
								{
									break;
								}
							}
							else
							{
								Project = ProjectChoice;
								break;
							}
						}
					}

					// if we got a valid vhoice, remember it
					if (!string.IsNullOrEmpty(Project))
					{
						// remember this for next run
						// case insensitive Contains
						if (!ManualProjects.Any(x => x.Equals(Project, StringComparison.OrdinalIgnoreCase)))
						{
							ManualProjects.Add(Project);
						}
						TurnkeySettings.SetUserSetting("User_ManualProjects", string.Join(";", ManualProjects));
					}
				}
				else
				{
					// if the choice was a mode switch, get the mode
					if (IndexToMode.TryGetValue(Choice, out CurrentMode))
					{
						continue;
					}
					Project = ProjectNames[Choice];
				}
			}

			// remember project for next run
			TurnkeySettings.SetUserSetting("User_LastSelectedProject", Project);

			return ProjectUtils.FindProjectFileFromName(Project);
		}

		private static DeviceInfo GetDeviceByPlatformAndName(UnrealTargetPlatform Platform, string DeviceName)
		{
			try
			{
				return AutomationTool.Platform.GetPlatform(Platform).GetDeviceByName(DeviceName);
			}
			catch (Exception Ex)
			{
				TurnkeyUtils.Log($"An error occurred trying to access device {DeviceName} for platform {Platform}: {Ex.Message}");

				// swallow errors and just return no device
				return null;
			}
		}

		private static DeviceInfo[] GetDevicesForPlatform(UnrealTargetPlatform Platform)
		{
			try
			{
				DeviceInfo[] Devices = AutomationTool.Platform.GetPlatform(Platform).GetDevices();
				
				// in the null case, return an empty array, for cleaner code for callers 
				return Devices != null ? Devices : new DeviceInfo[] { };
			}
			catch (Exception Ex)
			{
				TurnkeyUtils.Log($"An error occurred trying to access all devices for platform {Platform}: {Ex.Message}");

				// swallow errors and just return no devices
				return new DeviceInfo[] { } ;
			}
		}


		public static List<DeviceInfo> GetDevicesFromCommandLineOrUser(string[] CommandOptions, UnrealTargetPlatform Platform)
		{
			return GetDevicesFromCommandLineOrUser(CommandOptions, new List<UnrealTargetPlatform>() { Platform });
		}
		//public static DeviceInfo GetDeviceFromCommandLineOrUser(string[] CommandOptions, UnrealTargetPlatform Platform)
		//{
		//	Dictionary<UnrealTargetPlatform, List<DeviceInfo>> PlatformsAndDevices = GetDevicesFromCommandLineOrUser(CommandOptions, Platform);
		//}

		public static List<DeviceInfo> GetDevicesFromCommandLineOrUser(string[] CommandOptions, List<UnrealTargetPlatform> PossiblePlatforms)
		{
			List<DeviceInfo> ChosenDevices = null;

			// look at any devices on the commandline, and see if they have platforms or not
			string DeviceList = TurnkeyUtils.ParseParamValue("Device", null, CommandOptions);
			List<string> SplitDeviceList = null;
			if (DeviceList != null && DeviceList.ToLower() != "all")
			{
				SplitDeviceList = DeviceList.Split("+".ToCharArray()).ToList();

				// look if they have platform@ tags
				bool bAnyHavePlatform = SplitDeviceList.Any(x => x.Contains("@"));
				if (bAnyHavePlatform)
				{
					if (!SplitDeviceList.All(x => x.Contains("@")))
					{
						throw new AutomationException("If any device in -device has a platform indicator ('Platform@Device'), they must all have a platform indicator");
					}

					// now split it up for devices for each platform
					foreach (string DeviceToken in SplitDeviceList)
					{
						string[] Tokens = DeviceToken.Split("@".ToCharArray(), StringSplitOptions.RemoveEmptyEntries);
						if (Tokens.Length != 2)
						{
							throw new AutomationException("{0} did not have the Platform@Device format", DeviceToken);
						}
						UnrealTargetPlatform Platform;
						if (!UnrealTargetPlatform.TryParse(Tokens[0], out Platform))
						{
							TurnkeyUtils.Log("Platform indicator {0} is an invalid platform, skipping", Tokens[0]);
							continue;
						}

						string DeviceName = Tokens[1];

						// track it
						if (ChosenDevices == null)
						{
							ChosenDevices = new List<DeviceInfo>();
						}
						
						if (DeviceName.ToLower() == "all")
						{
							ChosenDevices.AddRange(GetDevicesForPlatform(Platform));
						}
						else
						{
							DeviceInfo Device = GetDeviceByPlatformAndName(Platform, DeviceName);
							if (Device != null)
							{
								ChosenDevices.Add(Device);
							}
						}
					}
					SplitDeviceList = null;
				}
			}

			// if we didn't get some platforms already from -device list, then get or ask the user for platforms
			if (ChosenDevices == null)
			{
				ChosenDevices = new List<DeviceInfo>();
				List<UnrealTargetPlatform> ChosenPlatforms;

				// use all platforms (with -device=all), or ask user if needed (GetPlatformsFromCommandLineOrUser would look at -platform=all, not -device=all)
				// if -platform was specified, use the function to get just the specified ones
				if (DeviceList != null && DeviceList.ToLower() == "all" && TurnkeyUtils.ParseParamValue("Platform", null, CommandOptions) == null)
				{
					ChosenPlatforms = GetAllValidPlatforms(PossiblePlatforms);
				}
				// if there's only one platform possible, use it
				else if (PossiblePlatforms != null && PossiblePlatforms.Count == 1)
				{
					ChosenPlatforms = PossiblePlatforms;
				}
				else
				{
					ChosenPlatforms = TurnkeyUtils.GetPlatformsFromCommandLineOrUser(CommandOptions, PossiblePlatforms);
				}

				if (ChosenPlatforms == null)
				{
					return null;
				}

				if (ChosenPlatforms.Count > 1 && SplitDeviceList != null && !(SplitDeviceList.Count == 1 && SplitDeviceList[0].ToLower() == "all"))
				{
					throw new AutomationException("When using -Device without platform specifiers ('Platform@Device'), a single platform must be specified (unless -Device=All is used)");
				}

				// get all the devices for the platforms
				if (!string.IsNullOrEmpty(DeviceList) && DeviceList.ToLower() == "all")
				{
					foreach (UnrealTargetPlatform Platform in ChosenPlatforms)
					{
						ChosenDevices.AddRange(GetDevicesForPlatform(Platform));
					}
				}
				// now if the list of devices was given, then attempt to find them in the platform
				else if (SplitDeviceList != null && SplitDeviceList.Count > 0)
				{
					foreach (UnrealTargetPlatform Platform in ChosenPlatforms)
					{
						foreach (string DeviceName in SplitDeviceList)
						{
							DeviceInfo Device = GetDeviceByPlatformAndName(Platform, DeviceName);
							if (Device != null)
							{
								ChosenDevices.Add(Device);
							}
						}
					}
				}
				// otherwise ask user for device
				else
				{
					List<string> Options = new List<string>();
					List<DeviceInfo> PossibleDevices = new List<DeviceInfo>();

					foreach (UnrealTargetPlatform Platform in ChosenPlatforms)
					{
						foreach (DeviceInfo Device in GetDevicesForPlatform(Platform))
						{
							PossibleDevices.Add(Device);
							Options.Add(string.Format("[{0} {1}] {2}", Platform, Device.Type, Device.Name));
						}
					}

					if (PossibleDevices.Count == 0)
					{
						Log("Unable to find any devices for platform(s): {0}", string.Join(", ", ChosenPlatforms));
						return null;
					}

					// get the choice
					int Choice = TurnkeyUtils.ReadInputInt("Select a device:", Options, true);

					if (Choice == 0)
					{
						return null;
					}

					// finally, add it to the proper list
					ChosenDevices.Add(PossibleDevices[Choice - 1]);
				}
			}

			// if we ended up with some platforms, but no devices, just return null
			return (ChosenDevices != null && ChosenDevices.Count > 0) ? ChosenDevices : null;
		}

		public static void GetPlatformsAndDevicesFromCommandLineOrUser(string[] CommandOptions, bool bSkipAskUserForDevice, out List<UnrealTargetPlatform> Platforms, out List<DeviceInfo> Devices, List<UnrealTargetPlatform> AllowedPlatforms=null)
		{
			Platforms = new List<UnrealTargetPlatform>();
			Devices = new List<DeviceInfo>();

			string DeviceString = TurnkeyUtils.ParseParamValue("Device", null, CommandOptions);
			if (string.IsNullOrEmpty(DeviceString))
			{
				Platforms.AddRange(TurnkeyUtils.GetPlatformsFromCommandLineOrUser(CommandOptions, AllowedPlatforms));
				// restrict devices to this platform
				AllowedPlatforms = Platforms;
			}

			// if there's no -device param, and we don't want to ask user for a device, skip this
			if (!string.IsNullOrEmpty(DeviceString) || !bSkipAskUserForDevice)
			{
				List<DeviceInfo> ChosenDevices = TurnkeyUtils.GetDevicesFromCommandLineOrUser(CommandOptions, AllowedPlatforms);
				
				if (ChosenDevices != null)
				{
					Devices = ChosenDevices;
					// pull the platforms out of the devices we have chosen
					Platforms = Devices.Select(x => x.Platform).ToHashSet().ToList();
				}
			}
		}

		public static string GetGenericOption(string[] CommandOptions, List<string> Options, string CommandLineOption)
		{
			return GetGenericOption(CommandOptions, Options, CommandLineOption, out _);
		}

		public static string GetGenericOption(string[] CommandOptions, List<string> Options, string CommandLineOption, out bool bWasOnCommandLine)
		{
			string ChosenValue = TurnkeyUtils.ParseParamValue(CommandLineOption, null, CommandOptions);

			if (ChosenValue != null)
			{
				bWasOnCommandLine = true;
			}
			else
			{
				bWasOnCommandLine = false;

				// default to previous selection if any
				string CachedOptionName = "User_LastSelectedGeneric_" + CommandLineOption;
				string LastSelectedOption = TurnkeySettings.GetUserSettingIfSet(CachedOptionName, "");
				int Default = Options.FindIndex(x => x.Equals(LastSelectedOption, StringComparison.OrdinalIgnoreCase));

				int Choice = ReadInputInt($"Choose the {CommandLineOption}:", Options, true, Default == -1 ? -1 : Default + 1);
				if (Choice == 0)
				{
					return null;
				}

				ChosenValue = Options[Choice - 1];

				// remember for next time
				TurnkeySettings.SetUserSetting(CachedOptionName, ChosenValue);
			}

			return ChosenValue;
		}

#endregion

#region Env vars

		private static Dictionary<string, string> SavedEnvVars;
		private static Dictionary<string, string> EnvVarsToSaveToBatchFile = new();
		
		private static Dictionary<string, string> GetCombinedEnvironmentVariables()
		{
			// Environment variables are case-insensitive on Windows
			Dictionary<string, string> Result = new(StringComparer.OrdinalIgnoreCase);

			// 1. Load Machine Variables First
			foreach (DictionaryEntry Pair in Environment.GetEnvironmentVariables(EnvironmentVariableTarget.Machine))
			{
				Result[(string)Pair.Key] = (string)Pair.Value;
			}

			// 2. Load User Variables Second (Applying Overwrite/Append rules)
			foreach (DictionaryEntry Pair in Environment.GetEnvironmentVariables(EnvironmentVariableTarget.User))
			{
				string Key = (string)Pair.Key;
				string UserValue = (string)Pair.Value;

				// Apply the special concatenation rule for PATH
				if (Key.Equals("PATH", StringComparison.OrdinalIgnoreCase))
				{
					string CombinedPath = Result.TryGetValue(Key, out string Value) ? Value : string.Empty;
                
					if (!string.IsNullOrEmpty(UserValue))
					{
						if (!string.IsNullOrEmpty(CombinedPath) && !CombinedPath.EndsWith(";"))
						{
							CombinedPath += ";";
						}
						
						CombinedPath += UserValue;
					}
					
					Result[Key] = CombinedPath;
				}
				else
				{
					// General Rule: User variable overwrites Machine variable
					Result[Key] = UserValue;
				}
			}

			return Result;
		}

		public static void StartTrackingExternalEnvVarChanges()
		{
			SavedEnvVars = GetCombinedEnvironmentVariables();
		}

		public static void EndTrackingExternalEnvVarChanges()
		{
			Dictionary<string, string> NewEnvVars = GetCombinedEnvironmentVariables();

			Log("Scanning for envvar changes...");

			// Find Additions and Modifications
			foreach (KeyValuePair<string, string> Pair in NewEnvVars)
			{
				if (!SavedEnvVars.ContainsKey(Pair.Key) || SavedEnvVars[Pair.Key] != Pair.Value)
				{
					Log("  Updating process env var {0} to {1}", Pair.Key, Pair.Value);
					Environment.SetEnvironmentVariable(Pair.Key, Pair.Value, EnvironmentVariableTarget.Process);
					EnvVarsToSaveToBatchFile[Pair.Key] = Pair.Value;
				}
			}

			// Find Deletions
			foreach (KeyValuePair<string, string> Pair in SavedEnvVars)
			{
				if (!NewEnvVars.ContainsKey(Pair.Key))
				{
					Log("  Deleting process env var {0}", Pair.Key);
					Environment.SetEnvironmentVariable(Pair.Key, null, EnvironmentVariableTarget.Process);

					if (!EnvVarsToSaveToBatchFile.Remove(Pair.Key))
					{
						EnvVarsToSaveToBatchFile[Pair.Key] = string.Empty;
					}
				}
			}

			SavedEnvVars = NewEnvVars;
			Log("... done! ");

			StringBuilder BatchContents = new();
			foreach (KeyValuePair<string, string> Pair in EnvVarsToSaveToBatchFile)
			{
				BatchContents.AppendLine("set {0}={1}", Pair.Key, Pair.Value);
				Log("  Recording variable to set by caller {0}={1}", Pair.Key, Pair.Value);
			}

			// write out a batch file for the caller of this to call to update vars, including all changes from previous commands
			string BatchPath = ExpandVariables("$(EngineDir)/Intermediate/Turnkey/PostTurnkeyVariables.bat");
			Log("  Writing updated envvars to {0}", BatchPath);
			Directory.CreateDirectory(Path.GetDirectoryName(BatchPath));
			File.WriteAllText(BatchPath, BatchContents.ToString());
		}

#endregion

#region Regex Matching

		static bool TryConvertToUint64(string InValue, out UInt64 OutValue)
		{ 
			if (InValue.StartsWith("0x"))
			{
				// must skip ovr the 0x
				return UInt64.TryParse(InValue.Substring(2), NumberStyles.HexNumber, CultureInfo.InvariantCulture, out OutValue);
			}
			return UInt64.TryParse(InValue, out OutValue);
		}

		public static bool IsValueValid(string Value, string AllowedValues, AutomationTool.Platform Platform)
		{
			if (string.IsNullOrEmpty(Value))
			{
				return false;
			}
			if (string.IsNullOrEmpty(AllowedValues))
			{
				return true;
			}

			// use a regex if the allowed string starts with "regex:"
			if (AllowedValues.StartsWith("regex:", StringComparison.InvariantCultureIgnoreCase))
			{
				return Regex.IsMatch(Value, AllowedValues.Substring(6));
			}

			// range type needs to have values that can convert to an integer (in base 10 or 16 with 0x)
			if (AllowedValues.StartsWith("range:", StringComparison.InvariantCultureIgnoreCase))
			{
				// match for "[Min]-[Max]" ([] meaning optional), and Min or Max can be a hex or decimal number,
				// and it must match the entire string, so, 0-10.0 would not match
				Match StreamMatch = new Regex(@"^(0x[0-9a-fA-F]*|[0-9]*)-(0x[0-9a-fA-F]*|[0-9]*)$").Match(AllowedValues.Substring(6));

				if (!StreamMatch.Success)
				{
					TurnkeyUtils.Log("Warning: range: type [{0}] was in a bad format. Must be a range of one or two positive integers in decimal or hex (ex: '0x1000-0x2000', or '435-')");
					return false;
				}

				// convert inputs to uint (unless already converted above by platform)
				UInt64 ValueInt = 0;
				if (!TryConvertToUint64(Value, out ValueInt) && !UEBuildPlatformSDK.GetSDKForPlatform(Platform.PlatformType.ToString()).TryConvertVersionToInt(Value, out ValueInt))
				{
					TurnkeyUtils.Log("Warning: range: input value [{0}] was not an unsigned integer, and platform couldn't convert it", Value);
					return false;
				}

				// min and max are optional, so use 0 and MaxValue if they aren't
				string MinString = StreamMatch.Groups[1].Value;
				string MaxString = StreamMatch.Groups[2].Value;
				UInt64 Min = 0, Max = UInt64.MaxValue;
				if (!string.IsNullOrEmpty(MinString))
				{
					// Regex verified they are in a good format, so we can use Parse
					TryConvertToUint64(MinString, out Min);
				}
				if (!string.IsNullOrEmpty(MaxString))
				{
					TryConvertToUint64(MaxString, out Max);
				}

				// finally perform the comparison
				return ValueInt >= Min && ValueInt <= Max;
			}

			// otherwise, perform a string comparison
			return string.Compare(Value, AllowedValues, true) == 0;
		}

#endregion

#region IO
		static private IOProvider IOProvider;

		public static void Log(string Message)
		{
			IOProvider.Log(Message, bAppendNewLine: true);
		}
		public static void Log(ref StringBuilder Message)
		{
			IOProvider.Log(Message.ToString(), bAppendNewLine: false);
			Message.Clear();
		}
		public static void Log(string Message, params object[] Params)
		{
			IOProvider.Log(string.Format(Message, Params), bAppendNewLine: true);
		}

		public static void Report(string Message)
		{
			IOProvider.Report(Message, bAppendNewLine: true);
		}
		public static void Report(ref StringBuilder Message)
		{
			IOProvider.Report(Message.ToString(), bAppendNewLine: false);
			Message.Clear();
		}
		public static void Report(string Message, params object[] Params)
		{
			IOProvider.Report(string.Format(Message, Params), bAppendNewLine: true);
		}

		public static void PauseForUser(string Message, params object[] Params)
		{
			IOProvider.PauseForUser(string.Format(Message, Params), bAppendNewLine: true);
		}

		public static string ReadInput(string Prompt, string Default = "")
		{
			return IOProvider.ReadInput(Prompt, Default, bAppendNewLine: true);
		}
		public static int ReadInputInt(ref StringBuilder Prompt, List<string> Options, bool bIsCancellable, int DefaultValue = -1)
		{
			string PromptString = Prompt.ToString();
			Prompt.Clear();
			return IOProvider.ReadInputInt(PromptString, Options, bIsCancellable, DefaultValue, bAppendNewLine: false);
		}
		public static int ReadInputInt(string Prompt, List<string> Options, bool bIsCancellable, int DefaultValue = -1)
		{
			return IOProvider.ReadInputInt(Prompt, Options, bIsCancellable, DefaultValue, bAppendNewLine: true);
		}

		public static bool GetUserConfirmation(string Message, bool bDefaultValue)
		{
			return IOProvider.GetUserConfirmation(Message, bDefaultValue, true );
		}
#endregion

#region Temp files [move to LocalCache]
		static List<string> PathsToCleanup = new List<string>();
		public static void AddPathToCleanup(string Path)
		{
			PathsToCleanup.Add(Path);
		}

		public static void CleanupPaths()
		{
			TurnkeyUtils.Log("Cleaning Temp Paths...");

			// cleanup any delay-cleanup files and directories
			foreach (string Path in PathsToCleanup)
			{
				if (Directory.Exists(Path))
				{
					InternalUtils.SafeDeleteDirectory(Path);
				}
				else if (File.Exists(Path))
				{
					InternalUtils.SafeDeleteFile(Path);
				}
			}

			PathsToCleanup.Clear();
		}
#endregion
	}
}
