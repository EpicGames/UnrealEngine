// Copyright Epic Games, Inc. All Rights Reserved.
using System;
using System.Collections.Generic;
using System.Linq;
using System.Text;
using System.IO;
using AutomationTool;
using UnrealBuildTool;
using EpicGames.Core;
using UnrealBuildTool.GDK;
using UnrealBuildBase;
using AutomationUtils.GDK;
using System.Runtime.Versioning;
using Microsoft.Extensions.Logging;
using static AutomationTool.CommandUtils;
using System.Text.Json;

// @note this code will soon be merged into MSGameStoreCustomStagingHandler

[CustomDeploymentHandlerAttribute("MSGameStore")]
public class MSGameStoreCustomDeploymentHandler : CustomDeploymentHandler
{
	protected virtual string IniSection_TargetSettings => "/Script/MSGamingSupport.MSGamingSettings";

	protected static int GDKEdition => GDKExports.GetGDKVersionNumber() ?? 0;

	protected static string GDKBinariesDir => Path.Combine(GDKExports.GetGSDKRoot(), "bin");

	public MSGameStoreCustomDeploymentHandler(Platform AutomationPlatform, ProjectParams Params, DeploymentContext SC)
		: base(AutomationPlatform, Params, SC)
	{
		if (GDKEdition == 0 || !Directory.Exists(GDKBinariesDir))
		{
			throw new AutomationException("GDK is not installed");
		}

		if (GDKEdition < 251000 && Params.NoBootstrapExe && SC.StageTargets.Any(X => !X.Receipt.Architectures.SingleArchitecture.bIsX64))
		{
			throw new AutomationException("October 2025 GDK or higher is required to create a GDK package with ARM64 unless you use the bootstrapper (omit -NoBootstrapExe)");
		}
	}

	public override bool SupportsPlatform(UnrealTargetPlatform Platform)
	{
		return Platform.IsInGroup(UnrealPlatformGroup.Windows);
	}

	public override bool GetPlatformPakCommandLine(ProjectParams Params, DeploymentContext SC, ref string CmdLine)
	{
		// GDK package patching works on 4k chunks
		StringBuilder ExtraPakCommandLine = new StringBuilder(" -blocksize=4KB");

		ConfigHierarchy EngineIni = ConfigCache.ReadHierarchy(ConfigHierarchyType.Engine, DirectoryReference.FromFile(SC.RawProjectPath), Platform.PlatformType, SC.CustomConfig);

		bool bAlignFilesLargeThanBlockInPak = false;
		if (EngineIni.GetBool(IniSection_TargetSettings, "bAlignFilesLargeThanBlockInPak", out bAlignFilesLargeThanBlockInPak) && bAlignFilesLargeThanBlockInPak)
		{
			ExtraPakCommandLine.Append(" -AlignFilesLargerThanBlock");
		}

		CmdLine += ExtraPakCommandLine.ToString();
		return true;
	}

	public override bool GetPlatformIoStoreCommandLine(ProjectParams Params, DeploymentContext SC, ref string CmdLine)
	{
		// GDK package patching works on 4k chunks
		CmdLine += " -compressionblockalignment=4096";
		return true;
	}

	public override bool? GetPlatformPatchesWithDiffPak(ProjectParams Params, DeploymentContext SC)
	{
		// GDK packages do not require diff paks
		return false;
	}

	public override bool PostStagingFileCopy(ProjectParams Params, DeploymentContext SC)
	{
		// if we are never going to package up the files, no need to deal with package metadata stuff
		if (Params.NeverPackage)
		{
			return false;
		}

		// get the package generator for the platform and make the packaging data
		MSGameStorePackageGenerator PackageGenerator = new(SC.RawProjectPath, SC.StageTargetPlatform.PlatformType, IniSection_TargetSettings, SC.CustomConfig, Platform.CrashReportPlatform);
		if (PackageGenerator.PrecachePackageLayout && !Params.PreStaged)
		{
			DirectoryReference PackageMetadataPath = DirectoryReference.Combine(SC.StageDirectory, ".pkgsrc");
			PackageGenerator.GeneratePackageMetadata(Params, SC, PackageMetadataPath);
		}

		return false;
	}


	public override void PostPackage(ProjectParams Params, DeploymentContext SC, int WorkingCL)
	{
		// get the package generator for the platform and make all packages
		MSGameStorePackageGenerator PackageGenerator = new(SC.RawProjectPath, SC.StageTargetPlatform.PlatformType, IniSection_TargetSettings, SC.CustomConfig, Platform.CrashReportPlatform);
		PackageGenerator.GeneratePackages(Params, SC);
	}

	[SupportedOSPlatform("windows")]
	public override bool Deploy(ProjectParams Params, DeploymentContext SC)
	{
		VerifyWindowsEnvironment();

		// check for the existence of the manifest file. This will tell us whether a previous staging step has been successful
		string ManifestPath = Path.Combine(SC.StageDirectory.FullName, "MicrosoftGame.config");
		if (!File.Exists(ManifestPath))
		{
			CommandUtils.Logger.LogError("Failed to find manifest. Did you stage the game? {Path}", ManifestPath);
			throw new AutomationException(ExitCode.Error_MissingExecutable, "Deploy failed. Could not find {1} {0}. You may need to stage this game.", ManifestPath, "MicrosoftGame.config");
		}

		// check for remote windows
		if (IsRemoteDevice(Params, SC))
		{
			if (Params.Package || Params.SkipPackage)
			{
				throw new AutomationException("remote PC GDK package deployment is not supported");
			}

			RemoteWinSupport.Deploy(Platform.TargetPlatformType, Params, SC);
			return true;
		}

		if (Params.Package || Params.SkipPackage)
		{
			// prepare package search paths
			List<DirectoryReference> PackagePaths = new List<DirectoryReference>();
			DirectoryReference RootPackagePath = DirectoryReference.Combine(SC.ProjectRoot, "Saved", "Packages", Platform.GetCookPlatform(false, Params.Client));
			if (Params.HasCreateReleaseVersion)
			{
				RootPackagePath = DirectoryReference.FromString(Params.GetCreateReleaseVersionPath(SC, Params.Client));
			}
			foreach (var Config in SC.StageTargetConfigurations)
			{
				PackagePaths.Add(DirectoryReference.Combine(RootPackagePath, $"{Config}"));
			}
			PackagePaths.Add(RootPackagePath); // also check the root package path as a last resort

			// find the first matching package
			string PackageFullName = GDKGameConfigInfo.GetPackageFullNameFromManifest(ManifestPath);
			string PackageFileName = null;
			foreach (DirectoryReference PackagePath in PackagePaths)
			{
				// locate the package
				if (Directory.Exists(PackagePath.FullName))
				{
					PackageFileName = Directory.EnumerateFiles(PackagePath.FullName, $"{PackageFullName}.msixvc").FirstOrDefault();
					if (!string.IsNullOrEmpty(PackageFileName))
					{
						break;
					}
				}

				CommandUtils.Logger.LogWarning("Could not find .msixvc package in {Path} for {PackageFullName}", PackagePath.FullName, PackageFullName);
			}
			if (string.IsNullOrEmpty(PackageFileName))
			{
				throw new AutomationException(ExitCode.Error_MissingExecutable, "Deploy failed. Could not find any .msixvc packages for {0}", PackageFullName);
			}

			// need to uninstall previous when installing a package, otherwise it fails
			ExecuteSDKCommand("wdapp.exe", $"uninstall \"{PackageFullName}\"");

			// check for special 'install launch chunk only' option: useful for testing chunk installation
			string ExtraInstallParams = "";
			if ((Environment.GetCommandLineArgs().Contains("-testchunkinstall")))
			{
				ExtraInstallParams = "/l";
			}

			// install the package
			bool bSuccess = ExecuteSDKCommand("wdapp.exe", $"install \"{PackageFileName}\" {ExtraInstallParams}");
			if (!bSuccess)
			{
				throw new AutomationException(ExitCode.Error_UnknownDeployFailure, "Deploy failed. Package installation failed");
			}
		}
		else
		{
			// register the application
			bool bSuccess = ExecuteSDKCommand("wdapp.exe", $"register \"{SC.StageDirectory}\"");
			if (!bSuccess)
			{
				throw new AutomationException(ExitCode.Error_UnknownDeployFailure, "Deploy failed. Staged build deploy failed");
			}
		}

		return true;
	}


	[SupportedOSPlatform("windows")]
	public override IProcessResult RunClient(ERunOptions ClientRunFlags, string ClientApp, string ClientCmdLine, ProjectParams Params, DeploymentContext SC)
	{
		VerifyWindowsEnvironment();

		if (IsRemoteDevice(Params, SC))
		{
			return RemoteWinSupport.RunClient(Platform.TargetPlatformType, ClientRunFlags, ClientApp, ClientCmdLine, Params);
		}

		// find the AUMID associated with the given ClientApp
		TargetReceipt Receipt = SC.StageTargets.First(X => X.Receipt.Launch!.GetFileName() == Path.GetFileName(ClientApp)).Receipt;
		UnrealTargetConfiguration Configuration = Receipt.Configuration;
		UnrealArch? Architecture = Params.NoBootstrapExe ? Receipt.Architectures.SingleArchitecture : null;
		string AUMIDName = GDKGameConfigGenerator.GetAUMIDFromProject(Platform.PlatformType, Logger, Params.RawProjectPath.Directory, Configuration, Architecture, Params.ShortProjectName, SC.CustomConfig);
		string PackageFamilyName = AUMIDName.Substring(0, AUMIDName.IndexOf('!'));

		// see if the application has been deployed (either a staged or packaged build - it doesn't matter at this stage)
		string WdAppListOutput = ExecuteSDKCommandAndGetOutput("wdapp.exe", "list /d /json");
		WdAppListOutput = WdAppListOutput.Substring(WdAppListOutput.IndexOf('{'));

		GDKInstalledApp InstalledApp = GDKInstalledApp.ParseAppList(JsonDocument.Parse(WdAppListOutput))
			.Where(App => App.GameConfig.PackageFamilyName == PackageFamilyName)
			.FirstOrDefault();

		string BaseLaunchDir;
		if (InstalledApp != null)
		{
			// if this is an installed package, it is no longer under the StagedBuild directory so we need to look up the new executable path
			if (InstalledApp.Type == GDKInstalledApp.AppType.Packaged)
			{
				string RelativeExePathFragment = new FileReference(ClientApp).MakeRelativeTo(SC.StageDirectory);
				ClientApp = Path.Combine(InstalledApp.DeployPath, RelativeExePathFragment);
			}

			BaseLaunchDir = InstalledApp.DeployPath;
		}
		else
		{
			if (Params.Package || Params.SkipPackage)
			{
				// game is packaged: we need to install the .msixvc file before it can be run
				throw new Exception("MSGameStore packages must be deployed before they can be run");
			}
			else if (Params.Stage || Params.SkipStage)
			{
				// game has been staged
				BaseLaunchDir = SC.StageDirectory.FullName;
			}
			else
			{
				// game has not been staged
				BaseLaunchDir = Path.GetDirectoryName(ClientApp);

				// create a placeholder MicrosoftGame.config
				Logger.LogInformation("Game has not been staged or deployed. Creating placeholder MicrosoftGame.config in {BaseLaunchDir}", BaseLaunchDir);
				List<AppXManifestExecutable> Executables = [ new( Configuration, Path.GetFileName(ClientApp)! ) ];
				GDKGameConfigGenerator.Create(SC.StageTargetPlatform.PlatformType, Logger, DirectoryReference.FromString(BaseLaunchDir), SC.ShortProjectName, Params.RawProjectPath, Executables, CustomConfig: SC.CustomConfig);
			}
		}

		// launch the executable
		return Platform.RunClient(ClientRunFlags, ClientApp, ClientCmdLine, Params);
	}


	public override bool PreGetFilesToDeployOrStage(ProjectParams Params, DeploymentContext SC)
	{
		// copy all custom install actions to the correct folder if we are not using package precaching
		MSGameStorePackageGenerator PackageGenerator = new(SC.RawProjectPath, SC.StageTargetPlatform.PlatformType, IniSection_TargetSettings, SC.CustomConfig, SC.StageTargetPlatform.CrashReportPlatform);
		if (!PackageGenerator.PrecachePackageLayout || Params.PreStaged)
		{
			ConfigHierarchy EngineIni = ConfigCache.ReadHierarchy(ConfigHierarchyType.Engine, DirectoryReference.FromFile(SC.RawProjectPath), SC.StageTargetPlatform.PlatformType, SC.CustomConfig);

			IEnumerable<MSGamingExports.CustomInstallAction> CustomInstallActions = MSGamingExports.CustomInstallAction.GetFromConfig(EngineIni, IniSection_TargetSettings);
			string TargetRoot = MSGamingExports.CustomInstallAction.GetTargetRootFolder(EngineIni, IniSection_TargetSettings);
			foreach (MSGamingExports.CustomInstallAction Action in CustomInstallActions)
			{
				StagedDirectoryReference TargetDir = string.IsNullOrEmpty(Action.TargetFolder) 
					? StagedDirectoryReference.Combine(StagedDirectoryReference.Root, TargetRoot) 
					: StagedDirectoryReference.Combine(StagedDirectoryReference.Root, TargetRoot, Action.TargetFolder);
				SC.StageFiles(StagedFileType.SystemNonUFS, Action.SourceFolder, StageFilesSearch.AllDirectories, TargetDir);
			}
		}

		return false; // do normal deploy too
	}

	public override void PostGetFilesToDeployOrStage(ProjectParams Params, DeploymentContext SC)
	{
		// generate the staging manifest. this is used when we are staging a cooked build. it must be done after everything else has been staged
		if (!Params.Prebuilt)
		{
			GDKAutomationUtils.GenerateGameConfigForStaging(Params, SC, IniSection_TargetSettings, CommandUtils.Logger);
		}
	}

	public override bool GetFilesToStageForDLC(ProjectParams Params, DeploymentContext SC)
	{
		if (!Params.Prebuilt && !Params.NeverPackage)
		{
			// generate the staging manifest. this is used when we are staging a cooked build
			GDKAutomationUtils.GenerateGameConfigForStaging(Params, SC, IniSection_TargetSettings, CommandUtils.Logger);
		}

		return true;
	}

	public override bool GetFilesToArchive(ProjectParams Params, DeploymentContext SC)
	{
		// if we packaged a build, archive that, instead of the raw staging directory
		if (Params.Package || Params.SkipPackage)
		{
			MSGameStorePackageGenerator PackageGenerator = new(SC.RawProjectPath, SC.StageTargetPlatform.PlatformType, IniSection_TargetSettings, SC.CustomConfig, Platform.CrashReportPlatform);
			PackageGenerator.GetPackageFilesToArchive(Params, SC);
		}
		else
		{
			// otherwise, just archive the staged build, but include any Win64 files as well (such as shared runtime DLLs)
			UnrealTargetPlatform[] AdditionalArchivePlatforms = { UnrealTargetPlatform.Win64 };
			SC.ArchiveFiles(SC.StageDirectory.FullName, AdditionalPlatforms: AdditionalArchivePlatforms);
		}

		return true;
	}

	private bool IsRemoteDevice(ProjectParams Params, DeploymentContext SC)
	{
		if (Params.Devices.Count != 1 || SC.StageTargetPlatform == null)
		{
			return false;
		}

		string DeviceId = RemoteWinSupport.GetTargetDeviceName(Params.Devices[0]);
		return SC.StageTargetPlatform.GetDevices().FirstOrDefault(x => x.Id.Equals(DeviceId, StringComparison.OrdinalIgnoreCase))?.Type == RemoteWinSupport.DeviceType;
	}


	protected bool ExecuteSDKCommand(string Command, string CommandLine)
	{
		string ExecutablePath = Path.Combine(GDKBinariesDir, Command);

		int SuccessCode;
		RunAndLog(ExecutablePath, CommandLine, out SuccessCode);

		return (SuccessCode == 0);
	}

	protected string ExecuteSDKCommandAndGetOutput(string Command, string CommandLine)
	{
		string ExecutablePath = Path.Combine(GDKBinariesDir, Command);

		return Run(ExecutablePath, CommandLine, Options: ERunOptions.NoLoggingOfRunCommand | ERunOptions.AppMustExist).Output;
	}


	[SupportedOSPlatform("windows")]
	private static void VerifyWindowsEnvironment()
	{
		if (!MSGamingExports.IsWindowsDeveloperMode())
		{
			CommandUtils.Logger.LogWarning("***** WINDOWS IS NOT IN DEVELOPER MODE. This means that deploying and launching are likely to fail. ******");
			CommandUtils.Logger.LogWarning("\tPlease enable it via the Windows Developer settings menu or contact your IT administrator.");
			CommandUtils.Logger.LogWarning("\tSee https://docs.microsoft.com/en-us/windows/uwp/get-started/enable-your-device-for-development for details.");
			// could spawn ms-settings:developers to open the developer settings page?
		}

		if (!MSGamingExports.IsMSGamingRuntimeInstalled())
		{
			string GRDKEnvVar = GDKExports.IsLegacyFolderStructure() ? "%GameDKLatest%" : "%GameDKCoreLatest%";
			CommandUtils.Logger.LogWarning("****** MICROSOFT GAMING RUNTIME IS NOT INSTALLED. This means that deploying and launching are likely to fail. ******");
			CommandUtils.Logger.LogWarning("\tPlease install it via {gdk}\\redist\\GamingServices.appxbundle or contact your IT administrator.", GRDKEnvVar);
			CommandUtils.Logger.LogWarning("\tSee https://learn.microsoft.com/en-us/gaming/gdk/_content/gc/get-started-with-pc-dev/config-test-pc-software/gr-configure-test-pc for details.");
			// could spawn ms-windows-store://pdp/?productid=9MWPM2CQNLHN to open the store page for it?
		}

		if (!MSGamingExports.IsWindowsVersionSupported())
		{
			CommandUtils.Logger.LogWarning("Your Windows version may be too low - it should be at least Windows 10 May 2019 Update (version 1903)");
		}
	}

}
