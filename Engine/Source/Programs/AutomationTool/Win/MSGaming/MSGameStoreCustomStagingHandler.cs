// Copyright Epic Games, Inc. All Rights Reserved.
using AutomationTool;
using AutomationUtils.GDK;
using EpicGames.Core;
using Microsoft.Extensions.Logging;
using System;
using System.Collections.Generic;
using System.IO;
using System.Linq;
using System.Runtime.Versioning;
using System.Text.Json;
using UnrealBuildBase;
using UnrealBuildTool;
using UnrealBuildTool.GDK;

[SupportedOSPlatform("windows")]
public class MSGameStoreCustomStagingHandler : CustomStagingHandler
{
	protected virtual string IniSection_TargetSettings => "/Script/MSGamingSupport.MSGamingSettings";

	protected static int GDKEdition => GDKExports.GetGDKVersionNumber() ?? 0;

	protected static string GDKBinariesDir => Path.Combine(GDKExports.GetGSDKRoot(), "bin");

	private bool bAutoGeneratePackage = true;


	protected override bool TryInitialize(ProjectParams Params, DeploymentContext SC)
	{
		// make sure the GDK is installed
		if (GDKEdition == 0 || !Directory.Exists(GDKBinariesDir))
		{
			// GDK is not installed
			return false;
		}

		// we only support Windows
		if (!SC.StageTargetPlatform.PlatformType.IsInGroup(UnrealPlatformGroup.Windows))
		{
			return false;
		}

		// already using the legacy MSGamingRuntime custom deployment handler which does a superset of our functionality, so we're not needed
		if (SC.CustomDeployment != null && SC.CustomDeployment is MSGameStoreCustomDeploymentHandler)
		{
			return false;
		}

		// see if our plugin is enabled
		if (!SC.StageTargets.Any( StageTarget => StageTarget.Receipt != null && StageTarget.Receipt.BuildPlugins.Contains("MSGameStore")))
		{
			return false;
		}

		// cache configuration, with custom config
		ConfigHierarchy EngineIni = ConfigCache.ReadHierarchy(ConfigHierarchyType.Engine, DirectoryReference.FromFile(SC.RawProjectPath), SC.StageTargetPlatform.PlatformType, SC.CustomConfig);
		if (EngineIni.GetBool(IniSection_TargetSettings, "bAutoGeneratePackage", out bool bShouldAutoGeneratePackage))
		{
			bAutoGeneratePackage = bShouldAutoGeneratePackage;
		}

		bool bIsRequired = bAutoGeneratePackage;
		if (!bIsRequired)
		{
			return false;
		}

		// sanity check
		if (GDKEdition < 251000 && Params.NoBootstrapExe && SC.StageTargets.Any(X => X.Receipt != null && !X.Receipt.Architectures.SingleArchitecture.bIsX64))
		{
			throw new AutomationException("October 2025 GDK or higher is required to create a GDK package with ARM64 unless you use the bootstrapper (omit -NoBootstrapExe)");
		}

		return true;
	}


	public override void GetFilesToStage(ProjectParams Params, DeploymentContext SC)
	{
		if (bAutoGeneratePackage)
		{
			// copy all custom install actions to the correct folder if we are not using package preaching
			MSGameStorePackageGenerator PackageGenerator = new(SC.RawProjectPath, SC.StageTargetPlatform.PlatformType, IniSection_TargetSettings, SC.CustomConfig, SC.StageTargetPlatform.CrashReportPlatform);
			if (!PackageGenerator.PrecachePackageLayout || Params.PreStaged)
			{
				ConfigHierarchy EngineIni = ConfigCache.ReadHierarchy(ConfigHierarchyType.Engine, DirectoryReference.FromFile(SC.RawProjectPath), SC.StageTargetPlatform.PlatformType, SC.CustomConfig);

				IEnumerable<MSGamingExports.CustomInstallAction> CustomInstallActions = MSGamingExports.CustomInstallAction.GetFromConfig(EngineIni, IniSection_TargetSettings);
				string TargetRoot = MSGamingExports.CustomInstallAction.GetTargetRootFolder(EngineIni, IniSection_TargetSettings);
				foreach (MSGamingExports.CustomInstallAction Action in CustomInstallActions)
				{
					StagedDirectoryReference TargetDir = string.IsNullOrEmpty(Action.TargetFolder) 
						? StagedDirectoryReference.Combine(StagedDirectoryReference.Root, TargetRoot) 
						: StagedDirectoryReference.Combine(StagedDirectoryReference.Root, TargetRoot, Action.TargetFolder);
					SC.StageFiles(StagedFileType.SystemNonUFS, Action.SourceFolder, StageFilesSearch.AllDirectories, TargetDir);
				}
			}
		}
	}

	public override void PostStagingFileCopy(ProjectParams Params, DeploymentContext SC)
	{
		if (Params.NeverPackage)
		{
			return;
		}

		if (bAutoGeneratePackage)
		{
			// get the package generator for the platform and make the packaging data
			MSGameStorePackageGenerator PackageGenerator = new(SC.RawProjectPath, SC.StageTargetPlatform.PlatformType, IniSection_TargetSettings, SC.CustomConfig, SC.StageTargetPlatform.CrashReportPlatform);
			if (PackageGenerator.PrecachePackageLayout && !Params.PreStaged)
			{
				DirectoryReference PackageMetadataBasePath = GetPackageMetadataBasePath(SC);
				CommandUtils.DeleteDirectory_NoExceptions(PackageMetadataBasePath.FullName);

				PackageGenerator.GeneratePackageMetadata(Params, SC, PackageMetadataBasePath);
			}
		}
	}


	public override void Package(ProjectParams Params, DeploymentContext SC, int WorkingCL)
	{
		if (bAutoGeneratePackage)
		{
			// get the package generator for the platform and make all packages
			DirectoryReference PackageMetadataBasePath = GetPackageMetadataBasePath(SC);
			string PackageOutputPathFragment = GetPackageOutputPathFragment(SC);

			MSGameStorePackageGenerator PackageGenerator = new(SC.RawProjectPath, SC.StageTargetPlatform.PlatformType, IniSection_TargetSettings, SC.CustomConfig, SC.StageTargetPlatform.CrashReportPlatform);
			PackageGenerator.GeneratePackages(Params, SC, PackageMetadataBasePath, PackageOutputPathFragment);
		}
	}

	public override void GetFilesToArchive(ProjectParams Params, DeploymentContext SC)
	{
		if (bAutoGeneratePackage)
		{
			// archive any package that we've made
			if (Params.Package || Params.SkipPackage)
			{
				string PackageOutputPathFragment = GetPackageOutputPathFragment(SC);
			
				MSGameStorePackageGenerator PackageGenerator = new(SC.RawProjectPath, SC.StageTargetPlatform.PlatformType, IniSection_TargetSettings, SC.CustomConfig, SC.StageTargetPlatform.CrashReportPlatform);
				PackageGenerator.GetPackageFilesToArchive(Params, SC, PackageOutputPathFragment);
			}
		}
	}



	public override bool TryDeploy(ProjectParams Params, DeploymentContext SC)
	{
		if (bAutoGeneratePackage)
		{
			if (Params.Package || Params.SkipPackage)
			{
				VerifyWindowsEnvironment();

				// prepare package search paths
				string PackageOutputPathFragment = GetPackageOutputPathFragment(SC);
				List<DirectoryReference> PackagePaths = new List<DirectoryReference>();
				DirectoryReference RootPackagePath = DirectoryReference.Combine(SC.ProjectRoot, "Saved", "Packages", SC.StageTargetPlatform.GetCookPlatform(false, Params.Client), PackageOutputPathFragment);
				if (Params.HasCreateReleaseVersion)
				{
					RootPackagePath = DirectoryReference.Combine(DirectoryReference.FromString(Params.GetCreateReleaseVersionPath(SC, Params.Client)), PackageOutputPathFragment);
				}
				foreach (var Config in SC.StageTargetConfigurations)
				{
					PackagePaths.Add(DirectoryReference.Combine(RootPackagePath, $"{Config}"));
				}
				PackagePaths.Add(DirectoryReference.Combine(RootPackagePath, "Combined"));
				PackagePaths.Add(RootPackagePath); // also check the root package path as a last resort

				// find the first matching package
				string PackageFileName = null;
				foreach (DirectoryReference PackagePath in PackagePaths)
				{
					// locate the package
					if (Directory.Exists(PackagePath.FullName))
					{
						PackageFileName = Directory.EnumerateFiles(PackagePath.FullName, $"*.msixvc").FirstOrDefault();
						if (!string.IsNullOrEmpty(PackageFileName))
						{
							break;
						}
					}
				}
				if (string.IsNullOrEmpty(PackageFileName))
				{
					CommandUtils.Logger.LogWarning("No MSGameStore generated package found - nothing to deploy");
					return false;
				}
				GDKGameConfigInfo GameConfig = new GDKGameConfigInfo(PackageFileName);

				// need to uninstall previous when installing a package, otherwise it fails
				// see if the application has been deployed (either a staged or packaged build - it doesn't matter at this stage)
				GDKInstalledApp InstalledApp = GetInstalledApp(GameConfig.PackageFamilyName);
				if (InstalledApp != null)
				{
					ExecuteSDKCommand("wdapp.exe", $"uninstall \"{InstalledApp.GameConfig.PackageFullName}\"");
				}

				// check for special 'install launch chunk only' option: useful for testing chunk installation
				string ExtraInstallParams = "";
				if ((Environment.GetCommandLineArgs().Contains("-testchunkinstall")))
				{
					ExtraInstallParams = "/l";
				}

				// install the package
				bool bSuccess = ExecuteSDKCommand("wdapp.exe", $"install \"{PackageFileName}\" {ExtraInstallParams}");
				if (!bSuccess)
				{
					throw new AutomationException(ExitCode.Error_UnknownDeployFailure, "Deploy failed. Package installation failed");
				}

				return true;
			}
		}

		return false;
	}


	public override IProcessResult TryRunClient(CommandUtils.ERunOptions ClientRunFlags, string ClientApp, string ClientCmdLine, ProjectParams Params, DeploymentContext SC)
	{
		if (bAutoGeneratePackage)
		{
			if (Params.Package || Params.SkipPackage) 
			{
				VerifyWindowsEnvironment();

				// find the AUMID associated with the given ClientApp
				TargetReceipt Receipt = SC.StageTargets.First(X => X.Receipt.Launch!.GetFileName() == Path.GetFileName(ClientApp)).Receipt;
				UnrealTargetConfiguration Configuration = Receipt.Configuration;
				UnrealArch? Architecture = Params.NoBootstrapExe ? Receipt.Architectures.SingleArchitecture : null;
				string AUMIDName = GDKGameConfigGenerator.GetAUMIDFromProject(SC.StageTargetPlatform.PlatformType, CommandUtils.Logger, Params.RawProjectPath.Directory, Configuration, Architecture, Params.ShortProjectName, SC.CustomConfig);
				string PackageFamilyName = AUMIDName.Substring(0, AUMIDName.IndexOf('!'));

				// see if the application has been deployed (either a staged or packaged build - it doesn't matter at this stage)
				GDKInstalledApp InstalledApp = GetInstalledApp(PackageFamilyName);

				if (InstalledApp != null && InstalledApp.Type == GDKInstalledApp.AppType.Packaged)
				{
					// if this is an installed package, it is no longer under the StagedBuild directory so we need to look up the new executable path
					string RelativeExePathFragment = new FileReference(ClientApp).MakeRelativeTo(SC.StageDirectory);
					ClientApp = Path.Combine(InstalledApp.DeployPath, RelativeExePathFragment);

					CommandUtils.PushDir(Path.GetDirectoryName(ClientApp));
					IProcessResult ClientProcess = CommandUtils.Run(ClientApp, ClientCmdLine, null, ClientRunFlags | CommandUtils.ERunOptions.NoWaitForExit);
					CommandUtils.PopDir();

					return ClientProcess;
				}

				CommandUtils.Logger.LogWarning("No MSGameStore installed package found - nothing to launch");
				return null;
			}
		}

		return null;
	}


	private static DirectoryReference GetPackageMetadataBasePath(DeploymentContext SC)
	{
		return DirectoryReference.Combine(SC.ProjectRoot, "Intermediate", SC.StageTargetPlatform.PlatformType.ToString(), "makepkgsrc");
	}

	private static string GetPackageOutputPathFragment(DeploymentContext SC)
	{
		return "MSGameStore";
	}

	private GDKInstalledApp GetInstalledApp(string PackageFamilyName)
	{
		string WdAppListOutput = ExecuteSDKCommandAndGetOutput("wdapp.exe", "list /d /json");
		WdAppListOutput = WdAppListOutput.Substring(WdAppListOutput.IndexOf('{'));

		GDKInstalledApp InstalledApp = GDKInstalledApp.ParseAppList(JsonDocument.Parse(WdAppListOutput))
			.Where(App => App.GameConfig.PackageFamilyName == PackageFamilyName)
			.FirstOrDefault();

		return InstalledApp;
	}

	protected bool ExecuteSDKCommand(string Command, string CommandLine)
	{
		string ExecutablePath = Path.Combine(GDKBinariesDir, Command);

		int SuccessCode;
		CommandUtils.RunAndLog(ExecutablePath, CommandLine, out SuccessCode);

		return (SuccessCode == 0);
	}

	protected string ExecuteSDKCommandAndGetOutput(string Command, string CommandLine)
	{
		string ExecutablePath = Path.Combine(GDKBinariesDir, Command);

		return CommandUtils.Run(ExecutablePath, CommandLine, Options: CommandUtils.ERunOptions.NoLoggingOfRunCommand | CommandUtils.ERunOptions.AppMustExist).Output;
	}

	private static void VerifyWindowsEnvironment()
	{
		if (!MSGamingExports.IsWindowsDeveloperMode())
		{
			CommandUtils.Logger.LogWarning("***** WINDOWS IS NOT IN DEVELOPER MODE. This means that deploying and launching are likely to fail. ******");
			CommandUtils.Logger.LogWarning("\tPlease enable it via the Windows Developer settings menu or contact your IT administrator.");
			CommandUtils.Logger.LogWarning("\tSee https://docs.microsoft.com/en-us/windows/uwp/get-started/enable-your-device-for-development for details.");
			// could spawn ms-settings:developers to open the developer settings page?
		}

		if (!MSGamingExports.IsMSGamingRuntimeInstalled())
		{
			string GRDKEnvVar = GDKExports.IsLegacyFolderStructure() ? "%GameDKLatest%" : "%GameDKCoreLatest%";
			CommandUtils.Logger.LogWarning("****** MICROSOFT GAMING RUNTIME IS NOT INSTALLED. This means that deploying and launching are likely to fail. ******");
			CommandUtils.Logger.LogWarning("\tPlease install it via {gdk}\\redist\\GamingServices.appxbundle or contact your IT administrator.", GRDKEnvVar);
			CommandUtils.Logger.LogWarning("\tSee https://learn.microsoft.com/en-us/gaming/gdk/_content/gc/get-started-with-pc-dev/config-test-pc-software/gr-configure-test-pc for details.");
			// could spawn ms-windows-store://pdp/?productid=9MWPM2CQNLHN to open the store page for it?
		}

		if (!MSGamingExports.IsWindowsVersionSupported())
		{
			CommandUtils.Logger.LogWarning("Your Windows version may be too low - it should be at least Windows 10 May 2019 Update (version 1903)");
		}
	}

}
