// Copyright Epic Games, Inc. All Rights Reserved.
using System.Collections.Generic;
using AutomationTool;
using UnrealBuildTool;
using EpicGames.Core;
using UnrealBuildBase;
using System;
using System.Linq;
using System.IO;
using UnrealBuildTool.GDK;
using Microsoft.Extensions.Logging;

namespace AutomationUtils.GDK
{
	public class MSGameStorePackageGenerator : GDKPackageGenerator
	{
		UnrealTargetPlatform? CrashReportPlatform;
		public MSGameStorePackageGenerator(FileReference InProjectPath, UnrealTargetPlatform InPlatformType, string InIniSection_TargetSettings, string CustomConfig, UnrealTargetPlatform? InCrashReportPlatform)
			: base(InProjectPath, InPlatformType, InIniSection_TargetSettings, CustomConfig)
		{
			CrashReportPlatform = InCrashReportPlatform ?? UnrealTargetPlatform.Win64;
		}

		protected override string GetPlatformSpecificPackageCommandlineOptions(ProjectParams Params, DeploymentContext SC, string DLCPackage)
		{
			return "/pc";
		}

		protected override void GetPlatformSpecificPackageSymbolPaths(HashSet<string> SymbolPaths, ProjectParams Params, DeploymentContext SC)
		{
			if (SC.bStageCrashReporter)
			{
				DirectoryReference CrashReporterBinaryDir = DirectoryReference.Combine(Unreal.EngineDirectory, "Binaries", CrashReportPlatform.ToString());
				SymbolPaths.Add(CrashReporterBinaryDir.FullName);
			}
		}

		public override string GetPlatformPackageFileExtension()
		{
			return ".msixvc";
		}

		protected override bool ShouldPackageFile(FileReference SrcFile, ProjectParams Params, DeploymentContext SC, UnrealTargetConfiguration? OnlyConfig = null)
		{
			// Exclude bootstrap executables that are associated with other configurations
			if (!Params.NoBootstrapExe && !Params.HasDLCName && OnlyConfig != null && SrcFile.HasExtension(".exe") && SrcFile.Directory == SC.StageDirectory)
			{
				Dictionary<UnrealTargetConfiguration, List<string>> BootstrapExecutables = GDKAutomationUtils.GetPerConfigBootstrapExecutables(Params, SC);
				bool bExcluded = BootstrapExecutables.Any( X => 
					X.Key != OnlyConfig &&
					X.Value.Any( F => SrcFile.GetFileName().Equals(F, StringComparison.InvariantCultureIgnoreCase))
				);
				if (bExcluded)
				{
					return false;
				}
			}

			return base.ShouldPackageFile(SrcFile, Params, SC, OnlyConfig);
		}

		protected override IEnumerable<StagedFileReference> GetPlatformFilesToPackage(ProjectParams Params, DeploymentContext SC, DirectoryReference PackageMetadataDir, UnrealTargetConfiguration? OnlyConfig)
		{
			List<StagedFileReference> PlatformFiles = [];

			ConfigHierarchy EngineIni = ConfigCache.ReadHierarchy(ConfigHierarchyType.Engine, DirectoryReference.FromFile(SC.RawProjectPath), SC.StageTargetPlatform.PlatformType, SC.CustomConfig);

			// copy the custom install actions to the package metadata folder & include them in the package
			IEnumerable<MSGamingExports.CustomInstallAction> CustomInstallActions = MSGamingExports.CustomInstallAction.GetFromConfig(EngineIni, IniSection_TargetSettings);
			string TargetRoot = MSGamingExports.CustomInstallAction.GetTargetRootFolder(EngineIni, IniSection_TargetSettings);
			foreach (MSGamingExports.CustomInstallAction Action in CustomInstallActions)
			{
				if (DirectoryReference.Exists(Action.SourceFolder))
				{
					StagedDirectoryReference StagedDir = string.IsNullOrEmpty(Action.TargetFolder) 
						? StagedDirectoryReference.Combine(StagedDirectoryReference.Root, TargetRoot) 
						: StagedDirectoryReference.Combine(StagedDirectoryReference.Root, TargetRoot, Action.TargetFolder);

					IEnumerable<FileReference> InputFiles = DirectoryReference.EnumerateFiles(Action.SourceFolder, "*", SearchOption.AllDirectories);
					foreach (FileReference InputFile in InputFiles)
					{
						string RelativePath = InputFile.MakeRelativeTo(Action.SourceFolder);

						FileReference OutputFile = FileReference.Combine(PackageMetadataDir, TargetRoot, Action.Name, RelativePath);
						CommandUtils.CopyFile_NoExceptions(InputFile.FullName, OutputFile.FullName, bQuiet:true);

						StagedFileReference StagedFile = StagedFileReference.Combine(StagedDir, RelativePath);
						PlatformFiles.Add(StagedFile);
					}
				}
			}

			return PlatformFiles;
		}


		protected override void PackageInternal(ProjectParams Params, DeploymentContext SC, DirectoryReference PackagePath, FileReference LayoutPath, DirectoryReference SourceFolder, ILogger Logger, string DLCPackage, UnrealTargetConfiguration? OnlyConfig = null, string PackageOutputPathFragment = "")
		{
			base.PackageInternal(Params, SC, PackagePath, LayoutPath, SourceFolder, Logger, DLCPackage, OnlyConfig, PackageOutputPathFragment);
			GenerateInstallBatchFile(PackagePath, Params, SC, (DLCPackage != null) );
		}

		private void GenerateInstallBatchFile(DirectoryReference OutputPath, ProjectParams Params, DeploymentContext SC, bool bIsDLC)
		{
			ConfigHierarchy EngineIni = ConfigCache.ReadHierarchy(ConfigHierarchyType.Engine, DirectoryReference.FromFile(SC.RawProjectPath), SC.StageTargetPlatform.PlatformType, SC.CustomConfig);

			// see if the install batch file is wanted (it is enabled by default)
			bool bGenerateInstallBatchFile = true;
			if (EngineIni.GetBool(IniSection_TargetSettings, "bGenerateInstallBatchFile", out bGenerateInstallBatchFile) && !bGenerateInstallBatchFile)
			{
				return;
			}

			// find the generated package
			string PackageFileName = Directory.GetFiles(OutputPath.FullName, "*.msixvc", SearchOption.TopDirectoryOnly).FirstOrDefault();
			if (string.IsNullOrEmpty(PackageFileName))
			{
				throw new AutomationException(ExitCode.Error_MissingExecutable, "Could not find an msixvc package in {0}.", OutputPath.FullName);
			}

			// create the game config info for the package, preferring the staged manifest for the base game if we find it
			GDKGameConfigInfo GameConfig;
			FileReference ManifestFileName = FileReference.Combine(SC.StageDirectory, "MicrosoftGame.config");
			if (FileReference.Exists(ManifestFileName) && !bIsDLC)
			{
				GameConfig = new GDKGameConfigInfo(ManifestFileName);
			}
			else
			{
				GameConfig = new GDKGameConfigInfo(PackageFileName);
			}

			// read source templates
			string TemplateFolder = Path.Combine(SC.EngineRoot.FullName, "Plugins", "Runtime", "Windows", "MSGameStore", "Build", "BatchTemplates");
			string InstallBatchFile = File.ReadAllText(Path.Combine(TemplateFolder, "TemplateInstall._bat"));
			string UninstallBatchFile = File.ReadAllText(Path.Combine(TemplateFolder, "TemplateUninstall._bat"));
			string RunBatchFile = File.ReadAllText(Path.Combine(TemplateFolder, "TemplateRun._bat"));

			// apply substitutions
			string InstallBatchFileName = string.Format($"install_{Params.ShortProjectName}.bat");
			string UninstallBatchFileName = string.Format($"uninstall_{Params.ShortProjectName}.bat");
			string RunBatchFileName = string.Format($"run_{Params.ShortProjectName}.bat");

			Dictionary<string, string> Substitutions = new Dictionary<string, string>();
			Substitutions.Add("__PACKAGEFULLNAME__", GameConfig.PackageFullName);
			Substitutions.Add("__PACKAGEFAMILYNAME__", GameConfig.PackageFamilyName);
			Substitutions.Add("__PACKAGEIDENTITYNAME__", GameConfig.Identity);
			Substitutions.Add("__PACKAGEIDENTITYPUBLISHER__", GameConfig.PublisherId);
			Substitutions.Add("__PACKAGEFILENAME__", Path.GetFileName(PackageFileName));
			Substitutions.Add("__PROJECTNAME__", Params.ShortProjectName);
			Substitutions.Add("__INSTALLBATCHFILENAME__", InstallBatchFileName);
			Substitutions.Add("__UNINSTALLBATCHFILENAME__", UninstallBatchFileName);
			Substitutions.Add("__RUNBATCHFILENAME__", RunBatchFileName);
			foreach (KeyValuePair<string, string> Substitution in Substitutions)
			{
				InstallBatchFile = InstallBatchFile.Replace(Substitution.Key, Substitution.Value);
				UninstallBatchFile = UninstallBatchFile.Replace(Substitution.Key, Substitution.Value);
				RunBatchFile = RunBatchFile.Replace(Substitution.Key, Substitution.Value);
			}

			// save batch files
			File.WriteAllText(Path.Combine(OutputPath.FullName, InstallBatchFileName), InstallBatchFile);
			File.WriteAllText(Path.Combine(OutputPath.FullName, UninstallBatchFileName), UninstallBatchFile);
			if (!bIsDLC)
			{
				File.WriteAllText(Path.Combine(OutputPath.FullName, RunBatchFileName), RunBatchFile);
			}
		}


		#region Packaging error/warning filter

		private static readonly Dictionary<string, string> ModFolderWarningsToSuppress = new Dictionary<string, string>()
		{
			{ "Warning: Most PC gaming customers expect their applications to be modifiable.", "Most PC gaming customers expect their applications to be modifiable." },
		};

		private static readonly Dictionary<string, string> MissingPDBWarningsToSuppress = new Dictionary<string, string>()
		{
			{"(?<msg>Warning: Symbol file not found for.*WinPixEventRuntime.dll.*)", "" }, //suppress WinPixEventRuntime warning completely because it should never be included in Shipping & it just creates noise
		};

		protected override void PrepareMessageFilterForMakePkg(SpewFilterHelper Filter, ConfigHierarchy EngineIni, ProjectParams Params, DeploymentContext SC)
		{
			base.PrepareMessageFilterForMakePkg(Filter, EngineIni, Params, SC);

			bool bUseModFolder = false;
			if (!EngineIni.GetBool(IniSection_TargetSettings, "bUseModFolder", out bUseModFolder) || !bUseModFolder)
			{
				Filter.ReplaceMessages(ModFolderWarningsToSuppress);
			}

			// suppress missing PDB warnings for known system items if this isn't a Shipping package
			if (Params.ClientConfigsToBuild.Count > 1 || !Params.ClientConfigsToBuild.Contains(UnrealTargetConfiguration.Shipping))
			{
				Filter.RegExReplaceMessages(MissingPDBWarningsToSuppress);
			}
		}


		#endregion
	}
}
