// Copyright Epic Games, Inc. All Rights Reserved.
using System;
using AutomationTool;
using UnrealBuildTool;
using Microsoft.Extensions.Logging;

namespace BuildScripts.Automation
{
	public class MSGameStorePlatformCheck : BuildCommand
	{
		public override void ExecuteBuild()
		{
			bool bPass = true;
			bool bIsWindows = OperatingSystem.IsWindows();

			if (bIsWindows)
			{
				bool bCheckWindowsVersion = ParseParamBool("CheckWindowsVersion", true);
				bool bCheckDeveloperMode = ParseParamBool("CheckDeveloperMode", true);
				bool bCheckGamingRuning = ParseParamBool("CheckGamingRuntime", true);

				if (bCheckWindowsVersion && !MSGamingExports.IsWindowsVersionSupported())
				{
					Logger.LogError("PC GDK requires at least Windows 10 May 2019 Update (version 1903)");
					bPass = false;
				}

				if (bCheckDeveloperMode && !MSGamingExports.IsWindowsDeveloperMode())
				{
					Logger.LogError("Windows is not in developer mode.");
					bPass = false;
				}

				if (bCheckGamingRuning && !MSGamingExports.IsMSGamingRuntimeInstalled())
				{
					Logger.LogError("Microsoft.GamingServices is not installed.");
					bPass = false;
				}
			}
			else
			{
				Logger.LogError("Unsupported host OS - only Windows is supported");
				bPass = false;
			}

			if (bPass)
			{
				Logger.LogInformation("This machine appears to be ready for PC GDK development");
			}
		}
	}
}
