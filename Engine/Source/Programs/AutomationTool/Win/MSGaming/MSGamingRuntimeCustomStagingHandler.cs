// Copyright Epic Games, Inc. All Rights Reserved.
using AutomationTool;
using AutomationUtils.GDK;
using EpicGames.Core;
using System;
using System.IO;
using System.Linq;
using System.Runtime.Versioning;
using UnrealBuildTool;

[SupportedOSPlatform("windows")]
public class MSGamingRuntimeCustomStagingHandler : CustomStagingHandler
{
	protected virtual string IniSection_TargetSettings => "/Script/MSGamingSupport.MSGamingSettings";

	protected static int GDKEdition => GDKExports.GetGDKVersionNumber() ?? 0;

	protected static string GDKBinariesDir => Path.Combine(GDKExports.GetGSDKRoot(), "bin");

	private bool bAutoGenerateStagingManifest = true; 


	protected override bool TryInitialize(ProjectParams Params, DeploymentContext SC)
	{
		// make sure the GDK is installed
		if (GDKEdition == 0 || !Directory.Exists(GDKBinariesDir))
		{
			// GDK is not installed
			return false;
		}

		// we only support Windows
		if (!SC.StageTargetPlatform.PlatformType.IsInGroup(UnrealPlatformGroup.Windows))
		{
			return false;
		}

		// already using the legacy MSGameStore custom deployment handler which does a superset of our functionality, so we're not needed
		if (SC.CustomDeployment != null && SC.CustomDeployment is MSGameStoreCustomDeploymentHandler )
		{
			return false;
		}

		// see if our plugin is enabled
		if (!SC.StageTargets.Any( StageTarget => StageTarget.Receipt != null && StageTarget.Receipt.BuildPlugins.Contains("MSGamingRuntime")))
		{
			return false;
		}

		// cache configuration, with custom config
		ConfigHierarchy EngineIni = ConfigCache.ReadHierarchy(ConfigHierarchyType.Engine, DirectoryReference.FromFile(SC.RawProjectPath), SC.StageTargetPlatform.PlatformType, SC.CustomConfig);
		if (EngineIni.GetBool(IniSection_TargetSettings, "bAutoGenerateStagingManifest", out bool bShouldAutoGenerateStagingManifest))
		{
			bAutoGenerateStagingManifest = bShouldAutoGenerateStagingManifest;
		}

		bool bIsRequired = bAutoGenerateStagingManifest;
		if (!bIsRequired)
		{
			return false;
		}

		// sanity check (unlikely to get this far because GRDK will throw an exception during the build, but keeping it here in case logic changes elsewhere)
		if (GDKEdition < 251000 && Params.NoBootstrapExe && SC.StageTargets.Any(X => !X.Receipt.Architectures.SingleArchitecture.bIsX64))
		{
			throw new AutomationException("October 2025 GDK or higher is required to use the GDK with ARM64 unless you use the bootstrapper (omit -NoBootstrapExe)");
		}

		return true;
	}


	public override void GetFilesToStage(ProjectParams Params, DeploymentContext SC)
	{
		if (bAutoGenerateStagingManifest)
		{
			if (!Params.Prebuilt)
			{
				// generate the staging manifest. this is used when we are staging a cooked build
				GDKAutomationUtils.GenerateGameConfigForStaging(Params, SC, IniSection_TargetSettings, CommandUtils.Logger);
			}
		}
	}

	public override void GetFilesToStageForDLC(ProjectParams Params, DeploymentContext SC)
	{
		if (bAutoGenerateStagingManifest)
		{
			if (!Params.Prebuilt && !Params.NeverPackage)
			{
				// generate the staging manifest. this is used when we are staging a cooked build
				GDKAutomationUtils.GenerateGameConfigForStaging(Params, SC, IniSection_TargetSettings, CommandUtils.Logger);
			}
		}
	}
}
