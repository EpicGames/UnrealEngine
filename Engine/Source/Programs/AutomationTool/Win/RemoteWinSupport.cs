// Copyright Epic Games, Inc. All Rights Reserved.
using AutomationTool;
using EpicGames.Core;
using Microsoft.Extensions.Logging;
using System;
using System.Collections.Generic;
using System.Diagnostics;
using System.IO;
using System.Linq;
using System.Net;
using System.Net.Sockets;
using System.Threading;
using UnrealBuildTool;

using static AutomationTool.CommandUtils;


/* Remote Windows device support currently requires the Microsoft.Gaming.RemoteIterationClient WinGet package. 
* (The remote devices need the Microsoft.Gaming.RemoteIterationEndpoint WinGet package)
* 
* 0.0.10-preview is the minimum supported version of the Microsoft.Gaming.RemoteIteration packages, and both versions must match
* 
* It is recommended to use the Xbox PC Toolbox installer to configure host and remote devices : https://aka.ms/ToolboxInstaller
* See https://aka.ms/GameRemoteDevtools for more details
* 
* At present, the list of devices is read from the engine ini. These devices should be paired already.
* 
*  %LOCALAPPDATA%\Unreal Engine\Engine\Config\UserEngine.ini  can be used for convenience
*  
*  [RemoteWin]
*  +DeviceNames=myfirstpc
*  +DeviceNames=mysecondpc
*/

public static class RemoteWinSupport
{
	public static readonly string DeviceType = "RemoteWin";
	private static string WdRemotePath => Environment.ExpandEnvironmentVariables("%LocalAppData%\\Microsoft\\WinGet\\Links\\wdRemote.exe");

	public static bool IsAvailable()
	{
		return File.Exists(WdRemotePath);
	}

	public static bool IsRemoteDevice( string DeviceName, UnrealTargetPlatform Platform )
	{
		return GetAllRemoteDeviceNames(Platform).Contains(DeviceName, StringComparer.OrdinalIgnoreCase);
	}
	
	public static List<string> GetAllRemoteDeviceNames(UnrealTargetPlatform Platform)
	{
		List<string> Result = [];

		// read device names from the engine ini. must already be paired
		// best place is %LOCALAPPDATA%\Unreal Engine\Engine\Config\UserEngine.ini
		ConfigHierarchy EngineConfig = ConfigCache.ReadHierarchy(ConfigHierarchyType.Engine, null, Platform);
		if (EngineConfig.GetArray("RemoteWin", "DeviceNames", out List<string> DeviceNames))
		{
			Result.AddRange(DeviceNames);
		}

		return Result;
	}

	public static List<DeviceInfo> GetDevices(UnrealTargetPlatform Platform)
	{
		List<DeviceInfo> Devices = [];

		foreach (string DeviceName in GetAllRemoteDeviceNames(Platform))
		{
			string SoftwareVersion = Environment.OSVersion.Version.ToString(); // NB. this is the local version, not the remote
			Devices.Add(new DeviceInfo(Platform, DeviceName, DeviceName, SoftwareVersion, DeviceType, true, true));
		}

		return Devices;
	}

	public static bool Deploy( string DeviceName, string SourcePath, string DestinationPath )
	{
		bool bHasPCGDK = File.Exists( Path.Combine(SourcePath, "MicrosoftGame.config" ));
		if (bHasPCGDK)
		{
			return RunWdRemote($"/action:deploy /source:{StringUtils.QuoteArgument(SourcePath)} /destination:{StringUtils.QuoteArgument(DestinationPath)}", DeviceName);
		}
		else
		{
			// copy requires 0.0.10-preview or higher of the wdremote tools
			return RunWdRemote($"/action:copy /source:{StringUtils.QuoteArgument(SourcePath)} /destination:{StringUtils.QuoteArgument(DestinationPath)}", DeviceName);
		}
	}

	public static bool Run(string DeviceName, string RemoteExecutablePath, string ClientCmdLine)
	{
		// escape all quotes
		// @fixme: this is only necessary because we're using wdremote.exe who's /args needs a quoted string. 
		//         this will break if ClientCmdLine already contains escaped quotes!
		ClientCmdLine = ClientCmdLine.Replace("\"", "\\\"");

		RemoteExecutablePath = StringUtils.QuoteArgument(RemoteExecutablePath);
		RemoteExecutablePath = RemoteExecutablePath.Replace('/', '\\');

		return RunWdRemote($"/action:launch /path:{RemoteExecutablePath} /args:\"{ClientCmdLine}\"", DeviceName);
	}

	public static bool Kill(string DeviceName)
	{
		return RunWdRemote("/action:terminate", DeviceName);
	}

	public static bool Deploy(UnrealTargetPlatform Platform, ProjectParams Params, DeploymentContext SC)
	{
		if (Params.Devices.Count == 0)
		{
			return false;
		}
		
		foreach (string Device in Params.Devices)
		{
			if (Platform == GetTargetDevicePlatform(Device, Platform))
			{
				string DeviceName = GetTargetDeviceName(Device);
				string SourcePath = SC.StageDirectory.FullName;
				string DestinationPath = GetDestinationRoot(Platform, Params);

				if (!Deploy(DeviceName, SourcePath, DestinationPath))
				{
					throw new AutomationException("Remote deployment to {0} failed", Params.Devices[0]);
				}
			}
		}

		return true;
	}

	public static IProcessResult RunClient(UnrealTargetPlatform Platform, ERunOptions ClientRunFlags, string ClientApp, string ClientCmdLine, ProjectParams Params)
	{
		if (Params.Devices.Count == 0)
		{
			return null;
		}

		ClientCmdLine = SanitizeCommandLine(ClientCmdLine);

		RemoteWinProcessResult Result = null;

		foreach (string Device in Params.Devices)
		{
			if (Platform == GetTargetDevicePlatform(Device, Platform))
			{
				string DeviceName = GetTargetDeviceName(Device);
				string ExecutablePath = GetRemoteExecutablePath(Platform, ClientApp, Params);

				string ExtraCmdLine = "";
				Placeholder_RemoteWinProcessMonitor ProcessMonitor = new(DeviceName, Logger);
				if (ProcessMonitor.IsAvailable)
				{
					ExtraCmdLine += $" -RemoteConsoleHost={ProcessMonitor.ServerAddress}";
				}

				if (!Run(DeviceName, ExecutablePath, ClientCmdLine + ExtraCmdLine))
				{
					throw new AutomationException("Remote launch on {0} failed", Device);
				}

				Logger.LogInformation("Running Package@Device:{ProcessName}@{DeviceName}", Path.GetFileName(ClientApp), DeviceName);
				Result = new(DeviceName, Path.GetFileName(ClientApp), ProcessMonitor, ClientRunFlags);
			}
		}


		if (Result != null && !ClientRunFlags.HasFlag(ERunOptions.NoWaitForExit))
		{
			Result.WaitForExit();
		}

		return Result;
	}

	private static string GetRemoteExecutablePath(UnrealTargetPlatform Platform, string ClientApp, ProjectParams Params)
	{
		// find the remote executable path
		string RelativeExecutablePath = Path.GetFileName(ClientApp);
		string[] Paths = ClientApp.Split([Path.DirectorySeparatorChar, Path.AltDirectorySeparatorChar], StringSplitOptions.RemoveEmptyEntries);
		if (Paths.Length >= 4 &&
			Paths[^3].Equals("Binaries", StringComparison.OrdinalIgnoreCase) &&
			Paths[^2].Equals(Platform.ToString(), StringComparison.OrdinalIgnoreCase))
		{
			RelativeExecutablePath = Path.Combine(Paths[^4..]); // MyGame/Binaries/Win64/MyGame-Test.exe etc
		}
		string ExecutablePath = Path.Combine(GetDestinationRoot(Platform, Params), RelativeExecutablePath);
		return ExecutablePath;
	}


	private static string SanitizeCommandLine(string ClientCmdLine)
	{
		// simplistic helper function - may need fleshing out for more complicated edge cases as they're discovered
		void RemoveParam(string Key, bool bOnlyIfUsingAbsolutePath = false)
		{
			int Start = ClientCmdLine.IndexOf(Key);
			if (Start >= 0)
			{
				int End;

				bool bQuote = false;
				for ( End = Start+1; End < ClientCmdLine.Length; End++ )
				{
					if ( ClientCmdLine[End] == '"' && ClientCmdLine[End-1] != '\\')
					{
						bQuote = !bQuote;
					}
					else if (!bQuote && ClientCmdLine[End] == ' ')
					{
						break;
					}
				}

				if (End > Start)
				{
					string Arg = ClientCmdLine[Start..End];
					if (bOnlyIfUsingAbsolutePath && Key.Contains('='))
					{
						string Value = StringUtils.StripQuoteArgument(ClientCmdLine[(Start + Key.Length)..End]);
						if (Path.IsPathRooted(Value))
						{
							Logger.LogWarning("remote launch ignoring parameter {key} due to absolute path: {val} ", Key[..^1], Value);
						}
						else
						{
							return;
						}
					}

					ClientCmdLine = ClientCmdLine.Replace(Arg,"");
				}
			}
		}

		// trim off any known absolute paths
		RemoveParam("-project=", bOnlyIfUsingAbsolutePath:true);
		RemoveParam("-abslog=");

		return ClientCmdLine;
	}

	public static string GetTargetDeviceName(string DeviceName)
	{
		string[] Tokens = DeviceName.Split(['@', ' ', '(']); // 'platform@devicename (extra)'   or '@device (extra)' or just 'device'
		if (Tokens.Length > 1)
		{
			return Tokens[1];
		}

		return DeviceName;
	}

	private static UnrealTargetPlatform GetTargetDevicePlatform(string DeviceName, UnrealTargetPlatform DefaultPlatform)
	{
		string[] Tokens = DeviceName.Split(['@', ' ', '(']); // 'platform@devicename (extra)'   or '@device (extra)' or just 'device'
		if (Tokens.Length > 1)
		{
			if (Tokens[0] == "Windows")
			{
				return UnrealTargetPlatform.Win64;
			}
			else if (Tokens[0].Length == 0)
			{
				return DefaultPlatform;
			}
			else if (UnrealTargetPlatform.TryParse( Tokens[0], out UnrealTargetPlatform Platform))
			{
				return Platform;
			}
		}
		else if (Tokens.Length == 1)
		{
			return DefaultPlatform;
		}

		throw new AutomationException("cannot get target platform for remote deployment from {0}", DeviceName);
	}

	private static string GetDestinationRoot(UnrealTargetPlatform Platform, ProjectParams Params)
	{
		string TargetName;
		if (Params.HasClientCookedTargets)
		{
			TargetName = Params.ClientCookedTargets[0];
		}
		else if (Params.HasServerCookedTargets)
		{
			TargetName = Params.ServerCookedTargets[0];
		}
		else
		{
			TargetName = Params.ShortProjectName;
		}

		return (Platform == UnrealTargetPlatform.Win64) ? TargetName : $"{TargetName}_{Platform}";
	}


	internal static bool RunWdRemote( string Parameters, string Device )
	{
		if (!File.Exists(WdRemotePath))
		{
			throw new AutomationException("{0} not found", WdRemotePath);
		}

		Parameters += $" /device:{Device}";
		RunAndLog(WdRemotePath, Parameters, out int SuccessCode);

		if (SuccessCode != 0)
		{
			Logger.LogError("wdremote {params} failed - exit code 0x{code}", Parameters, SuccessCode.ToString("X"));
		}

		return (SuccessCode == 0);
	}


	/*
	 * 
	 * Helper class for monitoring the remote process's output and lifetime via a simple tcp server. 
	 * 
	 * Likely placeholder until the new wdremote tools support process lifetime management & output redirection
	 * Paired with FPlaceholder_WindowsRemoteConsoleOutputDevice in the engine runtime
	 * 
	 * Not enabled in shipping builds by default (see WITH_REMOTEWIN_CONSOLE)
	 * 
	 */
	public class Placeholder_RemoteWinProcessMonitor
	{
		public Action<string> OutputReceived;


		public Placeholder_RemoteWinProcessMonitor( string InDeviceName, ILogger InLogger = null )
		{
			DeviceName = InDeviceName;
			Logger = InLogger;

			try
			{
				Listener = new(IPAddress.Any, 0);
				Listener.Start(1);

				IEnumerable<string> ServerIPAddresses = Dns.GetHostEntry(Dns.GetHostName(), AddressFamily.InterNetwork).AddressList.Select( X => X.ToString() );
				int Port = ((IPEndPoint)Listener.LocalEndpoint).Port;
				ServerAddress = string.Join( '+', ServerIPAddresses ) + $":{Port}";

				ListenThread = new Thread( new ThreadStart(RunServerThread) );

				IsAvailable = true;
			}
			catch( Exception e )
			{
				Logger?.LogWarning("Could not set up windows remote TTY socket for {device} : {message}", DeviceName, e.Message);
			}
		}

		public void Start()
		{
			if (IsAvailable)
			{
				LaunchCTS = bNoTimeout ? new() : new(TimeSpan.FromSeconds(LaunchTimeoutSeconds));
				ListenThread.Start();
			}
		}
		public void Stop()
		{
			if (IsAvailable)
			{
				LaunchCTS.Cancel();
				Listener.Stop();
			}
		}

		public void WaitForExit()
		{
			while (IsAvailable)
			{
				Thread.Sleep(0);
			}
		}

		private async void RunServerThread()
		{
			if (!LaunchCTS.IsCancellationRequested)
			{
				try
				{
					TcpClient Client = await Listener.AcceptTcpClientAsync(LaunchCTS.Token);
					if (Client != null)
					{
						bHadRemoteConnection = true;
						StreamReader ClientStream = new(Client.GetStream());

						string Line;
						CancellationTokenSource ReadCTS = bNoTimeout ? new() : new(TimeSpan.FromSeconds(HeartbeatTimeutSeconds));

						while ((Line = await ClientStream.ReadLineAsync(ReadCTS.Token)) != null)
						{
							ReadCTS.TryReset();
							if (Line != "remotewin_heartbeat")
							{
								OutputReceived?.Invoke(Line);
							}
						}

						Logger?.LogTrace("Remote windows process finished on {device}", DeviceName);
					}
					else
					{
						Logger?.LogTrace("Remote windows process - did not get a connection from {device}", DeviceName);
					}
				}
				catch (OperationCanceledException)
				{
					Logger?.LogWarning("Remote windows process did not launch quickly enough on {device}", DeviceName);
				}
				catch (Exception e)
				{
					Logger?.LogTrace("Remote windows process likely terminated on {device} ({message})", DeviceName, e.Message);
				}
			}

			IsAvailable = false;
		}

		public bool DidLaunch => bHadRemoteConnection;
		public bool IsAvailable {get; private set; } = false;
		public string ServerAddress {get; private set; }

		private const int LaunchTimeoutSeconds = 20;
		private const int HeartbeatTimeutSeconds = 10;
		private const bool bNoTimeout = false; // set to true for debugging purposes only

		private readonly string DeviceName;
		private readonly ILogger Logger;
		private readonly TcpListener Listener;
		private readonly Thread ListenThread;
		CancellationTokenSource LaunchCTS;
		private bool bHadRemoteConnection = false;


	}





	private class RemoteWinProcessResult : IProcessResult
	{
		private readonly string DeviceName;
		private readonly string ProcessName;
		private readonly Placeholder_RemoteWinProcessMonitor ProcessMonitor;
		private readonly bool bLogToConsole;
		private readonly bool bLogToBuffer;

		public RemoteWinProcessResult(string InDeviceName, string InProcessName, Placeholder_RemoteWinProcessMonitor InProcessMonitor, ERunOptions RunOptions)
		{
			DeviceName = InDeviceName;
			ProcessName = InProcessName;
			ProcessMonitor = InProcessMonitor;
			bLogToConsole = RunOptions.HasFlag(ERunOptions.AllowSpew);
			bLogToBuffer = !RunOptions.HasFlag(ERunOptions.NoStdOutCapture);
			if (!bLogToBuffer)
			{
				Output = null;
			}

			ExitCode = -1;

			if (ProcessMonitor.IsAvailable)
			{
				ProcessMonitor.OutputReceived = OnOutputReceived;
				ProcessMonitor.Start();
			}
		}

		private void OnOutputReceived( string Line )
		{
			if (bLogToBuffer)
			{
				Output += Line;
			}
			if (bLogToConsole)
			{
				Console.WriteLine(Line);
			}
		}

		public void StopProcess(bool KillDescendants = true)
		{
			ProcessMonitor.Stop();
			ExitCode = 1;
			Kill(DeviceName);
		}

		public bool HasExited => !ProcessMonitor.IsAvailable;
		public string GetProcessName() => ProcessName;
		public void OnProcessExited() { }
		public void DisposeProcess() { }
		public void StdOut(object sender, DataReceivedEventArgs e) { }
		public void StdErr(object sender, DataReceivedEventArgs e) { }
		public int ExitCode {get; set; }
		public bool bExitCodeSuccess => ExitCode == 0;
		public string Output {get; private set; } = "";
		public Process ProcessObject => null;
		public new string ToString() => $"RemoteWin RemoteWinProcessResult: {GetProcessName()} (ExitCode: {ExitCode})";

		public void WaitForExit()
		{
			if (ProcessMonitor.IsAvailable)
			{
				ProcessMonitor.WaitForExit();
				ExitCode = 0;
			}
		}

		public FileReference WriteOutputToFile(string FileName)
		{
			using (StreamWriter writer = new(FileName))
			{
				writer.Write(Output);
			}

			return new FileReference(FileName);
		}
	}


}
