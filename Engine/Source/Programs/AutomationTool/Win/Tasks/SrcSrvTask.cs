// Copyright Epic Games, Inc. All Rights Reserved.

using System.Collections.Generic;
using System.Linq;
using AutomationTool;
using UnrealBuildTool;
using System.Xml;

using EpicGames.Core;
using UnrealBuildBase;
using EpicGames.BuildGraph;
using System.Runtime.Versioning;

namespace Win.Automation
{
	/// <summary>
	/// Parameters for a task that adds source indexing information to symbol files (PDB).
	/// </summary>
	public class SrcSrvTaskParameters
	{
		/// <summary>
		/// List of output files. PDBs will be extracted from this list.
		/// </summary>
		[TaskParameter]
		public string BinaryFiles;

		/// <summary>
		/// List of source files to index and embed into the PDBs.
		/// </summary>
		[TaskParameter]
		public string SourceFiles;

		/// <summary>
		/// Branch to base all the depot source files from
		/// </summary>
		[TaskParameter]
		public string Branch;

		/// <summary>
		/// Changelist to sync files from
		/// </summary>
		[TaskParameter]
		public int Change;

		/// <summary>
		/// List of file path mappings to be applied during source indexing. It is needed only when using VFS (Virtual File System)
		/// feature when compiling the binary. VFS replaces the real local paths used with standardized paths like Z:\UEVFS\...
		/// Since these virtualized paths don't match paths used by the local workspace, we need to map them manually so that
		/// paths in object files/symbols matched the data we have in source indexing.
		/// It's a semicolon-separated list of path assignments e.g.
		///      "[Root]\Samples\Games\SampleGame=Z:\UEVFS\SampleGame;[Root]=Z:\UEVFS\Root"
		/// 
		/// [Root] will be replaced with Unreal.RootDirectory.
		/// You need to ensure that the mapping used here matches the conventions used by Unreal Build Tool.
		/// We may improve it in the future but current the VFS mapping used during compilation is not available outside of UBT.
		/// </summary>
		[TaskParameter(Optional = true)]
		public string VFSMapping { get; set; }
	}

	/// <summary>
	/// Task which strips symbols from a set of files
	/// Note that this task only supports source indexing for Windows-like platforms.
	/// SymStore can be considered a generalization of this task because in addition to uploading symbols
	/// it can also index sources and supports consoles as well (any missing platform that supports
	/// source indexing in general may add support for it by extending PublishSymbols).
	/// Check SymStoreTaskParameters.IndexSources/SourceFiles/Branch/Change
	/// </summary>
	[TaskElement("SrcSrv", typeof(SrcSrvTaskParameters))]
	public class SrcSrvTask : CustomTask
	{
		/// <summary>
		/// Parameters for this task
		/// </summary>
		SrcSrvTaskParameters Parameters;

		/// <summary>
		/// Construct a spawn task
		/// </summary>
		/// <param name="InParameters">Parameters for the task</param>
		public SrcSrvTask(SrcSrvTaskParameters InParameters)
		{
			Parameters = InParameters;
		}

		/// <summary>
		/// Execute the task.
		/// </summary>
		/// <param name="Job">Information about the current job</param>
		/// <param name="BuildProducts">Set of build products produced by this node.</param>
		/// <param name="TagNameToFileSet">Mapping from tag names to the set of files they include</param>
		[SupportedOSPlatform("windows")]
		public override void Execute(JobContext Job, HashSet<FileReference> BuildProducts, Dictionary<string, HashSet<FileReference>> TagNameToFileSet)
		{
			FileReference[] BinaryFiles = ResolveFilespec(Unreal.RootDirectory, Parameters.BinaryFiles, TagNameToFileSet).ToArray();
			FileReference[] SourceFiles = ResolveFilespec(Unreal.RootDirectory, Parameters.SourceFiles, TagNameToFileSet).ToArray();
			Execute(BinaryFiles, SourceFiles, Parameters.Branch, Parameters.Change, Parameters.VFSMapping);
		}

		[SupportedOSPlatform("windows")]
		internal static void Execute(FileReference[] BinaryFiles, FileReference[] SourceFiles, string Branch, int Change, string VFSMapping)
		{
			IEnumerable<FileReference> PdbFiles = BinaryFiles.Where(x => x.HasExtension(".pdb"));

			Win64Platform WindowsPlatform = Platform.GetPlatform(UnrealTargetPlatform.Win64) as Win64Platform;

			WindowsPlatform.AddSourceIndexToSymbols(PdbFiles, SourceFiles, Branch, Change, VFSMapping);
		}

        /// <summary>
        /// Output this task out to an XML writer.
        /// </summary>
        public override void Write(XmlWriter Writer)
        {
            Write(Writer, Parameters);
        }

		/// <summary>
		/// Find all the tags which are used as inputs to this task
		/// </summary>
		/// <returns>The tag names which are read by this task</returns>
		public override IEnumerable<string> FindConsumedTagNames()
		{
			foreach(string TagName in FindTagNamesFromFilespec(Parameters.BinaryFiles))
			{
				yield return TagName;
			}
			foreach(string TagName in FindTagNamesFromFilespec(Parameters.SourceFiles))
			{
				yield return TagName;
			}
		}

		/// <summary>
		/// Find all the tags which are modified by this task
		/// </summary>
		/// <returns>The tag names which are modified by this task</returns>
		public override IEnumerable<string> FindProducedTagNames()
		{
			yield break;
		}
	}

	public static partial class BgStateExtensions
	{
		/// <summary>
		/// Add source indexing information to symbol files (PDB).
		/// </summary>
		/// <param name="State"></param>
		/// <param name="BinaryFiles">List of output files. PDBs will be extracted from this list.</param>
		/// <param name="SourceFiles">List of source files to index and embed into the PDBs.</param>
		/// <param name="Branch">Branch to base all the depot source files from.</param>
		/// <param name="Change">Changelist to sync files from.</param>
		/// <param name="VFSMapping">List of Virtual File System transformations to apply to paths during source code indexing. Pass an empty string if no transformations are needed
		/// e.g. when VFS was not used when compiling binaries. Check description of SrcSrvTaskParameters.VFSMapping for details about the expected syntax.</param>
		/// <returns></returns>
		[SupportedOSPlatform("windows")]
		public static void SrcSrv(this BgContext State, HashSet<FileReference> BinaryFiles, HashSet<FileReference> SourceFiles, string Branch, int Change, string VFSMapping)
		{
			SrcSrvTask.Execute(BinaryFiles.ToArray(), SourceFiles.ToArray(), Branch, Change, VFSMapping);
		}
	}
}
