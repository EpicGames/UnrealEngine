<Project Sdk="Microsoft.NET.Sdk">
  <Import Project="..\..\Shared\UnrealEngine.csproj.props" />
  
  <PropertyGroup>
    <OutputType>Library</OutputType>
    <GenerateAssemblyInfo>false</GenerateAssemblyInfo>
    <GenerateTargetFrameworkAttribute>false</GenerateTargetFrameworkAttribute>
    <RootNamespace>Win.Automation</RootNamespace>
    <AssemblyName>Win.Automation</AssemblyName>
    <WarningsNotAsErrors>612,618</WarningsNotAsErrors>
    <OutputPath Condition="'$(ArtifactsPath)' == ''">..\..\..\..\Binaries\DotNET\AutomationTool\AutomationScripts\Platforms\Windows</OutputPath>
    <TreatWarningsAsErrors>true</TreatWarningsAsErrors>
  </PropertyGroup>

  <ItemGroup>
    <Compile Include="..\..\Shared\MetaData.cs">
      <Link>Properties\MetaData.cs</Link>
    </Compile>
  </ItemGroup>
  <ItemGroup>
    <ProjectReference Include="..\..\Shared\EpicGames.Core\EpicGames.Core.csproj" PrivateAssets="All"><Private>false</Private></ProjectReference>
    <ProjectReference Include="..\..\UnrealBuildTool\UnrealBuildTool.csproj" PrivateAssets="All"><Private>false</Private></ProjectReference>
    <ProjectReference Include="..\AutomationUtils\AutomationUtils.Automation.csproj" PrivateAssets="All"><Private>false</Private></ProjectReference>
    <ProjectReference Include="..\BuildGraph\BuildGraph.Automation.csproj" PrivateAssets="All"><Private>false</Private></ProjectReference>
    <ProjectReference Include="..\SteamDeck\SteamDeck.Automation.csproj" />
    <ProjectReference Include="..\Microsoft\Microsoft.Automation.csproj" PrivateAssets="All"><Private>false</Private></ProjectReference>
  </ItemGroup>
</Project>