// Copyright Epic Games, Inc. All Rights Reserved.
using System;
using System.Collections.Generic;
using System.Linq;
using System.Text;
using System.IO;
using System.Text.RegularExpressions;
using AutomationTool;
using AutomationUtils;
using UnrealBuildTool;
using Microsoft.Win32;
using System.Diagnostics;
using EpicGames.Core;
using UnrealBuildBase;
using System.Runtime.Versioning;
using System.Threading.Tasks;
using System.Reflection.PortableExecutable;
using Microsoft.Extensions.Logging;

using static AutomationTool.CommandUtils;
using System.Xml.Linq;

public class Win64Platform : Platform
{
	public Win64Platform()
		: base(UnrealTargetPlatform.Win64)
	{
	}

	protected Win64Platform(UnrealTargetPlatform PlatformType)
		: base(PlatformType)
	{
	}

	public override DeviceInfo[] GetDevices()
	{
		List<DeviceInfo> Devices = new List<DeviceInfo>();

		if (HostPlatform.Current.HostEditorPlatform == UnrealTargetPlatform.Win64)
		{
			DeviceInfo LocalMachine = new DeviceInfo(UnrealTargetPlatform.Win64, Unreal.MachineName, Unreal.MachineName,
				Environment.OSVersion.Version.ToString(), "Computer", true, true);

			Devices.Add(LocalMachine);

			Devices.AddRange(SteamDeckSupport.GetDevices(UnrealTargetPlatform.Win64));
			Devices.AddRange(RemoteWinSupport.GetDevices(UnrealTargetPlatform.Win64));
		}

		return Devices.ToArray();
	}

	public override void Deploy(ProjectParams Params, DeploymentContext SC)
	{
		// We only care about deploying for SteamDeck or other remote devices
		if (Params.Devices.Count == 1 && GetDevices().FirstOrDefault(x => x.Id == Params.DeviceNames[0])?.Type == "SteamDeck")
		{
			SteamDeckSupport.Deploy(UnrealTargetPlatform.Win64, Params, SC);
		}
		else if (Params.Devices.Count > 0 && GetDevices().FirstOrDefault(x => Params.DeviceNames.Contains(x.Id, StringComparer.OrdinalIgnoreCase))?.Type == "RemoteWin")
		{
			RemoteWinSupport.Deploy(PlatformType, Params, SC);
		}
	}

	public override IProcessResult RunClient(ERunOptions ClientRunFlags, string ClientApp, string ClientCmdLine, ProjectParams Params)
	{
		if (Params.Devices.Count == 1 && GetDevices().FirstOrDefault(x => x.Id == Params.DeviceNames[0])?.Type == "SteamDeck")
		{
			return SteamDeckSupport.RunClient(UnrealTargetPlatform.Win64, ClientRunFlags, ClientApp, ClientCmdLine, Params);
		}
		else if (Params.Devices.Count > 0 && GetDevices().FirstOrDefault(x => Params.DeviceNames.Contains(x.Id, StringComparer.OrdinalIgnoreCase))?.Type == "RemoteWin")
		{
			return RemoteWinSupport.RunClient(PlatformType, ClientRunFlags, ClientApp, ClientCmdLine, Params);
		}

		return base.RunClient(ClientRunFlags, ClientApp, ClientCmdLine, Params);
	}

	protected override string GetPlatformExeExtension()
	{
		return ".exe";
	}

	public override bool IsSupported { get { return true; } }

	public virtual UnrealTargetPlatform? BootstrapExePlatform
	{
		get { return null; }
	}


	public override void GetFilesToDeployOrStage(ProjectParams Params, DeploymentContext SC)
	{
		// Engine non-ufs (binaries)

		if (SC.bStageCrashReporter)
		{
			FileReference ReceiptFileName = TargetReceipt.GetDefaultPath(Unreal.EngineDirectory, "CrashReportClient", CrashReportPlatform ?? SC.StageTargetPlatform.PlatformType, UnrealTargetConfiguration.Shipping, null);
			if(FileReference.Exists(ReceiptFileName))
			{
				TargetReceipt Receipt = TargetReceipt.Read(ReceiptFileName);
				SC.StageBuildProductsFromReceipt(Receipt, true, false);
			}
		}

		// Stage all the build products
		foreach(StageTarget Target in SC.StageTargets)
		{
			SC.StageBuildProductsFromReceipt(Target.Receipt, Target.RequireFilesExist, Params.bTreatNonShippingBinariesAsDebugFiles);
		}

		// Copy the splash screen, windows specific
		FileReference SplashImage = FileReference.Combine(SC.ProjectRoot, "Content", "Splash", "Splash.bmp");
		if(FileReference.Exists(SplashImage))
		{
			SC.StageFile(StagedFileType.NonUFS, SplashImage);
		}

		// Stage cloud metadata
		DirectoryReference ProjectCloudPath = DirectoryReference.Combine(SC.ProjectRoot, "Platforms/Windows/Build/Cloud");
		if (DirectoryReference.Exists(ProjectCloudPath))
		{
			SC.StageFiles(StagedFileType.SystemNonUFS, ProjectCloudPath, StageFilesSearch.AllDirectories, new StagedDirectoryReference("Cloud"));
		}
		else
		{
			Logger.LogDebug("Can't find cloud directory {Arg0}", ProjectCloudPath.FullName);
		}

		// Stage the bootstrap executable
		if (!Params.NoBootstrapExe)
		{
			// one bootstrap per configuration, per target
			foreach (UnrealTargetConfiguration Configuration in SC.StageTargetConfigurations)
			{
				foreach (string TargetName in SC.StageTargets.Select( T => T.Receipt.TargetName ).Distinct())
				{
					Dictionary<UnrealArch,StagedFileReference> RemappedStagedFiles = [];
					BuildProduct PrimaryExecutable = null;
					StageTarget? PrimaryTarget = null;

					// collected the staged executable files for each staged architecture
					foreach (StageTarget Target in SC.StageTargets.Where( T => T.Receipt.Configuration == Configuration && T.Receipt.TargetName == TargetName))
					{
						BuildProduct Executable = Target.Receipt.BuildProducts.FirstOrDefault(x => x.Type == BuildProductType.Executable);
						if (Executable != null)
						{
							// only create bootstraps for executables
							List<StagedFileReference> StagedFiles = SC.FilesToStage.NonUFSFiles.Where(x => x.Value == Executable.Path).Select(x => x.Key).ToList();
							if (StagedFiles.Count > 0 && Executable.Path.HasExtension(".exe"))
							{
								UnrealArch Architecture = Target.Receipt.Architectures.SingleArchitecture;
								RemappedStagedFiles[Architecture] = DeploymentContext.ApplyDirectoryRemap(SC, StagedFiles.First());

								if (PrimaryExecutable == null || Architecture.bIsX64) // prefer x64 for the BootstrapExeName because it is undecorated
								{
									PrimaryExecutable = Executable;
									PrimaryTarget = Target;
								}
							}
						}
					}
					
					if (RemappedStagedFiles.Any())
					{
						string BootstrapArguments = "";
						if (!ShouldStageCommandLine(Params, SC))
						{
							if (!SC.IsCodeBasedProject)
							{
								BootstrapArguments = String.Format("..\\..\\..\\{0}\\{0}.uproject", SC.ShortProjectName);
							}
							else
							{
								BootstrapArguments = SC.ShortProjectName;
							}
						}

						string BootstrapExeName;
						if(SC.StageTargetConfigurations.Count > 1)
						{
							BootstrapExeName = PrimaryExecutable.Path.GetFileName();
						}
						else if(Params.IsCodeBasedProject)
						{
							BootstrapExeName = PrimaryTarget.Value.Receipt.TargetName + ".exe";
						}
						else
						{
							BootstrapExeName = SC.ShortProjectName + ".exe";
						}

						if (RemappedStagedFiles.Count == 1)
						{
							StageBootstrapExecutable(SC, BootstrapExeName, PrimaryExecutable.Path, RemappedStagedFiles.Values.First(), BootstrapArguments);
						}
						else
						{
							StageBootstrapExecutable(SC, BootstrapExeName, PrimaryExecutable.Path, null, BootstrapArguments, RemappedStagedFiles);
						}
					}
				}
			}
		}

		if (Params.Prereqs)
		{
			//SC.StageFile(StagedFileType.NonUFS, FileReference.Combine(SC.EngineRoot, "Extras", "Redist", "en-us", "UEPrereqSetup_x64.exe"));
			SC.StageFile(StagedFileType.NonUFS, FileReference.Combine(SC.EngineRoot, "Extras", "Redist", "en-us", "vc_redist.x64.exe"));
			SC.StageFile(StagedFileType.NonUFS, FileReference.Combine(SC.EngineRoot, "Extras", "Redist", "en-us", "vc_redist.arm64.exe")); // always staging arm64 redist in case we are run on an arm64 device in emulation mode

			StageGameInputWindowsRedist(SC);
		}

		if (!string.IsNullOrWhiteSpace(Params.AppLocalDirectory))
		{
			foreach (UnrealArch Architecture in SC.StageTargets.Select( T => T.Receipt.Architectures.SingleArchitecture ).Distinct() )
			{
				StageAppLocalDependencies(Params, SC, "Win64", Architecture);
			}
		}
	}

	/**
	 * Returns true if the project has set the config setting to include the GameInputRedist with its packaging
	 */
	protected bool ShouldStageGameInputWindowsRedist(DeploymentContext SC)
	{
		// Only include the Game Input redist file if the project has opted into via the Engine config setting for their project
		ConfigHierarchy EngineConfig = ConfigCache.ReadHierarchy(ConfigHierarchyType.Engine, DirectoryReference.FromFile(SC.RawProjectPath), UnrealTargetPlatform.Win64);
		
		//		; In YourGameProject/Config/DefaultEngine.ini...
		//		[GameInput]
		//		IncludeRedistFiles=True
		
		EngineConfig.GetBool("GameInput", "IncludeRedistFiles", out bool bShouldIncludeRedistFiles);

		return bShouldIncludeRedistFiles;
	}
	
	protected void StageGameInputWindowsRedist(DeploymentContext SC)
	{
		if (ShouldStageGameInputWindowsRedist(SC))
		{
			Console.WriteLine("Including Game Input prereq installer!");
			
			// Engine/Extras/Redist/en-us/GameInputRedist.msi
			SC.StageFile(StagedFileType.NonUFS, FileReference.Combine(SC.EngineRoot, "Extras", "Redist", "en-us", "GameInputRedist.msi"));	
		}
	}

	public override void ExtractPackage(ProjectParams Params, string SourcePath, string DestinationPath)
    {
    }

	public override void GetTargetFile(string RemoteFilePath, string LocalFile, ProjectParams Params)
	{
		var SourceFile = FileReference.Combine(new DirectoryReference(Params.BaseStageDirectory), GetCookPlatform(Params.HasServerCookedTargets, Params.HasClientTargetDetected), RemoteFilePath);
		CommandUtils.CopyFile(SourceFile.FullName, LocalFile);
	}

	void StageBootstrapExecutable(DeploymentContext SC, string ExeName, FileReference TargetFile, StagedFileReference StagedRelativeTargetPath, string StagedArguments, Dictionary<UnrealArch,StagedFileReference> PerArchitectureStagedRelativeTargetPath = null)
	{
		UnrealTargetPlatform BootstrapPlatform = (BootstrapExePlatform ?? SC.StageTargetPlatform.PlatformType);
		FileReference InputFile = FileReference.Combine(SC.LocalRoot, "Engine", "Binaries", BootstrapPlatform.ToString(), String.Format("BootstrapPackagedGame-{0}-Shipping.exe", BootstrapPlatform));
		if(FileReference.Exists(InputFile))
		{
			// Create the new bootstrap program
			DirectoryReference IntermediateDir = DirectoryReference.Combine(SC.ProjectRoot, "Intermediate", "Staging");
			DirectoryReference.CreateDirectory(IntermediateDir);

			FileReference IntermediateFile = FileReference.Combine(IntermediateDir, ExeName);
			CommandUtils.CopyFile(InputFile.FullName, IntermediateFile.FullName);
			CommandUtils.SetFileAttributes(IntermediateFile.FullName, ReadOnly: false);
	
			if (UnrealBuildTool.BuildHostPlatform.Current.Platform == UnrealTargetPlatform.Win64)
			{
				Logger.LogInformation("Patching bootstrap executable; {Arg0}", IntermediateFile.FullName);

				// Get the icon from the build directory if possible
				GroupIconResource GroupIcon = null;
				if(FileReference.Exists(FileReference.Combine(SC.ProjectRoot, "Build/Windows/Application.ico")))
				{
					GroupIcon = GroupIconResource.FromIco(FileReference.Combine(SC.ProjectRoot, "Build/Windows/Application.ico").FullName);
				}
				if(GroupIcon == null)
				{
					GroupIcon = GroupIconResource.FromExe(TargetFile.FullName);
				}

				// Update the resources in the new file
				using(ModuleResourceUpdate Update = new ModuleResourceUpdate(IntermediateFile.FullName, false))
				{
					const int IconResourceId = 101;
					if(GroupIcon != null) Update.SetIcons(IconResourceId, GroupIcon);

					if (PerArchitectureStagedRelativeTargetPath == null)
					{
						const int ExecFileResourceId = 201;
						Update.SetData(ExecFileResourceId, ResourceType.RawData, Encoding.Unicode.GetBytes(StagedRelativeTargetPath.ToString().Replace('/', '\\') + "\0"));
					}
					else
					{
						foreach (KeyValuePair<UnrealArch,StagedFileReference> ExtraTargetPath in PerArchitectureStagedRelativeTargetPath)
						{
							if (ExtraTargetPath.Key == UnrealArch.X64)
							{
								const int ExecFileResourceId_X64 = 201;
								Update.SetData(ExecFileResourceId_X64, ResourceType.RawData, Encoding.Unicode.GetBytes(ExtraTargetPath.Value.ToString().Replace('/', '\\') + "\0"));
							}
							else if (ExtraTargetPath.Key == UnrealArch.Arm64)
							{
								const int ExecFileResourceId_ARM64 = 203;
								Update.SetData(ExecFileResourceId_ARM64, ResourceType.RawData, Encoding.Unicode.GetBytes(ExtraTargetPath.Value.ToString().Replace('/', '\\') + "\0"));

							}
							else if (ExtraTargetPath.Key == UnrealArch.Arm64ec)
							{
								const int ExecFileResourceId_ARM64EC = 204;
								Update.SetData(ExecFileResourceId_ARM64EC, ResourceType.RawData, Encoding.Unicode.GetBytes(ExtraTargetPath.Value.ToString().Replace('/', '\\') + "\0"));
							}
							else
							{
								throw new AutomationException("Unexpected architecture {arch} for bootstrap executable", ExtraTargetPath.Key);
							}
						}
					}
					
					const int ExecArgsResourceId = 202;
					Update.SetData(ExecArgsResourceId, ResourceType.RawData, Encoding.Unicode.GetBytes(StagedArguments + "\0"));
					const int ExecFileOverrideResourceId = 205;
					Update.SetData(ExecFileOverrideResourceId, ResourceType.RawData, Encoding.Unicode.GetBytes("\0"));

					// Set a flag on the exec file so that the BootstrapPackagedGame code can check if we should run the GameInputRedistInstaller
					if (ShouldStageGameInputWindowsRedist(SC))
					{
						const int ExecFileIncludeGameInputRedist = 206;
						Update.SetData(ExecFileIncludeGameInputRedist, ResourceType.RawData, Encoding.Unicode.GetBytes("\0"));	
						Logger.LogInformation("Including the GameInputRedist.msi file with the packaged project");
					}
				}
			}
			else
			{
				Logger.LogInformation("Skipping patching of bootstrap executable (unsupported host platform)");
			}

			// Copy it to the staging directory
			SC.StageFile(StagedFileType.SystemNonUFS, IntermediateFile, new StagedFileReference(ExeName));
		}
	}

	public override string GetCookPlatform(bool bDedicatedServer, bool bIsClientOnly)
	{
		const string NoEditorCookPlatform = "Windows";
		const string ServerCookPlatform = "WindowsServer";
		const string ClientCookPlatform = "WindowsClient";

		if (bDedicatedServer)
		{
			return ServerCookPlatform;
		}
		else if (bIsClientOnly)
		{
			return ClientCookPlatform;
		}
		else
		{
			return NoEditorCookPlatform;
		}
	}

	public override string GetEditorCookPlatform()
	{
		return "WindowsEditor";
	}
	
	public override string GetPlatformPakCommandLine(ProjectParams Params, DeploymentContext SC)
	{
		string PakParams = " -patchpaddingalign=2048";
		if (!SC.DedicatedServer)
		{
			string OodleDllPath = DirectoryReference.Combine(SC.ProjectRoot, "Binaries/ThirdParty/Oodle/Win64/UnrealPakPlugin.dll").FullName;
			if (File.Exists(OodleDllPath))
			{
				PakParams += String.Format(" -customcompressor=\"{0}\"", OodleDllPath);
			}
		}
		return PakParams;
	}

	public override void Package(ProjectParams Params, DeploymentContext SC, int WorkingCL)
	{
		// If this is a content-only project and there's a custom icon, update the executable
		if (!Params.HasDLCName && !Params.IsCodeBasedProject)
		{
			FileReference IconFile = FileReference.Combine(Params.RawProjectPath.Directory, "Build", "Windows", "Application.ico");
			if(FileReference.Exists(IconFile))
			{
				Logger.LogInformation("Updating executable with custom icon from {IconFile}", IconFile);

				GroupIconResource GroupIcon = GroupIconResource.FromIco(IconFile.FullName);

				List<FileReference> ExecutablePaths = GetExecutableNames(SC);
				foreach (FileReference ExecutablePath in ExecutablePaths)
				{
					using (ModuleResourceUpdate Update = new ModuleResourceUpdate(ExecutablePath.FullName, false))
					{
						const int IconResourceId = 123; // As defined in Engine\Source\Runtime\Launch\Resources\Windows\resource.h
						if (GroupIcon != null)
						{
							Update.SetIcons(IconResourceId, GroupIcon);
						}
					}
				}
			}
		}

		PrintRunTime();
	}

	public override bool UseAbsLog
	{
		get { return BuildHostPlatform.Current.Platform == UnrealTargetPlatform.Win64; }
	}

	public override bool CanHostPlatform(UnrealTargetPlatform Platform)
	{
		if (Platform == UnrealTargetPlatform.Mac)
		{
			return false;
		}
		return true;
	}

    public override bool ShouldStageCommandLine(ProjectParams Params, DeploymentContext SC)
	{
		return false; // !String.IsNullOrEmpty(Params.StageCommandline) || !String.IsNullOrEmpty(Params.RunCommandline) || (!Params.IsCodeBasedProject && Params.NoBootstrapExe);
	}

	public override List<string> GetDebugFileExtensions()
	{
		return new List<string> { ".pdb", ".map" };
	}

	public override CustomDeploymentHandler GetCustomDeploymentHandler(ProjectParams Params, DeploymentContext SC)
	{
		ConfigHierarchy EngineIni = ConfigCache.ReadHierarchy(ConfigHierarchyType.Engine, Params.RawProjectPath.Directory, PlatformType, SC.CustomConfig);
		EngineIni.GetString("/Script/WindowsTargetPlatform.WindowsTargetSettings", "CustomDeployment", out string CustomDeploymentName);

		if (string.IsNullOrEmpty(CustomDeploymentName))
		{
			CustomDeploymentName = Params.CustomDeploymentHandler;
		}

		if (!string.IsNullOrEmpty(CustomDeploymentName))
		{
			return CustomDeploymentHandler.Create(CustomDeploymentName, this, Params, SC);
		}

		return base.GetCustomDeploymentHandler(Params, SC);
	}

	public override bool SignExecutables(DeploymentContext SC, ProjectParams Params)
	{
		// Sign everything we built
		List<FileReference> FilesToSign = GetExecutableNames(SC);
		CodeSign.SignMultipleFilesIfEXEOrDLL(FilesToSign);

		return true;
	}

	public void StageAppLocalDependencies(ProjectParams Params, DeploymentContext SC, string SourcePlatformDir, UnrealArch Architecture)
	{
		Dictionary<string, string> PathVariables = new Dictionary<string, string>();
		PathVariables["EngineDir"] = SC.EngineRoot.FullName;
		PathVariables["ProjectDir"] = SC.ProjectRoot.FullName;

		// support multiple comma-separated paths
		string[] AppLocalDirectories = Params.AppLocalDirectory.Split(';');
		foreach (string AppLocalDirectory in AppLocalDirectories)
		{
			string ExpandedAppLocalDir = Utils.ExpandVariables(AppLocalDirectory, PathVariables);
			string ArchitectureDir = Architecture.WindowsToolChain;

			DirectoryReference BaseAppLocalDependenciesPath = Path.IsPathRooted(ExpandedAppLocalDir) ? new DirectoryReference(CombinePaths(ExpandedAppLocalDir, SourcePlatformDir, ArchitectureDir)) : DirectoryReference.Combine(SC.ProjectRoot, ExpandedAppLocalDir, SourcePlatformDir, ArchitectureDir);
			if (!DirectoryReference.Exists(BaseAppLocalDependenciesPath))
			{
				BaseAppLocalDependenciesPath = BaseAppLocalDependenciesPath.ParentDirectory!; // no architecture subfolder - must be user-supplied dependency folder in need of upgrade
				if (DirectoryReference.Exists(BaseAppLocalDependenciesPath))
				{
					Logger.LogWarning("{path} should have {arch} subfolder with the redist DLLs inside. Falling back to use {path} directly", BaseAppLocalDependenciesPath, Architecture, BaseAppLocalDependenciesPath);
				}
			}
			if (DirectoryReference.Exists(BaseAppLocalDependenciesPath))
			{
				StageAppLocalDependenciesToDir(SC, BaseAppLocalDependenciesPath, Architecture, StagedDirectoryReference.Combine("Engine", "Binaries", PlatformType.ToString()));
				StageAppLocalDependenciesToDir(SC, BaseAppLocalDependenciesPath, Architecture, StagedDirectoryReference.Combine(SC.RelativeProjectRootForStage, "Binaries", PlatformType.ToString()));
			}
			else
			{
				Logger.LogWarning("Unable to deploy AppLocalDirectory dependencies. No such path: {BaseAppLocalDependenciesPath}", BaseAppLocalDependenciesPath);
			}
		}
	}

	static void StageAppLocalDependenciesToDir(DeploymentContext SC, DirectoryReference BaseAppLocalDependenciesPath, UnrealArch Architecture, StagedDirectoryReference StagedBinariesDir)
	{
		// x64 DLL dependencies are placed directly in StagedBinariesDir
		bool bUseArchitectureSubdirectory = !Architecture.bIsX64;

		string AssemblyName = Architecture.ToString().ToLower();
		StagedDirectoryReference StagedArchitectureBinariesDir = bUseArchitectureSubdirectory ? StagedDirectoryReference.Combine(StagedBinariesDir, AssemblyName) : StagedBinariesDir;

		// Check if there are any executables being staged in this directory. Usually we only need to stage runtime dependencies next to the executable, but we may be staging
		// other engine executables too (eg. CEF)
		List<StagedFileReference> FilesInTargetDir = SC.FilesToStage.NonUFSFiles.Keys.Where(x => x.IsUnderDirectory(StagedBinariesDir) && (x.HasExtension(".exe") || x.HasExtension(".dll"))).ToList();
		if(FilesInTargetDir.Count > 0)
		{
			Logger.LogInformation("Copying AppLocal dependencies from {BaseAppLocalDependenciesPath} to {StagedArchitectureBinariesDir}", BaseAppLocalDependenciesPath, StagedArchitectureBinariesDir);

			// Stage files in subdirs
			foreach (DirectoryReference DependencyDirectory in DirectoryReference.EnumerateDirectories(BaseAppLocalDependenciesPath))
			{	
				SC.StageFiles(StagedFileType.NonUFS, DependencyDirectory, StageFilesSearch.AllDirectories, StagedArchitectureBinariesDir);
			}
		}
	}


	public void UpdateStagedSxSManifest( DeploymentContext SC, UnrealArch Architecture, StagedDirectoryReference StagedBinariesDir)
	{
		// x64 doesn't need SxS manifest - DLL dependencies are placed directly in StagedBinariesDir
		if (Architecture.bIsX64)
		{
			return;
		}

		// try to update the staged SxS manifest
		string AssemblyName = Architecture.ToString().ToLower();
		DirectoryReference ArchitectureDir = DirectoryReference.Combine( SC.StageDirectory, StagedBinariesDir.Name, AssemblyName );
		FileReference ManifestFile = FileReference.Combine(ArchitectureDir, AssemblyName + ".manifest");
		if (FileReference.Exists(ManifestFile))
		{
			try
			{
				XDocument XmlManifest = XDocument.Load(ManifestFile.FullName);
				if (XmlManifest.Root != null && XmlManifest.Root.Name.LocalName == "assembly")
				{
					XNamespace XmlNs = XmlManifest.Root.Name.Namespace;

					// remove all existing file elements & add the new ones (this ensures that stale items from the Binaries folder are not included)
					XmlManifest.Root.Elements(XmlNs+"file").ToList().ForEach( X => X.Remove() );
					foreach (FileReference DLLFile in DirectoryReference.EnumerateFiles(ArchitectureDir, "*.dll"))
					{
						string DLLName = DLLFile.MakeRelativeTo(ArchitectureDir);
						XmlManifest.Root.Add( new XElement(XmlNs+"file", new XAttribute("name", DLLName ) ) );
					}

					XmlManifest.Save(ManifestFile.FullName);
				}
			}
			catch(Exception e)
			{
				Logger.LogWarning("Could not update SxS manifest {name} : {msg}", ManifestFile.GetFileName(), e.Message);
			}
		}
	}


	public override void PostStagingFileCopy(ProjectParams Params, DeploymentContext SC)
	{
		base.PostStagingFileCopy(Params, SC);

		// refresh the architecture SxS manifests if they are in use
		foreach (UnrealArch Architecture in SC.StageTargets.Select( Target => Target.Receipt.Architectures.SingleArchitecture ).Distinct())
		{
			UpdateStagedSxSManifest(SC, Architecture, StagedDirectoryReference.Combine("Engine", "Binaries", PlatformType.ToString()));
			UpdateStagedSxSManifest(SC, Architecture, StagedDirectoryReference.Combine(SC.RelativeProjectRootForStage, "Binaries", PlatformType.ToString()));
		}
	}


    /// <summary>
    /// Try to get the symstore.exe path from the given Windows SDK version
    /// </summary>
    /// <returns>Path to symstore.exe</returns>
	[SupportedOSPlatform("windows")]
    private static FileReference GetSymStoreExe()
    {
		// Trying first to look for auto sdk latest WindowsKits debugger tools
		DirectoryReference HostAutoSdkDir = null;
		if (UEBuildPlatformSDK.TryGetHostPlatformAutoSDKDir(out HostAutoSdkDir))
		{
			DirectoryReference WindowsKitsDebuggersDirAutoSdk = DirectoryReference.Combine(HostAutoSdkDir, "Win64", "Windows Kits", "Debuggers");

			if (DirectoryReference.Exists(WindowsKitsDebuggersDirAutoSdk))
			{

				FileReference SymStoreExe64 = FileReference.Combine(WindowsKitsDebuggersDirAutoSdk, "x64", "symstore.exe");
				if (FileReference.Exists(SymStoreExe64))
				{
					return SymStoreExe64;
				}
			}
		}

		List<KeyValuePair<string, DirectoryReference>> WindowsSdkDirs = WindowsExports.GetWindowsSdkDirs();
		foreach (DirectoryReference WindowsSdkDir in WindowsSdkDirs.Select(x => x.Value))
		{
			FileReference SymStoreExe64 = FileReference.Combine(WindowsSdkDir, "Debuggers", "x64", "symstore.exe");
			if (FileReference.Exists(SymStoreExe64))
			{
				return SymStoreExe64;
			}
		}
		throw new AutomationException("Unable to find a Windows SDK installation containing Debuggers/x64/symstore.exe");
    }

	[SupportedOSPlatform("windows")]
	public static bool TryGetPdbCopyLocation(out FileReference OutLocation)
	{
		// Trying first to look for auto sdk latest WindowsKits debugger tools
		DirectoryReference HostAutoSdkDir = null;
		if (UEBuildPlatformSDK.TryGetHostPlatformAutoSDKDir(out HostAutoSdkDir))
		{
			DirectoryReference WindowsKitsDebuggersDirAutoSdk = DirectoryReference.Combine(HostAutoSdkDir, "Win64", "Windows Kits", "Debuggers");

			if (DirectoryReference.Exists(WindowsKitsDebuggersDirAutoSdk))
			{
				FileReference PdbCopyExe64 = FileReference.Combine(WindowsKitsDebuggersDirAutoSdk, "x64", "pdbcopy.exe");
				if (FileReference.Exists(PdbCopyExe64))
				{
					OutLocation = PdbCopyExe64;
					return true;
				}
			}
		}

		// Try to find an installation of the Windows 10 SDK
		List<KeyValuePair<string, DirectoryReference>> WindowsSdkDirs = WindowsExports.GetWindowsSdkDirs();
		foreach (DirectoryReference WindowsSdkDir in WindowsSdkDirs.Select(x => x.Value))
		{
			FileReference PdbCopyExe64 = FileReference.Combine(WindowsSdkDir, "Debuggers", "x64", "pdbcopy.exe");
			if (FileReference.Exists(PdbCopyExe64))
			{
				OutLocation = PdbCopyExe64;
				return true;
			}
		}

		// Otherwise fail
		OutLocation = null;
		return false;
	}

	[SupportedOSPlatform("windows")]
	public override void StripSymbols(FileReference SourceFile, FileReference TargetFile)
	{
		bool bStripInPlace = false;

		if (SourceFile == TargetFile)
		{
			// PDBCopy only supports creation of a brand new stripped file so we have to create a temporary filename
			TargetFile = new FileReference(Path.Combine(TargetFile.Directory.FullName, Guid.NewGuid().ToString() + TargetFile.GetExtension()));
			bStripInPlace = true;
		}

		FileReference PdbCopyLocation;
		if(!TryGetPdbCopyLocation(out PdbCopyLocation))
		{
			throw new AutomationException("Unable to find installation of pdbcopy.exe, which is required to strip symbols. This tool is included as part of the 'Windows Debugging Tools' component of the Windows 10 SDK (https://developer.microsoft.com/en-us/windows/downloads/windows-10-sdk).");
		}

		ProcessStartInfo StartInfo = new ProcessStartInfo();
		StartInfo.FileName = PdbCopyLocation.FullName;
		StartInfo.Arguments = String.Format("\"{0}\" \"{1}\" -p", SourceFile.FullName, TargetFile.FullName);
		StartInfo.UseShellExecute = false;
		StartInfo.CreateNoWindow = true;
		Utils.RunLocalProcessAndLogOutput(StartInfo, Log.Logger);

		if (bStripInPlace)
		{
			// Copy stripped file to original location and delete the temporary file
			File.Copy(TargetFile.FullName, SourceFile.FullName, true);
			FileReference.Delete(TargetFile);
		}
	}

	[SupportedOSPlatform("windows")]
	public override bool PublishSymbols(DirectoryReference SymbolStoreDirectory, List<FileReference> Files,
			bool bIndexSources, List<FileReference> SourceFiles,
			string Product, string Branch, int Change, string BuildVersion = null, string VFSMapping = null)
    {
		Logger.LogInformation("Publishing symbols to \"{SymbolStoreDirectory}\" (source indexing: {bIndexSources})", SymbolStoreDirectory, bIndexSources);

		// Get the symstore.exe path, using the latest SDK version we can find.
		FileReference SymStoreExe = GetSymStoreExe();
		Logger.LogInformation("Using '{Path}' Version {Version}", SymStoreExe, FileVersionInfo.GetVersionInfo(SymStoreExe.FullName).FileVersion);

		List<FileReference> FilesToAdd = Files.Where(x => x.HasExtension(".pdb") || x.HasExtension(".exe") || x.HasExtension(".dll")).ToList();
		if(FilesToAdd.Count > 0)
		{
			DateTime Start = DateTime.Now;
			DirectoryReference TempSymStoreDir = DirectoryReference.Combine(Unreal.RootDirectory, "Saved", "SymStore");

			if (DirectoryReference.Exists(TempSymStoreDir))
			{
				CommandUtils.DeleteDirectory(TempSymStoreDir);
				DirectoryReference.CreateDirectory(TempSymStoreDir);
			}

			DirectoryReference TempSymStoreIndexedDir = DirectoryReference.Combine(Unreal.RootDirectory, "Engine", "Intermediate", "SymStoreIndexed");

			string TempFileName = Path.GetTempFileName();
			try
			{
				IEnumerable<FileReference> SymbolsToIndex = Enumerable.Empty<FileReference>();
				IEnumerable<FileReference> SymbolsAfterIndexing = Enumerable.Empty<FileReference>();

				if (bIndexSources)
				{
					// Skip read-only PDBs as we won't be able to add data to them. They are likely to be symbols for third-party libraries checked into the source control.
					SymbolsToIndex = FilesToAdd.Where(x => x.HasExtension(".pdb") && !(new FileInfo(x.FullName).IsReadOnly));

					// We can't write to the original symbol files because they may have been generated by a different build step,
					// and clobbering such build products is considered an error.
					// For this reason, we copy the symbol files and modify the copied.
					SymbolsAfterIndexing = CopySymbolsWithSourceIndexing(TempSymStoreIndexedDir, SymbolsToIndex, SourceFiles, Branch, Change, VFSMapping);
				}

				File.WriteAllLines(TempFileName, FilesToAdd.Except(SymbolsToIndex).Union(SymbolsAfterIndexing).Select(x => x.FullName), Encoding.ASCII);

				// Copy everything to the temp symstore
				ProcessStartInfo StartInfo = new ProcessStartInfo();
				StartInfo.FileName = SymStoreExe.FullName;
				StartInfo.Arguments = string.Format("add /f \"@{0}\" /s \"{1}\" /t \"{2}\" /o", TempFileName, TempSymStoreDir, Product);
				StartInfo.UseShellExecute = false;
				StartInfo.CreateNoWindow = true;
				if (Utils.RunLocalProcessAndLogOutput(StartInfo, Log.Logger) != 0)
				{
					return false;
				}
			}
			finally
			{
				File.Delete(TempFileName);
				CommandUtils.DeleteDirectory(TempSymStoreIndexedDir);
			}
			DateTime CompressDone = DateTime.Now;
			Logger.LogInformation("Took {Arg0}s to compress the symbol files to temp path {TempSymStoreDir}", (CompressDone - Start).TotalSeconds, TempSymStoreDir);

			int CopiedCount = 0;

			// Take each new compressed file made and try and copy it to the real symstore.  Exclude any symstore admin files
			foreach(FileReference File in DirectoryReference.EnumerateFiles(TempSymStoreDir, "*.*", SearchOption.AllDirectories).Where(File => IsSymbolFile(File)))
			{
				string RelativePath = File.MakeRelativeTo(DirectoryReference.Combine(TempSymStoreDir));
				FileReference ActualDestinationFile = FileReference.Combine(SymbolStoreDirectory, RelativePath);

				// Try and add a version file.  Do this before checking to see if the symbol is there already in the case of exact matches (multiple builds could use the same pdb, for example)
				if (!string.IsNullOrWhiteSpace(BuildVersion))
				{
					FileReference BuildVersionFile = FileReference.Combine(ActualDestinationFile.Directory, string.Format("{0}.version", BuildVersion));
					// Attempt to create the file. Just continue if it fails.
					try
					{
						DirectoryReference.CreateDirectory(BuildVersionFile.Directory);
						FileReference.WriteAllText(BuildVersionFile, string.Empty);
					}
					catch (Exception Ex)
					{
						Logger.LogWarning("Failed to write the version file, reason {Arg0}", Ex.ToString());
					}
				}

				// Don't bother copying the temp file if the destination file is there already.
				if (FileReference.Exists(ActualDestinationFile))
				{
					Logger.LogInformation("Destination file {Arg0} already exists, skipping", ActualDestinationFile.FullName);
					continue;
				}

				FileReference TempDestinationFile = new FileReference(ActualDestinationFile.FullName + Guid.NewGuid().ToString());
				try
				{
					CommandUtils.CopyFile(File.FullName, TempDestinationFile.FullName);
				}
				catch(Exception Ex)
				{
					throw new AutomationException("Couldn't copy the symbol file to the temp store! Reason: {0}", Ex.ToString());
				}
				// Move the file in the temp store over.
				try
				{
					FileReference.Move(TempDestinationFile, ActualDestinationFile);
					//LogVerbose("Moved {0} to {1}", TempDestinationFile, ActualDestinationFile);
					CopiedCount++;
				}
				catch (Exception Ex)
				{
					// If the file is there already, it was likely either copied elsewhere (and this is an ioexception) or it had a file handle open already.
					// Either way, it's fine to just continue on.
					if (FileReference.Exists(ActualDestinationFile))
					{
						Logger.LogInformation("Destination file {Arg0} already exists or was in use, skipping.", ActualDestinationFile.FullName);
						continue;
					}
					// If it doesn't exist, we actually failed to copy it entirely.
					else
					{
						Logger.LogWarning("Couldn't move temp file {Arg0} to the symbol store at location {Arg1}! Reason: {Arg2}", TempDestinationFile.FullName, ActualDestinationFile.FullName, Ex.ToString());
					}
				}
				// Delete the temp one no matter what, don't want them hanging around in the symstore
				finally
				{
					FileReference.Delete(TempDestinationFile);
				}
			}
			Logger.LogInformation("Took {Arg0}s to copy {CopiedCount} symbol files to the store at {SymbolStoreDirectory}", (DateTime.Now - CompressDone).TotalSeconds, CopiedCount, SymbolStoreDirectory);

			FileReference PingmeFile = FileReference.Combine(SymbolStoreDirectory, "pingme.txt");
			if (!FileReference.Exists(PingmeFile))
			{
				Logger.LogInformation("Creating {PingmeFile} to mark path as three-tiered symbol location", PingmeFile);
				File.WriteAllText(PingmeFile.FullName, "Exists to mark this as a three-tiered symbol location");
			}
		}
			
		return true;
    }

	[SupportedOSPlatform("windows")]
	private IEnumerable<FileReference> CopySymbolsWithSourceIndexing(DirectoryReference TargetDirectory, IEnumerable<FileReference> SymbolFiles,
	IEnumerable<FileReference> SourceFiles, string Branch, int Change, string VFSMapping)
	{
		if (DirectoryReference.Exists(TargetDirectory))
		{
			CommandUtils.DeleteDirectory(TargetDirectory);
			DirectoryReference.CreateDirectory(TargetDirectory);
		}

		List<FileReference> CopiedSymbolFiles = new List<FileReference>();
		List<FileReference> SkippedSymbolFiles = new List<FileReference>();

		// Copy all the symbol files to the given directory.
		foreach (FileReference File in SymbolFiles)
		{
			string RelativePath = File.MakeRelativeTo(Unreal.RootDirectory);

			if (RelativePath == File.FullName)
			{
				Logger.LogInformation("Skipping source indexing for \"{File}\" because it is not under the root directory \"{RootDirectory}\"", File.FullName, Unreal.RootDirectory);

				SkippedSymbolFiles.Add(File);

				continue;
			}

			FileReference DestinationFile = FileReference.Combine(TargetDirectory, RelativePath);

			try
			{
				CommandUtils.CopyFile(File.FullName, DestinationFile.FullName);
			}
			catch (Exception Ex)
			{
				throw new AutomationException("Couldn't copy the pdb file to the temp directory for indexing! Reason: {0}", Ex.ToString());
			}

			CopiedSymbolFiles.Add(DestinationFile);
		}

		// Index all symbol files in one go (indexing source code from the source control is shared work between all pdbs).
		AddSourceIndexToSymbols(CopiedSymbolFiles, SourceFiles, Branch, Change, VFSMapping);

		// If we skipped copying and source indexing of some files, we want to preserve the original references as this list
		// is later used for symbol uploading.
		CopiedSymbolFiles.AddRange(SkippedSymbolFiles);

		return CopiedSymbolFiles;
	}

	bool IsSymbolFile(FileReference File)
	{
		if (File.HasExtension(".dll") || File.HasExtension(".exe") || File.HasExtension(".pdb"))
		{
			return true;
		}
		if (File.HasExtension(".dl_") || File.HasExtension(".ex_") || File.HasExtension(".pd_"))
		{
			return true;
		}
		return false;
	}

	/// <summary>
	/// 
	/// </summary>
	/// <param name="PdbFiles"></param>
	/// <param name="SourceFiles"></param>
	/// <param name="Branch"></param>
	/// <param name="Change"></param>
	/// <param name="VFSMappingString"></param>
	[SupportedOSPlatform("windows")]
	public void AddSourceIndexToSymbols(IEnumerable<FileReference> PdbFiles, IEnumerable<FileReference> SourceFiles, string Branch, int Change, string VFSMappingString)
	{
		Logger.LogInformation("Adding source control information to PDB files...");

		string DepotFilter = ".../Source/...";

		// On Windows we get real (non-virtualized) paths as SourceFiles. We pass null as VFSMapping to BuildSourceDatabase
		// so that our source database was indexed with these real paths. We only need to virtualize them when adding source info into PDB for the debugger.
		Dictionary<string, P4HaveRecord> SourceDatabase = SourceIndexingUtils.BuildSourceDatabase(DepotFilter, null);

		if (SourceDatabase == null)
		{
			throw new AutomationException($"Failed to query the source code information for '{DepotFilter}'.");
		}

		var VFSMapping = SourceIndexingUtils.ParseVFSMapping(VFSMappingString);

		SourceIndexingUtils.OutputVFSMapping(VFSMapping);

		// Get the pdbstr.exe path, using the latest SDK version we can find.
		FileReference PdbStrExe = GetPdbStrExe();

		// Get the path to the generated SRCSRV.INI file
		FileReference SrcSrvIni = FileReference.Combine(Unreal.RootDirectory, "Engine", "Intermediate", "SrcSrv.ini");
		DirectoryReference.CreateDirectory(SrcSrvIni.Directory);

		// Generate the SRCSRV.INI file
		using (StreamWriter Writer = new StreamWriter(SrcSrvIni.FullName))
		{
			int MissingFilesCount = 0;

			Writer.WriteLine("SRCSRV: ini------------------------------------------------");
			Writer.WriteLine("VERSION=1");
			Writer.WriteLine("VERCTRL=Perforce");
			Writer.WriteLine("SRCSRV: variables------------------------------------------");
			Writer.WriteLine("SRCSRVTRG=%sdtrg%");
			Writer.WriteLine("SRCSRVCMD=%sdcmd%");
			Writer.WriteLine("SDCMD=p4.exe print -o %srcsrvtrg% \"//%var2%#%var3%\"");
			Writer.WriteLine("SDTRG=%targ%\\%fnbksl%(%var2%)#%var3%");
			Writer.WriteLine("SRCSRV: source files ---------------------------------------");
			foreach (FileReference SourceFile in SourceFiles)
			{
				P4HaveRecord SourceInfo;

				if (SourceDatabase.TryGetValue(SourceFile.FullName, out SourceInfo))
				{
					Writer.WriteLine("{0}*{1}*{2}", SourceIndexingUtils.ConvertLocalToVirtualized(SourceFile.FullName, VFSMapping), SourceInfo.DepotFile.Replace("//", ""), SourceInfo.Revision);
				}
				else
				{
					++MissingFilesCount;
				}
			}
			Writer.WriteLine("SRCSRV: end------------------------------------------------");

			if (MissingFilesCount > 0)
			{
				Logger.LogInformation("Skipped {MissingFilesCount} files (out of {SourceFileCount}) for which source control files couldn't be located.", MissingFilesCount, SourceFiles.Count());
			}
		}

		// Execute PDBSTR on the PDB files in parallel.
		Parallel.ForEach(PdbFiles, (PdbFile, State) => { ExecutePdbStrTool(PdbStrExe, PdbFile, SrcSrvIni, State); });
	}

	/// <summary>
	/// Executes the PdbStr tool.
	/// </summary>
	/// <param name="PdbStrExe">Path to pdbstr.exe</param>
	/// <param name="PdbFile">The PDB file to embed source information for</param>
	/// <param name="SrcSrvIni">Ini file containing settings to embed</param>
	/// <param name="State">The current loop state</param>
	/// <returns>True if the tool executed successfully</returns>
	static void ExecutePdbStrTool(FileReference PdbStrExe, FileReference PdbFile, FileReference SrcSrvIni, ParallelLoopState State)
	{
		FileInfo PdbInfo = new FileInfo(PdbFile.FullName);
		FileInfo IniInfo = new FileInfo(SrcSrvIni.FullName);

		using (Process Process = new Process())
		{
			List<string> Messages = new List<string>();

			Messages.Add(String.Format("Writing source server data: {0}", PdbFile));

			DataReceivedEventHandler OutputHandler = (s, e) => { if (e.Data != null) { Messages.Add(e.Data); } };
			Process.StartInfo.FileName = PdbStrExe.FullName;
			Process.StartInfo.Arguments = String.Format("-w -p:\"{0}\" -i:\"{1}\" -s:srcsrv", PdbFile.FullName, SrcSrvIni.FullName);
			Process.StartInfo.UseShellExecute = false;
			Process.StartInfo.RedirectStandardOutput = true;
			Process.StartInfo.RedirectStandardError = true;
			Process.StartInfo.RedirectStandardInput = false;
			Process.StartInfo.CreateNoWindow = true;
			Process.OutputDataReceived += OutputHandler;
			Process.ErrorDataReceived += OutputHandler;
			Process.Start();
			Process.BeginOutputReadLine();
			Process.BeginErrorReadLine();
			Process.WaitForExit();

			if (Process.ExitCode != 0)
			{
				Messages.Add($"Failed to embed source server data for {PdbFile} (exit code: {Process.ExitCode})");
			}

			lock (State)
			{
				foreach (string Message in Messages)
				{
					Logger.LogInformation("{Text}", Message);
				}
			}
		}
	}

	/// <summary>
	/// Try to get the pdbstr.exe path from the Windows SDK
	/// </summary>
	/// <returns>Path to pdbstr.exe</returns>
	[SupportedOSPlatform("windows")]
	static FileReference GetPdbStrExe()
	{
		List<KeyValuePair<string, DirectoryReference>> WindowsSdkDirs = WindowsExports.GetWindowsSdkDirs();

		// Trying first to look for auto sdk latest WindowsKits debugger tools
		DirectoryReference HostAutoSdkDir = null;
		if (UEBuildPlatformSDK.TryGetHostPlatformAutoSDKDir(out HostAutoSdkDir))
		{
			DirectoryReference WindowsKitsDebuggersDirAutoSdk = DirectoryReference.Combine(HostAutoSdkDir, "Win64", "Windows Kits", "Debuggers");

			if (DirectoryReference.Exists(WindowsKitsDebuggersDirAutoSdk))
			{
				FileReference CheckPdbStrExe64 = FileReference.Combine(WindowsKitsDebuggersDirAutoSdk, "x64", "srcsrv", "pdbstr.exe");
				if (FileReference.Exists(CheckPdbStrExe64))
				{
					return CheckPdbStrExe64;
				}
			}
		}

		foreach (DirectoryReference WindowsSdkDir in WindowsSdkDirs.Select(x => x.Value))
		{
			FileReference CheckPdbStrExe64 = FileReference.Combine(WindowsSdkDir, "Debuggers", "x64", "srcsrv", "pdbstr.exe");
			if (FileReference.Exists(CheckPdbStrExe64))
			{
				return CheckPdbStrExe64;
			}
		}

        throw new AutomationException("Unable to find a Windows SDK installation containing Debuggers/x64/srcsrv/pdbstr.exe");
	}

	public override string[] SymbolServerDirectoryStructure
    {
        get
        {
            return new string[]
            {
                "{0}*.pdb;{0}*.exe;{0}*.dll", // Binary File Directory (e.g. QAGameClient-Win64-Test.exe --- .pdb, .dll and .exe are allowed extensions)
                "*",                          // Hash Directory        (e.g. A92F5744D99F416EB0CCFD58CCE719CD1)
            };
        }
    }
	
	// Lock file no longer needed since files are moved over the top from the temp symstore
	public override bool SymbolServerRequiresLock
	{
		get
		{
			return false;
		}
	}
}
