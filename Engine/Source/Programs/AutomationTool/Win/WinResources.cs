// Copyright Epic Games, Inc. All Rights Reserved.
using EpicGames.Core;
using System;
using System.Collections.Generic;
using System.Linq;
using System.Text;
using System.IO;
using System.Runtime.InteropServices;

public enum ResourceType
{
	Icon = 3,
	RawData = 10,
	GroupIcon = 14,
	Version = 16,
}

public class IconResource
{
	public byte Width;
	public byte Height;
	public byte ColorCount;
	public ushort Planes;
	public ushort BitCount;
	public byte[] Data;

	public IconResource(byte InWidth, byte InHeight, byte InColorCount, ushort InPlanes, ushort InBitCount, byte[] InData)
	{
		Width = InWidth;
		Height = InHeight;
		ColorCount = InColorCount;
		Planes = InPlanes;
		BitCount = InBitCount;
		Data = InData;
	}

	public override string ToString()
	{
		return String.Format("{0}x{1}@{2}bpp", Width, Height, BitCount);
	}
}

public class GroupIconResource
{
	public IconResource[] Icons;

	public GroupIconResource(IconResource[] InIcons)
	{
		Icons = InIcons;
	}

	public static GroupIconResource FromIco(string FileName)
	{
		using(FileStream Stream = new FileStream(FileName, FileMode.Open, FileAccess.Read))
		{
			BinaryReader Reader = new BinaryReader(Stream);

			Reader.ReadUInt16(); // Reserved
			Reader.ReadUInt16(); // Type
			ushort Count = Reader.ReadUInt16();

			IconResource[] Icons = new IconResource[Count];
			uint[] Offsets = new uint[Count];

			for(int Idx = 0; Idx < Count; Idx++)
			{
				Icons[Idx] = ReadIconHeader(Reader);
				Offsets[Idx] = Reader.ReadUInt32();
			}

			for(int Idx = 0; Idx < Icons.Length; Idx++)
			{
				Stream.Seek(Offsets[Idx], SeekOrigin.Begin);
				Stream.Read(Icons[Idx].Data, 0, Icons[Idx].Data.Length);
			}

			return new GroupIconResource(Icons);
		}
	}

	public static GroupIconResource FromExe(string FileName)
	{
		using(ModuleResourceLibary Module = new ModuleResourceLibary(FileName))
		{
			byte[] GroupData = Module.ReadFirstResource(ResourceType.GroupIcon);
			return FromByteArray(GroupData, Module);
		}
	}

	public static GroupIconResource FromByteArray(byte[] GroupData, ModuleResourceLibary Module)
	{
		if (GroupData != null)
		{
			using (MemoryStream Input = new MemoryStream(GroupData))
			{
				BinaryReader Reader = new BinaryReader(Input);

				Reader.ReadUInt16(); // Reserved
				Reader.ReadUInt16(); // Type
				ushort Count = Reader.ReadUInt16();

				IconResource[] Icons = new IconResource[Count];
				for (int Idx = 0; Idx < Count; Idx++)
				{
					Icons[Idx] = ReadIconHeader(Reader);
					int IconId = Reader.ReadUInt16();
					Icons[Idx].Data = Module.ReadResource(IconId, ResourceType.Icon);
				}
				return new GroupIconResource(Icons);
			}
		}
		else
		{
			return null;
		}
	}

	public byte[] ToByteArray()
	{
		using(MemoryStream Output = new MemoryStream())
		{
			BinaryWriter Writer = new BinaryWriter(Output);

			Writer.Write((ushort)0); // Reserved
			Writer.Write((ushort)1); // Type (1 = icons)
			Writer.Write((ushort)Icons.Length); 

			for(int Idx = 0; Idx < Icons.Length; Idx++)
			{
				WriteIconHeader(Writer, Icons[Idx]);
				Writer.Write((ushort)(Idx + 1));
			}

			return Output.ToArray();
		}
	}

	public static IconResource ReadIconHeader(BinaryReader Reader)
	{
		byte Width = Reader.ReadByte();
		byte Height = Reader.ReadByte();
		byte ColorCount = Reader.ReadByte();
		if(Reader.ReadByte() != 0) throw new InvalidDataException();
		ushort Planes = Reader.ReadUInt16();
		ushort BitCount = Reader.ReadUInt16();
		byte[] Data = new byte[Reader.ReadUInt32()];

		return new IconResource(Width, Height, ColorCount, Planes, BitCount, Data);
	}

	public void WriteIconHeader(BinaryWriter Writer, IconResource Icon)
	{
		Writer.Write(Icon.Width);
		Writer.Write(Icon.Height);
		Writer.Write(Icon.ColorCount);
		Writer.Write((byte)0);
		Writer.Write(Icon.Planes);
		Writer.Write(Icon.BitCount);
		Writer.Write(Icon.Data.Length);
	}
}

public class ModuleResourceLibary : IDisposable
{
	const uint LOAD_LIBRARY_AS_DATAFILE = 0x00000002; 

	delegate bool EnumResourceNamesDelegate(IntPtr ModuleHandle, IntPtr Type, IntPtr Name, IntPtr Param);

	[DllImport("kernel32.dll", SetLastError=true)]
	extern static IntPtr LoadLibraryEx(string FileName, IntPtr FileHandle, uint Flags);

	[DllImport("kernel32.dll", SetLastError=true)]
	extern static bool FreeLibrary(IntPtr Handle);

	[DllImport("kernel32.dll", SetLastError=true)]
	static extern bool EnumResourceNames(IntPtr ModuleHandle, IntPtr Type, EnumResourceNamesDelegate EnumDelegate, IntPtr Param);

	[DllImport("kernel32.dll", SetLastError=true)]
	static extern IntPtr FindResource(IntPtr ModuleHandle, IntPtr Name, IntPtr Type);

	[DllImport("kernel32.dll", SetLastError=true)]
	static extern IntPtr LoadResource(IntPtr ModuleHandle, IntPtr ResourceHandle);

	[DllImport("kernel32.dll", SetLastError=true)]
	static extern IntPtr LockResource(IntPtr ResourceDataHandle);

	[DllImport("kernel32.dll", SetLastError=true)]
	static extern uint SizeofResource(IntPtr ModuleHandle, IntPtr ResourceDataHandle);

	IntPtr ModuleHandle;

	public ModuleResourceLibary(string FileName)
	{
		ModuleHandle = LoadLibraryEx(FileName, new IntPtr(0), LOAD_LIBRARY_AS_DATAFILE);
	}

	public void Dispose()
	{
		FreeLibrary(ModuleHandle);
	}

	public byte[] ReadResource(int ResourceId, ResourceType Type)
	{
		return ReadResourceInternal(new IntPtr(ResourceId), new IntPtr((int)Type));
	}

	public byte[] ReadFirstResource(ResourceType Type)
	{
		byte[] Result = null;
		EnumResourceNames(ModuleHandle, new IntPtr((int)Type), (Handle, FoundType, Name, Param) => { Result = ReadResourceInternal(Name, FoundType); return false; }, new IntPtr(0));
		return Result;
	}

	byte[] ReadResourceInternal(IntPtr Name, IntPtr Type)
	{
		IntPtr ResourceHandle = FindResource(ModuleHandle, Name, Type);

		IntPtr ResourceDataHandle = LoadResource(ModuleHandle, ResourceHandle);
		IntPtr ResourceData = LockResource(ResourceDataHandle);
	
		uint ResourceSize = SizeofResource(ModuleHandle, ResourceHandle);

		byte[] ResourceDataArray = new byte[ResourceSize];
		Marshal.Copy(ResourceData, ResourceDataArray, 0, (int)ResourceSize);

		return ResourceDataArray;
	}
}

public class ModuleResourceUpdate : IDisposable
{
	[DllImport("kernel32.dll", SetLastError=true)]
	static extern IntPtr BeginUpdateResource(string pFileName, [MarshalAs(UnmanagedType.Bool)]bool bDeleteExistingResources);

	[DllImport("kernel32.dll", SetLastError=true)]
	static extern bool UpdateResource(IntPtr hUpdate, IntPtr lpType, IntPtr lpName, ushort wLanguage, IntPtr lpData, uint cbData);

	[DllImport("kernel32.dll", SetLastError=true)]
	static extern bool EndUpdateResource(IntPtr hUpdate, bool fDiscard);

	const ushort DefaultLanguage = 1033; // en-us

	IntPtr UpdateHandle;
	List<IntPtr> UnmanagedPointers = new List<IntPtr>();

	public ModuleResourceUpdate(string OutputFile, bool bRemoveExisting)
	{
		UpdateHandle = BeginUpdateResource(OutputFile, bRemoveExisting);
		EpicGames.Core.Log.WriteLine(EpicGames.Core.LogEventType.Console, "Begin update resource '{0}' ({1})", OutputFile, Marshal.GetLastWin32Error());
	}

	public void SetData(int ResourceId, ResourceType Type, byte[] Data)
	{
		IntPtr UnmanagedPointer = Marshal.AllocHGlobal(Data.Length);
		UnmanagedPointers.Add(UnmanagedPointer);

		Marshal.Copy(Data, 0, UnmanagedPointer, Data.Length);

		bool UpdateReturn = UpdateResource(UpdateHandle, new IntPtr((int)Type), new IntPtr(ResourceId), DefaultLanguage, UnmanagedPointer, (uint)Data.Length);

		EpicGames.Core.Log.WriteLine(EpicGames.Core.LogEventType.Console, "Add resource {0}, length {1} ({2})", ResourceId, Data.Length, UpdateReturn);

		if (!UpdateReturn)
		{
			throw new Exception("Couldn't update resource");
		}
	}

	public void SetIcons(int ResourceId, GroupIconResource GroupIcon)
	{
		// Write the icon group resource
		SetData(ResourceId, ResourceType.GroupIcon, GroupIcon.ToByteArray());

		// Write the individual icons
		for(int Idx = 0; Idx < GroupIcon.Icons.Length; Idx++)
		{
			SetData(Idx + 1, ResourceType.Icon, GroupIcon.Icons[Idx].Data);
		}
	}

	public void Dispose()
	{
		bool EndReturn = EndUpdateResource(UpdateHandle, false);
		EpicGames.Core.Log.WriteLine(EpicGames.Core.LogEventType.Console, "End update resource ({0}/{1})", EndReturn, Marshal.GetLastWin32Error());

		foreach(IntPtr UnmanagedPointer in UnmanagedPointers)
		{
			Marshal.FreeHGlobal(UnmanagedPointer);
		}
		UnmanagedPointers.Clear();
	}
}

