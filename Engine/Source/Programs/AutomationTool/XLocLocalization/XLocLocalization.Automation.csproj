<Project Sdk="Microsoft.NET.Sdk">
  <Import Project="..\..\Shared\UnrealEngine.csproj.props" />
  
  <PropertyGroup>
    <OutputType>Library</OutputType>
    <GenerateAssemblyInfo>false</GenerateAssemblyInfo>
    <GenerateTargetFrameworkAttribute>false</GenerateTargetFrameworkAttribute>
    <RootNamespace>AutomationTool</RootNamespace>
    <AssemblyName>XLocLocalization.Automation</AssemblyName>
    <WarningsNotAsErrors>612,618</WarningsNotAsErrors>
    <OutputPath Condition="'$(ArtifactsPath)' == ''">..\..\..\..\Binaries\DotNET\AutomationTool\AutomationScripts\XLocLocalization</OutputPath>
    <TreatWarningsAsErrors>true</TreatWarningsAsErrors>
    <CopyLocalLockFileAssemblies>true</CopyLocalLockFileAssemblies>
    <SatelliteResourceLanguages>en</SatelliteResourceLanguages> <!-- remove non english resource languages -->
  </PropertyGroup>

  <ItemGroup>
    <ProjectReference Include="..\..\Shared\EpicGames.Core\EpicGames.Core.csproj" PrivateAssets="All"><Private>false</Private></ProjectReference>
    <ProjectReference Include="..\..\UnrealBuildTool\UnrealBuildTool.csproj" PrivateAssets="All"><Private>false</Private></ProjectReference>
    <ProjectReference Include="..\AutomationUtils\AutomationUtils.Automation.csproj" PrivateAssets="All"><Private>false</Private></ProjectReference>
    <ProjectReference Include="..\Localization\Localization.Automation.csproj" PrivateAssets="All"><Private>false</Private></ProjectReference>
  </ItemGroup>
  <ItemGroup>
    <PackageReference Include="System.Security.Cryptography.Pkcs" Version="6.0.5" />
    <PackageReference Include="System.ServiceModel.Http" Version="4.10.3" PrivateAssets="all" />
  </ItemGroup>
</Project>