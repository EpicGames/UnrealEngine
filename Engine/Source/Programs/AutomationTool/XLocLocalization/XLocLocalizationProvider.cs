// Copyright Epic Games, Inc. All Rights Reserved.
using System;
using System.Collections.Generic;
using System.IO;
using System.Text;
using System.Linq;
using System.Net;
using System.Xml;
using System.ServiceModel;
using System.Security.Cryptography;
using AutomationTool;
using UnrealBuildTool;
using EpicGames.Localization;
using System.Threading.Tasks;
using Microsoft.Extensions.Logging;

using static AutomationTool.CommandUtils;

#pragma warning disable SYSLIB0014

namespace EpicGames.XLocLocalization
{
	public struct XLocConfig
	{
		public string Server;
		public string APIKey;
		public string APISecret;
		public string LocalizationId;
	};

	public struct XLocUtils
	{
		public static string MD5HashString(string Str)
		{
			var HashBytes = MD5.Create().ComputeHash(Encoding.UTF8.GetBytes(Str));

			StringBuilder HashString = new StringBuilder();
			foreach (var HashByte in HashBytes)
			{
				HashString.Append(HashByte.ToString("x2"));
			}
			return HashString.ToString();
		}

		public static WebResponse GetWebResponse(WebRequest Request)
		{
			// GetResponse sometimes throws rather than just letting you use the error code in the response
			try
			{
				return Request.GetResponse();
			}
			catch (WebException Ex)
			{
				return Ex.Response;
			}
		}
	}

	public abstract class XLocLocalizationProvider : LocalizationProvider
	{
		public XLocLocalizationProvider(LocalizationProviderArgs InArgs)
			: base(InArgs)
		{
			Config = new XLocConfig();
		}

		public async override Task DownloadProjectFromLocalizationProvider(string ProjectName, ProjectImportExportInfo ProjectImportInfo)
		{
			var XLocApiClient = CreateXLocApiClient();

			try
			{
				var AuthToken = RequestAuthTokenWithRetry(XLocApiClient);

				// Get the latest files for each culture.
				foreach (var Culture in ProjectImportInfo.CulturesToGenerate)
				{
					// Skip the native culture, as XLoc doesn't have an entry for it
					if (Culture == ProjectImportInfo.NativeCulture)
					{
						continue;
					}

					DownloadLatestPOFile(XLocApiClient, AuthToken, Culture, null, ProjectImportInfo);
					foreach (var Platform in ProjectImportInfo.SplitPlatformNames)
					{
						DownloadLatestPOFile(XLocApiClient, AuthToken, Culture, Platform, ProjectImportInfo);
					}
				}
			}
			finally
			{
				XLocApiClient.Close();
			}

			await Task.CompletedTask;
		}

		private void DownloadLatestPOFile(XLocApiClient XLocApiClient, string AuthToken, string Culture, string Platform, ProjectImportExportInfo ProjectImportInfo)
		{
			var XLocFilename = GetXLocFilename(ProjectImportInfo.PortableObjectName, Platform);

			// This will throw if the requested culture is invalid, but we don't want to let that kill the whole gather
			var LatestBuildXml = "";
			try
			{
				var EpicCultureToXLocLanguageId = GetEpicCultureToXLocLanguageId();
				LatestBuildXml = RequestLatestBuild(XLocApiClient, AuthToken, EpicCultureToXLocLanguageId[Culture], XLocFilename);
			}
			catch (Exception Ex)
			{
				Logger.LogWarning(Ex, "RequestLatestBuild failed for {Culture}. {Ex}", Culture, Ex);
				return;
			}
			if (String.IsNullOrEmpty(LatestBuildXml))
			{
				Logger.LogInformation("[IGNORED] '{0}' has no build data ({1})", XLocFilename, Culture);
				return;
			}

			var POFileUri = "";
			var BuildsXmlDoc = new XmlDocument();
			BuildsXmlDoc.LoadXml(LatestBuildXml);

			var BuildElem = BuildsXmlDoc["Build"];
			if (BuildElem != null)
			{
				var BuildFilesElem = BuildElem["BuildFiles"];
				if (BuildFilesElem != null)
				{
					foreach (XmlNode BuildFile in BuildFilesElem)
					{
						bool IsCorrectFile = false;

						// Is this the file we want?
						var GameFileElem = BuildFile["GameFile"];
						if (GameFileElem != null)
						{
							var GameFileNameElem = GameFileElem["Name"];
							if (GameFileNameElem != null && GameFileNameElem.InnerText.Equals(XLocFilename, StringComparison.OrdinalIgnoreCase))
							{
								IsCorrectFile = true;
							}
						}

						if (IsCorrectFile)
						{
							var BuildFileDownloadUriElem = BuildFile["DownloadUri"];
							if (BuildFileDownloadUriElem != null)
							{
								POFileUri = BuildFileDownloadUriElem.InnerText;
								break;
							}
						}
					}
				}
			}

			if (String.IsNullOrEmpty(POFileUri))
			{
				Console.WriteLine("[IGNORED] '{0}' was not found in the build data ({1})", XLocFilename, Culture);
			}
			else
			{
				var DestinationDirectory = String.IsNullOrEmpty(Platform)
						? new DirectoryInfo(CommandUtils.CombinePaths(RootWorkingDirectory, ProjectImportInfo.DestinationPath))
						: new DirectoryInfo(CommandUtils.CombinePaths(RootWorkingDirectory, ProjectImportInfo.DestinationPath, ProjectImportExportInfo.PlatformLocalizationFolderName, Platform));
				var CultureDirectory = (ProjectImportInfo.bUseCultureDirectory) ? new DirectoryInfo(Path.Combine(DestinationDirectory.FullName, Culture)) : DestinationDirectory;
				if (!CultureDirectory.Exists)
				{
					CultureDirectory.Create();
				}

				var HTTPRequest = WebRequest.Create(POFileUri);
				HTTPRequest.Method = "GET";
				using (var Response = (HttpWebResponse)XLocUtils.GetWebResponse(HTTPRequest))
				{
					if (Response.StatusCode != HttpStatusCode.OK)
					{
						Logger.LogWarning("HTTP Request to '{Url}' failed. {Status}", POFileUri, Response.StatusDescription);
						return;
					}

					using (var ResponseStream = Response.GetResponseStream())
					{
						var ExportFile = new FileInfo(Path.Combine(CultureDirectory.FullName, ProjectImportInfo.PortableObjectName));

						// Write out the updated PO file so that the gather commandlet will import the new data from it
						{
							var ExportFileWasReadOnly = false;
							if (ExportFile.Exists)
							{
								// We're going to clobber the existing PO file, so make sure it's writable (it may be read-only if in Perforce)
								ExportFileWasReadOnly = ExportFile.IsReadOnly;
								ExportFile.IsReadOnly = false;
							}

							using (var FileStream = ExportFile.Open(FileMode.Create))
							{
								ResponseStream.CopyTo(FileStream);
								Logger.LogInformation("[SUCCESS] Exporting: '{File}' as '{ExportFile}' ({Culture})", XLocFilename, ExportFile.FullName, Culture);
							}

							if (ExportFileWasReadOnly)
							{
								ExportFile.IsReadOnly = true;
							}
						}

						// Also update the back-up copy so we can diff against what we got from XLoc, and what the gather commandlet produced
						{
							var ExportFileCopy = new FileInfo(Path.Combine(ExportFile.DirectoryName, String.Format("{0}_FromXLoc{1}", Path.GetFileNameWithoutExtension(ExportFile.Name), ExportFile.Extension)));

							var ExportFileCopyWasReadOnly = false;
							if (ExportFileCopy.Exists)
							{
								// We're going to clobber the existing PO file, so make sure it's writable (it may be read-only if in Perforce)
								ExportFileCopyWasReadOnly = ExportFileCopy.IsReadOnly;
								ExportFileCopy.IsReadOnly = false;
							}

							ExportFile.CopyTo(ExportFileCopy.FullName, true);

							if (ExportFileCopyWasReadOnly)
							{
								ExportFileCopy.IsReadOnly = true;
							}

							// Add/check out backed up POs from OneSky.
							if (CommandUtils.P4Enabled)
							{
								UnrealBuild.AddBuildProductsToChangelist(PendingChangeList, new List<string>() { ExportFileCopy.FullName });
							}
						}
					}
				}
			}
		}

		public async override Task UploadProjectToLocalizationProvider(string ProjectName, ProjectImportExportInfo ProjectExportInfo)
		{
			var XLocApiClient = CreateXLocApiClient();
			var TransferServiceClient = CreateTransferServiceClient();

			try
			{
				var AuthToken = RequestAuthTokenWithRetry(XLocApiClient);

				// Upload the .po file for the native culture first
				UploadLatestPOFile(TransferServiceClient, AuthToken, ProjectExportInfo.NativeCulture, null, ProjectExportInfo);
				foreach (var Platform in ProjectExportInfo.SplitPlatformNames)
				{
					UploadLatestPOFile(TransferServiceClient, AuthToken, ProjectExportInfo.NativeCulture, Platform, ProjectExportInfo);
				}

				if (bUploadAllCultures)
				{
					// Upload the remaining .po files for the other cultures
					foreach (var Culture in ProjectExportInfo.CulturesToGenerate)
					{
						// Skip native culture as we uploaded it above
						if (Culture != ProjectExportInfo.NativeCulture)
						{
							UploadLatestPOFile(TransferServiceClient, AuthToken, Culture, null, ProjectExportInfo);
							foreach (var Platform in ProjectExportInfo.SplitPlatformNames)
							{
								UploadLatestPOFile(TransferServiceClient, AuthToken, Culture, Platform, ProjectExportInfo);
							}
						}
					}
				}
			}
			finally
			{
				XLocApiClient.Close();
				TransferServiceClient.Close();
			}

			await Task.CompletedTask;
		}

		private void UploadLatestPOFile(TransferServiceClient TransferServiceClient, string AuthToken, string Culture, string Platform, ProjectImportExportInfo ProjectExportInfo)
		{
			var SourceDirectory = String.IsNullOrEmpty(Platform)
					? new DirectoryInfo(CommandUtils.CombinePaths(RootWorkingDirectory, ProjectExportInfo.DestinationPath))
					: new DirectoryInfo(CommandUtils.CombinePaths(RootWorkingDirectory, ProjectExportInfo.DestinationPath, ProjectImportExportInfo.PlatformLocalizationFolderName, Platform));
			var CultureDirectory = (ProjectExportInfo.bUseCultureDirectory) ? new DirectoryInfo(Path.Combine(SourceDirectory.FullName, Culture)) : SourceDirectory;

			var FileToUpload = new FileInfo(Path.Combine(CultureDirectory.FullName, ProjectExportInfo.PortableObjectName));
			var XLocFilename = GetXLocFilename(ProjectExportInfo.PortableObjectName, Platform);

			using (var FileStream = FileToUpload.OpenRead())
			{
				// We need to leave the language ID field blank for the native culture to avoid XLoc trying to process it as translated source, rather than raw source text
				var EpicCultureToXLocLanguageId = GetEpicCultureToXLocLanguageId();
				var LanguageId = (Culture == ProjectExportInfo.NativeCulture) ? "" : EpicCultureToXLocLanguageId[Culture];

				var FileUploadMetaData = new XLoc.Contracts.GameFileUploadInfo();
				FileUploadMetaData.CaseSensitive = false;
				FileUploadMetaData.FileName = XLocFilename;
				FileUploadMetaData.HistoricalTranslation = false;
				FileUploadMetaData.LanguageId = LanguageId;
				FileUploadMetaData.LocalizationId = Config.LocalizationId;
				FileUploadMetaData.PlatformId = "!";

				Console.WriteLine("Uploading: '{0}' as '{1}' ({2})", FileToUpload.FullName, XLocFilename, Culture);

				try
				{
					TransferServiceClient.UploadGameFile(Config.APIKey, AuthToken, FileUploadMetaData, FileToUpload.Length, FileStream);
					Console.WriteLine("[SUCCESS] Uploading: '{0}' ({1})", FileToUpload.FullName, Culture);
				}
				catch (Exception Ex)
				{
					Console.WriteLine("[FAILED] Uploading: '{0}' ({1}) - {2}", FileToUpload.FullName, Culture, Ex);
				}
			}
		}

		protected XLocApiClient CreateXLocApiClient()
		{
			var Binding = new BasicHttpBinding();
			Binding.Name = "basicHttpXLocApi";
			Binding.CloseTimeout = new TimeSpan(0, 1, 0);
			Binding.OpenTimeout = new TimeSpan(0, 1, 0);
			Binding.ReceiveTimeout = new TimeSpan(0, 10, 0);
			Binding.SendTimeout = new TimeSpan(0, 1, 0);
			Binding.AllowCookies = false;
			Binding.BypassProxyOnLocal = false;
			Binding.MaxBufferSize = 2147483647;
			Binding.MaxBufferPoolSize = 5242880;
			Binding.MaxReceivedMessageSize = 2147483647;
			Binding.TextEncoding = Encoding.UTF8;
			Binding.TransferMode = TransferMode.Buffered;
			Binding.UseDefaultWebProxy = true;
			Binding.ReaderQuotas.MaxDepth = 32;
			Binding.ReaderQuotas.MaxStringContentLength = 2147483647;
			Binding.ReaderQuotas.MaxArrayLength = 65536;
			Binding.ReaderQuotas.MaxBytesPerRead = 4096;
			Binding.ReaderQuotas.MaxNameTableCharCount = 65536;

			var Endpoint = new EndpointAddress(new Uri(String.Format("http://{0}/api/XLocApiService.svc", Config.Server)));

			return new XLocApiClient(Binding, Endpoint);
		}

		protected TransferServiceClient CreateTransferServiceClient()
		{
			var Binding = new BasicHttpBinding();
			Binding.Name = "transfer";
			Binding.MaxReceivedMessageSize = 67108864;
			Binding.TransferMode = TransferMode.Streamed;

			var Endpoint = new EndpointAddress(new Uri(String.Format("http://{0}/api/XLocTransferService.svc", Config.Server)));

			return new TransferServiceClient(Binding, Endpoint);
		}

		protected string RequestAuthToken(XLocApiClient XLocApiClient)
		{
			return XLocApiClient.GetAuthToken(Config.APIKey, XLocUtils.MD5HashString(Config.APIKey + Config.APISecret));
		}

		protected string RequestAuthTokenWithRetry(XLocApiClient XLocApiClient)
		{
			const int MAX_COUNT = 3;

			int Count = 0;
			for (; ; )
			{
				try
				{
					return RequestAuthToken(XLocApiClient);
				}
				catch
				{
					if (++Count < MAX_COUNT)
					{
						Logger.LogWarning("RequestAuthToken attempt {Count}/{Total} failed. Retrying...", Count, MAX_COUNT);
						continue;
					}

					Logger.LogWarning("RequestAuthToken attempt {Count}/{Total} failed.", Count, MAX_COUNT);
					throw;
				}
			}
		}

		protected string RequestLatestBuild(XLocApiClient XLocApiClient, string AuthToken, string LanguageId, string RemoteFilename)
		{
			return XLocApiClient.GetLatestBuildByFile(Config.APIKey, AuthToken, XLocUtils.MD5HashString(Config.APIKey + Config.APISecret + Config.LocalizationId + LanguageId + RemoteFilename), Config.LocalizationId, LanguageId, RemoteFilename);
		}

		private string GetXLocFilename(string BaseFilename, string Platform)
		{
			var XLocFilename = BaseFilename;
			if (!String.IsNullOrEmpty(Platform))
			{
				// Apply the platform suffix. XLoc will take care of merging the files from different platforms together.
				var XLocFilenameWithSuffix = Path.GetFileNameWithoutExtension(XLocFilename) + "_" + Platform + Path.GetExtension(XLocFilename);
				XLocFilename = XLocFilenameWithSuffix;
			}
			if (!String.IsNullOrEmpty(LocalizationBranchName))
			{
				// Apply the branch suffix. XLoc will take care of merging the files from different branches together.
				var XLocFilenameWithSuffix = Path.GetFileNameWithoutExtension(XLocFilename) + "_" + LocalizationBranchName + Path.GetExtension(XLocFilename);
				XLocFilename = XLocFilenameWithSuffix;
			}
			if (!String.IsNullOrEmpty(RemoteFilenamePrefix))
			{
				// Apply the prefix (this is used to avoid collisions with plugins that use the same name for their PO files)
				XLocFilename = RemoteFilenamePrefix + "_" + XLocFilename;
			}
			return XLocFilename;
		}

		protected XLocConfig Config;

		virtual protected Dictionary<string, string> GetEpicCultureToXLocLanguageId()
		{
			return new Dictionary<string, string>
			{
				{ "en", "E" },			// English
				{ "en-US", "9" },		// English (US)
				{ "en-GB", "6" },		// English (British)
				{ "en-HK", "en-HK" },	// English (Hong Kong)
				{ "fr", "F" },			// French
				{ "fr-CA", "Q"},		// French (Canadian)
				{ "it", "I" },			// Italian
				{ "de", "G" },			// German
				{ "es", "S" },			// Spanish
				{ "es-419", "Y" },		// Spanish (Latin American)
				{ "da", "D" },			// Danish
				{ "nl", "U" },			// Dutch
				{ "fi", "B" },			// Finnish
				{ "sv", "W" },			// Swedish
				{ "ru", "R" },			// Russian
				{ "pl", "P" },			// Polish
				{ "ar", "A" },			// Arabic
				{ "ko", "K" },			// Korean
				{ "ja", "J" },			// Japanese
				{ "zh-Hans", "2" },		// Chinese (Simplified)
				{ "zh-Hant", "1" },		// Chinese (Traditional)
				{ "tr", "3" },			// Turkish
				{ "th", "T" },			// Thai
				{ "pt", "O" },			// Portuguese
				{ "pt-BR", "$" },		// Portuguese (Brazilian)
			};
		}
	}
}
