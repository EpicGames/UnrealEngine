<?xml version="1.0" encoding="utf-8"?>
<assembly manifestVersion="1.0" xmlns="urn:schemas-microsoft-com:asm.v1">
	<trustInfo xmlns="urn:schemas-microsoft-com:asm.v2">
		<security>
			<requestedPrivileges xmlns="urn:schemas-microsoft-com:asm.v3">
				<requestedExecutionLevel level="asInvoker" uiAccess="false" />
			</requestedPrivileges>
		</security>
	</trustInfo>
	<compatibility xmlns="urn:schemas-microsoft-com:compatibility.v1">
		<application>
			<!-- Windows 10 and Windows 11 -->
			<supportedOS Id="{8e0f7a12-bfb3-4fe8-b9a5-48fd50a15a9a}" />
		</application>
	</compatibility>
	<application xmlns="urn:schemas-microsoft-com:asm.v3">
		<windowsSettings>
			<longPathAware xmlns="http://schemas.microsoft.com/SMI/2016/WindowsSettings">true</longPathAware>
		</windowsSettings>
	</application>
</assembly>
